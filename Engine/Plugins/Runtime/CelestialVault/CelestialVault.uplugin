{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "CelestialVault",
	"Description": "A DaySequence implementation of a Celestial Vault for Earth using ephemeris",
	"Category": "Other",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"CanContainContent": true,
  	"IsExperimentalVersion": false,
  	"IsBetaVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "CelestialVault",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "CelestialVaultEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "DaySequence",
			"Enabled": true
		}
	]
}