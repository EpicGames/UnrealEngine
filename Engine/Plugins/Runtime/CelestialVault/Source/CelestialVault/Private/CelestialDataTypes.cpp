// Copyright Epic Games, Inc. All Rights Reserved.

#include "CelestialDataTypes.h"
#include "CelestialMaths.h"
#include "CelestialVault.h"

FString FCelestialBody::ToString() const
{
	return FString::Printf(TEXT(
			"Name: %s\n"
			"RA J2000 (Topocentric): %s\n"
			"DEC J2000 (Topocentric): %s\n"
			"RA of Date (Topocentric): %s\n"
			"DEC of Date (Topocentric): %s\n"
			"RA of Date (Geocentric): %s\n"
			"DEC of Date (Geocentric): %s\n"
			"Distance: %.0f AU (%.0f Parsecs)\n"
			"Radius: %.0f km\n"
			"Magnitude: %.2f\n"),
		*Name,
		*UCelestialMaths::Conv_RightAscensionToString(RAJ2000),
		*UCelestialMaths::Conv_DeclinationToString(DECJ2000),
		*UCelestialMaths::Conv_RightAscensionToString(RA),
		*UCelestialMaths::Conv_DeclinationToString(DEC),
		*UCelestialMaths::Conv_RightAscensionToString(RAGeocentric),
		*UCelestialMaths::Conv_DeclinationToString(DECGeocentric),
		DistanceInAU, UCelestialMaths::AstronomicalUnitsToParsec(DistanceInAU),
		Radius,
		Magnitude
	);
}

void FCelestialBody::ComputeUELocalTransform(double ExtraScale, double UEDistanceOverride)
{
	if (DistanceInAU <= 0)
	{
		UE_LOGF(LogCelestialVault, Error, "Invalid DistanceInAU (%f <= 0) for Celestial Body %ls - Check your inputs", DistanceInAU, *Name);
		UELocalTransform = FTransform::Identity;
		DirectionTowardEarth = -FVector::XAxisVector;
		return; 
	}
	
	// What Distance to use
	double UEDistance = UCelestialMaths::AstronomicalUnitsToMeters(DistanceInAU) * 100.0; // We want UE Units there
	if (UEDistanceOverride > 0.0)
	{
		UEDistance = UEDistanceOverride;
	}

	// Location — use geocentric RA/DEC for placement; topocentric parallax is handled by the camera being offset from PlanetCenterComponent (Earth's center).
	FVector BodyLocation = UCelestialMaths::RADECToXYZ_RH(RAGeocentric * 15, DECGeocentric, UEDistance);
	BodyLocation.Y *= -1; // Convert to UE Frame by inverting Y

	// Rotation - Orient the Body mesh towards the earth -
	// Use Look at the UE Origin, this is an acceptable approximation if the vault is big enough - We could use the Earth center if needed
	FVector BodyToEarth = (FVector::ZeroVector - BodyLocation).GetSafeNormal();
	FQuat BodyRotation = FQuat(BodyToEarth.Rotation());

	// Scale
	// The mesh plane is 100 UE Units (1m), and located at UEDistance. Use Thales theorem to compute its effective scale at this distance to ensure the right apparent diameter
	//                                                                             Body
	//                                                                             |
	//                                    UEObject                                 |  Body Radius
	//                                     |  UE Radius                            |
	//   Earth ----------------------------|---------------------------------------/
	//                                    UE Distance                             Body Distance
	//
	//    Body Radius (m)        UE Radius (UE units) 
	// => ------------------ = -------------------------
	//    Body Distance (m)      UE Distance (UE Units)
	//
	
	double UERadius = (UEDistance * Radius * 1000.0) / UCelestialMaths::AstronomicalUnitsToMeters(DistanceInAU);  // * 1000 is to transform the Body Radius (km) into (m)
	double Scale = UERadius / 50.0 * ExtraScale; // The plane half-length is 50 UE Units
	FVector BodyScale3D = FVector(Scale, Scale, Scale);

	// Build Transform
	UELocalTransform = FTransform(BodyRotation, BodyLocation, BodyScale3D);

	// Direction
	DirectionTowardEarth = BodyToEarth;
}

FString FStellarBody::ToString() const
{
	FString BaseString = Super::ToString();

	return FString::Printf(TEXT(
			"%s"
			"-----------------\n"
			"ColorIndex: %.2f\n"
			"HipparcosID %d\n"
			"HenryDraperID %d\n"
			"YaleBrightStarID %d\n"),
		*BaseString,
		ColorIndex,
		HipparcosID,
		HenryDraperID,
		YaleBrightStarID
	);
}


FString FPlanetaryBody::ToString() const
{
	FString BaseString = Super::ToString();

	FString OrbitInfoString = "";
	if (OrbitType == EOrbitType::VSOP87)
	{
		FString VSOP87BodyTypeName = FString("Unknown");
		if (UEnum* EnumClass = StaticEnum<EVSOP87BodyType>())
		{
			VSOP87BodyTypeName = EnumClass->GetNameStringByValue(static_cast<int64>(VSOP87BodyType));
		}

		OrbitInfoString = FString::Printf(TEXT("VSOP87BodyType: %s"), *VSOP87BodyTypeName);
	}
	else if (OrbitType == EOrbitType::Elliptic)
	{
		// TODO
		OrbitInfoString = "Elliptic Orbit";
	}


	return FString::Printf(TEXT(
			"%s"
			"-----------------\n"
			"%s\n"
			"ApparentDiameterDegrees: %s / %.8f°\n"
			"ScaledApparentDiameterDegrees: %.8f°\n"
			"Age: %.2f\n"
			"IlluminationPercentage: %.2f\n"),
		*BaseString,
		*OrbitInfoString,
		*UCelestialMaths::Conv_DeclinationToString(ApparentDiameterDegrees), ApparentDiameterDegrees,
		ScaledApparentDiameterDegrees,
		Age,
		IlluminationPercentage
	);
}

void FPlanetaryBody::ComputeUELocalTransform(double ExtraScale, double UEDistanceOverride)
{
	FCelestialBody::ComputeUELocalTransform(ExtraScale, UEDistanceOverride);

	// We need to scale the Apparent diameter too here. 
	ScaledApparentDiameterDegrees = ApparentDiameterDegrees * ExtraScale;
}
