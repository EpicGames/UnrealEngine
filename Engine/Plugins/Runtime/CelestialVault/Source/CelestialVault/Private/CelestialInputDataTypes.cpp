// Copyright Epic Games, Inc. All Rights Reserved.


#include "CelestialInputDataTypes.h"


// Earth Preset
FPlanetaryBodyInputData FPlanetaryBodyInputData::Earth(TEXT("Earth"), EOrbitType::VSOP87, EVSOP87BodyType::Earth, 6378.0);
// Moon Preset
FPlanetaryBodyInputData FPlanetaryBodyInputData::Moon(TEXT("Moon"), EOrbitType::VSOP87, EVSOP87BodyType::Moon, 1737.4);
