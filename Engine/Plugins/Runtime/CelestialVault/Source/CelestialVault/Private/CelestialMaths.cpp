// Copyright Epic Games, Inc. All Rights Reserved.


#include "CelestialMaths.h"
#include "CelestialVault.h"
#include "MathUtil.h"
#include "VSOP87.h"

#pragma region Static Members

// From VSOP87.doc
// REFERENCE SYSTEM
// ================
//
// The coordinates of the main version VSOP87 and of the versions A, B, and E
// are given in the inertial frame defined by the dynamical equinox and ecliptic
// J2000 (JD2451545.0).
//
// The rectangular coordinates of VSOP87A and VSOP87E defined in dynamical ecliptic
// frame J2000 can be connected to the equatorial frame FK5 J2000 with the
// following rotation :
//
//   X        +1.000000000000  +0.000000440360  -0.000000190919   X
//   Y     =  -0.000000479966  +0.917482137087  -0.397776982902   Y
//   Z FK5     0.000000000000  +0.397776982902  +0.917482137087   Z VSOP87A
// 
FMatrix const UCelestialMaths::VSOPEclipticToFK5Equatorial = FMatrix( 
		FVector(+1.000000000000, +0.000000440360, -0.000000190919),	
		FVector(-0.000000479966, +0.917482137087, -0.397776982902),	
		FVector(+0.000000000000, +0.397776982902, +0.917482137087),		 
		FVector(+0.000000000000, 0.000000000000, +0.000000000000)).GetTransposed(); 	// No Origin offset

double const UCelestialMaths::SpeedOfLightMetersPerSeconds = 299792458.0;
double const UCelestialMaths::AstronomicalUnitsMeters = 149597870700.0;
double const UCelestialMaths::ParsecAstronomicalUnits = 206264.80624709636;
double const UCelestialMaths::NewMoonReferenceJulianDate = 2460705.025196759; // Known new Moon was January 29th, 2025, at 12:36:17 UTC
double const UCelestialMaths::SynodicMonthAverage = 29.530588853;


double UCelestialMaths::WGS84::SemiMajorAxis = 6378137.0;
double UCelestialMaths::WGS84::SemiMinorAxis = 6356752.314245;
double UCelestialMaths::WGS84::Flattening = 1.0 / 298.257223563;

#pragma endregion

#pragma region Colors

FLinearColor UCelestialMaths::BVtoLinearColor(float BV)
{
	// From https://en.wikipedia.org/wiki/Color_index

	// Model valid only between [-0.4, 2.0] 
	BV = FMath::Clamp(BV, -0.4f, 2.0f);

	// Compute Effective Temperature
	const float K1 = 0.92f * BV + 1.7f;
	const float K2 = 0.92f * BV + 0.62f;
	float Temperature = 4600.0f * (1.0f / K1 + 1.0f / K2);

	// Convert to Color
	return FLinearColor::MakeFromColorTemperature( Temperature);;
}

double UCelestialMaths::GetIlluminationPercentage(double NormalizedAge)
{
	return 0.5 * (1.0-FMath::Cos(2.0*UE_PI * NormalizedAge));
}

#pragma endregion

#pragma region VSOP87

void UCelestialMaths::GetSolarSystemBodyLocationVelocity_VSOP87_AU(EVSOP87BodyType VSOP87BodyType, double JulianDate, FVector& LocationAU, FVector& Velocity)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(GetSolarSystemBodyLocationVelocity_VSOP87_AU);
	
	// Warning - We need to use VSOP 87 Time for these functions! 
	double VSOPTime = JulianDateToVSOP87Time(JulianDate);

	// Get the body coordinates in the inertial frame defined by the dynamical equinox and ecliptic J2000 (JD2451545.0).
	Velocity = FVector::ZeroVector;
	LocationAU = FVector::ZeroVector;
	
	switch (VSOP87BodyType)
	{
	case EVSOP87BodyType::Mercury:
		LocationAU = UVSOP87::GetMercuryLocation(VSOPTime, EVSOP87Accuracy::Milli);
		break;
	case EVSOP87BodyType::Venus:
		LocationAU = UVSOP87::GetVenusLocation(VSOPTime, EVSOP87Accuracy::Milli);
		break;
	case EVSOP87BodyType::Earth:
		LocationAU = UVSOP87::GetEarthLocation(VSOPTime, EVSOP87Accuracy::Full);
		Velocity = UVSOP87::GetEarthVelocity(VSOPTime, EVSOP87Accuracy::Full);
		break;
	case EVSOP87BodyType::Mars:
		LocationAU = UVSOP87::GetMarsLocation(VSOPTime, EVSOP87Accuracy::Milli);
		break;
	case EVSOP87BodyType::Jupiter:
		LocationAU = UVSOP87::GetJupiterLocation(VSOPTime, EVSOP87Accuracy::Milli);
		break;
	case EVSOP87BodyType::Saturn:
		LocationAU = UVSOP87::GetSaturnLocation(VSOPTime, EVSOP87Accuracy::Milli);
		break;
	case EVSOP87BodyType::Uranus:
		LocationAU = UVSOP87::GetUranusLocation(VSOPTime, EVSOP87Accuracy::Milli);
		break;
	case EVSOP87BodyType::Neptune:
		LocationAU = UVSOP87::GetNeptuneLocation(VSOPTime, EVSOP87Accuracy::Milli);
		break;
	case EVSOP87BodyType::Moon:
		LocationAU = UVSOP87::GetMoonLocation(VSOPTime, EVSOP87Accuracy::Full);
		break;
	case EVSOP87BodyType::Sun:
		// Sun is at Origin, so returning 0 is what we want! 
		break;
	}
}

FVector UCelestialMaths::GetSolarSystemBodyLocation_VSOP87_Relativistic(FVector ReferenceBodyLocation_Heliocentric_AU, EVSOP87BodyType VSOP87BodyType, double JulianDate)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(GetSolarSystemBodyLocation_VSOP87_Relativistic);
	
	double JulianDateLightAdjusted = JulianDate;
	FVector BodyPositionAU = FVector::ZeroVector;

	for (int32 i = 0; i < 3; i++) // 3 iterations are good enough to converge
	{
		FVector Velocities;
		GetSolarSystemBodyLocationVelocity_VSOP87_AU(VSOP87BodyType, JulianDateLightAdjusted, BodyPositionAU, Velocities);
		double PlanetaryBodyDistanceAU = FVector::Distance(ReferenceBodyLocation_Heliocentric_AU, BodyPositionAU);
		double lightPropagationTimeInDays = SecondsToDay( AstronomicalUnitsToMeters(PlanetaryBodyDistanceAU) / SpeedOfLightMetersPerSeconds);
		JulianDateLightAdjusted = JulianDate - lightPropagationTimeInDays;
	}
	return BodyPositionAU;
}

#pragma endregion


#pragma region Celestial Bodies

FPlanetaryBodyKinematicState UCelestialMaths::GetPlanetaryBodyKinematicState_AU(double JulianDate, EVSOP87BodyType VSOP87BodyType)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(GetPlanetaryBodyKinematicState_AU);
	
	FPlanetaryBodyKinematicState BodyKinematicState;

	// Get Body Location in VSOP Rectangular Coordinates (X, Y, Z)
	GetSolarSystemBodyLocationVelocity_VSOP87_AU(VSOP87BodyType, JulianDate, BodyKinematicState.Location_VSOP87, BodyKinematicState.Velocity_VSOP87);

	// Transform to FK5J2000 Frame
	BodyKinematicState.Location_Heliocentric_FK5J2000 = VSOPEclipticToFK5Equatorial.TransformVector(BodyKinematicState.Location_VSOP87);
	BodyKinematicState.Velocity_FK5J2000 = VSOPEclipticToFK5Equatorial.TransformVector(BodyKinematicState.Velocity_VSOP87);

	return BodyKinematicState;
}

void UCelestialMaths::GetBodyCelestialCoordinatesAU(double JulianDate, EVSOP87BodyType VSOP87BodyType, double ObserverLatitude, double ObserverLongitude, bool bGeoCentricObserver,
                                                    bool bIgnoreRelativisticEffect, double& RAJ2000Hours, double& DECJ2000Degrees, double& RAHours, double& DECDegrees, double& RAGeocentricHours, double& DECGeocentricDegrees, double& DistanceBodyToEarthAU, double& DistanceBodyToSunAU, double& DistanceEarthToSunAU)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(GetBodyCelestialCoordinatesAU);

	// Compute Earth Reference location
	FPlanetaryBodyKinematicState EarthKinematicState = GetPlanetaryBodyKinematicState_AU(JulianDate, EVSOP87BodyType::Earth);

	// Call the actual computation method with the Earth as a reference.
	GetBodyCelestialCoordinatesAU_UsingKnownState(JulianDate, VSOP87BodyType, EarthKinematicState, ObserverLatitude, ObserverLongitude, bGeoCentricObserver, bIgnoreRelativisticEffect, RAJ2000Hours, DECJ2000Degrees, RAHours, DECDegrees, RAGeocentricHours, DECGeocentricDegrees, DistanceBodyToEarthAU, DistanceBodyToSunAU, DistanceEarthToSunAU);
}

void UCelestialMaths::GetBodyCelestialCoordinatesAU_UsingKnownState(double JulianDate, EVSOP87BodyType VSOP87BodyType, const FPlanetaryBodyKinematicState& EarthKinematicState, double ObserverLatitude, double ObserverLongitude, bool bGeoCentricObserver, bool bIgnoreRelativisticEffect, double& RAJ2000Hours, double& DECJ2000Degrees, double& RAHours, double& DECDegrees, double& RAGeocentricHours, double& DECGeocentricDegrees, double& DistanceBodyToEarthAU, double& DistanceBodyToSunAU, double& DistanceEarthToSunAU)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(GetBodyCelestialCoordinatesAU_UsingKnownState);
	
	// Compute the Precession and Nutation Matrices - The formulas expect a Date in Terrestrial Time
	double TAISeconds = JulianDateToInternationalAtomicTime(JulianDate);
	double TTSeconds = InternationalAtomicTimeToTerrestrialTime(TAISeconds);
	double JulianDateTT = SecondsToDay(TTSeconds);
	FMatrix PrecessionMatrix = GetPrecessionMatrix(JulianDateTT);
	FMatrix NutationMatrix = GetNutationMatrix(JulianDateTT);

	// Note about Coordinates systems

	// The VSOP frame is a "heliocentric ecliptic frame of the J2000 equinox"
	//    Origin = Sun (Heliocentric)
	//    Coordinates Axes = Mean ecliptic & equinox of J2000.0
	//       XY-plane: the mean ecliptic plane at epoch J2000.0
	//       X-axis: pointing toward mean equinox at J2000.0
	//       Z-axis: perpendicular to the ecliptic

	// The FK5 system is a geocentric celestial reference frame:
	//    Origin = Earth (GeoCentric)
	//    Coordinates Axes = Mean equator and mean equinox of J2000.0.
	//       X-axis Points toward the mean equinox of J2000.0 (intersection of Earth’s equator with the ecliptic at J2000).
	//       Z-axis Points toward the mean north celestial pole at J2000.0 (Earth’s spin axis direction).
	//       Y-axis Completes the right-handed coordinate system (90° east of X along the equator).
	//    FK5 does not rotate with Earth’s daily rotation → the frame is fixed in space

	// When getting the coordinates from any VSOP87 Call, it's important to change the axis frame to FK5J2000 using the VSOPEclipticToFK5Equatorial Matrix
	// All the locations hereby defined will be using the FK5J2000 Frame, with HelioCentric or GeoCentric to precise the offset due to the Earth location

	// Get Body Location in FK5J2000 Rectangular Coordinates (X, Y, Z)
	FVector BodyVelocity_VSOP87 = FVector::ZeroVector;
	FVector BodyLocation_VSOP87 = FVector::ZeroVector;
	if (bIgnoreRelativisticEffect)
	{
		GetSolarSystemBodyLocationVelocity_VSOP87_AU(VSOP87BodyType, JulianDate, BodyLocation_VSOP87, BodyVelocity_VSOP87);
	}
	else
	{
		BodyLocation_VSOP87 = GetSolarSystemBodyLocation_VSOP87_Relativistic(EarthKinematicState.Location_VSOP87, VSOP87BodyType, JulianDate);		
	}
	
	FVector BodyLocation_Heliocentric_FK5J2000_AU = VSOPEclipticToFK5Equatorial.TransformVector(BodyLocation_VSOP87);


	// In the FK5J2000AU Frame (S = Sun, B = Body, E = Earth, O = Observer
	//   B
	//
	//
	//            O
	//  S        E
	//
	// We want to compute the RA, DEC in two flavors: 
	//   - TopoCentric --> This will be OB (Relative to the Observer, for display) 
	//   - GeoCentric  --> This will be EB (Relative to the Earth Center, for objects placement)
	// 
	// With 
	//  * SB = BodyLocation_Heliocentric_FK5J2000_AU
	//  * SE = EarthKinematicState.Location_Heliocentric_FK5J2000
	//  * EB = BodyLocation_GeoCentric_FK5J2000_AU
	//  * EO = ObserverLocationGeoCentric_FK5J2000AU
	//  * OB = BodyLocation_TopoCentric_FK5J2000_AU
	
	// For the GeoCentric RA/DEC, we need to compute the EB vector 
	// EB = ES + SB = SB - SE 
	//   => BodyLocation_GeoCentric_FK5J2000_AU = BodyLocation_Heliocentric_FK5J2000_AU - EarthKinematicState.Location_Heliocentric_FK5J2000;
	FVector BodyLocation_GeoCentric_FK5J2000_AU = BodyLocation_Heliocentric_FK5J2000_AU - EarthKinematicState.Location_Heliocentric_FK5J2000;
	
	// For the TopoCentric RA/DEC, we need to compute the OB vector
	// OB = OE + EB
	// OB = EB - EO
	//   => BodyLocation_TopoCentric_FK5J2000_AU = BodyLocation_GeoCentric_FK5J2000_AU - ObserverLocationGeoCentric_FK5J2000AU

	
	// Let's compute the EO, the Observer location in the GeoCentric Frame
	// If the end user works in a geocentric frame in UE, using Latitude = 0 and Longitude = 0 will not work. It's on the surface of the Atlantic Ocean
	// We need to handle the special case where the user requests for a GeoCentric observer. In this case EO = FVector::ZeroVector
	FVector ObserverLocationGeoCentric_FK5J2000AU = FVector::ZeroVector;
	if (!bGeoCentricObserver)
	{
		ObserverLocationGeoCentric_FK5J2000AU = GetObserverGeocentricLocationAU(ObserverLatitude, ObserverLongitude, 0, JulianDate);
		ObserverLocationGeoCentric_FK5J2000AU = NutationMatrix.TransformPosition(ObserverLocationGeoCentric_FK5J2000AU);
		ObserverLocationGeoCentric_FK5J2000AU = PrecessionMatrix.TransformPosition(ObserverLocationGeoCentric_FK5J2000AU);
	}

	// Topocentric J2000 RA/DEC — matches Horizons "Astrometric RA & Dec" with an observer location.
	FVector BodyLocation_TopoCentric_FK5J2000_AU = BodyLocation_GeoCentric_FK5J2000_AU - ObserverLocationGeoCentric_FK5J2000AU;
	double RAJ2000Degrees;
	XYZToRADEC_RH(BodyLocation_TopoCentric_FK5J2000_AU, RAJ2000Degrees, DECJ2000Degrees, DistanceBodyToEarthAU);
	RAJ2000Hours = RAJ2000Degrees / 15.0;

	
	// Topocentric RA/DEC of date — We need to apply the Precession and Nutation matrices to account for the earth axis oscillations. 
	FVector BodyLocation_TopoCentric_Aberrated_AU = ComputeAberration(BodyLocation_TopoCentric_FK5J2000_AU, EarthKinematicState.Velocity_FK5J2000);
	FVector BodyLocation_TopoCentric_Precessed_AU = PrecessionMatrix.InverseTransformVector(BodyLocation_TopoCentric_Aberrated_AU);
	FVector BodyLocation_TopoCentric_OfDate_AU = NutationMatrix.InverseTransformVector(BodyLocation_TopoCentric_Precessed_AU);

	double RATopocentricDegrees = 0.0;
	double DistanceTopocentric = 0.0;
	XYZToRADEC_RH(BodyLocation_TopoCentric_OfDate_AU, RATopocentricDegrees, DECDegrees, DistanceTopocentric);
	RAHours = RATopocentricDegrees / 15.0;
	
	
	// Geocentric RA/DEC of date — used for body placement in the scene.
	// Geocentric only differs from Topocentric for close bodies (Moon: up to ~57', Sun: ~8.7”).
	FVector BodyLocation_GeoCentric_Aberrated_AU = ComputeAberration(BodyLocation_GeoCentric_FK5J2000_AU, EarthKinematicState.Velocity_FK5J2000);
	FVector BodyLocation_GeoCentric_Precessed_AU = PrecessionMatrix.InverseTransformVector(BodyLocation_GeoCentric_Aberrated_AU);
	FVector BodyLocation_GeoCentric_OfDate_AU = NutationMatrix.InverseTransformVector(BodyLocation_GeoCentric_Precessed_AU);

	double RAGeocentricDegrees = 0.0;
	XYZToRADEC_RH(BodyLocation_GeoCentric_OfDate_AU, RAGeocentricDegrees, DECGeocentricDegrees, DistanceBodyToEarthAU);
	RAGeocentricHours = RAGeocentricDegrees / 15.0;

	DistanceBodyToEarthAU = BodyLocation_GeoCentric_FK5J2000_AU.Length();
	DistanceBodyToSunAU = BodyLocation_Heliocentric_FK5J2000_AU.Length();
	DistanceEarthToSunAU = EarthKinematicState.Location_Heliocentric_FK5J2000.Length();
}



double UCelestialMaths::GetPlanetaryBodyMagnitude(EVSOP87BodyType VSOP87BodyType, double DistanceToSunAU, double DistanceToEarthAU, double DistanceEarthToSunAU, double& PhaseAngle)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(GetPlanetaryBodyMagnitude);
	
	// From Computing Apparent Planetary Magnitudes for The Astronomical Almanac
	// James L. Hilton US Naval Observatory

	if (VSOP87BodyType == EVSOP87BodyType::Sun)
	{
		// Early out for the sun, and also protects again a divide by zero with DistanceToSunAU in the PhaseAngle Computation
		return -26.74;
	}
	
	// V = 5 log10 ( r d ) + V1(0) + C1 α + C2 α2 + ... with
	//   r = planet’s distance from the Sun
	//   d = planet’s distance from the earth
	//	 a = illumination phase angle (in degrees)
	//   V1(0) sometimes referred to as the planet’s absolute magnitude or geometric magnitude is the magnitude when observed at α = 0
	//   ΣnCn αn is called the phase function 
	

	
	
	double DistanceFactor = 5.0 * FMath::LogX(10.0, FMath::Abs(DistanceToEarthAU * DistanceToSunAU));
	if (VSOP87BodyType == EVSOP87BodyType::Moon)
	{
		// For the moon, the distance factor is different. // Schaefer 1998
		// mV = −12.73 + 0.026∣α∣ + 4.0×10−9α4 + 5log10(Δ0/Δ)
		// Δ0 =384400 km (average distance), Δ = Moon-Earth distance
		DistanceFactor = 5.0 * FMath::LogX(10.0, FMath::Abs(DistanceToEarthAU) / MetersToAstronomicalUnits(384400.0 * UE_KM_TO_M));
	}
	
	double ApparentMagnitude = 0.0;
	if (DistanceToSunAU <= 0 || DistanceToEarthAU <= 0)
	{
		PhaseAngle = 0.0;
		return 0.0;
	}
	
	double PhaseInput = (DistanceToSunAU * DistanceToSunAU + DistanceToEarthAU * DistanceToEarthAU - DistanceEarthToSunAU * DistanceEarthToSunAU) / (2.0 * DistanceToSunAU * DistanceToEarthAU); 
	PhaseAngle = FMath::RadiansToDegrees(FMath::Acos( FMath::Clamp(PhaseInput , -1.0, 1.0)));
	double PhaseAngle2 = PhaseAngle * PhaseAngle;
	double PhaseAngle3 = PhaseAngle2 * PhaseAngle;
	double PhaseAngle4 = PhaseAngle3 * PhaseAngle;
	double PhaseAngle5 = PhaseAngle4 * PhaseAngle;
	double PhaseAngle6 = PhaseAngle5 * PhaseAngle;
	double PhaseFunction = 0.0;


	switch (VSOP87BodyType)
	{
	case EVSOP87BodyType::Mercury:
		ApparentMagnitude = -0.613;
		PhaseFunction = 6.3280E-02 * PhaseAngle - 1.6336E-03 * PhaseAngle2 + 3.3644E-05 * PhaseAngle3 - 3.4265E-07 * PhaseAngle4 + 1.6893E-09 *
			PhaseAngle5 - 3.0334E-12 * PhaseAngle6;
		break;
	case EVSOP87BodyType::Venus:
		if (PhaseAngle < 163.7)
		{
			ApparentMagnitude = -4.384;
			PhaseFunction = -1.044E-03 * PhaseAngle + 3.687E-04 * PhaseAngle2 - 2.814E-06 * PhaseAngle3 + 8.938E-09 * PhaseAngle4;
		}
		else // 163.7 <  α < 179 - let's go to 180...
		{
			ApparentMagnitude = 236.05828;
			PhaseFunction = -2.81914 * PhaseAngle + 8.39034E-03 * PhaseAngle2;
		}
		break;
	case EVSOP87BodyType::Earth:
		ApparentMagnitude = -3.99;
		PhaseFunction = -1.060E-3 * PhaseAngle + 2.054E-4 * PhaseAngle2;
		break;
	case EVSOP87BodyType::Mars:
		if (PhaseAngle < 50.0)
		{
			ApparentMagnitude = -1.601;
			PhaseFunction = 0.02267 * PhaseAngle - 0.0001302 * PhaseAngle2;
		}
		else
		{
			ApparentMagnitude = -0.367;
			PhaseFunction = -0.02573 * PhaseAngle + 0.0003445 * PhaseAngle2;
		}
		break;
	case EVSOP87BodyType::Jupiter:
		if (PhaseAngle < 12.0)
		{
			ApparentMagnitude = -9.395;
			PhaseFunction = -3.7E-04 * PhaseAngle - 6.16E-04 * PhaseAngle2;
		}
		else
		// 12 < α < 130 - The phase curve of Jupiter as seen from Earth cannot exceed α = 12 so we should be good. Add this one just in case...
		{
			ApparentMagnitude = -9.428;
			PhaseFunction = -2.5 * FMath::LogX(10.0,
				1.0
				- 1.507 * (PhaseAngle / 180.0)
				- 0.363 * FMath::Pow(PhaseAngle / 180.0, 2.0)
				- 0.062 * FMath::Pow(PhaseAngle / 180.0, 3.0)
				+ 2.809 * FMath::Pow(PhaseAngle / 180.0, 4.0)
				- 1.876 * FMath::Pow(PhaseAngle / 180.0, 5.0));
		}
		break;
	case EVSOP87BodyType::Saturn:
		// Keep it simple and ignore the ring effects
		if (PhaseAngle < 6.0)
		{
			ApparentMagnitude = -8.95;
			PhaseFunction = -3.7E-04 * PhaseAngle + 6.16E-04 * PhaseAngle2;
		}
		else // 6 < α < 150 -
		{
			ApparentMagnitude = -8.94;
			PhaseFunction = 2.446E-4 * PhaseAngle + 2.672E-4 * PhaseAngle2 - 1.505E-6 * PhaseAngle3 + 4.767E-9 * PhaseAngle4;
		}
		break;
	case EVSOP87BodyType::Uranus:
		ApparentMagnitude = -7.19;
		PhaseFunction = -8.4E-04 * 82.0; // PhaseAngle doesn't have any impact, and 82 is the most important planetographic latitude
		break;
	case EVSOP87BodyType::Neptune:
		ApparentMagnitude = -7.00;
		PhaseFunction = 7.944E-3 * PhaseAngle + 9.617E-5 * PhaseAngle2;
		break;
	case EVSOP87BodyType::Moon:
		ApparentMagnitude = -12.73;
		PhaseFunction = 0.026 * PhaseAngle + 4E-9 * PhaseAngle4;
		break;
	case EVSOP87BodyType::Sun:
		return -26.74;
	default: ;
	}

	return DistanceFactor + ApparentMagnitude + PhaseFunction;
}

#pragma endregion

#pragma region Time

FDateTime UCelestialMaths::LocalTimeToUTCTime(FDateTime LocalTime, double TimeZoneOffset, bool IsDst)
{
	if (IsDst)
	{
		TimeZoneOffset += 1.0;
	}

	return LocalTime - FTimespan::FromHours(TimeZoneOffset);
}

FDateTime UCelestialMaths::UTCTimeToLocalTime(FDateTime UTCTime, double TimeZoneOffset, bool IsDst)
{
	if (IsDst)
	{
		TimeZoneOffset += 1.0;
	}

	return UTCTime + FTimespan::FromHours(TimeZoneOffset);
}

double UCelestialMaths::UTCDateTimeToJulianDate(FDateTime UTCDateTime)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UTCDateTimeToJulianDate);
	
	// From https://www.celestialprogramming.com/julian.html

	// Get Individual values for YMD and HMS
	int32 Year;
	int32 Month;
	int32 Day;
	UTCDateTime.GetDate(Year, Month, Day);
	const FTimespan Time = UTCDateTime.GetTimeOfDay();
	const int32 Hours = Time.GetHours();
	const int32 Minutes = Time.GetMinutes();
	const double Seconds = Time.GetTotalSeconds() - Minutes * 60.0 - Hours * 3600.0;

	// Prepare the Input DateTime for JulianDate computations
	bool bIsGregorian = true;
	if (Year < 1582 || (Year == 1582 && (Month < 10 || (Month == 10 && Day < 5))))
	{
		bIsGregorian = false;
	}

	if (Month < 3)
	{
		Year = Year - 1;
		Month = Month + 12;
	}

	int32 b = 0;
	if (bIsGregorian)
	{
		int32 a = FloorForJulianDate(Year / 100.0);
		b = 2 - a + FloorForJulianDate(a / 4.0);
	}

	// Compute the Julian Date
	double JulianDate = FloorForJulianDate(365.25 * (Year + 4716.0)) + FloorForJulianDate(30.6001 * (Month + 1)) + Day + b - 1524.5;
	JulianDate += Hours / 24.0;
	JulianDate += Minutes / 24.0 / 60.0;
	JulianDate += Seconds / 24.0 / 60.0 / 60.0;
	return JulianDate;
}	

FDateTime UCelestialMaths::JulianDateToUTCDateTime(double JulianDate)
{
	// From https://www.celestialprogramming.com/julian.html
	// From Meeus, CH7, p63

	double temp = JulianDate + 0.5;
	int32 IntegralPart = StaticCast<int32>(FMath::TruncToDouble(temp));
	double FractionalPart = temp - IntegralPart;

	// If Integral Part < 2299161, take A = IntegralPart
	int32 A = IntegralPart;
	if (IntegralPart >= 2299161)
	{
		int32 Alpha = FloorForJulianDate((IntegralPart - 1867216.25) / 36524.25);
		A = IntegralPart + 1 + Alpha - FloorForJulianDate(Alpha / 4.0);
	}

	// Compute ABCDE values
	int32 B = A + 1524;
	int32 C = FloorForJulianDate((B - 122.1) / 365.25);
	int32 D = FloorForJulianDate(365.25 * C);
	int32 E = FloorForJulianDate((B - D) / 30.6001);

	int32 day = B - D - static_cast<int32>(30.6001 * E);
	int32 month = E - 1;
	if (E > 13)
	{
		month = E - 13;
	}
	int32 year = C - 4716;
	if (month < 3)
	{
		year = C - 4715;
	}

	// Split in H,M,S,MS
	int32 hour = StaticCast<int32>(FMath::TruncToDouble(FractionalPart * 24.0));
	FractionalPart -= hour / 24.0;
	int32 minute = StaticCast<int32>(FMath::TruncToDouble(FractionalPart * 24.0 * 60.0));
	FractionalPart -= minute / (24.0 * 60.0);
	int32 seconds = StaticCast<int32>(FMath::TruncToDouble(FractionalPart * 24.0 * 60.0 * 60.0));
	FractionalPart -= seconds / (24.0 * 60.0 * 60.0);
	int32 milliseconds = StaticCast<int32>(FMath::RoundToDouble(FractionalPart * 24.0 * 60.0 * 60.0 * 1000.0));

	return FDateTime(year, month, day, hour, minute, seconds, milliseconds);
}

double UCelestialMaths::DateTimeToGreenwichMeanSiderealTime(FDateTime UTCDateTime)
{
    return ModPositive(DateTimeToGreenwichMeanSiderealTimeUnwrapped(UTCDateTime), 360.0);
}

double UCelestialMaths::DateTimeToGreenwichMeanSiderealTimeUnwrapped(FDateTime UTCDateTime)
{
    double JulianDate = UTCDateTimeToJulianDate(UTCDateTime);
	double JulianCentury = JulianDateToJulianCenturies(JulianDate);
	return 280.46061837 + 360.98564736629*(JulianDate-2451545.0) + 0.000387933 * JulianCentury * JulianCentury - 1.0/38710000.0 * JulianCentury * JulianCentury * JulianCentury;
}

double UCelestialMaths::JulianDateToGreenwichMeanSiderealTime(double JulianDate) {
	//The IAU Resolutions on Astronomical Reference Systems, Time Scales, and Earth Rotation Models Explanation and Implementation (George H. Kaplan)
	// https://arxiv.org/pdf/astro-ph/0602086.pdf - page 30
	// T is the number of centuries of TDB (or TT) from J2000.0
	// Formula for arcseconds: EarthRotationAngle + 0.014506 + 4612.15739966*T + 1.39667721*T^2 − 0.00009344*T^3 + 0.00001882*T^4
	// Here our Earth angle is already in degrees
	const double JulianCentury = JulianDateToJulianCenturies(JulianDate);

	const double EarthRotationAngle = GetEarthRotationAngle(JulianDate);
	
	double GMST = EarthRotationAngle + ArcsecondsToDegrees(
				0.014506 +
				4612.15739966 * JulianCentury +
				1.39667721 * JulianCentury * JulianCentury +
				-0.00009344 * JulianCentury * JulianCentury * JulianCentury +
				0.00001882 * JulianCentury * JulianCentury * JulianCentury * JulianCentury) ;

	return ModPositive(GMST, 360.0);
}

double UCelestialMaths::LocalSideralTime(double LongitudeDegrees, double GreenwichMeanSideralTime)
{
	return ModPositive(GreenwichMeanSideralTime + LongitudeDegrees, 360.0);
}

double UCelestialMaths::JulianDateToGreenwichApparentSiderealTime(double JulianDate)
{
	// From https://aa.usno.navy.mil/faq/GAST
	// The Greenwich apparent sidereal time is obtained by adding a correction to the Greenwich mean sidereal time computed above.
	// The correction term is called the nutation in right ascension or the equation of the equinoxes. Thus,
	// GAST = GMST + eqeq.
	
	const double GMST = JulianDateToGreenwichMeanSiderealTime(JulianDate);
	const double EE = EquationOfTheEquinoxes(JulianDate);
	double gast = GMST + EE;

	return ModPositive(gast,360.0);
}

double UCelestialMaths::JulianDateToJulianCenturies(double JulianDate)
{
	return (JulianDate - 2451545.0) / 36525.0;
}

double UCelestialMaths::GetLeapSeconds(double JulianDate)
	{
		//Source IERS Resolution B1 and http://maia.usno.navy.mil/ser7/tai-utc.dat
		//This function must be updated any time a new leap second is introduced

		if (JulianDate > 2457754.5) return 37.0;
		if (JulianDate > 2457204.5) return 36.0;
		if (JulianDate > 2456109.5) return 35.0;
		if (JulianDate > 2454832.5) return 34.0;
		if (JulianDate > 2453736.5) return 33.0;
		if (JulianDate > 2451179.5) return 32.0;
		if (JulianDate > 2450630.5) return 31.0;
		if (JulianDate > 2450083.5) return 30.0;
		if (JulianDate > 2449534.5) return 29.0;
		if (JulianDate > 2449169.5) return 28.0;
		if (JulianDate > 2448804.5) return 27.0;
		if (JulianDate > 2448257.5) return 26.0;
		if (JulianDate > 2447892.5) return 25.0;
		if (JulianDate > 2447161.5) return 24.0;
		if (JulianDate > 2446247.5) return 23.0;
		if (JulianDate > 2445516.5) return 22.0;
		if (JulianDate > 2445151.5) return 21.0;
		if (JulianDate > 2444786.5) return 20.0;
		if (JulianDate > 2444239.5) return 19.0;
		if (JulianDate > 2443874.5) return 18.0;
		if (JulianDate > 2443509.5) return 17.0;
		if (JulianDate > 2443144.5) return 16.0;
		if (JulianDate > 2442778.5) return 15.0;
		if (JulianDate > 2442413.5) return 14.0;
		if (JulianDate > 2442048.5) return 13.0;
		if (JulianDate > 2441683.5) return 12.0;
		if (JulianDate > 2441499.5) return 11.0;
		if (JulianDate > 2441317.5) return 10.0;
		if (JulianDate > 2439887.5) return 4.21317 + (JulianDate - 2439126.5) * 0.002592;
		if (JulianDate > 2439126.5) return 4.31317 + (JulianDate - 2439126.5) * 0.002592;
		if (JulianDate > 2439004.5) return 3.84013 + (JulianDate - 2438761.5) * 0.001296;
		if (JulianDate > 2438942.5) return 3.74013 + (JulianDate - 2438761.5) * 0.001296;
		if (JulianDate > 2438820.5) return 3.64013 + (JulianDate - 2438761.5) * 0.001296;
		if (JulianDate > 2438761.5) return 3.54013 + (JulianDate - 2438761.5) * 0.001296;
		if (JulianDate > 2438639.5) return 3.44013 + (JulianDate - 2438761.5) * 0.001296;
		if (JulianDate > 2438486.5) return 3.34013 + (JulianDate - 2438761.5) * 0.001296;
		if (JulianDate > 2438395.5) return 3.24013 + (JulianDate - 2438761.5) * 0.001296;
		if (JulianDate > 2438334.5) return 1.945858 + (JulianDate - 2437665.5) * 0.0011232;
		if (JulianDate > 2437665.5) return 1.845858 + (JulianDate - 2437665.5) * 0.0011232;
		if (JulianDate > 2437512.5) return 1.372818 + (JulianDate - 2437300.5) * 0.001296;
		if (JulianDate > 2437300.5) return 1.422818 + (JulianDate - 2437300.5) * 0.001296;
		return 0.0;
	}

double UCelestialMaths::InternationalAtomicTimeToTerrestrialTime(double TAI)
{
	// From https://www2.mps.mpg.de/homes/fraenz/systems/systems2art/node2.html
	//  TT = terrestrial time in SI seconds
	//  TT= TAI + 32.184 seconds;
	return TAI + 32.184;
}

double UCelestialMaths::JulianDateToInternationalAtomicTime(double JulianDate)
{
	return GetLeapSeconds(JulianDate) + DaysToSeconds(JulianDate);
}

double UCelestialMaths::JulianDateToVSOP87Time(double JulianDate)
{
	// Convert time, because, the VSOP87 coordinates expect the time in TerrestrialTime
	double TAI = JulianDateToInternationalAtomicTime(JulianDate); 
	double TT = InternationalAtomicTimeToTerrestrialTime(TAI); // adds the 32.184s

	// Caution here. In the VSOP87 equations, Time is not Julian centuries! 
	// The denominator is 365250, not 36525.
	// That’s because VSOP87 expresses time in units of 10,000 Julian years (1 Julian millennium = 100 centuries = 36,525 × 100 days).
	return (SecondsToDay(TT)-2451545.0) / 365250.0; // We divide by 10 because VSOP expects T : time expressed in Thousands of Julian Years (tjy) elapsed from J2000 (JD2451545.0).
}

#pragma endregion

#pragma region Angles

double UCelestialMaths::ModPositive(double Value, double Modulo)
{
	double Result = FMath::Fmod(Value, Modulo);
	if (Result < 0.0)
	{
		Result += Modulo;
	}
	return Result; 
}

void UCelestialMaths::DegreesToHMS(double DecimalDegrees, int32& Hours, int32& Minutes, double& Seconds)
{
	DecimalDegrees = FMath::Fmod(DecimalDegrees, 360.0);
	if (DecimalDegrees < 0.0)
	{
		DecimalDegrees += 360.0;
	}

	double AngleHours = DecimalDegrees / 15.0;

	Hours = StaticCast<uint32>(AngleHours);
	Minutes = StaticCast<uint32>((AngleHours - Hours) * 60.0);
	Seconds = (AngleHours - Hours) * 3600.0 - Minutes * 60.0;
}

void UCelestialMaths::DegreesToDMS(double DecimalDegrees, bool& Sign, int32& Degrees, int32& Minutes, double& Seconds)
{
	Sign = true;
	DecimalDegrees = FMath::Fmod(DecimalDegrees, 360.0);

	if (DecimalDegrees < 0.0)
	{
		Sign = false;
		DecimalDegrees *= -1.0;
	}

	Degrees = StaticCast<uint32>(DecimalDegrees);
	Minutes = StaticCast<uint32>((DecimalDegrees - Degrees) * 60.0);
	Seconds = (DecimalDegrees - Degrees) * 3600.0 - Minutes * 60.0;
}

#pragma endregion

#pragma region Earth

double UCelestialMaths::GetEarthRotationAngle(double JulianDate)
{
	//The IAU Resolutions on Astronomical Reference Systems, Time Scales, and Earth Rotation Models Explanation and Implementation (George H. Kaplan)
	// https://arxiv.org/pdf/astro-ph/0602086.pdf - page 30, Eq 2.11
	//
	// DU is the number of UT1 days from 2000 January 1, 12h UT1: DU = JD(UT1)– 2451545.0.
	// The angle θ is given in terms of rotations (units of 2π radians or 360d)
	// 
	// θ =0.7790572732640 +0.00273781191135448 DU + frac(JD(UT1)) --> We need to multiply it by 2xPI

	const double DU = JulianDate - 2451545.0;
	const double JulianDateFraction = FMath::Fractional(JulianDate);
	const double Rotations = (0.779057273264 + 0.00273781191135448 * DU + JulianDateFraction);

	return ModPositive(Rotations * 360.0, 360.0);
}

FVector UCelestialMaths::GeodeticLatLonToECEFXYZAU(double Latitude, double Longitude, double Altitude)
{
	// Algorithm from Explanatory Supplement to the Astronomical Almanac 3rd ed. P294
	double LatitudeRadians = FMath::DegreesToRadians(Latitude);
	double LongitudeRadians = FMath::DegreesToRadians(Longitude);
	
	const double a = MetersToAstronomicalUnits(WGS84::SemiMajorAxis);
	const double C = 1.0 / FMath::Sqrt(FMath::Cos(LatitudeRadians) * FMath::Cos(LatitudeRadians) + (1.0 - WGS84::Flattening) * (1.0 - WGS84::Flattening) * (FMath::Sin(LatitudeRadians) * FMath::Sin(LatitudeRadians)));

	const double S = (1.0 - WGS84::Flattening) * (1.0 - WGS84::Flattening) * C;
	const double h = MetersToAstronomicalUnits(Altitude);

	return FVector(
		(a * C + h) * FMath::Cos(LatitudeRadians) * FMath::Cos(LongitudeRadians),
		(a * C + h) * FMath::Cos(LatitudeRadians) * FMath::Sin(LongitudeRadians),
		(a * S + h) * FMath::Sin(LatitudeRadians)); 
}

void UCelestialMaths::ECEFXYZAUToGeodeticLatLon(FVector ECEFLocationAU, double &Latitude, double &Longitude, double &AltitudeMeters)
{
	// Convert coordinates from AU to meters
	const double Xm = AstronomicalUnitsToMeters(ECEFLocationAU.X);
	const double Ym = AstronomicalUnitsToMeters(ECEFLocationAU.Y);
	const double Zm = AstronomicalUnitsToMeters(ECEFLocationAU.Z);
	const double ProjectedRadius = FMath::Sqrt(Xm * Xm + Ym * Ym);
	const double e2 = 1.0 - (WGS84::SemiMinorAxis * WGS84::SemiMinorAxis) / (WGS84::SemiMajorAxis * WGS84::SemiMajorAxis);  // first eccentricity squared
	const double ep2 = (WGS84::SemiMajorAxis * WGS84::SemiMajorAxis) / (WGS84::SemiMinorAxis * WGS84::SemiMinorAxis) - 1.0; // second eccentricity squared

	// Longitude
	const double LongitudeRadians = FMath::Atan2(Ym, Xm);
	Longitude = FMath::RadiansToDegrees(LongitudeRadians);
	
	// Bowring formula
	const double Theta = FMath::Atan2(Zm * WGS84::SemiMajorAxis, ProjectedRadius * WGS84::SemiMinorAxis);
	const double SinTheta = FMath::Sin(Theta);
	const double CosTheta = FMath::Cos(Theta);

	// Latitude 
	const double LatitudeRadians = FMath::Atan2(Zm + ep2 * WGS84::SemiMinorAxis * SinTheta * SinTheta * SinTheta,  ProjectedRadius - e2 * WGS84::SemiMajorAxis * CosTheta * CosTheta * CosTheta);
	Latitude = FMath::RadiansToDegrees(LatitudeRadians);

	// Altitude
	if (ProjectedRadius > UE_SMALL_NUMBER)
	{
		const double SinLatitude = FMath::Sin(LatitudeRadians);
		const double GeocentricNormal = WGS84::SemiMajorAxis / FMath::Sqrt(1.0 - e2 * SinLatitude * SinLatitude);
		AltitudeMeters = ProjectedRadius / FMath::Cos(LatitudeRadians) - GeocentricNormal;	
	}
	else
	{
		AltitudeMeters = FMath::Abs(Zm) - WGS84::SemiMinorAxis; 
	}
	
}

double UCelestialMaths::WGS84GeocentricRadius(double Latitude)
{
	const double Phi = FMath::DegreesToRadians(Latitude);
	const double CosPhi = FMath::Cos(Phi);
	const double SinPhi = FMath::Sin(Phi);

	const double Numerator = FMath::Square(WGS84::SemiMajorAxis * WGS84::SemiMajorAxis * CosPhi) + FMath::Square(WGS84::SemiMinorAxis * WGS84::SemiMinorAxis * SinPhi);
	const double Denominator = FMath::Square(WGS84::SemiMajorAxis * CosPhi) + FMath::Square(WGS84::SemiMinorAxis * SinPhi);

	return FMath::Sqrt(Numerator / Denominator);
}

FVector UCelestialMaths::GetObserverGeocentricLocationAU(double Latitude,double Longitude, double Altitude, double JulianDate )
{
	const FVector ObserverECEF = GeodeticLatLonToECEFXYZAU(Latitude, Longitude, Altitude);
	const double GreenwichApparentSiderealTime = JulianDateToGreenwichApparentSiderealTime(JulianDate);

	// Construct the 4x4 rotation matrix to rotate the ECEF position around the Earth axis, depending on the GAST
	FMatrix RotationMatrix = GetRotationZMatrix(-FMath::DegreesToRadians(GreenwichApparentSiderealTime));
	return RotationMatrix.InverseTransformPosition(ObserverECEF);
}

double UCelestialMaths::EquationOfTheEquinoxes(double JulianDate) {
	// The IAU Resolutions on Astronomical Reference Systems, Time Scales, and Earth Rotation Models Explanation and Implementation (George H. Kaplan)
	// https://arxiv.org/pdf/astro-ph/0602086.pdf
	// eq 5.12 p58
	// The expression for the mean obliquity of the ecliptic (the angle between the mean equator and ecliptic, or, equivalently, between the ecliptic pole and mean celestial pole of date) is:
	// E =E0 −46.836769T −0.0001831T2 +0.00200340T3 −0.000000576T4 −0.0000000434T5
	// with E0 = 84381.406 arcseconds
	// With T = the number of Julian centuries of TDB since 2000 Jan 1, 12h TDB. If the dates and times are expressed as Julian dates, then T = (t − 2451545.0)/36525.
	const double JulianCenturies = JulianDateToJulianCenturies(JulianDate);

	const double MeanObliquity = ArcsecondsToDegrees(
			84381.406 +
			-46.836769 * JulianCenturies +
			-0.0001831 * JulianCenturies * JulianCenturies +
			0.0020034 * JulianCenturies * JulianCenturies * JulianCenturies +
			-0.000000576 * JulianCenturies * JulianCenturies * JulianCenturies * JulianCenturies +
			-0.0000000434 * JulianCenturies * JulianCenturies * JulianCenturies * JulianCenturies * JulianCenturies);

	double DeltaPsiDegrees;
	double DeltaEpsilonDegrees; 
	Nutation2000BTruncated(JulianCenturies, DeltaPsiDegrees, DeltaEpsilonDegrees);
	return DeltaPsiDegrees * FMath::Cos(FMath::DegreesToRadians(MeanObliquity + DeltaEpsilonDegrees));
}

void UCelestialMaths::Nutation2000BTruncated(double JulianCenturies, double& DeltaPsiDegrees, double& DeltaEpsilonDegrees)
{
	//The IAU Resolutions on Astronomical Reference Systems, Time Scales, and Earth Rotation Models Explanation and Implementation (George H. Kaplan)
	//https://arxiv.org/pdf/astro-ph/0602086.pdf
	//IAU 2000B Nutation truncated to 6 terms

	const double JC = JulianCenturies;
	const double JC2 = JC * JC;
	const double JC3 = JC * JC2;
	const double JC4 = JC * JC3;

	//Fundamental Arguments p46 eq 5.17, 5.18, 5.19
	// The last five arguments are the same fundamental luni-solar arguments used in previous nutation theories, but with updated expresssions. They are, respectively,
	// l the mean anomaly of the Moon; 
	// l′the mean anomaly of the Sun; --> Lp
	// F the mean argument of latitude of the Moon; 
	// D the mean elongation of the Moon from the Sun
	// Ω the mean longitude of the Moon’s mean ascending node:

	const double Lp = ArcsecondsToRadians(1287104.79305 + 129596581.0481 * JC - 0.5532 * JC2 + 0.000136 * JC3 - 0.00001149 * JC4);
	const double F = ArcsecondsToRadians(335779.526232 + 1739527262.8478 * JC - 12.7512 * JC2 - 0.001037 * JC3 + 0.00000417 * JC4);
	const double D = ArcsecondsToRadians(1072260.70369 + 1602961601.209 * JC - 6.3706 * JC2 + 0.006593 * JC3 - 0.00003169 * JC4);
	const double Omega = ArcsecondsToRadians(450160.398036 - 6962890.5431 * JC + 7.4722 * JC2 + 0.007702 * JC3 - 0.00005939 * JC4);

	//Terms summed from lowest to highest to reduce floating point rounding errors.  See coefficients Page 88.
	// Constants are first multiplied by 10000000 to reduce errors
	double DeltaPsiArcSeconds = 0.0;
	double DeltaEpsilonArcSeconds = 0.0;

	// FundamentalArgument #6
	double FundamentalArgument = Lp + 2.0 * (F - D + Omega);
	DeltaPsiArcSeconds += (-516821.0 + 1226.0 * JC) * FMath::Sin(FundamentalArgument) + -524.0 * FMath::Cos(FundamentalArgument);
	DeltaEpsilonArcSeconds += (224386.0 + -677.0 * JC) * FMath::Cos(FundamentalArgument) + -174.0 * FMath::Sin(FundamentalArgument);

	// FundamentalArgument #5 - Just Lp
	DeltaPsiArcSeconds += (1475877.0 + -3633.0 * JC) * FMath::Sin(Lp) + 11817.0 * FMath::Cos(Lp);
	DeltaEpsilonArcSeconds += (73871.0 + -184.0 * JC) * FMath::Cos(Lp) + -1924.0 * FMath::Sin(Lp);

	// FundamentalArgument #4
	FundamentalArgument = 2.0 * Omega;
	DeltaPsiArcSeconds += (2074554.0 + 207.0 * JC) * FMath::Sin(FundamentalArgument) + -698.0 * FMath::Cos(FundamentalArgument);
	DeltaEpsilonArcSeconds += (-897492.0 + 470.0 * JC) * FMath::Cos(FundamentalArgument) + -291.0 * FMath::Sin(FundamentalArgument);

	// FundamentalArgument #3
	FundamentalArgument = 2.0 * (F + Omega);
	DeltaPsiArcSeconds += (-2276413.0 + -234.0 * JC) * FMath::Sin(FundamentalArgument) + 2796.0 * FMath::Cos(FundamentalArgument);
	DeltaEpsilonArcSeconds += (978459.0 + -485.0 * JC) * FMath::Cos(FundamentalArgument) + 1374.0 * FMath::Sin(FundamentalArgument);

	// FundamentalArgument #2
	FundamentalArgument = 2.0 * (F - D + Omega);
	DeltaPsiArcSeconds += (-13170906.0 + -1675.0 * JC) * FMath::Sin(FundamentalArgument) + -13696.0 * FMath::Cos(FundamentalArgument);
	DeltaEpsilonArcSeconds += (5730336.0 + -3015.0 * JC) * FMath::Cos(FundamentalArgument) + -4587.0 * FMath::Sin(FundamentalArgument);

	// FundamentalArgument #1
	DeltaPsiArcSeconds += (-172064161.0 + -174666.0 * JC) * FMath::Sin(Omega) + 33386.0 * FMath::Cos(Omega);
	DeltaEpsilonArcSeconds += (92052331.0 + 9086.0 * JC) * FMath::Cos(Omega) + 15377.0 * FMath::Sin(Omega);

	DeltaPsiDegrees = ArcsecondsToDegrees(DeltaPsiArcSeconds / 10000000.0);
	DeltaEpsilonDegrees = ArcsecondsToDegrees(DeltaEpsilonArcSeconds / 10000000.0);
}

FTransform UCelestialMaths::GetEarthCenterTransformECEF(double Latitude, double Longitude, double Altitude)
{
	// Compute the Location part 
	FVector ECEFLocation = GeodeticLatLonToECEFXYZAU(Latitude, Longitude, Altitude) * AstronomicalUnitsMeters;

	// Compute the 3 Axis vectors
	FMatrix AxisMatrix; 
	// See ECEF standard : https://commons.wikimedia.org/wiki/File:ECEF_ENU_Longitude_Latitude_right-hand-rule.svg
	if (FMathd::Abs(ECEFLocation.X) < FMathd::Epsilon &&
		FMathd::Abs(ECEFLocation.Y) < FMathd::Epsilon)
	{
		// Special Case - On earth axis... 
		double Sign = 1.0;
		if (FMathd::Abs(ECEFLocation.Z) < FMathd::Epsilon)
		{
			// At origin - Should not happen, but consider it's the same as north pole
			// Leave Sign = 1
		}
		else
		{
			// At South or North pole - Axis are set to be continuous with other points
			Sign = FMathd::SignNonZero(ECEFLocation.Z);
		}

		AxisMatrix = FMatrix(
			FVector::YAxisVector, 			// East = Y
			-FVector::XAxisVector * Sign,	// North = Sign * X
			FVector::ZAxisVector * Sign,	// Up = Sign*Z
			ECEFLocation);
	}
	else
	{
		double Tolerance = 1.E-50; // Normalize with a very low threshold, because default is 10-8, too high for double computations

		// Compute the ellipsoid normal (Earth...)
		FVector OneOverRadiiSquared = FVector(1.0 / (WGS84::SemiMajorAxis * WGS84::SemiMajorAxis), 1.0 / (WGS84::SemiMajorAxis * WGS84::SemiMajorAxis), 1.0 / (WGS84::SemiMinorAxis * WGS84::SemiMinorAxis)); 
		FVector GeodeticSurfaceNormal( ECEFLocation.X * OneOverRadiiSquared.X, ECEFLocation.Y * OneOverRadiiSquared.Y, ECEFLocation.Z * OneOverRadiiSquared.Z);
		GeodeticSurfaceNormal.Normalize(Tolerance);

		// Get other axes
		FVector Up = GeodeticSurfaceNormal;
		FVector East(-ECEFLocation.Y, ECEFLocation.X, 0.0); 
		East.Normalize(Tolerance); 
		FVector North = Up.Cross(East);

		// Set Matrix
		AxisMatrix = FMatrix(	East, North, Up, ECEFLocation);
	}
	
	return FTransform(AxisMatrix.Inverse());
}

FTransform UCelestialMaths::GetEarthCenterTransformUEFrame(double Latitude, double Longitude, double Altitude, bool Geocentric)
{
	// Get the transformation in proper Celestial World units (meters, right handed)
	FTransform ECEFFrameToWorldFrame = FTransform(); // By default, in geocentric mode, Center is a Zero Transform
	if (!Geocentric)
	{
		// Consider tangent frame at lat/long
		ECEFFrameToWorldFrame = GetEarthCenterTransformECEF(Latitude, Longitude, 0);
	}
	

	// UE Frame are expressed in Left-handed coordinates, and units are in meters - Convert to UE Transform 
	FMatrix WorldFrameToUEFrame = FMatrix( 
		FVector(1.0, 0.0, 0.0),			// Easting (X) is UE World X
		FVector(0.0, -1.0, 0.0),			// Northing (Y) is UE World -Y because of left-handed convention
		FVector(0.0, 0.0, 1.0),			// Up (Z) is UE World Z 
		FVector(0.0, 0.0, 0.0));	// No Origin offset
	FMatrix UEFrameToWorldFrame = WorldFrameToUEFrame.Inverse();

	// Update the rotation part
	FMatrix TransformMatrix = UEFrameToWorldFrame * ECEFFrameToWorldFrame.ToMatrixNoScale() * WorldFrameToUEFrame;
	// Get Origin, and convert UE units to meters
	FVector UEOrigin = TransformMatrix.GetOrigin() * FVector(100.0, 100.0, 100.0);
	TransformMatrix.SetOrigin(UEOrigin);

	// Apply the transform
	return FTransform(TransformMatrix);
}

FMatrix UCelestialMaths::GetPrecessionMatrix(double JulianDate)
{
	// Fukushima-Williams IAU 2006
	double t =  (JulianDate - 2451545.5) / 36525.0; // We don't use the Julian Centuries because precession expects JD TT

	double gamma = ArcsecondsToRadians(
		-0.052928 +
		10.556378 * t +
		0.4932044 * t * t -
		0.00031238 * t * t * t -
		0.000002788 * t * t * t * t +
		0.000000026 * t * t * t * t * t);
	double phi = ArcsecondsToRadians(
		84381.412819 -
		46.811016 * t +
		0.0511268 * t * t +
		0.00053289 * t * t * t -
		0.00000044 * t * t * t * t -
		0.0000000176 * t * t * t * t * t);
	double psi = ArcsecondsToRadians(
		-0.041775 +
		5038.481484 * t +
		1.5584175 * t * t -
		0.00018522 * t * t * t -
		0.000026452 * t * t * t * t -
		0.0000000148 * t * t * t * t * t);
	double eps = ArcsecondsToRadians(
		84381.406 -
		46.836769 * t -
		0.0001831 * t * t +
		0.0020034 * t * t * t -
		0.000000576 * t * t * t * t -
		0.0000000434 * t * t * t * t * t);

	FMatrix a = GetRotationZMatrix(gamma);
	FMatrix a2 = FRotationMatrix::Make(FRotator(0, FMath::RadiansToDegrees(gamma), 0));
	
	FMatrix Mphi = GetRotationXMatrix(phi);
	FMatrix Mphi2 = FRotationMatrix::Make(FRotator(FMath::RadiansToDegrees(phi), 0, 0));
	
	FMatrix b = Mphi * a;
	FMatrix b2 = Mphi2 * a2;
	FMatrix Mpsi = GetRotationZMatrix(-psi);
	FMatrix c = Mpsi * b;
	FMatrix Meps = GetRotationXMatrix(-eps);
	FMatrix d = Meps * c;
	
	return d;
}

FMatrix UCelestialMaths::GetNutationMatrix(double JulianDate)
{
	double t =  (JulianDate - 2451545.5) / 36525.0; // We don't use the Julian Centuries because precession expects JD TT

	double DeltaPsiDegrees = 0.0;
	double DeltaEpsilonDegrees = 0.0;
	Nutation2000BTruncated(t, DeltaPsiDegrees, DeltaEpsilonDegrees);
	double DeltaPsiRadians = FMath::DegreesToRadians(DeltaPsiDegrees);
	double DeltaEpsilonRadians = FMath::DegreesToRadians(DeltaEpsilonDegrees);
	
	double eps = ArcsecondsToRadians(
		84381.406 -
		46.836769 * t -
		0.0001831 * t * t +
		0.0020034 * t * t * t -
		0.000000576 * t * t * t * t -
		0.0000000434 * t * t * t * t * t);

	FMatrix a = GetRotationXMatrix(eps);
	FMatrix b = GetRotationZMatrix(-DeltaPsiRadians) * a;
	FMatrix c = GetRotationXMatrix(-(eps + DeltaEpsilonRadians)) * b;

	return c;
}

FVector UCelestialMaths::ComputeAberration(const FVector& TopocentricTargetLocationAU, const FVector& EarthVelocity)
{
	//"MEAN AND APPARENT PLACE COMPUTATIONS IN THE NEW IAU SYSTEM. III. APPARENT, TOPOCENTRIC, AND ASTROMETRIC PLACES OF PLANETS AND STARS"
	//G. H. Kaplan, J. A. Hughes, P. K. Seidelmann, and C. A. Smith
	//U. S. Naval Observatory, Washington, DC 20392

	//http://articles.adsabs.harvard.edu/pdf/1989AJ.....97.1197K

	double C = 173.1446326846693; //Speed of light in AU per Day

	//Eq 16
	double t = TopocentricTargetLocationAU.Length() / C;
	double B = EarthVelocity.Length() / C;
	double cosD = TopocentricTargetLocationAU.Dot(EarthVelocity) / (TopocentricTargetLocationAU.Length() * EarthVelocity.Length());
	double y = FMath::Sqrt(1 - B * B);
	double f1 = B * cosD;
	double f2 = (1 + f1 / (1 + y)) * t;

	//Eq 17
	FVector AberrationLocation = ( y * TopocentricTargetLocationAU + f2 * EarthVelocity) / (1+f1);
	return AberrationLocation;
}

#pragma endregion

#pragma region Sun & Moon

FStellarBody UCelestialMaths::GetSunInformation(double JulianDate, double ObserverLatitude, double ObserverLongitude, bool bGeoCentric)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(GetSunInformation);
	
	// Compute Earth Reference location
	FPlanetaryBodyKinematicState EarthKinematicState = GetPlanetaryBodyKinematicState_AU(JulianDate, EVSOP87BodyType::Earth);

	return GetSunInformation_UsingKnownState(JulianDate, EarthKinematicState, ObserverLatitude, ObserverLongitude, bGeoCentric);
}

FStellarBody UCelestialMaths::GetSunInformation_UsingKnownState(double JulianDate, const FPlanetaryBodyKinematicState& EarthKinematicState, double ObserverLatitude, double ObserverLongitude, bool bGeoCentric)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(GetSunInformation_UsingKnownState);
	// Get the Celestial Body Coordinates
	double RAJ2000Hours;
	double DECJ2000Degrees;
	double RAGeocentricHours;
	double DECGeocentricDegrees;
	double RAHours;
	double DECDegrees;
	double DistanceBodyToEarthAU;
	double DistanceBodyToSunAU;
	double DistanceEarthToSunAU;
	GetBodyCelestialCoordinatesAU_UsingKnownState(JulianDate, EVSOP87BodyType::Sun, EarthKinematicState, ObserverLatitude, ObserverLongitude, bGeoCentric, false, RAJ2000Hours, DECJ2000Degrees, RAHours, DECDegrees, RAGeocentricHours, DECGeocentricDegrees, DistanceBodyToEarthAU, DistanceBodyToSunAU, DistanceEarthToSunAU);

	// Set the Sun properties. Some properties can be hardcoded for the Sun (CI, Magnitude, Radius)
	FStellarBody SunInfo;
	SunInfo.ColorIndex = 0.656;
	SunInfo.Color = UCelestialMaths::BVtoLinearColor(SunInfo.ColorIndex);
	SunInfo.OrbitType = EOrbitType::VSOP87;
	SunInfo.RAJ2000 = RAJ2000Hours;
	SunInfo.DECJ2000 = DECJ2000Degrees;
	SunInfo.RA = RAHours;
	SunInfo.DEC = DECDegrees;
	SunInfo.RAGeocentric = RAGeocentricHours;
	SunInfo.DECGeocentric = DECGeocentricDegrees;
	SunInfo.DistanceInAU = DistanceEarthToSunAU;
	SunInfo.Magnitude = -26.74;
	SunInfo.Radius = 695700;

	// Make sure we build the UE Transformation Cache
	SunInfo.ComputeUELocalTransform(); // No change in scale or distance for the Sun - Default arguments
	
	return SunInfo;
}

double UCelestialMaths::GetMoonNormalizedAgeSimple(double JulianDate)
{
	double DeltaDays = JulianDate - NewMoonReferenceJulianDate;
	double MoonAgeDays = ModPositive(DeltaDays, SynodicMonthAverage);

	return MoonAgeDays / SynodicMonthAverage;
}

#pragma endregion

#pragma region Utilities

void UCelestialMaths::XYZToRADEC_RH(FVector XYZ, double& RADegrees, double& DECDegrees, double& Radius)
{
	if (XYZ.IsNearlyZero())
	{
		UE_LOGF(LogCelestialVault, Error, "Zero-length XYZ vector in XYZToRADEC_RH - Check your inputs");
		RADegrees = 0.0;
		DECDegrees = 0.0;
		Radius = 0.0;
		return;
	}

	//Convert from Cartesian to polar coordinates
	Radius = XYZ.Length();
	double l = FMath::Atan2(XYZ.Y, XYZ.X);
	double t = FMath::Acos(XYZ.Z / Radius);

	//Make sure RA is positive, and Dec is in range +/-90
	if (l < 0.0) {
		l += 2.0 * UE_PI ;
	}
	t = 0.5 * UE_PI - t;

	RADegrees = FMath::RadiansToDegrees(l);
	DECDegrees = FMath::RadiansToDegrees(t);
}

FVector UCelestialMaths::RADECToXYZ_RH(double RADegrees, double DECDegrees, double Radius)
{
	double RARadians = FMath::DegreesToRadians(RADegrees);
	double DECRadians = FMath::DegreesToRadians(DECDegrees);
	
	double X = Radius * FMath::Cos(DECRadians) * FMath::Cos(RARadians);
	double Y = Radius * FMath::Cos(DECRadians) * FMath::Sin(RARadians);
	double Z = Radius * FMath::Sin(DECRadians);

	return FVector(X, Y, Z);
}

FString UCelestialMaths::GetPreciseVectorString(FVector Vector, int32 MinimumFractionalDigits)
{
	FNumberFormattingOptions NumberFormatOptions;
	NumberFormatOptions.MinimumFractionalDigits = MinimumFractionalDigits;

	return FString::Printf(TEXT("X=%s Y=%s Z=%s"),
		*FText::AsNumber(Vector.X, &NumberFormatOptions).ToString(),
		*FText::AsNumber(Vector.Y, &NumberFormatOptions).ToString(),
		*FText::AsNumber(Vector.Z, &NumberFormatOptions).ToString()
		);
}

FString UCelestialMaths::Conv_CelestialBodyToString(const FCelestialBody& CelestialBody) 
{
	return CelestialBody.ToString();
}

FString UCelestialMaths::Conv_PlanetaryBodyToString(const FPlanetaryBody& PlanetaryBody)
{
	return PlanetaryBody.ToString();
}

FString UCelestialMaths::Conv_StellarBodyToString(const FStellarBody& StellarBody)
{
	return StellarBody.ToString();
}

FString UCelestialMaths::Conv_RightAscensionToString(const double& RightAscensionHours)
{
	FString RAString;
	int32 H = 0;
	int32 M = 0;
	double S = 0.0;
	DegreesToHMS(RightAscensionHours * 15.0, H, M,S);
	return Conv_HMSToString(H,M,S); 
}

FString UCelestialMaths::Conv_DeclinationToString(const double& DeclinationDegrees)
{
	int32 DecD = 0;
	int32 DecM = 0;
	double DecS = 0.0;
	bool DecSign = false;
	DegreesToDMS(DeclinationDegrees, DecSign, DecD, DecM, DecS  );
	return Conv_DMSToString(DecSign, DecD, DecM, DecS);
}

FString UCelestialMaths::Conv_HMSToString( int32 Hours, int32 Minutes, double Seconds)
{
	return FString::Printf(TEXT("%dh%02dm%05.2fs"), Hours, Minutes, Seconds); 
}

FString UCelestialMaths::Conv_DMSToString(bool Sign, int32 Degrees, int32 Minutes, double Seconds)
{
	return FString::Printf(TEXT("%c%d°%02d\'%05.2f\""), Sign? '+' : '-' , Degrees, Minutes, Seconds); 
}

FString UCelestialMaths::Conv_DegreesToHMSString(double Degrees)
{
	int32 H = 0;
	int32 M = 0;
	double S = 0.0;
	DegreesToHMS(Degrees, H, M,S);
	return Conv_HMSToString(H,M,S); 
}

FString UCelestialMaths::Conv_DegreesToDMSString(double Degrees)
{
	int32 D = 0;
	int32 M = 0;
	double S = 0.0;
	bool Sign = false;
	DegreesToDMS(Degrees, Sign, D, M, S  );
	return Conv_DMSToString(Sign, D, M, S);
}

#pragma endregion

#pragma region Private Functions and variables

int32 UCelestialMaths::FloorForJulianDate(double JulianDate)
{
	if (JulianDate > 0.0)
	{
		return StaticCast<int32>(FMath::Floor(JulianDate));
	}
	if (JulianDate == FMath::Floor(JulianDate))
	{
		return StaticCast<int32>(JulianDate);
	}
	return StaticCast<int32>(FMath::Floor(JulianDate)-1);
}

FMatrix UCelestialMaths::GetRotationXMatrix(double AngleRadians)
{
	// Compute cosine and sine of the angle
	double CosTheta = FMath::Cos(AngleRadians);
	double SinTheta = FMath::Sin(AngleRadians);
	
	// Construct the 4x4 rotation matrix manually
	return FMatrix(
			FPlane(1.0, 0.0,      0.0,      0.0), // Row 1
			FPlane(0.0, CosTheta, SinTheta,0.0), // Row 2
			FPlane(0.0, -SinTheta, CosTheta, 0.0), // Row 3
			FPlane(0.0, 0.0,      0.0,      1.0)  // Row 4 (required for FMatrix, but ignored here)
		);
}

FMatrix UCelestialMaths::GetRotationYMatrix(double AngleRadians)
{
	// Compute cosine and sine of the angle
	double CosTheta = FMath::Cos(AngleRadians);
	double SinTheta = FMath::Sin(AngleRadians);
	
	// Construct the 4x4 rotation matrix manually
	return FMatrix(
		FPlane(CosTheta, 0.0, -SinTheta,0.0), // Row 1
		FPlane(0.0,      1.0, 0.0,     0.0), // Row 2
		FPlane(SinTheta,0.0, CosTheta,0.0), // Row 3
		FPlane(0.0,      0.0, 0.0,     1.0)  // Row 4 (required for FMatrix, but ignored here)
		);
}

FMatrix UCelestialMaths::GetRotationZMatrix(double AngleRadians)
{
	// Compute cosine and sine of the angle
	double CosTheta = FMath::Cos(AngleRadians);
	double SinTheta = FMath::Sin(AngleRadians);
	
	// Construct the 4x4 rotation matrix manually
	return FMatrix(
			FPlane(CosTheta, SinTheta, 0.0, 0.0), // Row 1
			FPlane(-SinTheta,  CosTheta, 0.0, 0.0), // Row 2
			FPlane(0.0,      0.0,       1.0, 0.0), // Row 3
			FPlane(0.0,      0.0,       0.0, 1.0)  // Row 4 (required for FMatrix, but ignored here)
		);
}

#pragma endregion