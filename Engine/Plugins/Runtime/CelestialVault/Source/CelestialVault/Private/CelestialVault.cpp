// Copyright Epic Games, Inc. All Rights Reserved.

#include "CelestialVault.h"

#include "HAL/IConsoleManager.h"

#define LOCTEXT_NAMESPACE "FCelestialVaultModule"

DEFINE_LOG_CATEGORY(LogCelestialVault);


void FCelestialVaultModule::StartupModule()
{
	// This code will execute after your module is loaded into memory; the exact timing is specified in the .uplugin file per-module
	
	// Starting with 5.8, Setting r.EyeAdaptation.CachedLightingPreExposure wrong will cause various cached lighting values to be clamped, 
	// which means that e.g. indirect lighting will be too dark. 
	// The default value is not adapted to our high luminance range, so we need to adjust it. 
	if (IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.EyeAdaptation.CachedLightingPreExposure")))
	{
		const EConsoleVariableFlags SetBy = (EConsoleVariableFlags)(CVar->GetFlags() & ECVF_SetByMask);

		// We still want to allow the user to override it in its DefautEngine.ini file, so we override only the initial value set by constructor.
		if (SetBy == ECVF_SetByConstructor)
		{
			CVar->Set(8, ECVF_SetByCode);
		}
	}
}

void FCelestialVaultModule::ShutdownModule()
{
	// This function may be called during shutdown to clean up your module.  For modules that support dynamic reloading,
	// we call this function before unloading the module.
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FCelestialVaultModule, CelestialVault)



