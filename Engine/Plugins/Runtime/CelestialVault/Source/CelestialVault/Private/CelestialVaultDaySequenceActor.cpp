// Copyright Epic Games, Inc. All Rights Reserved.

#include "CelestialVaultDaySequenceActor.h"

// Components
#include "Components/DirectionalLightComponent.h"
#include "Components/ExponentialHeightFogComponent.h"
#include "Components/PostProcessComponent.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "Components/SkyAtmosphereComponent.h"
#include "Components/SkyLightComponent.h"
#include "Components/VolumetricCloudComponent.h"
#include "Components/StaticMeshComponent.h"

// UE Objects
#include "Materials/MaterialInterface.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "UObject/ConstructorHelpers.h"
#include "DaySequenceSubsystem.h"
#include "DaySequenceCollectionAsset.h"
#include "Materials/MaterialParameterCollectionInstance.h"
#include "Materials/MaterialParameterCollection.h"
#include "DaySequence/Private/DaySequencePlayer.h"
#include "Engine/World.h"
#include "Engine/StaticMesh.h"
#include "Curves/CurveFloat.h"

// Celestial Objects
#include "CelestialMaths.h"
#include "CelestialDataTypes.h"
#include "CelestialVault.h"
#include "ViewportCameraProvider.h"


#if CELESTIAL_VAULT_ENABLE_DRAW_DEBUG

#include "Engine/Engine.h"
#include "Debug/DebugDrawService.h"
#include "SceneView.h"
#include "Engine/Canvas.h"

// Celestial Vault Global ShowFlag
TCustomShowFlag<EShowFlagShippingValue::Dynamic> CelestialVaultCustomShowFlag(
	TEXT("CelestialVault"),
	false,
	EShowFlagGroup::SFG_Visualize,
	NSLOCTEXT("ShowFlagDisplayName", "CelestialVault", "Celestial Vault")
);
#endif

#define UE_ONE_KILOMETER 100000.0

// Sets default values
ACelestialVaultDaySequenceActor::ACelestialVaultDaySequenceActor(const FObjectInitializer& Init)
: Super(Init)
{
	ExponentialHeightFogComponent = CreateOptionalDefaultSubobject<UExponentialHeightFogComponent>(TEXT("ExponentialHeightFog"));
	if (ExponentialHeightFogComponent)
	{
		ExponentialHeightFogComponent->SetupAttachment(RootComponent);
		ExponentialHeightFogComponent->FogCutoffDistance = 20000000.0; // We don't want the fog to apply on the deep sky
		ExponentialHeightFogComponent->EndDistance = 500000.0; // We need a non-zero value here to avoid artifacts in round worlds, and quite high to have a proper haze
	}
	
	static ConstructorHelpers::FObjectFinder<UCurveFloat> HighlightContrastCurve(TEXT("/CelestialVault/Data/CF_CelestialHighlightContrastCurve.CF_CelestialHighlightContrastCurve"));
	GlobalPostProcessVolume = CreateOptionalDefaultSubobject<UPostProcessComponent>(TEXT("GlobalPostProcessVolume"));
	if (GlobalPostProcessVolume)
	{
		GlobalPostProcessVolume->SetupAttachment(RootComponent);
		GlobalPostProcessVolume->Settings.bOverride_AutoExposureMinBrightness = true;
		GlobalPostProcessVolume->Settings.AutoExposureMinBrightness = -2.0f;
		GlobalPostProcessVolume->Settings.bOverride_LocalExposureHighlightContrastCurve = true;
		GlobalPostProcessVolume->Settings.LocalExposureHighlightContrastCurve = HighlightContrastCurve.Object;

		GlobalPostProcessVolume->Settings.bOverride_AutoExposureSpeedUp = true;
		GlobalPostProcessVolume->Settings.AutoExposureSpeedUp = 100.0f;
		GlobalPostProcessVolume->Settings.bOverride_AutoExposureSpeedDown = true;
		GlobalPostProcessVolume->Settings.AutoExposureSpeedDown = 100.0f;
	}
	
	// Components Attached to Root
	SkyLightComponent = CreateDefaultSubobject<USkyLightComponent>(TEXT("Sky Light"));
	if (SkyLightComponent)
	{
		SkyLightComponent->SetupAttachment(RootComponent);
		SkyLightComponent->bRealTimeCapture = true;
		SkyLightComponent->bLowerHemisphereIsBlack = false;
	}
	

	VolumetricCloudComponent = CreateOptionalDefaultSubobject<UVolumetricCloudComponent>(TEXT("Volumetric Cloud"));
	if (VolumetricCloudComponent)
	{
		VolumetricCloudComponent->SetupAttachment(RootComponent);
		if (!IsTemplate())
		{
			// We don't want to load this material for the CDO as it will hold on to it forever and it is quite a large asset.
			static ConstructorHelpers::FObjectFinder<UMaterialInterface> VolumetricCloudDefaultMaterialRef(TEXT("/CelestialVault/Materials/m_SimpleVolumetricCloud_TOD_Inst.m_SimpleVolumetricCloud_TOD_Inst"));
			VolumetricCloudComponent->SetMaterial(VolumetricCloudDefaultMaterialRef.Object);
		}	
	}
	

	NorthOffsetComponent = CreateDefaultSubobject<USceneComponent>(TEXT("North Offset Transform"));
	if (NorthOffsetComponent)
	{
		NorthOffsetComponent->SetupAttachment(RootComponent);	
	}
	

	// Add an editor-only compass to visualize the North Offset
	CompassComponent = CreateOptionalDefaultSubobject<UStaticMeshComponent>(TEXT("Compass"));
	if (CompassComponent)
	{
		CompassComponent->SetupAttachment(NorthOffsetComponent);
		CompassComponent->SetCollisionEnabled(ECollisionEnabled::NoCollision);
		CompassComponent->SetGenerateOverlapEvents(false);
		CompassComponent->SetCastShadow(false);
		CompassComponent->SetAffectDynamicIndirectLighting(false);
		CompassComponent->SetCanEverAffectNavigation(false);
		CompassComponent->bIsEditorOnly = true;
		CompassComponent->SetHiddenInGame(true);
		if (!IsTemplate())
		{
			static ConstructorHelpers::FObjectFinder<UStaticMesh> CompassDefaultMesh(TEXT("/CelestialVault/Editor/SM_Compass.SM_Compass"));
			CompassComponent->SetStaticMesh(CompassDefaultMesh.Object);
		}	
	}
	
	
	PlanetCenterComponent = CreateDefaultSubobject<USceneComponent>(TEXT("Planet Center Transform"));
	if (PlanetCenterComponent)
	{
		PlanetCenterComponent->SetupAttachment(NorthOffsetComponent);	
	}
	
	
	// Components Attached to Planet Center
	SkyAtmosphereComponent = CreateDefaultSubobject<USkyAtmosphereComponent>(TEXT("Sky Atmosphere"));
	if (SkyAtmosphereComponent)
	{
		SkyAtmosphereComponent->SetupAttachment(PlanetCenterComponent);
		SkyAtmosphereComponent->TransformMode = ESkyAtmosphereTransformMode::PlanetCenterAtComponentTransform;	
	}
	


	// Rotating Celestial Vault 
	CelestialVaultComponent = CreateDefaultSubobject<USceneComponent>(TEXT("Rotating Celestial Vault"));
	if (CelestialVaultComponent)
	{
		CelestialVaultComponent->SetupAttachment(PlanetCenterComponent);	
	}
	
	// Observer-centered frame for at-infinity sky content. Attached to Root (We replicate the observer location with a t translation)
	// so that fixed stars and the deep-sky dome are not subject to the ~R_earth/R_vault parallax
	// bias introduced by the Earth-centered PlanetCenterComponent.
	TopocentricVaultComponent = CreateDefaultSubobject<USceneComponent>(TEXT("Topocentric Vault"));
	if (TopocentricVaultComponent)
	{
		TopocentricVaultComponent->SetupAttachment(NorthOffsetComponent);
	}

	// Sidereal-rotation child of the topocentric vault. Shares its yaw track with
	// CelestialVaultComponent via CelestialVaultSequence so the at-infinity sky rotates in lockstep.
	TopocentricSiderealComponent = CreateDefaultSubobject<USceneComponent>(TEXT("Topocentric Sidereal"));
	if (TopocentricSiderealComponent)
	{
		TopocentricSiderealComponent->SetupAttachment(TopocentricVaultComponent);	
	}
	
	
	// Components attached to the Celestial Vaul
	
	// Deep Sky background
	static ConstructorHelpers::FObjectFinder<UStaticMesh> SkySphereDefaultMesh(TEXT("/CelestialVault/Meshes/SM_CelestialVault.SM_CelestialVault"));
	static ConstructorHelpers::FObjectFinder<UMaterialInterface> SkySphereDefaultMaterial(TEXT("/CelestialVault/Materials/MI_CelestialVault.MI_CelestialVault"));
	DeepSkyComponent = CreateOptionalDefaultSubobject<UStaticMeshComponent>(TEXT("Deep Sky"));
	if (DeepSkyComponent)
	{
		DeepSkyComponent->SetupAttachment(TopocentricSiderealComponent);
		DeepSkyComponent->SetStaticMesh(SkySphereDefaultMesh.Object);
		DeepSkyComponent->SetCollisionEnabled(ECollisionEnabled::NoCollision);
		DeepSkyComponent->SetGenerateOverlapEvents(false);
		DeepSkyComponent->SetCastShadow(false);
		DeepSkyComponent->SetAffectDynamicIndirectLighting(false);
		DeepSkyComponent->SetCanEverAffectNavigation(false);
		DeepSkyComponent->SetMaterial(0, SkySphereDefaultMaterial.Object.Get());
		DeepSkyComponent->SetRelativeScale3D(FVector(CelestialVaultDistance*1000.0));	
	}


	// Deep Sky background
	static ConstructorHelpers::FObjectFinder<UMaterialInterface> VelocityProxyDefaultMaterial(TEXT("/CelestialVault/Materials/M_VelocityProxy.M_VelocityProxy"));
	VelocityVectorsProxyComponent = CreateOptionalDefaultSubobject<UStaticMeshComponent>(TEXT("Velocity Vectors Proxy"));
	if (VelocityVectorsProxyComponent)
	{
		VelocityVectorsProxyComponent->SetupAttachment(DeepSkyComponent);
		VelocityVectorsProxyComponent->SetStaticMesh(SkySphereDefaultMesh.Object);
		VelocityVectorsProxyComponent->SetCollisionEnabled(ECollisionEnabled::NoCollision);
		VelocityVectorsProxyComponent->SetGenerateOverlapEvents(false);
		VelocityVectorsProxyComponent->SetCastShadow(false);
		VelocityVectorsProxyComponent->SetAffectDynamicIndirectLighting(false);
		VelocityVectorsProxyComponent->SetCanEverAffectNavigation(false);
		VelocityVectorsProxyComponent->SetMaterial(0, VelocityProxyDefaultMaterial.Object.Get());
		VelocityVectorsProxyComponent->SetRelativeScale3D(FVector(1.001));	
	}
	

	// Stars ISM
	static ConstructorHelpers::FObjectFinder<UStaticMesh> PlaneXMesh(TEXT("/CelestialVault/Meshes/SM_Plane_FacingX.SM_Plane_FacingX"));
	static ConstructorHelpers::FObjectFinder<UMaterialInterface> StarsDefaultMaterial(TEXT("/CelestialVault/Materials/MI_StarsStar_EnergyConservative.MI_StarsStar_EnergyConservative"));
	StarsComponent = CreateOptionalDefaultSubobject<UInstancedStaticMeshComponent>(TEXT("Stars"));
	if (StarsComponent)
	{
		StarsComponent->SetupAttachment(DeepSkyComponent);
		StarsComponent->SetStaticMesh(PlaneXMesh.Object);
		StarsComponent->SetCollisionEnabled(ECollisionEnabled::NoCollision);
		StarsComponent->SetGenerateOverlapEvents(false);
		StarsComponent->SetCastShadow(false);
		StarsComponent->SetAffectDynamicIndirectLighting(false);
		StarsComponent->SetCanEverAffectNavigation(false);
		StarsComponent->SetMaterial(0, StarsDefaultMaterial.Object);
		StarsComponent->SetTranslucentSortPriority(-1);
		StarsComponent->bUseAttachParentBound = true;	
	}
	

	// Planets ISM
	static ConstructorHelpers::FObjectFinder<UMaterialInterface> PlanetsDefaultMaterial(TEXT("/CelestialVault/Materials/MI_SolarSystemPlanets.MI_SolarSystemPlanets"));
	PlanetsComponent = CreateOptionalDefaultSubobject<UInstancedStaticMeshComponent>(TEXT("Planets"));
	if (PlanetsComponent)
	{
		PlanetsComponent->SetupAttachment(CelestialVaultComponent);
		PlanetsComponent->SetStaticMesh(PlaneXMesh.Object);
		PlanetsComponent->SetCollisionEnabled(ECollisionEnabled::NoCollision);
		PlanetsComponent->SetGenerateOverlapEvents(false);
		PlanetsComponent->SetCastShadow(false);
		PlanetsComponent->SetAffectDynamicIndirectLighting(false);
		PlanetsComponent->SetCanEverAffectNavigation(false);
		PlanetsComponent->SetMaterial(0, PlanetsDefaultMaterial.Object);
	}
	
	

	// Moon - Disc
	static ConstructorHelpers::FObjectFinder<UMaterialInterface> MoonDiscDefaultMaterial(TEXT("/CelestialVault/Materials/MI_Moon.MI_Moon"));
	MoonDiscComponent = CreateOptionalDefaultSubobject<UStaticMeshComponent>(TEXT("Moon Disk"));
	if (MoonDiscComponent)
	{
		MoonDiscComponent->SetupAttachment(CelestialVaultComponent);
		MoonDiscComponent->SetStaticMesh(PlaneXMesh.Object);
		MoonDiscComponent->SetCollisionEnabled(ECollisionEnabled::NoCollision);
		MoonDiscComponent->SetGenerateOverlapEvents(false);
		MoonDiscComponent->SetCastShadow(true); // Eclipses?
		MoonDiscComponent->SetAffectDynamicIndirectLighting(false);
		MoonDiscComponent->SetCanEverAffectNavigation(false);
		MoonDiscComponent->SetMaterial(0, MoonDiscDefaultMaterial.Object.Get());
	}
	

	// Moon - Light
	MoonLightComponent = CreateDefaultSubobject<UDirectionalLightComponent>(TEXT("Moon Light"));
	if (MoonLightComponent)
	{
		MoonLightComponent->SetupAttachment(CelestialVaultComponent);
		MoonLightComponent->SetAtmosphereSunLightIndex(1);	// Make Moon the secondary directional light that contributes to the sky atmosphere.
		MoonLightComponent->SetForwardShadingPriority(0); // Give Moon forward shading priority.
		MoonLightComponent->SetIntensity(MoonLightIntensity);
		MoonLightComponent->SetUseTemperature(true);
		MoonLightComponent->SetTemperature(9000.f);
		MoonLightComponent->SetWorldRotation(FRotator(-45.f, 0.0f, 0.0f));
		MoonLightComponent->bCastCloudShadows = true; // Otherwise we still have hard shadows with an overcast sky 
	}
	
	// Attach the sunlight relative to the rotating Vault
	SunLightComponent = CreateDefaultSubobject<UDirectionalLightComponent>(TEXT("Sun Light"));
	if (SunLightComponent)
	{
		SunLightComponent->SetupAttachment(CelestialVaultComponent);
		SunLightComponent->SetAtmosphereSunLightIndex(0);	// Make Sun the first directional light that contributes to the sky atmosphere.
		SunLightComponent->SetForwardShadingPriority(1);   // Give Sun forward shading priority.
		SunLightComponent->SetIntensity(SunLightIntensity);
		SunLightComponent->bCastCloudShadows = true; // Otherwise we still have hard shadows with an overcast sky
	}
	
	// Sequence and Data Assets
	if (!IsTemplate())
	{
		// Override the default collection (which animates the moon and sky material)
		static ConstructorHelpers::FObjectFinder<UDaySequenceCollectionAsset> DefaultCollection(TEXT("/CelestialVault/DSCA_CelestialVault.DSCA_CelestialVault"));
		DaySequenceCollections.Add(DefaultCollection.Object.Get());

		static ConstructorHelpers::FObjectFinder<UDataTable> DefaultStarsCatalog(TEXT("/CelestialVault/Data/DT_HYGCatalog_10K.DT_HYGCatalog_10K"));
		CelestialStarCatalog = DefaultStarsCatalog.Object.Get();

		static ConstructorHelpers::FObjectFinder<UDataTable> DefaultPlanetaryBodiesCatalog(TEXT("/CelestialVault/Data/DT_SolarSystemPlanets.DT_SolarSystemPlanets"));
		PlanetsCatalog = DefaultPlanetaryBodiesCatalog.Object.Get();
	}

	// MPC
	static ConstructorHelpers::FObjectFinder<UMaterialParameterCollection> MPC(TEXT("/CelestialVault/Materials/MPC_CelestialVault.MPC_CelestialVault"));
	CelestialVaultMPC = MPC.Object.Get();

	SetTimePerCycle(24);
	SetInitialTimeOfDay(12);
}

bool ACelestialVaultDaySequenceActor::IsDaylightSavingsNow() const
{
	return UDaylightSavings::IsDaylightSavings(GetDateAndTime(), DaylightSavingsMode, Latitude, DaylightSavingsStart, DaylightSavingsEnd, DaylightSavingsSwitchHour);
}

bool ACelestialVaultDaySequenceActor::IsDaylightSavingsAtTimeOfDay(double TimeOfDay) const
{
	FDateTime LocalTime = GetDate() + FTimespan::FromHours(TimeOfDay);
	return UDaylightSavings::IsDaylightSavings(LocalTime,  DaylightSavingsMode, Latitude, DaylightSavingsStart, DaylightSavingsEnd, DaylightSavingsSwitchHour);
}

//////// Protected Functions

void ACelestialVaultDaySequenceActor::BeginPlay()
{
	Super::BeginPlay();

	// We don't inherit from a BaseDaySequenceActor (not the same components), so we need to register ourselves
	if (const UWorld* World = GetWorld())
	{
		if (UDaySequenceSubsystem* DaySequenceSubsystem = World->GetSubsystem<UDaySequenceSubsystem>())
		{
			DaySequenceSubsystem->SetDaySequenceActor(this);
		}
		SequenceLoopCount = 0;
		LastViewportCameraLocation = FVector::ZeroVector;
	}
}

void ACelestialVaultDaySequenceActor::OnConstruction(const FTransform& Transform)
{
	Super::OnConstruction(Transform);

	// We don't inherit from a BaseDaySequenceActor (not the same components), so we need to register ourselves
	if (const UWorld* World = GetWorld())
	{
		if (UDaySequenceSubsystem* DaySequenceSubsystem = World->GetSubsystem<UDaySequenceSubsystem>())
		{
			if (DaySequenceSubsystem->GetDaySequenceActor(/* bFindFallbackOnNull */ false) != this)
			{
				DaySequenceSubsystem->SetDaySequenceActor(this);
			}
		}

		// Replace the Moon Material with a MID
		if (MoonDiscComponent)
		{
			UMaterialInstanceDynamic * MoonDiscMaterial = Cast<UMaterialInstanceDynamic>(MoonDiscComponent->GetMaterial(0));
			if (!MoonDiscMaterial)
			{
				MoonDiscMaterial = MoonDiscComponent->CreateAndSetMaterialInstanceDynamic(0); // Make the material Dynamic to control the Phase
			}
			MoonDiscMaterial->SetScalarParameterValue(FName("MoonAge"), MoonAge);	
		}
		
		RebuildAll();  
	}
}

void ACelestialVaultDaySequenceActor::Tick(float DeltaTime)
{
	Super::Tick(DeltaTime);

#if WITH_EDITOR
	// In Editor, DaySequenceUpdated is not called, but the actor is ticked
	// In this case, we'll update the Bodies motion manually on tick to react to the TimeOfDay Slider changes
	if (UWorld* World = GetWorld())
	{
		// We don't want to update the bodies motion each frame in Game, as the DaySequence actor has a Sequence Update Interval to allow for sliced updates. We don't want to override this feature.   
		if (!World->IsGameWorld())
		{
			// In editor...
			UpdateBodiesMotion();	
		}
	}
#endif

	if (CelestialVaultMPC)
	{
		if (UWorld* World = GetWorld())
		{
			if (UMaterialParameterCollectionInstance* CelestialVaultMPCInstance = World->GetParameterCollectionInstance(CelestialVaultMPC))
			{
				CelestialVaultMPCInstance->SetScalarParameterValue(TEXT("TimeOfDay"), (SequenceLoopCount * 24 + GetTimeOfDay())*3600);
			}	
		}
	}
	
	if (IViewportCameraProvider* Provider = GetCameraProvider())
	{
		FVector CameraLocation;
		FRotator CameraRotation;

		if (Provider->GetCamera(CameraLocation, CameraRotation))
		{
			
			// Check if we moved by more than ObserverBasedEffectsMovementThreshold
			if ( FVector::DistSquared(LastViewportCameraLocation, CameraLocation) >  (ObserverBasedEffectsMovementThreshold * ObserverBasedEffectsMovementThreshold * UE_ONE_KILOMETER * UE_ONE_KILOMETER))
			{
				LastViewportCameraLocation = CameraLocation;
				
				// Adjust the SkyAtmosphere radius
				if (bAdjustSkyAtmosphereToLocalRadius)
				{
					if (PlanetCenterComponent)
					{
						FVector CameraECEFLocationMeters = PlanetCenterComponent->GetComponentTransform().InverseTransformPosition(CameraLocation) * FVector(0.01, -0.01, 0.01);
						const FVector CameraECEFLocationAU = CameraECEFLocationMeters * UCelestialMaths::MetersToAstronomicalUnits(1);

						double CameraLatitude, CameraLongitude, CameraAltitudeMeters;
						UCelestialMaths::ECEFXYZAUToGeodeticLatLon(CameraECEFLocationAU, CameraLatitude, CameraLongitude, CameraAltitudeMeters);
						//UE_LOGF(LogCelestialVault, Verbose, "Current Camera Location: Latitude=%f, Longitude=%f, Altitude=%f m", CameraLatitude, CameraLongitude, CameraAltitudeMeters);

						double WG84SRadiusMeters = UCelestialMaths::WGS84GeocentricRadius(CameraLatitude);
						double AdjustedRadiusKm = FMath::GetMappedRangeValueClamped(FVector2D(AdjustSkyAtmosphereFadeMinAltitude * 1000.0, AdjustSkyAtmosphereFadeMaxAltitude * 1000.0), FVector2D(WG84SRadiusMeters, UCelestialMaths::WGS84::SemiMajorAxis), CameraAltitudeMeters) / 1000.0;
						//UE_LOGF(LogCelestialVault, Verbose, "WG84 Radius: %f km - Retained = %f km", WG84SRadiusMeters / 1000.0, AdjustedRadiusKm);
						SkyAtmosphereComponent->SetBottomRadius(AdjustedRadiusKm);
					}
				}
			
				// To Avoid Parallax issues with the stars, we need to center the Deep Sky around the current observer 
				if (TopocentricVaultComponent)
				{
					TopocentricVaultComponent->SetWorldLocation(CameraLocation);
				}
			 
				// Trigger UpdateBodiesMotion to adjust the Lights directions if the observer moved
				UpdateBodiesMotion();
			}
		}
	}
}

void ACelestialVaultDaySequenceActor::SequencePlayerUpdated(float CurrentTime, float PreviousTime)
{
	Super::SequencePlayerUpdated(CurrentTime, PreviousTime);

	if (IDaySequencePlayer* Player = GetSequencePlayer())
	{
		if (Player->GetCurrentNumLoops() != SequenceLoopCount)
		{
			SequenceLoopCount = Player->GetCurrentNumLoops(); 
			UE_LOGF(LogCelestialVault, Verbose, "Day Sequence Loop: We are now on Day %d", SequenceLoopCount);

			// New day, refresh the Body Motion Cache.  
			ResetBodyMotionCache();
		}

		// Update Celestial Bodies Motion.
		UpdateBodiesMotion();
	}
	
}

void ACelestialVaultDaySequenceActor::PostRegisterAllComponents()
{
	Super::PostRegisterAllComponents();

#if CELESTIAL_VAULT_ENABLE_DRAW_DEBUG
	if (!DrawDebugDelegateHandle.IsValid())
	{
		DrawDebugDelegateHandle = UDebugDrawService::Register(*CelestialVaultSF, FDebugDrawDelegate::CreateUObject(this, &ACelestialVaultDaySequenceActor::DebugDrawCallback));
	}
#endif 
}

void ACelestialVaultDaySequenceActor::UnregisterAllComponents(bool bForReregister)
{
#if CELESTIAL_VAULT_ENABLE_DRAW_DEBUG
	if (DrawDebugDelegateHandle.IsValid())
	{
		UDebugDrawService::Unregister(DrawDebugDelegateHandle);
		DrawDebugDelegateHandle.Reset();
	}
#endif 	
	Super::UnregisterAllComponents(bForReregister);
}

void ACelestialVaultDaySequenceActor::Stop()
{
	UDaySequencePlayer* Player = GetSequencePlayerInternal();
	UWorld* World = GetWorld();
	if (HasValidRootSequence() && Player && World && World->IsGameWorld())
	{
		Player->Stop();
		StopDaySequenceUpdateTimer();
		SetTimeOfDay( GetInitialTimeOfDay());
		SequenceLoopCount = 0;
	}
}

#if CELESTIAL_VAULT_ENABLE_DRAW_DEBUG

void ACelestialVaultDaySequenceActor::DebugDrawCallback(UCanvas* Canvas, APlayerController* PC) const
{
	// Get the current editor view’s flags (pseudo; adapt to your context)
	const FEngineShowFlags& Flags = Canvas->SceneView->Family->EngineShowFlags;
	if (CelestialVaultCustomShowFlag.IsEnabled(Flags))
	{
		// Draw in Green
		Canvas->SetDrawColor(FColor::Green);

		// Build message string before displaying it at once
		FString DebugMessage;
		DebugMessage += FString("------ Celestial Vault ------\n");
		DebugMessage += FString("    -- Time -- \n");

		FDateTime LocalTime = GetDateAndTime();
		FDateTime UTCTime = UCelestialMaths::LocalTimeToUTCTime(LocalTime, GMT_TimeZone, IsDaylightSavingsNow());
		double JulianDate = UCelestialMaths::UTCDateTimeToJulianDate(UTCTime);
		double JulianCenturies = UCelestialMaths::JulianDateToJulianCenturies(JulianDate); 
		double GMST = UCelestialMaths::DateTimeToGreenwichMeanSiderealTime(UTCTime);
		double LMST = UCelestialMaths::LocalSideralTime(Longitude, GMST); // TODO /!\ ECEF

		 
		DebugMessage += FString::Printf(TEXT("    Local Time: %s"), *LocalTime.ToFormattedString(TEXT("%d %b %Y, %H:%M:%S"))) + LINE_TERMINATOR;
		DebugMessage += FString::Printf(TEXT("    UTC Time: %s"), *UTCTime.ToFormattedString(TEXT("%d %b %Y, %H:%M:%S"))) + LINE_TERMINATOR;
		DebugMessage += FString::Printf(TEXT("    Julian Date: %.5f"), JulianDate) + LINE_TERMINATOR;
		DebugMessage += FString::Printf(TEXT("    Julian Centuries: %.5f"), JulianCenturies) + LINE_TERMINATOR;
		DebugMessage += FString::Printf(TEXT("    Daylight Savings: %s"), IsDaylightSavingsNow() ? TEXT("Yes") : TEXT("No") ) + LINE_TERMINATOR;
		
		DebugMessage += FString::Printf(TEXT("    GMST at 0:00: %s (%.5f)°"), *UCelestialMaths::Conv_DegreesToHMSString(UCelestialMaths::ModPositive(GMST0_Unwrapped,360)), UCelestialMaths::ModPositive(GMST0_Unwrapped,360)) + LINE_TERMINATOR;
		DebugMessage += FString::Printf(TEXT("    GMST: %s (%.5f)°"), *UCelestialMaths::Conv_DegreesToHMSString(GMST), GMST) + LINE_TERMINATOR;
		DebugMessage += FString::Printf(TEXT("    LMST: %s (%.5f)°"), *UCelestialMaths::Conv_DegreesToHMSString(LMST), LMST) + LINE_TERMINATOR;

		// Sun and Moon Info
		DebugMessage += LINE_TERMINATOR;
		DebugMessage += FString("    -- Sun -- \n");
		FStellarBody SunAtTime = ComputeSunInfo(JulianDate);
		DebugMessage += SunAtTime.ToString() + LINE_TERMINATOR;

		DebugMessage += FString("    -- Moon -- \n");
		FPlanetaryBody MoonAtTime = ComputeMoonInfo(JulianDate);
		DebugMessage += MoonAtTime.ToString() + LINE_TERMINATOR;
		
		// Draw
		Canvas->DrawText(GetDefault<UEngine>()->GetMediumFont(), DebugMessage, 10, 10);
	}
}

#endif

//////// Public Functions
FDateTime ACelestialVaultDaySequenceActor::GetDate() const
{
	if (bUseCurrentDate)
	{
		return FDateTime::Now().GetDate();
	}

	int32 ClampedMonth = FMath::Clamp(static_cast<int>(Month), 1, 12);
	if ( Day > FDateTime::DaysInMonth(Year, ClampedMonth))
	{
		int32 ClampedDay = FDateTime::DaysInMonth(Year, ClampedMonth);
		UE_LOGF(LogCelestialVault, Warning, "Day value (%d) over the number of days in month - Using %d instead", Day, ClampedDay);
		return FDateTime(Year, ClampedMonth, ClampedDay, 0,0,0);
	}
		
	return FDateTime(Year, ClampedMonth, Day, 0,0,0) + FTimespan(SequenceLoopCount, 0,0, 0);;
}

FDateTime ACelestialVaultDaySequenceActor::GetDateAndTime() const
{
	return GetDate() + FTimespan::FromHours(GetTimeOfDay());
}

double ACelestialVaultDaySequenceActor::GetJulianDate() const
{
	FDateTime LocalTime = GetDateAndTime();
	FDateTime UTCTime = UCelestialMaths::LocalTimeToUTCTime(LocalTime, GMT_TimeZone, IsDaylightSavingsNow());
	return UCelestialMaths::UTCDateTimeToJulianDate(UTCTime);
}

FStellarBody ACelestialVaultDaySequenceActor::ComputeSunInfo(double JulianDate) const
{
	// Compute Earth Reference location
	FPlanetaryBodyKinematicState EarthKinematicState = UCelestialMaths::GetPlanetaryBodyKinematicState_AU(JulianDate, EVSOP87BodyType::Earth);

	return ComputeSunInfo_UsingKnownState(JulianDate, EarthKinematicState);
}

FStellarBody ACelestialVaultDaySequenceActor::ComputeCurrentSunInfo() const
{
	return ComputeSunInfo(GetJulianDate());
}

FStellarBody ACelestialVaultDaySequenceActor::ComputeSunInfo_UsingKnownState(double JulianDate, const FPlanetaryBodyKinematicState& EarthKinematicState) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ComputeSunInfo_UsingKnownState);

	
	FStellarBody ResultSunInfo = UCelestialMaths::GetSunInformation_UsingKnownState(JulianDate, EarthKinematicState, Latitude, Longitude, bLevelIsGeocentric);

	// Location
	FVector SunLocation = UCelestialMaths::RADECToXYZ_RH(ResultSunInfo.RAGeocentric * 15.0, ResultSunInfo.DECGeocentric, UCelestialMaths::AstronomicalUnitsToMeters(ResultSunInfo.DistanceInAU) * 100);
	SunLocation.Y *= -1.0; // Convert to UE Frame by inverting Y
	ResultSunInfo.UELocalTransform.SetLocation(SunLocation);
	ResultSunInfo.DirectionTowardEarth = (FVector::ZeroVector - SunLocation).GetSafeNormal();
	return ResultSunInfo;
}

FPlanetaryBody ACelestialVaultDaySequenceActor::ComputeMoonInfo(double JulianDate) const
{
	// Compute Earth Reference location
	FPlanetaryBodyKinematicState EarthKinematicState = UCelestialMaths::GetPlanetaryBodyKinematicState_AU(JulianDate, EVSOP87BodyType::Earth);
	
	return ComputeMoonInfo_UsingKnownState(JulianDate, EarthKinematicState);
}

FPlanetaryBody ACelestialVaultDaySequenceActor::ComputeCurrentMoonInfo() const
{
	return ComputeMoonInfo(GetJulianDate());
}

FPlanetaryBody ACelestialVaultDaySequenceActor::ComputeMoonInfo_UsingKnownState(double JulianDate, const FPlanetaryBodyKinematicState& EarthKinematicState) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ComputeMoonInfo_UsingKnownState);

	
	double UEDistance = CelestialVaultDistance * UE_ONE_KILOMETER * MoonVaultPercentage / 100.0;
	FPlanetaryBody ResultMoonInfo = GetPlanetaryBodyInfo_UsingKnownState(JulianDate, FPlanetaryBodyInputData::Moon,  EarthKinematicState, true); // Ignore the relativistic effects for the moon. It's close to us, and save some iterations in the computations. 

	// Set the Moon specifics, and compute the Transform
	ResultMoonInfo.Age = UCelestialMaths::GetMoonNormalizedAgeSimple(JulianDate);;
	ResultMoonInfo.IlluminationPercentage = UCelestialMaths::GetIlluminationPercentage(ResultMoonInfo.Age);
	ResultMoonInfo.ComputeUELocalTransform(MoonScale, UEDistance);
	
	if (bManualControl) // Override 
	{
		// Phase/Age
		ResultMoonInfo.Age = MoonAge;
		ResultMoonInfo.IlluminationPercentage = UCelestialMaths::GetIlluminationPercentage(MoonAge);

		// Location
		FStellarBody SunInfoTemp = ComputeSunInfo_UsingKnownState(JulianDate,  EarthKinematicState);
		ResultMoonInfo.RAGeocentric = UCelestialMaths::ModPositive(SunInfoTemp.RAGeocentric + MoonOffset_RA, 24.0);
		ResultMoonInfo.DECGeocentric = UCelestialMaths::ModPositive(SunInfoTemp.DECGeocentric + MoonOffset_DEC + 180.0, 360.0) - 180.0;
		// We changed the RA, so we need to update the transform
		ResultMoonInfo.ComputeUELocalTransform(MoonScale, UEDistance);
	}
	return ResultMoonInfo;
}


void ACelestialVaultDaySequenceActor::SetMoonDiscAge(float InMoonAge)
{
	if (MoonDiscComponent)
	{
		UMaterialInstanceDynamic * MoonDiscMaterial = Cast<UMaterialInstanceDynamic>(MoonDiscComponent->GetMaterial(0));
		if (MoonDiscMaterial)
		{
			MoonDiscMaterial->SetScalarParameterValue(FName("MoonAge"), InMoonAge);
		}
	}

	if (MoonLightComponent)
	{
		MoonLightComponent->SetIntensity(MoonLightIntensity * UCelestialMaths::GetIlluminationPercentage(InMoonAge));
	}
}

bool ACelestialVaultDaySequenceActor::GetClosestStar(FVector ObserverLocation, FVector LookupDirection, double ThresholdAngleDegrees, FStellarBody& FoundStarInfo, FTransform& StarTransform)
{
	StarTransform = FTransform::Identity;
	
	// We query only if we generated the StarInfo  
	if (!bKeepStarsInfo || !StarsComponent)
	{
		return false;
	}
	
	double CosThresholdAngle = FMath::Cos(FMath::DegreesToRadians(ThresholdAngleDegrees));
	LookupDirection.Normalize();
	double MinCos = -1.0;
	bool StarFound = false;
	
	FTransform StarsComponentTransform = StarsComponent->GetComponentTransform();
	for (FStellarBody StarInfo : Stars)
	{
		// We need to compute the world Transform for the body, because the celestial vault has rotated and the Body UE Transform is local to the Sky. 
		FTransform WorldTransform =  StarInfo.UELocalTransform * StarsComponentTransform;
			
		FVector DirectionToInstance = WorldTransform.GetLocation() - ObserverLocation;
		DirectionToInstance.Normalize();

		double CosDeltaAngle = FVector::DotProduct(LookupDirection, DirectionToInstance);
		if (CosDeltaAngle > CosThresholdAngle && CosDeltaAngle > MinCos)
		{
			// Inside the cone angle, and closer to latest
			MinCos = CosDeltaAngle;
			FoundStarInfo = StarInfo;
			StarTransform = WorldTransform;
			StarFound = true;
		}
	}

	return StarFound;
}

bool ACelestialVaultDaySequenceActor::GetClosestPlanetaryBody(FVector ObserverLocation, FVector LookupDirection, double ThresholdAngleDegrees, FPlanetaryBody& FoundPlanetaryBodyInfo, FTransform& BodyTransform)
{
	BodyTransform = FTransform::Identity;
	
	double CosThresholdAngle = FMath::Cos(FMath::DegreesToRadians(ThresholdAngleDegrees));
	LookupDirection.Normalize();
	double MinCos = -1.0;
	bool BodyFound = false;
	
	// Check for the moon first
	if (MoonDiscComponent)
	{
		// We need to use the ISM component and query the world transform because the celestial vault has rotated
		FVector MoonLocation = MoonDiscComponent->GetComponentLocation();
		FVector DirectionToMoon = MoonLocation - ObserverLocation;
		DirectionToMoon.Normalize();

		double CosDeltaAngle = FVector::DotProduct(LookupDirection, DirectionToMoon);
		if (CosDeltaAngle > CosThresholdAngle && CosDeltaAngle > MinCos)
		{
			// Inside the cone angle. At least we have found the moon. 
			MinCos = CosDeltaAngle;
			BodyTransform = MoonDiscComponent->GetComponentTransform();

			// Compute the moon info for the exact moment in time
			FDateTime LocalTime = GetDateAndTime();
			FDateTime UTCTime = UCelestialMaths::LocalTimeToUTCTime(LocalTime, GMT_TimeZone, IsDaylightSavingsNow());
			double JulianDate = UCelestialMaths::UTCDateTimeToJulianDate(UTCTime);
			FoundPlanetaryBodyInfo = ComputeMoonInfo(JulianDate);
			BodyFound = true;
		}
	}

	// Check for planets now
	if (PlanetsComponent)
	{
		// We query only if we generated the Planet Info  
		if (bKeepPlanetsInfos)
		{
			FTransform PlanetComponentTransform = PlanetsComponent->GetComponentTransform();
			for (FPlanetaryBody BodyInfo : Planets)
			{
				// We need to compute the world Transform for the body, because the celestial vault has rotated and the Body UE Transform is local to the Sky. 
				FTransform WorldTransform = BodyInfo.UELocalTransform * PlanetComponentTransform;

				FVector DirectionToInstance = WorldTransform.GetLocation() - ObserverLocation;
				DirectionToInstance.Normalize();

				double CosDeltaAngle = FVector::DotProduct(LookupDirection, DirectionToInstance);
				if (CosDeltaAngle > CosThresholdAngle && CosDeltaAngle > MinCos)
				{
					// Inside the cone angle, and closer to latest found body
					MinCos = CosDeltaAngle;
					FoundPlanetaryBodyInfo = BodyInfo;
					BodyTransform = WorldTransform;
					BodyFound = true;
				}
			}
		}
		else
		{
			
		}
		
	}

	return BodyFound;
}

bool ACelestialVaultDaySequenceActor::GetPlanetaryBodyByVSOP87Type(EVSOP87BodyType VSOP87Type, FPlanetaryBody& FoundPlanetaryBodyInfo, FTransform& BodyTransform)
{
	BodyTransform = FTransform::Identity;
		
	if (VSOP87Type == EVSOP87BodyType::Moon && MoonDiscComponent) 
	{
		BodyTransform = MoonDiscComponent->GetComponentTransform();

		// Compute the moon info for the exact moment in time
		FDateTime LocalTime = GetDateAndTime();
		FDateTime UTCTime = UCelestialMaths::LocalTimeToUTCTime(LocalTime, GMT_TimeZone, IsDaylightSavingsNow());
		double JulianDate = UCelestialMaths::UTCDateTimeToJulianDate(UTCTime);
		FoundPlanetaryBodyInfo = ComputeMoonInfo(JulianDate);
		
		return true;
	}
	else
	{
		// We query only if we generated the Planets Info  
		if (bKeepPlanetsInfos)
		{
			for (FPlanetaryBody PlanetaryBodyInfo : Planets)
			{
				if (PlanetaryBodyInfo.VSOP87BodyType == VSOP87Type)
				{
					FoundPlanetaryBodyInfo = PlanetaryBodyInfo;
					FTransform PlanetComponentTransform = PlanetsComponent->GetComponentTransform();
					BodyTransform = FoundPlanetaryBodyInfo.UELocalTransform * PlanetComponentTransform;
					return true;
				}
			}
		}
	}
	return false;
}

double ACelestialVaultDaySequenceActor::GetCelestialVaultAngle(double TimeOfDay) const
{
	// In order to animate the Celestial vault rotation, we'll use the GMST. But if we modulate the GMST angle, we'll have the 360->0 problem.
	// We want a value without that problem, but we can't trick using a monotonically increasing value, because when DST changes, it's like if the GMST was going backwards at the DST switch hour. 
	// Therefore, we keep trace of and "absolute/unwrapped" GMST0, and substract it from the GMST "absolute/unwrapped".
	// In order to still have the proper GMST rotation at the beginning, we modulate it as a start reference.  
	FDateTime LocalTime = GetDate() + FTimespan::FromHours(TimeOfDay);
	FDateTime UTCTime = UCelestialMaths::LocalTimeToUTCTime(LocalTime, GMT_TimeZone, IsDaylightSavingsAtTimeOfDay(TimeOfDay));
	return UCelestialMaths::ModPositive(GMST0_Unwrapped, 360) + UCelestialMaths::DateTimeToGreenwichMeanSiderealTimeUnwrapped(UTCTime) - GMST0_Unwrapped;
}

#if WITH_EDITOR

void ACelestialVaultDaySequenceActor::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	//Get the name of the property that was changed  
	FName PropertyName = (PropertyChangedEvent.Property != nullptr) ? PropertyChangedEvent.Property->GetFName() : NAME_None;

	bool bBuildAll = false;
	bool bBuildStars = false;
	bool bBuildPlanetaryBodies = false;
	bool bRebuildSequence = false;

	// Geometry properties
	if (PropertyName == GET_MEMBER_NAME_CHECKED(ACelestialVaultDaySequenceActor, CelestialVaultDistance) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(ACelestialVaultDaySequenceActor, StarsVaultPercentage) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(ACelestialVaultDaySequenceActor, PlanetsVaultPercentage) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(ACelestialVaultDaySequenceActor, MoonVaultPercentage))
	{
		if (DeepSkyComponent)
		{
			// Sphere mesh 1 meter so 100 units, so to have km, we just have to multiply by 1000. 
			DeepSkyComponent->SetRelativeScale3D(FVector(CelestialVaultDistance * 1000.0 ));	
		}
		
		// TODO UpdateGeoReferencing();

		bBuildAll |= true;
		bRebuildSequence = true;
	}

	// North Offset
	if (PropertyName == GET_MEMBER_NAME_CHECKED(ACelestialVaultDaySequenceActor, NorthOffset))
	{
		if (NorthOffsetComponent)
		{
			NorthOffsetComponent->SetRelativeRotation(FRotator(0, NorthOffset, 0));
		}
	}
		
	
	// Time properties
	if (PropertyName == GET_MEMBER_NAME_CHECKED(ACelestialVaultDaySequenceActor, bUseCurrentDate) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(ACelestialVaultDaySequenceActor, Year) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(ACelestialVaultDaySequenceActor, Month) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(ACelestialVaultDaySequenceActor, Day) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(ACelestialVaultDaySequenceActor, GMT_TimeZone) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(ACelestialVaultDaySequenceActor, Latitude) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(ACelestialVaultDaySequenceActor, Longitude ||
		PropertyName == GET_MEMBER_NAME_CHECKED(ACelestialVaultDaySequenceActor, DaylightSavingsMode) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(ACelestialVaultDaySequenceActor, DaylightSavingsStart) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(ACelestialVaultDaySequenceActor, DaylightSavingsEnd) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(ACelestialVaultDaySequenceActor, DaylightSavingsSwitchHour) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(ACelestialVaultDaySequenceActor, bLevelIsGeocentric)))
	{
		// DateTime has changed, we need to update the Reference CelestialVault Angle.
		FDateTime LocalTime0 = GetDate();
		FDateTime UTCTime0 = UCelestialMaths::LocalTimeToUTCTime(LocalTime0, GMT_TimeZone, IsDaylightSavingsAtTimeOfDay(0));
		GMST0_Unwrapped = UCelestialMaths::DateTimeToGreenwichMeanSiderealTimeUnwrapped(UTCTime0);
		
		CelestialVaultComponent->SetRelativeRotation( FRotator(0.0, UCelestialMaths::ModPositive(GMST0_Unwrapped, 360), 0.0) );
		TopocentricSiderealComponent->SetRelativeRotation( FRotator(0.0, UCelestialMaths::ModPositive(GMST0_Unwrapped, 360), 0.0) );

		UpdatePlanetCenter();
		
		bBuildAll = true;
		bRebuildSequence = true;
	}

	// Properties impacting the Stars
	if (PropertyName == GET_MEMBER_NAME_CHECKED(ACelestialVaultDaySequenceActor, CelestialStarCatalog) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(ACelestialVaultDaySequenceActor, FictionalStarCatalog) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(ACelestialVaultDaySequenceActor, MaxVisibleMagnitude) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(ACelestialVaultDaySequenceActor, bKeepStarsInfo))
	{
		bBuildStars |= true;
		bRebuildSequence = true;
	}

	// Properties impacting the Planetary Bodies
	if (PropertyName == GET_MEMBER_NAME_CHECKED(ACelestialVaultDaySequenceActor, PlanetsCatalog) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(ACelestialVaultDaySequenceActor, PlanetsScale) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(ACelestialVaultDaySequenceActor, bKeepPlanetsInfos) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(ACelestialVaultDaySequenceActor, MoonScale)||
		PropertyName == GET_MEMBER_NAME_CHECKED(ACelestialVaultDaySequenceActor, bManualControl)||
		PropertyName == GET_MEMBER_NAME_CHECKED(ACelestialVaultDaySequenceActor, MoonAge) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(ACelestialVaultDaySequenceActor, MoonLightIntensity) || 
		PropertyName == GET_MEMBER_NAME_CHECKED(ACelestialVaultDaySequenceActor, SunLightIntensity) || 
		PropertyName == GET_MEMBER_NAME_CHECKED(ACelestialVaultDaySequenceActor, MoonOffset_RA) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(ACelestialVaultDaySequenceActor, MoonOffset_DEC))
	{
		bBuildPlanetaryBodies |= true;
		bRebuildSequence = true;
	}
	
	if (bBuildAll || bBuildStars)
	{
		InitStarsComponent();
	}

	if (bBuildAll || bBuildPlanetaryBodies)
	{
		InitPlanetsComponent();
	}

	if (bRebuildSequence)
	{
		ResetBodyMotionCache();
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void ACelestialVaultDaySequenceActor::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	Super::PostEditChangeChainProperty(PropertyChangedEvent);

	// Check the Validity of Daylight Savings Dates
	if (FProperty* MemberProperty = PropertyChangedEvent.PropertyChain.GetHead()->GetValue()) // The Struct property that has changed is the Head of the Chain
	{
		// Is it a FDaylightSavingsRule property that changed? 
		FStructProperty* StructProperty = CastField<FStructProperty>(MemberProperty);
		if (StructProperty && StructProperty->Struct == FDaylightSavingsRule::StaticStruct())
		{
			// Now, check which property of this DaylightSavingsRule has changed
			if (FProperty* InnerProperty = PropertyChangedEvent.PropertyChain.GetTail()->GetValue())
			{
				const FName PropName = InnerProperty->GetFName();

				// Validate only if Fixed-Day related properties changed
				if (PropName == GET_MEMBER_NAME_CHECKED(FDaylightSavingsRule, DayOfMonth) ||
					PropName == GET_MEMBER_NAME_CHECKED(FDaylightSavingsRule, Month))
				{
					if (FDaylightSavingsRule* DaylightSavingsRule = StructProperty->ContainerPtrToValuePtr<FDaylightSavingsRule>(this))
					{
						DaylightSavingsRule->ClampToValidDate(Year);
					}
				}
			}
		}
	}
}
#endif

//////// Private Functions

void ACelestialVaultDaySequenceActor::InitStarsComponent() 
{
	TRACE_CPUPROFILER_EVENT_SCOPE(InitStarsComponent);
	
	if (StarsComponent)
	{
		StarsComponent->ClearInstances();
		StarsComponent->SetNumCustomDataFloats(4);
		Stars.Empty();
		
		// Check for the Catalog
		if (!CelestialStarCatalog && !FictionalStarCatalog)
		{
			UE_LOGF(LogCelestialVault, Warning, "Please define at least a Celestial or Fictional StarCatalog");
			return;
		}

		if (CelestialStarCatalog)
		{
			// Check for the Catalog Data types
			const UScriptStruct* CelestialStarCatalogRowStruct = CelestialStarCatalog->GetRowStruct();
			if (CelestialStarCatalogRowStruct != FCelestialStarInputData::StaticStruct())
			{
				UE_LOGF(LogCelestialVault, Warning, "Invalid DataTable row structure for CelestialStarCatalog! It should be of type %ls", *FCelestialStarInputData::StaticStruct()->GetName());
			}
			else
			{
				FDateTime LocalTime = GetDateAndTime();
				FDateTime UTCTime = UCelestialMaths::LocalTimeToUTCTime(LocalTime, GMT_TimeZone, IsDaylightSavingsNow());
				double JulianDate = UCelestialMaths::UTCDateTimeToJulianDate(UTCTime);
				FPlanetaryBodyKinematicState EarthKinematicState = UCelestialMaths::GetPlanetaryBodyKinematicState_AU(JulianDate, EVSOP87BodyType::Earth);
				// Compute the Precession and Nutation Matrices - The formulas expect a Date in Terrestrial Time
				double TAISeconds = UCelestialMaths::JulianDateToInternationalAtomicTime(JulianDate);
				double TTSeconds = UCelestialMaths::InternationalAtomicTimeToTerrestrialTime(TAISeconds);
				double JulianDateTT = UCelestialMaths::SecondsToDay(TTSeconds);
				FMatrix PrecessionMatrix = UCelestialMaths::GetPrecessionMatrix(JulianDateTT);
				FMatrix NutationMatrix = UCelestialMaths::GetNutationMatrix(JulianDateTT);
				
				// Generate Celestial Stars
				CelestialStarCatalog->ForeachRow<FCelestialStarInputData>("AEarthDaySequenceActor::InitStarsComponent", [&](const FName& Key, const FCelestialStarInputData& CelestialStarInputData)
					{
						if (CelestialStarInputData.Magnitude <= MaxVisibleMagnitude)
						{
							// Compute the RA and DEC of Date
							FVector StarLocation_TopoCentric_FK5J2000_AU = UCelestialMaths::RADECToXYZ_RH(CelestialStarInputData.RA * 15.0, CelestialStarInputData.DEC, CelestialStarInputData.DistanceInPC);
							FVector StarLocation_TopoCentric_FK5J2000_Aberrated_AU = UCelestialMaths::ComputeAberration(StarLocation_TopoCentric_FK5J2000_AU, EarthKinematicState.Velocity_FK5J2000);
							// Transform the Body Topographic location with the Precession and Nutation Matrices at Date. 
							FVector StarLocation_TopoCentric_FK5J2000_Precessed_AU = PrecessionMatrix.InverseTransformVector(StarLocation_TopoCentric_FK5J2000_Aberrated_AU);
							FVector StarLocation_TopoCentric_FK5J2000_OfDate_AU = NutationMatrix.InverseTransformVector(StarLocation_TopoCentric_FK5J2000_Precessed_AU);
							double RADegrees = 0.0;
							double DECDegrees = 0.0;
							double DistanceStarToEarthAU = 0.0;
							UCelestialMaths::XYZToRADEC_RH(StarLocation_TopoCentric_FK5J2000_OfDate_AU, RADegrees, DECDegrees, DistanceStarToEarthAU);
							double RAHours = RADegrees / 15.0;

							
							// Location - Convert to UE Left-handed Frame (Invert the Y coordinate)
							// Special case for the Stars. To avoid recomputing the ISM bounds, we use the parent bounds, and therefore inherit from the transformation of the Celestial Vault.
							// The mesh being 100 units radius, we have a local transform corresponding to a StarsVaultPercentage distance. 
							FVector StarLocation = UCelestialMaths::RADECToXYZ_RH(RADegrees, DECDegrees, StarsVaultPercentage) * FVector(1.0, -1.0, 1.0);

							// Color
							FLinearColor StarColor = UCelestialMaths::BVtoLinearColor(CelestialStarInputData.ColorIndex);

							// Create ISM instance
							int32 NewIndex = StarsComponent->AddInstance(FTransform(StarLocation), false);
							TArray<float> NewCustomData;
							NewCustomData.Add(StarColor.R);
							NewCustomData.Add(StarColor.G);
							NewCustomData.Add(StarColor.B);
							NewCustomData.Add(CelestialStarInputData.Magnitude);
							StarsComponent->SetCustomData(NewIndex, NewCustomData);

							// Keep trace of the Star Information for further runtime queries					
							if (bKeepStarsInfo)
							{
								FStellarBody StarInfo;
								StarInfo.RAJ2000 = CelestialStarInputData.RA;
								StarInfo.DECJ2000 = CelestialStarInputData.DEC;
								StarInfo.RA = RAHours;
								StarInfo.DEC = DECDegrees;
								// Stellar topocentric parallax is physically negligible (~1e-13") — topocentric == geocentric for stars.
								StarInfo.RAGeocentric = RAHours;
								StarInfo.DECGeocentric = DECDegrees;
								StarInfo.DistanceInAU = UCelestialMaths::ParsecsToAstronomicalUnits(CelestialStarInputData.DistanceInPC);
								StarInfo.Name = CelestialStarInputData.Name;
								StarInfo.Magnitude = CelestialStarInputData.Magnitude;
								StarInfo.Color = StarColor;
								StarInfo.HipparcosID = CelestialStarInputData.HipparcosID;
								StarInfo.HenryDraperID = CelestialStarInputData.HenryDraperID;
								StarInfo.YaleBrightStarID = CelestialStarInputData.YaleBrightStarID;
								StarInfo.ColorIndex = CelestialStarInputData.ColorIndex;

								// Don't compute the UE Local Transform but use the one we use for the ISM component. 
								StarInfo.UELocalTransform = FTransform(StarLocation);
								// Maybe add other computed values here... 
								Stars.Add(StarInfo);
							}
						}
					}
				);
			}
		}

		if (FictionalStarCatalog)
		{
			const UScriptStruct* FictionalStarCatalogRowStruct = FictionalStarCatalog->GetRowStruct();
			if (FictionalStarCatalogRowStruct != FStarInputData::StaticStruct())
			{
				UE_LOGF(LogCelestialVault, Warning, "Invalid DataTable row structure for FictionalStarCatalog! It should be of type %ls", *FStarInputData::StaticStruct()->GetName() );
			}
			else
			{
				// Generate Fictional Stars
				FictionalStarCatalog->ForeachRow<FStarInputData>("AEarthDaySequenceActor::InitStars / Fictional", [&](const FName& Key, const FStarInputData& StarInputData)
					{
						if (StarInputData.Magnitude <= MaxVisibleMagnitude)
						{
							// Location - Convert to UE Left-handed Frame (Invert the Y coordinate)
							FVector StarLocation = UCelestialMaths::RADECToXYZ_RH(StarInputData.RA * 15.0, StarInputData.DEC, StarsVaultPercentage) * FVector(1.0, -1.0, 1.0);

							// Color
							FLinearColor StarColor = StarInputData.Color;

							// Create ISM instance
							int32 NewIndex = StarsComponent->AddInstance(FTransform(StarLocation), false);
							TArray<float> NewCustomData;
							NewCustomData.Add(StarColor.R);
							NewCustomData.Add(StarColor.G);
							NewCustomData.Add(StarColor.B);
							NewCustomData.Add(StarInputData.Magnitude);
							StarsComponent->SetCustomData(NewIndex, NewCustomData);

							// Keep trace of the Star Information for further runtime queries					
							if (bKeepStarsInfo)
							{
								FStellarBody StarInfo;
								StarInfo.RA = StarInputData.RA;
								StarInfo.DEC = StarInputData.DEC;
								// Stellar topocentric parallax is physically negligible (~1e-13") — topocentric == geocentric for stars.
								StarInfo.RAGeocentric = StarInputData.RA;
								StarInfo.DECGeocentric = StarInputData.DEC;
								StarInfo.DistanceInAU = UCelestialMaths::ParsecsToAstronomicalUnits(StarInputData.DistanceInPC);
								StarInfo.Name = StarInputData.Name;
								StarInfo.Magnitude = StarInputData.Magnitude;
								StarInfo.Color = StarColor;
								// Don't compute the UE Local Transform but use the one we use for the ISM component. 
								StarInfo.UELocalTransform = FTransform(StarLocation);
								Stars.Add(StarInfo);
							}
						}
					}
				);
			}
		}
		
		UE_LOGF(LogCelestialVault, Verbose, "%d Stars added ", StarsComponent->GetInstanceCount());
		StarsComponent->MarkRenderInstancesDirty();
	}		
}

void ACelestialVaultDaySequenceActor::InitPlanetsComponent()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(InitPlanetsComponent);
	
	// Get the proper Julian day, at Midnight... The Daysequence will rotate the sky vault later 
	FDateTime LocalTime0 = GetDate();
	FDateTime UTCTime0 = UCelestialMaths::LocalTimeToUTCTime(LocalTime0, GMT_TimeZone, IsDaylightSavingsNow());
	double JulianDay = UCelestialMaths::UTCDateTimeToJulianDate(UTCTime0);

	// Init the Planets from the catalog
	if (PlanetsComponent)
	{
		PlanetsComponent->ClearInstances();
		PlanetsComponent->SetNumCustomDataFloats(2);
		Planets.Empty();

		// Safety check on Catalog Data
		if (!PlanetsCatalog)
		{
			UE_LOGF(LogCelestialVault, Warning, "PlanetaryBodiesCatalog is null!");
			return;
		}

		const UScriptStruct* RowStruct = PlanetsCatalog->GetRowStruct();
		if (RowStruct != FPlanetaryBodyInputData::StaticStruct() )
		{
			UE_LOGF(LogCelestialVault, Warning, "Invalid DataTable row structure for the Planetary Bodies Catalog! It should be of type %ls", *FPlanetaryBodyInputData::StaticStruct()->GetName() );
			return;
		}

		// Compute Earth Reference location
		FPlanetaryBodyKinematicState EarthKinematicState = UCelestialMaths::GetPlanetaryBodyKinematicState_AU(JulianDay, EVSOP87BodyType::Earth);

		
		// Init from the DataTable
		PlanetsCatalog->ForeachRow<FPlanetaryBodyInputData>("AEarthDaySequenceActor::InitPlanetsComponent", [&](const FName& Key, const FPlanetaryBodyInputData& InputPlanetaryBody)
			{
				// Compute the Body location relative to the Earth reference state for the JuliandDay. 
				FPlanetaryBody BodyInfo = GetPlanetaryBodyInfo_UsingKnownState(JulianDay, InputPlanetaryBody, EarthKinematicState);
				
				// Compute the UE Transform tp locate the Body on the Sphere
				double UEBodyDistance = CelestialVaultDistance * UE_ONE_KILOMETER * PlanetsVaultPercentage / 100.0;
				BodyInfo.ComputeUELocalTransform(PlanetsScale, UEBodyDistance);
			
				// Add the new ISM Instance
				int32 NewIndex = PlanetsComponent->AddInstance(BodyInfo.UELocalTransform);
			
				// Add the Custom data (column index to sample the planets atlas texture)
				TArray<float> NewCustomData;
				NewCustomData.Add(InputPlanetaryBody.TextureColumnIndex);
				NewCustomData.Add(BodyInfo.Magnitude);
				PlanetsComponent->SetCustomData(NewIndex, NewCustomData);

				// Keep trace of the PlanetaryBody for further queries (the Datatable is readonly, so store in another object)
				if (bKeepPlanetsInfos)
				{
					Planets.Add(BodyInfo);	
				}
			}
		);
		PlanetsComponent->MarkRenderInstancesDirty();
	}
}

FPlanetaryBody ACelestialVaultDaySequenceActor::GetPlanetaryBodyInfo_UsingKnownState(double JulianDay, const FPlanetaryBodyInputData& InputPlanetaryBody, const FPlanetaryBodyKinematicState& EarthKinematicState, bool bIgnoreRelativisticEffect) const
{
	FPlanetaryBody BodyInfo;
	BodyInfo.VSOP87BodyType = InputPlanetaryBody.VSOP87BodyType;
	BodyInfo.OrbitType = InputPlanetaryBody.OrbitType;
	BodyInfo.Name = InputPlanetaryBody.Name;
	BodyInfo.Radius = InputPlanetaryBody.Radius;

	// Compute location
	double RAJ2000Hours, DECJ2000Degrees, RAGeocentricHours, DECGeocentricDegrees, RAHours, DECDegrees, DistanceToEarthAU, DistanceToSunAU, DistanceEarthToSunAU;

	UCelestialMaths::GetBodyCelestialCoordinatesAU_UsingKnownState(JulianDay, InputPlanetaryBody.VSOP87BodyType, EarthKinematicState, Latitude, Longitude, bLevelIsGeocentric, bIgnoreRelativisticEffect, RAJ2000Hours, DECJ2000Degrees, RAHours, DECDegrees, RAGeocentricHours, DECGeocentricDegrees, DistanceToEarthAU, DistanceToSunAU, DistanceEarthToSunAU);
	BodyInfo.RAJ2000 = RAJ2000Hours;
	BodyInfo.DECJ2000 = DECJ2000Degrees;
	BodyInfo.RA = RAHours;
	BodyInfo.DEC = DECDegrees;
	BodyInfo.RAGeocentric = RAGeocentricHours;
	BodyInfo.DECGeocentric = DECGeocentricDegrees;
	BodyInfo.DistanceInAU = DistanceToEarthAU;

	// Compute Magnitude
	double Phase; // TODO - Check the PhaseAngle Computations. 
	BodyInfo.Magnitude = UCelestialMaths::GetPlanetaryBodyMagnitude(InputPlanetaryBody.VSOP87BodyType, DistanceToSunAU, DistanceToEarthAU , DistanceEarthToSunAU, Phase);
	BodyInfo.Age = Phase;
	
	// Compute the True and the Scaled apparent Diameters
	BodyInfo.ApparentDiameterDegrees = FMath::RadiansToDegrees(FMath::Atan2(InputPlanetaryBody.Radius * 1000.0, UCelestialMaths::AstronomicalUnitsToMeters(DistanceToEarthAU)) * 2.0);
	return BodyInfo;
}

void ACelestialVaultDaySequenceActor::UpdatePlanetCenter()
{
	// Get the transformation in proper UE Frame units, and set it to the Component 
	PlanetCenterTransform = UCelestialMaths::GetEarthCenterTransformUEFrame(Latitude, Longitude, 0, bLevelIsGeocentric);
	PlanetCenterComponent->SetRelativeTransform(PlanetCenterTransform);
	// Mirror PlanetCenter's rotation onto the observer-centered vault so at-infinity content (Stars, DeepSky) shares the same ENU basis but with zero translation.
	TopocentricVaultComponent->SetWorldRotation(PlanetCenterComponent->GetComponentTransform().GetRotation());

	// Adjust the SkyAtmosphere to the new radius

	if (bLevelIsGeocentric)
	{
		SkyAtmosphereComponent->SetBottomRadius(6378.137f); // Reset to max Earth ellipsoidal radius. 
	}
	else
	{
		double Radius = PlanetCenterTransform.GetLocation().Length();
		SkyAtmosphereComponent->SetBottomRadius(Radius / UE_ONE_KILOMETER);
	}
}

void ACelestialVaultDaySequenceActor::RebuildAll()
{
	if (const UWorld* World = GetWorld())
	{
		// Make sure the actor is properly located at the Origin
		SetActorTransform(FTransform::Identity);
		
		// DateTime has changed, we need to update the Reference CelestialVault Angle.
		FDateTime LocalTime0 = GetDate();
		FDateTime UTCTime0 = UCelestialMaths::LocalTimeToUTCTime(LocalTime0, GMT_TimeZone, IsDaylightSavingsAtTimeOfDay(0));
		GMST0_Unwrapped = UCelestialMaths::DateTimeToGreenwichMeanSiderealTimeUnwrapped(UTCTime0);
		
		CelestialVaultComponent->SetRelativeRotation( FRotator(0.0, UCelestialMaths::ModPositive(GMST0_Unwrapped, 360.0), 0.0) );
		TopocentricSiderealComponent->SetRelativeRotation( FRotator(0.0, UCelestialMaths::ModPositive(GMST0_Unwrapped, 360.0), 0.0) );

		UpdatePlanetCenter();

		// Rebuild Sky and Sequence
		InitStarsComponent();
		InitPlanetsComponent();
		ResetBodyMotionCache();
		UpdateBodiesMotion();
	}
}

void ACelestialVaultDaySequenceActor::UpdateBodiesMotion()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UpdateCelestialBodiesMotion);
	
	double TimeOfDay = GetTimeOfDay();
	TimeOfDay = FMath::Fmod(TimeOfDay, 24.0); // After incrementing the number of loops, the Sequence player returns a TOD that could be 24. Make a FMod to make sure we are at the beginning of the next day  
		
	EnsureKeysAroundTimeOfDay(TimeOfDay); 
	
	FTransform GeocentricTransform = CelestialVaultComponent->GetComponentTransform();
	
	FVector MoonDiscLocation = MoonDiscLocationCurve.Eval(TimeOfDay);
	if (MoonDiscComponent)
	{
		MoonDiscComponent->SetRelativeLocation(MoonDiscLocation);
		MoonDiscComponent->SetRelativeRotation(MoonDiscRotationCurve.Eval(TimeOfDay));
		SetMoonDiscAge(MoonAgeCurve.Eval(TimeOfDay));
	}
	
	if ( MoonLightComponent)
	{
		// Update World direction based on Moon and Observer Locations
		FVector MoonWorldLocation = GeocentricTransform.TransformPosition( MoonDiscLocation);
		FVector MoonDirection = MoonWorldLocation - LastViewportCameraLocation;
		MoonLightComponent->SetWorldRotation(FQuat::FindBetweenVectors(FVector::ForwardVector, -MoonDirection));
	}

	if ( SunLightComponent)
	{
		// Update World direction based on Sun and Observer Locations
		FVector SunLocation = SunLocationCurve.Eval(TimeOfDay);
		FVector SunWorldLocation = GeocentricTransform.TransformPosition(SunLocation);
		FVector SunDirection = SunWorldLocation - LastViewportCameraLocation;
		SunLightComponent->SetWorldRotation(FQuat::FindBetweenVectors(FVector::ForwardVector, -SunDirection));
		
		SunLightComponent->SetIntensity(SunLightIntensity);
	}

	// Don't use a curve cache for the Celestial Vault rotation. It's a basic angular rotation.
	double VaultYaw = GetCelestialVaultAngle(TimeOfDay );
	CelestialVaultComponent->SetRelativeRotation(FRotator(0.0, VaultYaw, 0.0));
	TopocentricSiderealComponent->SetRelativeRotation(FRotator(0.0, VaultYaw, 0.0));
}

/// Bodies Motion Cache
void ACelestialVaultDaySequenceActor::ResetBodyMotionCache()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(BuildCurvesCache);
	UE_LOGF(LogCelestialVault, Verbose, "Reset Body Motion Cache for Day %d", SequenceLoopCount);

	MoonAgeCurve.Reset();
	MoonDiscLocationCurve.Reset();
	MoonDiscRotationCurve.Reset();
	SunLocationCurve.Reset();

	BodyMotionCacheMax = 0.0;
	BodyMotionCacheMin = 24.0;
}

void ACelestialVaultDaySequenceActor::EnsureKeysAroundTimeOfDay(double TimeOfDay)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(EnsureKeysAroundTimeOfDay);
	
	// The Cache will be split into even Time Step intervals, using BodyMotionCacheKeyFramesCount
	const double TimeStep = 24.0 / FMath::Max(static_cast<signed>(BodyMotionCacheKeyFramesCount), 1);

	double LeftKeyValue = FMath::Floor(TimeOfDay / TimeStep) * TimeStep;
	double RightKeyValue = LeftKeyValue + TimeStep;

	// Nothing in cache, add two points
	if (BodyMotionCacheMin == 24 && BodyMotionCacheMax == 0)
	{
		// First query, we don't have KeyFrames yet, so add them
		AddBodyMotionCacheKey(LeftKeyValue);
		AddBodyMotionCacheKey(RightKeyValue);
	}

	// If needed, Extend left: add ALL intermediate keys at constant Step (this allows for arbitrary jumps in time of day)
	while (LeftKeyValue < BodyMotionCacheMin)
	{
		AddBodyMotionCacheKey(BodyMotionCacheMin - TimeStep);
	}

	// Same extention to the Right. 
	while (RightKeyValue > BodyMotionCacheMax)
	{
		AddBodyMotionCacheKey(BodyMotionCacheMax + TimeStep);
	}
}

void ACelestialVaultDaySequenceActor::AddBodyMotionCacheKey(double TimeOfDay)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(AddBodyMotionCacheKey);
	UE_LOGF(LogCelestialVault, Verbose, "Adding new Keyframe in the BodyMotion Cache, for TOD = %f", TimeOfDay);
	
	// We need the Julian Date for the celestial computation. For this, we need to consider the initial day, the number of times the sequence has looped, and the current time of day.
	// Compute the Local time
	FDateTime LocalTime = GetDate() + FTimespan::FromHours(TimeOfDay);
	// Convert DateTime to Julian Date 
	FDateTime UTCTime = UCelestialMaths::LocalTimeToUTCTime(LocalTime, GMT_TimeZone, IsDaylightSavingsAtTimeOfDay(TimeOfDay));
	double JulianDate = UCelestialMaths::UTCDateTimeToJulianDate(UTCTime);

	// Compute Earth Reference location - We'll use it as a cache for Moon/Sun faster computations
	FPlanetaryBodyKinematicState EarthKinematicState = UCelestialMaths::GetPlanetaryBodyKinematicState_AU(JulianDate, EVSOP87BodyType::Earth);
		
	// Get Moon data
	FPlanetaryBody  MoonInfo = ComputeMoonInfo_UsingKnownState(JulianDate, EarthKinematicState);
	MoonAgeCurve.AddPoint(TimeOfDay, MoonInfo.Age);
	MoonDiscLocationCurve.AddPoint(TimeOfDay, MoonInfo.UELocalTransform.GetLocation());
	MoonDiscRotationCurve.AddPoint(TimeOfDay, MoonInfo.UELocalTransform.GetRotation());
	// Scale won't change over time but at least make sure we apply it once
	if (MoonDiscComponent)
	{
		MoonDiscComponent->SetRelativeScale3D(MoonInfo.UELocalTransform.GetScale3D());	
	}
	
	// Get Sun Data
	FStellarBody SunInfo = ComputeSunInfo_UsingKnownState(JulianDate, EarthKinematicState);
	SunLocationCurve.AddPoint(TimeOfDay, SunInfo.UELocalTransform.GetLocation());

	// Update our known interval cache. 
	BodyMotionCacheMax = FMath::Max(TimeOfDay, BodyMotionCacheMax);
	BodyMotionCacheMin = FMath::Min(TimeOfDay, BodyMotionCacheMin);
}

