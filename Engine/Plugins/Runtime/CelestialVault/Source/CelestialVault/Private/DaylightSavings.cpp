// Copyright Epic Games, Inc. All Rights Reserved.



#include "DaylightSavings.h"

FDateTime UDaylightSavings::ToDate(const FDaylightSavingsRule& DaylightSavingsRule, int Year)
{
	FDateTime Result;

	switch (DaylightSavingsRule.Rule)
	{
	case EDaylightSavingRuleKind::NthDay:
		{
			int DayInMonth = NthWeekdayInMonth(Year, DaylightSavingsRule.Month, DaylightSavingsRule.WeekDay, DaylightSavingsRule.Nth);
			if (DayInMonth == 0 )
			{
				Result = FDateTime::MinValue();
			}
			else
			{
				Result = FDateTime(Year,FMath::Clamp(static_cast<int>(DaylightSavingsRule.Month), 1, 12), DayInMonth );	
			}
		}
		break;
	case EDaylightSavingRuleKind::LastDay:
		Result = FDateTime(Year, FMath::Clamp(static_cast<int>(DaylightSavingsRule.Month), 1, 12), LastWeekdayInMonth(Year, DaylightSavingsRule.Month, DaylightSavingsRule.WeekDay)); 
		break;
	case EDaylightSavingRuleKind::FixedDate:
		Result = FDateTime(Year, FMath::Clamp(static_cast<int>(DaylightSavingsRule.Month), 1, 12), FMath::Clamp(DaylightSavingsRule.DayOfMonth, 1 , FDateTime::DaysInMonth(Year, FMath::Clamp(static_cast<int>(DaylightSavingsRule.Month), 1, 12))));
		break;
	}

	return Result;
}

FString UDaylightSavings::ToString(const FDaylightSavingsRule& DaylightSavingsRule)
{
	FString Result = FString();
	switch (DaylightSavingsRule.Rule)
	{
	case EDaylightSavingRuleKind::NthDay:
		Result = FString::Printf(TEXT("%d%s %s of %s"), DaylightSavingsRule.Nth, *GetOrdinalSuffix(DaylightSavingsRule.Nth), *GetDayString(DaylightSavingsRule.WeekDay), *GetMonthString(DaylightSavingsRule.Month));
		break;
	case EDaylightSavingRuleKind::LastDay:
		Result =  FString::Printf(TEXT("Last %s of %s"), *GetDayString(DaylightSavingsRule.WeekDay), *GetMonthString(DaylightSavingsRule.Month));
		break;
	case EDaylightSavingRuleKind::FixedDate:
		Result =  FString::Printf(TEXT("%d%s of %s"), DaylightSavingsRule.DayOfMonth, *GetOrdinalSuffix(DaylightSavingsRule.DayOfMonth), *GetMonthString(DaylightSavingsRule.Month));
		break;
	}

	return Result;
}

EWeekDay UDaylightSavings::GetWeekDay(FDateTime Date)
{
	// Uses Sakamoto’s algorithm internally, which returns 0=Sunday..6=Saturday.
	static int month_table[12] = { 0, 3, 2, 5, 0, 3, 5, 1, 4, 6, 2, 4 }; 
	int Year = Date.GetYear();
	if (Date.GetMonth() < 3)
	{
		Year -= 1;
	}
	
	int Day = ( Year + Year / 4 - Year / 100 + Year / 400 + month_table[Date.GetMonth() - 1] + Date.GetDay() ) % 7;

	// We want our Week to start on Monday (BP enums needs to start at 0), so Mapping from Sunday-based to Monday-based:
	return static_cast<EWeekDay>((Day+6)%7);
}

int UDaylightSavings::NthWeekdayInMonth(int Year, EMonth Month, EWeekDay Weekday, int Nth)
{
	if (Nth <= 0)
	{
		return 0;
	}
	
	EWeekDay FirstWeekDayOfMonth = GetWeekDay(FDateTime(Year,FMath::Clamp(static_cast<int>(Month), 1, 12), 1));
	int DayOffset = (static_cast<int>(Weekday) - static_cast<int>(FirstWeekDayOfMonth) + 7) % 7;

	int Day = 1 + DayOffset + 7 * (Nth - 1);
	if (Day > FDateTime::DaysInMonth(Year, FMath::Clamp(static_cast<int>(Month), 1, 12)))
	{
		return 0; // does not exist
	}

	return Day;
}

int UDaylightSavings::LastWeekdayInMonth(int Year, EMonth Month, EWeekDay Weekday)
{
	int LastDay = FDateTime::DaysInMonth(Year, FMath::Clamp(static_cast<int>(Month), 1, 12)); 
	EWeekDay LastWeekdayOfMonth = GetWeekDay(FDateTime(Year, FMath::Clamp(static_cast<int>(Month), 1, 12), LastDay));

	int DayOffsetBackwards = (static_cast<int>(LastWeekdayOfMonth) - static_cast<int>(Weekday) + 7) % 7;
	return LastDay - DayOffsetBackwards;
}

FString UDaylightSavings::GetMonthString(EMonth Month)
{
	FString Result = FString();
	if (UEnum* EnumClass = StaticEnum<EMonth>())
	{
		Result = EnumClass->GetNameStringByValue(static_cast<int64>(Month));
	}

	return Result;
}

FString UDaylightSavings::GetDayString(EWeekDay Day)
{
	FString Result = FString();
	if (UEnum* EnumClass = StaticEnum<EWeekDay>())
	{
		Result = EnumClass->GetNameStringByValue(static_cast<int64>(Day));
	}

	return Result;
}

FString UDaylightSavings::GetOrdinalSuffix(int32 Number)
{
	int32 Tens = Number % 100;

	// Exceptions : 11, 12, 13 -> "th"
	if (Tens >= 11 && Tens <= 13)
	{
		return TEXT("th");
	}

	// Otherwise look at the unit
	switch (Number % 10)
	{
	case 1:
		return TEXT("st");
	case 2:
		return TEXT("nd");
	case 3:
		return TEXT("rd");
	default:
		return TEXT("th");
	}
}

bool UDaylightSavings::IsDaylightSavings(FDateTime DateTime, EDaylightSavingsMode Mode, double Latitude, FDaylightSavingsRule StartDay, FDaylightSavingsRule EndDay, int SwitchHour)
{
	if (Mode == EDaylightSavingsMode::None)
	{
		return false;
	}

	int32 EndTimeYear = DateTime.GetYear();

	if (Mode == EDaylightSavingsMode::SouthernHemisphere ||
		(Mode == EDaylightSavingsMode::Automatic && Latitude < 0))
	{
		EndTimeYear++;
	}
	
	FDateTime StartTime = ToDate(StartDay, DateTime.GetYear());
	if (StartTime == FDateTime::MinValue())
	{
		// invalid Date - return false by default
		return false;
	}
	StartTime += FTimespan::FromHours(SwitchHour);


	FDateTime EndTime = ToDate(EndDay,  EndTimeYear);
	if (EndTime == FDateTime::MinValue())
	{
		// invalid Date - return false by default
		return false;
	}
	EndTime += FTimespan::FromHours(SwitchHour);

	return (DateTime >= StartTime && DateTime <= EndTime);
}
