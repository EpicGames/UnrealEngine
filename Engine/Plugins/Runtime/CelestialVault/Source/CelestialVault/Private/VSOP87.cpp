// Copyright Epic Games, Inc. All Rights Reserved.


#include "VSOP87.h"

// Include Coefficients data
#include "VSOP87/VSOP87A_Mercury.h"
#include "VSOP87/VSOP87A_Venus.h"
#include "VSOP87/VSOP87A_Earth.h"
#include "VSOP87/VSOP87A_Emb.h"
#include "VSOP87/VSOP87A_Mars.h"
#include "VSOP87/VSOP87A_Jupiter.h"
#include "VSOP87/VSOP87A_Saturn.h"
#include "VSOP87/VSOP87A_Uranus.h"
#include "VSOP87/VSOP87A_Neptune.h"

// Public
FVector UVSOP87::GetMercuryLocation(double VSOP87Time, EVSOP87Accuracy Accuracy)
{
	// Find the right indices array for the wanted accuracy
	int AccuracyIndex = static_cast<int>(Accuracy);
	const auto& IndexesForAccuracy = VSOP87A_MERCURY_TRUNC_COUNT[AccuracyIndex];

	// The VSOP 87 coefficients are separated by Coordinate, X,Y,Z. Evaluate for each axis. 
	const double X = EvaluateCoordinate(VSOP87A_MERCURY_X, IndexesForAccuracy[0], VSOP87Time);
	const double Y = EvaluateCoordinate(VSOP87A_MERCURY_Y, IndexesForAccuracy[1], VSOP87Time);
	const double Z = EvaluateCoordinate(VSOP87A_MERCURY_Z, IndexesForAccuracy[2], VSOP87Time);

	return FVector(X, Y, Z);
}

FVector UVSOP87::GetMercuryVelocity(double VSOP87Time, EVSOP87Accuracy Accuracy)
{
	// Find the right indices array for the wanted accuracy
	int AccuracyIndex = static_cast<int>(Accuracy);
	const auto& IndexesForAccuracy = VSOP87A_MERCURY_TRUNC_COUNT[AccuracyIndex];

	// The VSOP 87 coefficients are separated by Coordinate, X,Y,Z. Evaluate for each axis. 
	const double X = EvaluateVelocity(VSOP87A_MERCURY_X, IndexesForAccuracy[0], VSOP87Time);
	const double Y = EvaluateVelocity(VSOP87A_MERCURY_Y, IndexesForAccuracy[1], VSOP87Time);
	const double Z = EvaluateVelocity(VSOP87A_MERCURY_Z, IndexesForAccuracy[2], VSOP87Time);

	return FVector(X, Y, Z);
}

FVector UVSOP87::GetVenusLocation(double VSOP87Time, EVSOP87Accuracy Accuracy)
{
	// Find the right indices array for the wanted accuracy
	int AccuracyIndex = static_cast<int>(Accuracy);
	const auto& IndexesForAccuracy = VSOP87A_VENUS_TRUNC_COUNT[AccuracyIndex];

	// The VSOP 87 coefficients are separated by Coordinate, X,Y,Z. Evaluate for each axis. 
	const double X = EvaluateCoordinate(VSOP87A_VENUS_X, IndexesForAccuracy[0], VSOP87Time);
	const double Y = EvaluateCoordinate(VSOP87A_VENUS_Y, IndexesForAccuracy[1], VSOP87Time);
	const double Z = EvaluateCoordinate(VSOP87A_VENUS_Z, IndexesForAccuracy[2], VSOP87Time);

	return FVector(X, Y, Z);
}

FVector UVSOP87::GetVenusVelocity(double VSOP87Time, EVSOP87Accuracy Accuracy)
{
	// Find the right indices array for the wanted accuracy
	int AccuracyIndex = static_cast<int>(Accuracy);
	const auto& IndexesForAccuracy = VSOP87A_VENUS_TRUNC_COUNT[AccuracyIndex];

	// The VSOP 87 coefficients are separated by Coordinate, X,Y,Z. Evaluate for each axis. 
	const double X = EvaluateVelocity(VSOP87A_VENUS_X, IndexesForAccuracy[0], VSOP87Time);
	const double Y = EvaluateVelocity(VSOP87A_VENUS_Y, IndexesForAccuracy[1], VSOP87Time);
	const double Z = EvaluateVelocity(VSOP87A_VENUS_Z, IndexesForAccuracy[2], VSOP87Time);

	return FVector(X, Y, Z);
}

FVector UVSOP87::GetEarthLocation(double VSOP87Time, EVSOP87Accuracy Accuracy)
{
	// Find the right indices array for the wanted accuracy
	int AccuracyIndex = static_cast<int>(Accuracy);
	const auto& IndexesForAccuracy = VSOP87A_EARTH_TRUNC_COUNT[AccuracyIndex];

	// The VSOP 87 coefficients are separated by Coordinate, X,Y,Z. Evaluate for each axis. 
	const double X = EvaluateCoordinate(VSOP87A_EARTH_X, IndexesForAccuracy[0], VSOP87Time);
	const double Y = EvaluateCoordinate(VSOP87A_EARTH_Y, IndexesForAccuracy[1], VSOP87Time);
	const double Z = EvaluateCoordinate(VSOP87A_EARTH_Z, IndexesForAccuracy[2], VSOP87Time);

	return FVector(X, Y, Z);
}

FVector UVSOP87::GetEarthVelocity(double VSOP87Time, EVSOP87Accuracy Accuracy)
{
	// Find the right indices array for the wanted accuracy
	int AccuracyIndex = static_cast<int>(Accuracy);
	const auto& IndexesForAccuracy = VSOP87A_EARTH_TRUNC_COUNT[AccuracyIndex];

	// The VSOP 87 coefficients are separated by Coordinate, X,Y,Z. Evaluate for each axis. 
	const double X = EvaluateVelocity(VSOP87A_EARTH_X, IndexesForAccuracy[0], VSOP87Time);
	const double Y = EvaluateVelocity(VSOP87A_EARTH_Y, IndexesForAccuracy[1], VSOP87Time);
	const double Z = EvaluateVelocity(VSOP87A_EARTH_Z, IndexesForAccuracy[2], VSOP87Time);

	return FVector(X, Y, Z);
}

FVector UVSOP87::GetEMBLocation(double VSOP87Time, EVSOP87Accuracy Accuracy)
{
	// Find the right indices array for the wanted accuracy
	int AccuracyIndex = static_cast<int>(Accuracy);
	const auto& IndexesForAccuracy = VSOP87A_EMB_TRUNC_COUNT[AccuracyIndex];

	// The VSOP 87 coefficients are separated by Coordinate, X,Y,Z. Evaluate for each axis. 
	const double X = EvaluateCoordinate(VSOP87A_EMB_X, IndexesForAccuracy[0], VSOP87Time);
	const double Y = EvaluateCoordinate(VSOP87A_EMB_Y, IndexesForAccuracy[1], VSOP87Time);
	const double Z = EvaluateCoordinate(VSOP87A_EMB_Z, IndexesForAccuracy[2], VSOP87Time);

	return FVector(X, Y, Z);
}

FVector UVSOP87::GetEMBVelocity(double VSOP87Time, EVSOP87Accuracy Accuracy)
{
	// Find the right indices array for the wanted accuracy
	int AccuracyIndex = static_cast<int>(Accuracy);
	const auto& IndexesForAccuracy = VSOP87A_EMB_TRUNC_COUNT[AccuracyIndex];

	// The VSOP 87 coefficients are separated by Coordinate, X,Y,Z. Evaluate for each axis. 
	const double X = EvaluateVelocity(VSOP87A_EMB_X, IndexesForAccuracy[0], VSOP87Time);
	const double Y = EvaluateVelocity(VSOP87A_EMB_Y, IndexesForAccuracy[1], VSOP87Time);
	const double Z = EvaluateVelocity(VSOP87A_EMB_Z, IndexesForAccuracy[2], VSOP87Time);

	return FVector(X, Y, Z);
}

FVector UVSOP87::GetMoonLocation(double VSOP87Time, EVSOP87Accuracy Accuracy)
{
	constexpr double MoonEarthMassRatio = 0.01230073677; 
	FVector EarthLocation = GetEarthLocation(VSOP87Time, Accuracy);
	FVector EMBLocation = GetEMBLocation(VSOP87Time, Accuracy);
	FVector Offset = (EMBLocation - EarthLocation) * (1 + 1 / MoonEarthMassRatio);
	return Offset + EarthLocation;
}

FVector UVSOP87::GetMoonVelocity(double VSOP87Time, EVSOP87Accuracy Accuracy)
{
	constexpr double MoonEarthMassRatio = 0.01230073677; 
	FVector EarthVelocity = GetEarthVelocity(VSOP87Time, Accuracy);
	FVector EMBVelocity = GetEMBVelocity(VSOP87Time, Accuracy);
	FVector Offset = (EMBVelocity - EarthVelocity) * (1 + 1 / MoonEarthMassRatio);
	return Offset + EarthVelocity;
}

FVector UVSOP87::GetMarsLocation(double VSOP87Time, EVSOP87Accuracy Accuracy)
{
	// Find the right indices array for the wanted accuracy
	int AccuracyIndex = static_cast<int>(Accuracy);
	const auto& IndexesForAccuracy = VSOP87A_MARS_TRUNC_COUNT[AccuracyIndex];

	// The VSOP 87 coefficients are separated by Coordinate, X,Y,Z. Evaluate for each axis. 
	const double X = EvaluateCoordinate(VSOP87A_MARS_X, IndexesForAccuracy[0], VSOP87Time);
	const double Y = EvaluateCoordinate(VSOP87A_MARS_Y, IndexesForAccuracy[1], VSOP87Time);
	const double Z = EvaluateCoordinate(VSOP87A_MARS_Z, IndexesForAccuracy[2], VSOP87Time);

	return FVector(X, Y, Z);
}

FVector UVSOP87::GetMarsVelocity(double VSOP87Time, EVSOP87Accuracy Accuracy)
{
	// Find the right indices array for the wanted accuracy
	int AccuracyIndex = static_cast<int>(Accuracy);
	const auto& IndexesForAccuracy = VSOP87A_MARS_TRUNC_COUNT[AccuracyIndex];

	// The VSOP 87 coefficients are separated by Coordinate, X,Y,Z. Evaluate for each axis. 
	const double X = EvaluateVelocity(VSOP87A_MARS_X, IndexesForAccuracy[0], VSOP87Time);
	const double Y = EvaluateVelocity(VSOP87A_MARS_Y, IndexesForAccuracy[1], VSOP87Time);
	const double Z = EvaluateVelocity(VSOP87A_MARS_Z, IndexesForAccuracy[2], VSOP87Time);

	return FVector(X, Y, Z);
}

FVector UVSOP87::GetJupiterLocation(double VSOP87Time, EVSOP87Accuracy Accuracy)
{
	// Find the right indices array for the wanted accuracy
	int AccuracyIndex = static_cast<int>(Accuracy);
	const auto& IndexesForAccuracy = VSOP87A_JUPITER_TRUNC_COUNT[AccuracyIndex];

	// The VSOP 87 coefficients are separated by Coordinate, X,Y,Z. Evaluate for each axis. 
	const double X = EvaluateCoordinate(VSOP87A_JUPITER_X, IndexesForAccuracy[0], VSOP87Time);
	const double Y = EvaluateCoordinate(VSOP87A_JUPITER_Y, IndexesForAccuracy[1], VSOP87Time);
	const double Z = EvaluateCoordinate(VSOP87A_JUPITER_Z, IndexesForAccuracy[2], VSOP87Time);

	return FVector(X, Y, Z);
}

FVector UVSOP87::GetJupiterVelocity(double VSOP87Time, EVSOP87Accuracy Accuracy)
{
	// Find the right indices array for the wanted accuracy
	int AccuracyIndex = static_cast<int>(Accuracy);
	const auto& IndexesForAccuracy = VSOP87A_JUPITER_TRUNC_COUNT[AccuracyIndex];

	// The VSOP 87 coefficients are separated by Coordinate, X,Y,Z. Evaluate for each axis. 
	const double X = EvaluateVelocity(VSOP87A_JUPITER_X, IndexesForAccuracy[0], VSOP87Time);
	const double Y = EvaluateVelocity(VSOP87A_JUPITER_Y, IndexesForAccuracy[1], VSOP87Time);
	const double Z = EvaluateVelocity(VSOP87A_JUPITER_Z, IndexesForAccuracy[2], VSOP87Time);

	return FVector(X, Y, Z);
}

FVector UVSOP87::GetSaturnLocation(double VSOP87Time, EVSOP87Accuracy Accuracy)
{
	// Find the right indices array for the wanted accuracy
	int AccuracyIndex = static_cast<int>(Accuracy);
	const auto& IndexesForAccuracy = VSOP87A_SATURN_TRUNC_COUNT[AccuracyIndex];

	// The VSOP 87 coefficients are separated by Coordinate, X,Y,Z. Evaluate for each axis. 
	const double X = EvaluateCoordinate(VSOP87A_SATURN_X, IndexesForAccuracy[0], VSOP87Time);
	const double Y = EvaluateCoordinate(VSOP87A_SATURN_Y, IndexesForAccuracy[1], VSOP87Time);
	const double Z = EvaluateCoordinate(VSOP87A_SATURN_Z, IndexesForAccuracy[2], VSOP87Time);

	return FVector(X, Y, Z);
}

FVector UVSOP87::GetSaturnVelocity(double VSOP87Time, EVSOP87Accuracy Accuracy)
{
	// Find the right indices array for the wanted accuracy
	int AccuracyIndex = static_cast<int>(Accuracy);
	const auto& IndexesForAccuracy = VSOP87A_SATURN_TRUNC_COUNT[AccuracyIndex];

	// The VSOP 87 coefficients are separated by Coordinate, X,Y,Z. Evaluate for each axis. 
	const double X = EvaluateVelocity(VSOP87A_SATURN_X, IndexesForAccuracy[0], VSOP87Time);
	const double Y = EvaluateVelocity(VSOP87A_SATURN_Y, IndexesForAccuracy[1], VSOP87Time);
	const double Z = EvaluateVelocity(VSOP87A_SATURN_Z, IndexesForAccuracy[2], VSOP87Time);

	return FVector(X, Y, Z);
}

FVector UVSOP87::GetUranusLocation(double VSOP87Time, EVSOP87Accuracy Accuracy)
{
	// Find the right indices array for the wanted accuracy
	int AccuracyIndex = static_cast<int>(Accuracy);
	const auto& IndexesForAccuracy = VSOP87A_URANUS_TRUNC_COUNT[AccuracyIndex];

	// The VSOP 87 coefficients are separated by Coordinate, X,Y,Z. Evaluate for each axis. 
	const double X = EvaluateCoordinate(VSOP87A_URANUS_X, IndexesForAccuracy[0], VSOP87Time);
	const double Y = EvaluateCoordinate(VSOP87A_URANUS_Y, IndexesForAccuracy[1], VSOP87Time);
	const double Z = EvaluateCoordinate(VSOP87A_URANUS_Z, IndexesForAccuracy[2], VSOP87Time);

	return FVector(X, Y, Z);
}

FVector UVSOP87::GetUranusVelocity(double VSOP87Time, EVSOP87Accuracy Accuracy)
{
	// Find the right indices array for the wanted accuracy
	int AccuracyIndex = static_cast<int>(Accuracy);
	const auto& IndexesForAccuracy = VSOP87A_URANUS_TRUNC_COUNT[AccuracyIndex];

	// The VSOP 87 coefficients are separated by Coordinate, X,Y,Z. Evaluate for each axis. 
	const double X = EvaluateVelocity(VSOP87A_URANUS_X, IndexesForAccuracy[0], VSOP87Time);
	const double Y = EvaluateVelocity(VSOP87A_URANUS_Y, IndexesForAccuracy[1], VSOP87Time);
	const double Z = EvaluateVelocity(VSOP87A_URANUS_Z, IndexesForAccuracy[2], VSOP87Time);

	return FVector(X, Y, Z);
}

FVector UVSOP87::GetNeptuneLocation(double VSOP87Time, EVSOP87Accuracy Accuracy)
{
	// Find the right indices array for the wanted accuracy
	int AccuracyIndex = static_cast<int>(Accuracy);
	const auto& IndexesForAccuracy = VSOP87A_NEPTUNE_TRUNC_COUNT[AccuracyIndex];

	// The VSOP 87 coefficients are separated by Coordinate, X,Y,Z. Evaluate for each axis. 
	const double X = EvaluateCoordinate(VSOP87A_NEPTUNE_X, IndexesForAccuracy[0], VSOP87Time);
	const double Y = EvaluateCoordinate(VSOP87A_NEPTUNE_Y, IndexesForAccuracy[1], VSOP87Time);
	const double Z = EvaluateCoordinate(VSOP87A_NEPTUNE_Z, IndexesForAccuracy[2], VSOP87Time);

	return FVector(X, Y, Z);
}

FVector UVSOP87::GetNeptuneVelocity(double VSOP87Time, EVSOP87Accuracy Accuracy)
{
	// Find the right indices array for the wanted accuracy
	int AccuracyIndex = static_cast<int>(Accuracy);
	const auto& IndexesForAccuracy = VSOP87A_NEPTUNE_TRUNC_COUNT[AccuracyIndex];

	// The VSOP 87 coefficients are separated by Coordinate, X,Y,Z. Evaluate for each axis. 
	const double X = EvaluateVelocity(VSOP87A_NEPTUNE_X, IndexesForAccuracy[0], VSOP87Time);
	const double Y = EvaluateVelocity(VSOP87A_NEPTUNE_Y, IndexesForAccuracy[1], VSOP87Time);
	const double Z = EvaluateVelocity(VSOP87A_NEPTUNE_Z, IndexesForAccuracy[2], VSOP87Time);

	return FVector(X, Y, Z);
}

// Private

double UVSOP87::EvaluateCoordinate(const VSOP87Block (&Blocks)[6], const int32 IndexesForAccuracy[6], double VSOP87Time)
{
	double TimePowerN = 1.0;
	double Result = 0.0;
	//        5       
	// X(t) = ∑ t^n * ∑An,k * Cos(Bn,k + Cn,k * t)
	//        n=0     k
	//
	// n = the current exponent of VSOP87Time  t^0 = 1, so we can start with PowerOfTime = 1.0
	// k = The number of all VSOP87 exponents.
	//
	// If we iterate over all the exponents, we obtain the full accuracy. But there is no need to go very far away for our use case.
	// We defined different quality thresholds, associated with a threshold on the A factors, as this.
	// 
	// 	 | Variant name | Threshold on |A|  |
	//   |--------------|-------------------|
	//   | Full         | 0 (no truncation) |
	//   | XXLarge      | 1e-8              |
	//   | XLarge       | 5e-8              |
	//   | Large        | 1e-7              |
	//   | Medium       | 5e-7              |
	//   | Small        | 1e-6              |
	//   | XSmall       | 5e-6              |
	//   | Milli        | 1e-5              |
	//   | Micro        | 5e-5              |
	//   | Nano         | 1e-4              |
	//   | Pico         | 1e-3              |
	//
	// When creating the VSOPA_Planet files, we sorted the coefficients by |A| and kept a trace of the limit index in the VSOP87A_<Planet>_TRUNC_COUNT array
	// We can therefore stop iterating when needed.   
	
	
	// iterate over the Exponents of time
	for (int TimeExponent = 0; TimeExponent < 6; ++TimeExponent)
	{
		// Blocks 
		const VSOP87Block& BlockForExponent = Blocks[TimeExponent]; // Get the ABC coefficients for the appropriate time exponent
		int32 BlockCountForAccuracy = FMath::Min(IndexesForAccuracy[TimeExponent], BlockForExponent.TermsCount); // Find the index will stop iterating at

		double SumForExponent = 0.0;
		for (int32 BlockIndex = 0; BlockIndex < BlockCountForAccuracy; ++BlockIndex)
		{
			const VSOP87Term& Term = BlockForExponent.Terms[BlockIndex];
			SumForExponent += Term.A * FMath::Cos(Term.B + Term.C * VSOP87Time);
		}

		// Add to the result
		Result += SumForExponent * TimePowerN;

		// Prepare for the next power of Time
		TimePowerN *= VSOP87Time;
	}
	
	return Result;
}

double UVSOP87::EvaluateVelocity(const VSOP87Block (&Blocks)[6], const int32 IndexesForAccuracy[6], double VSOP87Time)
{
	// Velocities are computed by deriving this expression 
	//        5       
	// X(t) = ∑ t^n * ∑An,k * Cos(Bn,k + Cn,k * t)
	//        n=0     k
	//
	// Inner derivative
	//  d/Td Cos(B+CT) = −C * Sin(B+CT)
	// 
	// Outer derivative --> UV' = U'V + UV' 
	//	d/Td [T^n * A * Cos(B+CT)] = n * T^(n−1) * A * Cos(B+CT) − T^n * AC * Sin(B+CT) 
	// 
	//            5       
	// dX/dt(t) = ∑   ∑ [nT^(n−1)An,k * Cos(ϕn,k) − T^n * An,k * Cn,k * Sin(ϕn,k)]
	//           n=0  k
	// Where ϕn,k = Bn,k + Cn,k * T
	
	// Here, we can't compute T^n at each loop by accumulating, because we need T^n-1 too.
	// Let's keep the powers of time in a array. 
	double PowersOfTime[6];
	PowersOfTime[0] = 1.0; // T^0
	for (int n = 1; n < 6; ++n)
	{
		PowersOfTime[n] = PowersOfTime[n - 1] * VSOP87Time;
	}
		

	double dXdT = 0.0; // derivative w.r.t. VSOP87Time

	// iterate over the Exponents of time
	for (int TimeExponent = 0; TimeExponent < 6; ++TimeExponent)
	{
		const VSOP87Block& BlockForExponent = Blocks[TimeExponent]; // Get the ABC coefficients for the appropriate time exponent
		int32 BlockCountForAccuracy = FMath::Min(IndexesForAccuracy[TimeExponent], BlockForExponent.TermsCount); // Find the index will stop iterating at

		for (int32 BlockIndex = 0; BlockIndex < BlockCountForAccuracy; ++BlockIndex)
		{
			const VSOP87Term& Term = BlockForExponent.Terms[BlockIndex];

			double Phi = Term.B + Term.C * VSOP87Time;
			double CosPhi = FMath::Cos(Phi);
			double SinPhi = FMath::Sin(Phi);

			// d/dT[ VSOP87Time^n * A * Cos(phi) ] = n * VSOP87Time^(n-1) * A * Cos(phi) - VSOP87Time^n * A * C * sin(phi)
			//                                              (1)  =0 when n==0                     (2) 
			double Contribution = 0.0;

			// (1)
			if (TimeExponent > 0) 
			{
				Contribution += TimeExponent * PowersOfTime[TimeExponent - 1] * Term.A * CosPhi;
			}
			// (2)
			Contribution -= PowersOfTime[TimeExponent] * Term.A * Term.C * SinPhi;

			dXdT += Contribution;
		}
	}

	// Convert derivative from per-VSOP87Time to per-JD 
	// VSOP87Time = (JD - 2451545.0) / 365250.0  -> dT/dJD = 1/365250
	return dXdT / 365250.0; // AU/day
}

