// Copyright Epic Games, Inc. All Rights Reserved.


#include "ViewportCameraProvider.h"
#include "Engine/Engine.h"
#include "GameFramework/PlayerController.h"

static IViewportCameraProvider* GCameraProvider = nullptr;



//////////////////////////////////////////////////////////////////////////

bool FRuntimeCameraProvider::GetCamera(FVector& OutLocation, FRotator& OutRotation) const
{
	if (!GEngine) return false;

	UWorld* World = GEngine->GetCurrentPlayWorld();
	if (!World) return false;

	APlayerController* PC = World->GetFirstPlayerController();
	if (!PC) return false;

	PC->GetPlayerViewPoint(OutLocation, OutRotation);
	return true;
}

IViewportCameraProvider* GetCameraProvider()
{
	if (!GCameraProvider)
	{
		static FRuntimeCameraProvider DefaultProvider;
		GCameraProvider = &DefaultProvider;
	}

	return GCameraProvider;
}

void SetCameraProvider(IViewportCameraProvider* InProvider)
{
	GCameraProvider = InProvider;
}