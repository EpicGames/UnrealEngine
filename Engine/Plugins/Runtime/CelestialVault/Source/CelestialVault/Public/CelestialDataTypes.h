// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "CelestialDataTypes.generated.h"

/** Different type of orbital motion for the Celestial bodies. Will be used to branch the computational logics */
UENUM(BlueprintType) 
enum class EOrbitType : uint8
{
	Star,		// Far away star - No orbit
	Elliptic,	// Generic Ellipsoidal Orbit
	VSOP87		// Orbit defined by the VSOP87 Computations
};

/** Different type Solar System bodies in the VSOP87 Equations */
UENUM(BlueprintType) 
enum class EVSOP87BodyType : uint8
{
	Sun,
	Mercury,
	Venus,
	Earth,
	Mars,
	Jupiter,
	Saturn,
	Uranus ,
	Neptune ,
	Moon 
};

/**
 * Runtime structure to store the properties common to all Celestial Bodies
 */
USTRUCT(BlueprintInternalUseOnly)
struct FCelestialBody
{
	GENERATED_BODY()
	virtual ~FCelestialBody() = default;   // to remove Warning C4265

public: 
	/** The Body Name */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category= "Celestial Vault")
	FString Name = "";

	/** One of the predefined Orbit types (Solar system planets), any custom one for fantasy planets */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category= "Celestial Vault")
	EOrbitType OrbitType = EOrbitType::Elliptic;
	
	/** The Body J2000 Right Ascension (topocentric, from observer lat/lon) - In hours! 
	 * 
	 * Note: The "true" catalog J2000 is always geocentric;
	 * We decided to express this value in the topocentric variant for validation against JPL Horizons or Stellarium, which use data generated with a surface observer.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category= "Celestial Vault", meta=(ForceUnits="Hours", DisplayName="Right Ascension J2000", MakeStructureDefaultValue="0.000000"))
	double RAJ2000 = 0.0;

	/** The Body J2000 Declination (topocentric, from observer lat/lon) - In Degrees 
	 * 
	 * Note: The "true" catalog J2000 is always geocentric;
	 * We decided to express this value in the topocentric variant for validation against JPL Horizons or Stellarium, which use data generated with a surface observer.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category= "Celestial Vault", meta=(ForceUnits="Degrees", DisplayName="Declination J2000", MakeStructureDefaultValue="0.000000"))
	double DECJ2000 = 0.0;
	
	/** The Body Right Ascension of date (topocentric, from observer lat/lon)*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category= "Celestial Vault", meta=(ForceUnits="Hours"))
	double RA = 0.0;

	/** The Body Declination of date (topocentric, from observer lat/lon)*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category= "Celestial Vault", meta=(ForceUnits="Degrees"))
	double DEC = 0.0;

	/** The Body Right Ascension of date (geocentric) - used for 3D scene placement. In hours. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category= "Celestial Vault", meta=(ForceUnits="Hours"))
	double RAGeocentric = 0.0;

	/** The Body Declination of date (geocentric) - used for 3D scene placement. In Degrees. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category= "Celestial Vault", meta=(ForceUnits="Degrees"))
	double DECGeocentric = 0.0;

	/** The Body distance to Earth - In Astronomical Units! */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category= "Celestial Vault")
	double DistanceInAU = 0.0;

	/** The Body Radius in Kilometers */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category= "Celestial Vault", meta=(ForceUnits="Kilometers"))
	double Radius = 0.0;

	/** The Magnitude of the Body seen from Earth */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category= "Celestial Vault")
	double Magnitude = 0.0;

	/** Keep track of the Vault-relative transform, at ToD = 0 for animated bodies (Moon) */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category= "Celestial Vault")
	FTransform UELocalTransform = FTransform::Identity;

	/** Keep track of the location toward the Earth */ 
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category= "Celestial Vault")
	FVector DirectionTowardEarth = FVector::ZeroVector;

	// Can't use a UFUNCTION inside a struct --> Use the Celestial Maths Blueprint Function Library
	FString ToString() const;


	
	/** From the properties of this Celestial Body, compute the associated UE Transform, considering the following arguments
	 *   - ExtraScale - By default, the base scale is computed by assuming the body is rendered by a 1meter billboard plane. This ExtraScale factor artificially makes the object larger.
	 *   - DistanceOverride - if not set (or <0), we compute the UE Object location based on the actual Celestial Body Location. If set, we consider this distance instead. (Useful to place the body on a Plato-like sphere)
	 */ 
	virtual void ComputeUELocalTransform(double ExtraScale = 1.0, double UEDistanceOverride = -1.0 );
};

/** Additional Properties specific to Stellar Bodies (Stars) */
USTRUCT(BlueprintType)
struct FStellarBody : public FCelestialBody
{
	GENERATED_BODY()

	/** Star Color Index, also named B-V */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category= "Celestial Vault", meta=(DisplayName="ColorIndex", MakeStructureDefaultValue="0.000000"))
	double ColorIndex = 0.0;

	/** Star RGB Color - Can be computed from the B-V value if the star is from an official Catalog */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category= "Celestial Vault", meta=(DisplayName="Color"))
	FLinearColor Color = FLinearColor(1.0f,1.0f,1.0f,1.0f);

	/** Star Hipparcos ID if present in the Hipparcos Catalog */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category= "Celestial Vault", meta=(DisplayName="Hipparcos Catalog ID", MakeStructureDefaultValue="0"))
	int32 HipparcosID = 0;

	/** Star Henry Draper ID if present in the Henry Draper Catalog */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category= "Celestial Vault", meta=(DisplayName="Henry Draper Catalog ID", MakeStructureDefaultValue="0"))
	int32 HenryDraperID = 0;

	/** Star YaleBrightStar ID if present in the Yale Bright Star Catalog */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category= "Celestial Vault", meta=(DisplayName="Yale Bright Star Catalog ID", MakeStructureDefaultValue="0"))
	int32 YaleBrightStarID = 0;

	// Can't use a UFUNCTION inside a struct --> Use the Celestial Maths Blueprint Function Library
	FString ToString() const;
};

/** Additional Properties specific to Planetary Bodies (Planets/Moons) */
USTRUCT(BlueprintType)
struct FPlanetaryBody : public FCelestialBody
{
	GENERATED_BODY()

	/** VSOP87 Identifier of the Planetary Body */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category= "Celestial Vault")
	EVSOP87BodyType VSOP87BodyType = EVSOP87BodyType::Earth;

	/** The True apparent diameter of the Body seen from Earth (Angular size, in degrees) */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category= "Celestial Vault",  meta = (ForceUnits="Degrees"))
	double ApparentDiameterDegrees = 0.0;

	/**
	 * The Scaled apparent diameter of the Body seen from Earth (Angular size, in degrees) 
	 *   Takes the fake scaling factor into consideration, so useful for camera FOV tracking. 
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category= "Celestial Vault",  meta = (ForceUnits="Degrees"))
	double ScaledApparentDiameterDegrees = 0.0;
	
	/** indication of the body disc age (phase). 0 = New, 0.25 = First quarter, 0.5 = Full, 1 = Next New */ 
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category= "Celestial Vault")
	double Age = 0.5;

	/** indication of percentage of the illumination when the Body is fully visible*/ 
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category= "Celestial Vault")
	double IlluminationPercentage = 1.0;

	FString ToString() const;

	virtual void ComputeUELocalTransform(double ExtraScale = 1.0, double UEDistanceOverride = -1.0 ) override;
};


/**
 * Runtime structure to store the Kinematic State - Location, Velocity of a Planetary Body, in VSOP87 and FK5J2000 frames
 */
USTRUCT(BlueprintInternalUseOnly)
struct FPlanetaryBodyKinematicState
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category= "Celestial Vault")
	FVector Location_VSOP87 = FVector::ZeroVector;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category= "Celestial Vault")
	FVector Velocity_VSOP87 = FVector::ZeroVector;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category= "Celestial Vault")
	FVector Location_Heliocentric_FK5J2000 = FVector::ZeroVector;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category= "Celestial Vault")
	FVector Velocity_FK5J2000 = FVector::ZeroVector;
};

