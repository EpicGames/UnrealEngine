// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/DataTable.h"
#include "CelestialDataTypes.h"
#include "CelestialInputDataTypes.generated.h"

// FTableRowBase
//   |- FPlanetaryBodyInputData
//   \- FStarInputData
//        \- FCelestialStarInputData

/**
 * TableRow Base type to describe the Creation parameters of a planetary body (Planet, Moon)
 * Will only be used at creation time, from a proper Data Table
 */
USTRUCT(BlueprintType)
struct FPlanetaryBodyInputData : public FTableRowBase
{
	GENERATED_BODY()

	FPlanetaryBodyInputData(const FString& InName, EOrbitType InOrbitType, double InRadius) : Name(InName), OrbitType(InOrbitType), Radius(InRadius) {}
	FPlanetaryBodyInputData(const FString& InName, EOrbitType InOrbitType, EVSOP87BodyType InVSOP87BodyType, double InRadius) : Name(InName), OrbitType(InOrbitType), VSOP87BodyType(InVSOP87BodyType), Radius(InRadius) {}
	FPlanetaryBodyInputData() {}
public:
	/** The body Name */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category= "Celestial Vault")
	FString Name = "";

	/** Can be VSOP87 or Elliptic */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category= "Celestial Vault")
	EOrbitType OrbitType = EOrbitType::Elliptic;

	/** VSOP87 Identifier of the Planetary Body */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category= "Celestial Vault")
	EVSOP87BodyType VSOP87BodyType = EVSOP87BodyType::Earth;
	
	/** The Body Radius in Kilometers */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category= "Celestial Vault", meta=(ForceUnits="Kilometers"))
	double Radius = 1000.0;

	/** The planetary body Material expects Bodies textures in a single row - This is the 0-based index of the texture to use */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category= "Celestial Vault")
	int32 TextureColumnIndex = 0;

	// TODO_Beta
	// Add the Elliptic parameters for other Moon/Planets
	// Ellipsoid - Minor/Major axes + offset to X axis
	// Revolution period (year) + angle offset to time
	// Phase if we want to fake it

public: 
	static FPlanetaryBodyInputData Earth; // Earth Preset
	static FPlanetaryBodyInputData Moon; // Moon Preset
};

/**
 * TableRow Base type to describe the Creation parameters of a Basic Star
 * Will only be used at creation time, from a proper Data Table
 * This Struct contains the minimal needed data for a Fictional Star
 */
USTRUCT(BlueprintType)
struct FStarInputData : public FTableRowBase 
{
	GENERATED_BODY()
public:
	/** The Star Right Ascension in the Celestial Frame - In hours! */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category= "Celestial Vault", meta=(ForceUnits="Hours", DisplayName="Right Ascension", MakeStructureDefaultValue="0.000000"))
	double RA = 0.0;

	/** The Star Declination in the Celestial Frame - In Degrees */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category= "Celestial Vault", meta=(ForceUnits="Degrees", DisplayName="Declination", MakeStructureDefaultValue="0.000000"))
	double DEC = 0.0;

	/** Earth to Star distance (in Parsecs)  */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category= "Celestial Vault", meta=(DisplayName="Distance (in parsecs)", MakeStructureDefaultValue="100.000000"))
	double DistanceInPC = 100.0;
	
	/** Star Name */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category= "Celestial Vault", meta=(DisplayName="Name"))
	FString Name = "";

	/** Star Magnitude */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category= "Celestial Vault", meta=(DisplayName="Magnitude", MakeStructureDefaultValue="1.000000"))
	double Magnitude = 1.0;

	/** Star RGB Color - Useless if computed from the B-V */ 
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category= "Celestial Vault", meta=(DisplayName="Color"))
	FLinearColor Color = FLinearColor(1.0f,1.0f,1.0f,1.0f);
};

/**
 * TableRow Base type to describe the Creation parameters of a Catalog-based Star
 * Will only be used at creation time, from a proper Data Table
 * This Struct extends the FStarInputData class with additional Catalog Properties
 */
USTRUCT(BlueprintType)
struct FCelestialStarInputData : public FStarInputData 
{
	GENERATED_BODY()
public:
	/** Star Hipparcos ID */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category= "Celestial Vault", meta=(DisplayName="Hipparcos Catalog ID", MakeStructureDefaultValue="0"))
	int32 HipparcosID = -1;

	/** Star Henry Draper ID */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category= "Celestial Vault", meta=(DisplayName="Henry Draper Catalog ID", MakeStructureDefaultValue="0"))
	int32 HenryDraperID = -1;

	/** Star YaleBrightStar ID */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category= "Celestial Vault", meta=(DisplayName="Yale Bright Star Catalog ID", MakeStructureDefaultValue="0"))
	int32 YaleBrightStarID = -1;
	
	/** Star Color Index, also named B-V */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category= "Celestial Vault", meta=(DisplayName="ColorIndex", MakeStructureDefaultValue="0.000000"))
	double ColorIndex = -1.0;
};

