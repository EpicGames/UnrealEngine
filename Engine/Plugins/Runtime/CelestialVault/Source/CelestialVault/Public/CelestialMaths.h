// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "CelestialDataTypes.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "CelestialMaths.generated.h"

/**
 *  Units Conventions
 *    Distances: 
 *      Distances are expressed in Astronomical units - Specified by the "AU" mention in the function name
 *
 *    Time:
 *      Local/UTC Time: All function parameters contains either the "Local" or "UTC" prefix in their name to specify if they expect a Local or UTC Time. 

 *      Most Celestial functions are expecting an absolute time expressed using a Julian Date.
 *      When the name "JulianDay" is used, it means the Julian Date when t=0 (midnight, beginning of the day)
 *      By definition, a Julian Date is finishing by 0.5 at Midnight. 
 *
 *    Angles: 
 *      GMST and GAST angle are expressed in Degrees
 *      Some Celestial data involve ArcSeconds (1 Degree = 3600 Arcseconds) - Conversion functions are provided
 */
UCLASS()
class CELESTIALVAULT_API UCelestialMaths : public UBlueprintFunctionLibrary
{
    GENERATED_BODY()

public:

#pragma region Colors

    /** Returns the RGB normalized components [0..1] from the Color Index (B-V) Value **/
    UFUNCTION(BlueprintCallable, Category="Celestial Vault|Color")
    static FLinearColor BVtoLinearColor(float BV);

	/** return the illumination factor (0..1) of a Body, considering his normalized age and the crescent effects */
	UFUNCTION(BlueprintCallable, Category="Celestial Vault|Color")
	static double GetIlluminationPercentage(double NormalizedAge);

#pragma endregion

#pragma region VSOP87

	    /**
     * Returns the location of a specific Solar System body, in the VSOP Coordinate system
     * 
     *   The VSOP frame is a "heliocentric ecliptic frame of the J2000 equinox"
     *     Origin = Sun (Heliocentric)
     *     Coordinates Axes = Mean ecliptic & equinox of J2000.0
	 *        XY-plane: the mean ecliptic plane at epoch J2000.0
     *        X-axis: pointing toward mean equinox at J2000.0
     *        Z-axis: perpendicular to the ecliptic
     *        
     *   The returned location is expressed in Astronomical Units (AU)
     *   The relativistic effects are ignored (See GetBodyLocation_FK5J2000_AU_Relativistic )
     */
    UFUNCTION(BlueprintCallable, Category="Celestial Vault|VSOP87")
    static void GetSolarSystemBodyLocationVelocity_VSOP87_AU(EVSOP87BodyType VSOP87BodyType, double JulianDate, FVector& LocationAU, FVector& Velocity);

    /**
     * Returns the location of a specific  Solar System Body, in the VSOP Coordinate system
     * 
	 *   The VSOP frame is a "heliocentric ecliptic frame of the J2000 equinox"
	 *     Origin = Sun (Heliocentric)
	 *     Coordinates Axes = Mean ecliptic & equinox of J2000.0
	 *        XY-plane: the mean ecliptic plane at epoch J2000.0
	 *        X-axis: pointing toward mean equinox at J2000.0
	 *        Z-axis: perpendicular to the ecliptic
     *       
     *   The returned location is expressed in Astronomical Units (AU)
     *       
     * The returned Location and the Observer Body location are expressed in Astronomical Units (AU)
     * The relativistic effects are considered (time taken for the light to reach the Observer body location)
     * This function therefore returns the location of the Planetary Body as if it was seen from the Observer Body Location at this JulianDate
    */
    UFUNCTION(BlueprintCallable, Category="Celestial Vault|VSOP87")
    static FVector GetSolarSystemBodyLocation_VSOP87_Relativistic(FVector ReferenceBodyLocation_Heliocentric_AU, EVSOP87BodyType VSOP87BodyType, double JulianDate);

	/**
	* Returns the Body location and velocity in the VSOP87 and FK5J2000 frames
	*
	*	This is a convenience function to keep a cache of a Body location (eg. Earth) and accelerate the computations when interested in several other bodies. 
	*/
	UFUNCTION(BlueprintCallable, Category="Celestial Vault|VSOP87")
	static FPlanetaryBodyKinematicState GetPlanetaryBodyKinematicState_AU(double JulianDate, EVSOP87BodyType VSOP87BodyType);

#pragma endregion

#pragma region Celestial Bodies
	
    /** Return the location of a Planetary Body relative to the Earth, expressed in Celestial Coordinates (RA, DEC, Distance)
     * It requires the Observer location on Earth for more precise computations
     * This function also returns the distance between bodies, as it can help for Magnitude computations
     *
	 * By default, the relativistic effects are considered (time taken for the light to reach the Observer body location)
	 * This function therefore returns the location of the Planetary Body as if it was seen from the Observer Body Location at this JulianDate
	 * With bIgnoreRelativisticEffect=true, one can ignore this and have the instant location. It can also save some computation cycles for close bodies like Moon.
	 * With bGeoCentricObserver=true, the observer is expected to be at the Earth center, and the Latitude/longitude are ignored. 
     * **/
    UFUNCTION(BlueprintCallable, Category="Celestial Vault|Planetary bodies")
    static void GetBodyCelestialCoordinatesAU(double JulianDate, EVSOP87BodyType VSOP87BodyType, double ObserverLatitude, double ObserverLongitude, bool bGeoCentricObserver,
                                              bool bIgnoreRelativisticEffect, double& RAJ2000Hours, double& DECJ2000Degrees, double& RAHours, double& DECDegrees, double& RAGeocentricHours, double& DECGeocentricDegrees, double& DistanceBodyToEarthAU, double& DistanceBodyToSunAU, double& DistanceEarthToSunAU);

	/** Return the location of a Planetary Body relative to the Earth, expressed in Celestial Coordinates (RA, DEC, Distance)
	 * It requires the Observer location on Earth for more precise computations
	 * This function also returns the distance between bodies, as it can help for Magnitude computations
	 *
	 * By default, the relativistic effects are considered (time taken for the light to reach the Observer body location)
	 * This function therefore returns the location of the Planetary Body as if it was seen from the Observer Body Location at this JulianDate
	 * With bIgnoreRelativisticEffect=true, one can ignore this and have the instant location. It can also save some computation cycles for close bodies like Moon. 
	 * With bGeoCentricObserver=true, the observer is expected to be at the Earth center, and the Latitude/longitude are ignored. 
     *
	 * This override can run faster if we provide it with a cache of the Earth Location and Velocity, because it won't recompute it. 
	 * **/
	UFUNCTION(BlueprintCallable, Category="Celestial Vault|Planetary bodies")
	static void GetBodyCelestialCoordinatesAU_UsingKnownState(double JulianDate, EVSOP87BodyType VSOP87BodyType, const FPlanetaryBodyKinematicState& EarthKinematicState, double ObserverLatitude, double ObserverLongitude, bool bGeoCentricObserver,
	                                                          bool bIgnoreRelativisticEffect, double& RAJ2000Hours, double& DECJ2000Degrees, double& RAHours, double& DECDegrees, double& RAGeocentricHours, double& DECGeocentricDegrees, double& DistanceBodyToEarthAU, double& DistanceBodyToSunAU, double& DistanceEarthToSunAU);
	
    /** Return the Magnitude of a Planetary Body as seen from the Earth **/
    UFUNCTION(BlueprintCallable, Category="Celestial Vault|Planetary bodies")
    static double GetPlanetaryBodyMagnitude(EVSOP87BodyType VSOP87BodyType, double DistanceToSunAU, double DistanceToEarthAU, double DistanceEarthToSunAU, double& PhaseAngle);

#pragma endregion

#pragma region Time

    /** Return the UTC for a specific Local Time, using the TimeZone and Daylight Saving Information **/
    UFUNCTION(BlueprintCallable, Category="Celestial Vault|Time")
    static FDateTime LocalTimeToUTCTime(FDateTime LocalTime, double TimeZoneOffset, bool IsDst);

    /** Return the Local Time for a specific UTC Time, using the TimeZone and Daylight Saving Information **/
    UFUNCTION(BlueprintCallable, Category="Celestial Vault|Time")
    static FDateTime UTCTimeToLocalTime(FDateTime UTCTime, double TimeZoneOffset, bool IsDst);
    
    /** Return the Julian Date for a specific UTC Time **/
    UFUNCTION(BlueprintCallable, Category="Celestial Vault|Time")
    static double UTCDateTimeToJulianDate(FDateTime UTCDateTime);

    /** Return the UTC Time for a specific Julian Date **/
    UFUNCTION(BlueprintCallable, Category="Celestial Vault|Time")
    static FDateTime JulianDateToUTCDateTime(double JulianDate);

    /**
     * Return the Greenwich Mean Sidereal Time (GMST) for a specific DateTime, in Degrees
     * 
     * By definition, the provided DateTime has to be the DateTime at the Greenwitch Meridian, so it's a UTC DateTime
     */
    UFUNCTION(BlueprintCallable, Category="Celestial Vault|Time")
    static double DateTimeToGreenwichMeanSiderealTime(FDateTime UTCDateTime);

    /**
     * Return the Greenwich Mean Sidereal Time (GMST) for a specific DateTime, in Degrees, without doing any ModPositive, so unwrapped
     * 
     * By definition, the provided DateTime has to be the DateTime at the Greenwitch Meridian, so it's a UTC DateTime
     */
    UFUNCTION(BlueprintCallable, Category="Celestial Vault|Time")
    static double DateTimeToGreenwichMeanSiderealTimeUnwrapped(FDateTime UTCDateTime);


    /** Return the Greenwich Mean Sidereal Time (GMST) for a specific Julian Date, In Degrees */
    UFUNCTION(BlueprintCallable, Category="Celestial Vault|Time")
    static double JulianDateToGreenwichMeanSiderealTime(double JulianDate);

    /** Return the Sidereal Time for a specific Longitude and GMST **/
    UFUNCTION(BlueprintCallable, Category="Celestial Vault|Time")
    static double LocalSideralTime(double LongitudeDegrees, double GreenwichMeanSideralTime);
    
    /**
     * Return the Greenwich Apparent Sidereal Time (GAST) for a specific UTC DateTime, in Degrees.
     * 
     * The Greenwich apparent sidereal time is obtained by adding a correction to the Greenwich mean sidereal time.
     * The correction term is called the nutation in right ascension or the equation of the equinoxes.
     */
    UFUNCTION(BlueprintCallable, Category="Celestial Vault|Time")
    static double JulianDateToGreenwichApparentSiderealTime(double JulianDate);

    /**
     * Returns the Julien Centuries
     *
     * Julian Centuries = (JulianDate - 2451545.0) / 36525.0
     */
    UFUNCTION(BlueprintCallable, Category="Celestial Vault|Time")
    static double JulianDateToJulianCenturies(double JulianDate);

    /** Returns the Leap Seconds for a specific Julian Date
     *
     * A leap second is a one-second adjustment that is occasionally applied to Coordinated Universal Time (UTC), to accommodate the difference between
     * precise time (International Atomic Time (TAI), as measured by atomic clocks) and imprecise observed solar time (UT1),
     * which varies due to irregularities and long-term slowdown in the Earth's rotation.
     */
    UFUNCTION(BlueprintCallable, Category="Celestial Vault|Time")
    static double GetLeapSeconds(double JulianDate);

    /** Returns the Terrestrial Time in SI seconds
     * 
     *  TT = TAI + 32.184 seconds;
     */
    UFUNCTION(BlueprintCallable, Category="Celestial Vault|Time")
    static double InternationalAtomicTimeToTerrestrialTime(double TAI);

    /** Returns the International Atomic Time in SI seconds
     *
     *  TAI = GetLeapSeconds(JulianDate) + DaysToSeconds(JulianDate);
     */
    UFUNCTION(BlueprintCallable, Category="Celestial Vault|Time")
    static double JulianDateToInternationalAtomicTime(double JulianDate);

	/** Converts Seconds to Decimal Days */
    UFUNCTION(BlueprintCallable, Category="Celestial Vault|Time")
    static inline double SecondsToDay(double Seconds) { return Seconds / 86400.0; }

	/** Converts decimal Days to Seconds */
    UFUNCTION(BlueprintCallable, Category="Celestial Vault|Time")
    static inline double DaysToSeconds(double Days) { return Days * 86400.0; }

    /** Converts a Julian Date to a proper VSOP87Time, suitable for the VSOP87 computations
     * 
	 * In the VSOP87 equations, Time is not Julian centuries!
	 *  > Given a Julian date JD expressed in dynamical time T = (TAI+32.184s), the VSOP equations exprect a Time = (T-2451545)/365250
	 *	> The denominator is 365250, not 36525.
	 *	> That’s because VSOP87 expresses time in units of 10,000 Julian years (1 Julian millennium = 100 centuries = 36,525 × 100 days).
     */
    UFUNCTION(BlueprintCallable, Category="Celestial Vault|Time")
    static double JulianDateToVSOP87Time(double JulianDate);
	
#pragma endregion

#pragma region Angles
    
    /** Convert Arcseconds to Degrees
     * 
     * 1 Degree = 3600 Arcseconds */
    UFUNCTION(BlueprintCallable, Category="Celestial Vault|Angles")
    static double ArcsecondsToDegrees(double Arcseconds) { return Arcseconds / 3600.0; };

    /** Convert Arcseconds to Radians
     * 
     * 2 PI Rad = 360 Degrees = 360 * 3600 Arcseconds
     */ 
    UFUNCTION(BlueprintCallable, Category="Celestial Vault|Angles")
    static double ArcsecondsToRadians(double Arcseconds) { return Arcseconds / 3600.0 * UE_PI / 180.0; };
    
    /** Convert Degrees to Arcseconds
     * 
     * 1 Degree = 3600 Arcseconds */
    UFUNCTION(BlueprintCallable, Category="Celestial Vault|Angles")
    static double DegreesToArcseconds(double Degrees) { return Degrees * 3600.0; };

    /** Special Mod function that makes sure to always return positive values */
    UFUNCTION(BlueprintCallable, Category="Celestial Vault|Angles")
    static double ModPositive(double Value, double Modulo);

    /** Convert Decimal degrees to Hours, Minutes, Seconds ( One Hour equals 15 degrees) */
    UFUNCTION(BlueprintCallable, Category="Celestial Vault|Time")
    static void DegreesToHMS(double DecimalDegrees, int32& Hours, int32& Minutes, double& Seconds);

    /** Convert decimal degrees to Degrees, Minutes, Seconds, with the appropriate Sign (True if Positive) **/ 
    UFUNCTION(BlueprintCallable, Category="Celestial Vault|Time")
    static void DegreesToDMS(double DecimalDegrees, bool& Sign, int32& Degrees, int32& Minutes, double& Seconds);

#pragma endregion 

#pragma region Earth

	class WGS84
	{
	public:
		/**  Semi Major Axis in Meters **/ 
		static double SemiMajorAxis;

		/**  Semi Minor Axis in Meters **/
		static double SemiMinorAxis;
		
		static double Flattening;
	};
	
    /** Retrurn the Earth Rotation Angle (In Degrees) as measured by GMST (Greenwich Mean Sidereal Time)
     *
     * It refers to the angle of Earth's rotation relative to the fixed stars, specifically the hour angle of the mean vernal equinox as observed from the Greenwich meridian.
     * It represents how far Earth has rotated since the mean equinox crossed the Greenwich meridian;
     * It is essentially a way to measure Earth's rotation in angular terms based on a celestial reference point.
     */
    UFUNCTION(BlueprintCallable, Category="Celestial Vault|Earth")
    static double GetEarthRotationAngle(double JulianDate);

    /** Convert Geodetic Lat Lon to Geocentric XYZ position vector in ECEF coordinates, for the WGS84 Ellipsoid.
     * 
     *   Be careful, XYZ coordinates are
     *    - ECEF Coordinates in the ECEF Right-Handed Frame (not the Left-handed UE ones in UE Units)
     *    - Expressed in Astronomical Units (AU)  
     */
    UFUNCTION(BlueprintCallable, Category="Celestial Vault|Earth")
    static FVector GeodeticLatLonToECEFXYZAU(double Latitude, double Longitude, double Altitude);


    /** Converts a ECEF Location for the WGS84 Ellipsoid into Geodetic Latitude and Longitude.  
     *
	 *   Be careful, XYZ coordinates are
	 *    - ECEF Coordinates in the ECEF Right-Handed Frame (not the Left-handed UE ones in UE Units)
	 *    - Expressed in Astronomical Units (AU)
	 *
	 *   Altitude is returned as Meters
     */
    UFUNCTION(BlueprintCallable, Category="Celestial Vault|Earth")
	static void ECEFXYZAUToGeodeticLatLon(FVector ECEFLocationAU, double &Latitude, double &Longitude, double &AltitudeMeters);

    /**
     *  Return the WGS84 Radius (in Meters) for a specific Latitude 
     */
    UFUNCTION(BlueprintCallable, Category="Celestial Vault|Earth")
	static double WGS84GeocentricRadius(double Latitude);

    

    /** Return the Geocentric position of an observer located at the Earth surface, considering the rotation at this specific JulianDate, using the Greenwich Apparent Didereal Time.
     *
     *  The position is expressed relatively to the earth center, but on the solar system reference frame.
     *  Coordinates are Expressed in Astronomical Units (AU)
     */
    UFUNCTION(BlueprintCallable, Category="Celestial Vault|Earth")
    static FVector GetObserverGeocentricLocationAU(double Latitude, double Longitude, double Altitude, double JulianDate);

    /** Return the nutation in right ascension ( aka the equation of the equinoxes) in Degrees
     * 
     * This correction term is used when computing the Greenwich apparent sidereal from the Greenwich mean sidereal time
     */
    UFUNCTION(BlueprintCallable, Category="Celestial Vault|Earth")
    static double EquationOfTheEquinoxes(double JulianDate);

    /** Approximation of the IAU2000A/B nutation model used in the Equation Of The Equinoxes, accurate enough for VSOP87 computations  */
    UFUNCTION(BlueprintCallable, Category="Celestial Vault|Earth")
    static void Nutation2000BTruncated(double JulianDate, double& DeltaPsiDegrees, double& DeltaEpsilonDegrees);

    /** Return the Transformation to apply to a WGS84 Ellipsoid model so that its location in Lat,long,Altitude is tangent to the Origin
     *  The Transform is expressed in the ECEF Frame (Location units are meters, right-handed ECEF Frame)
     */
    UFUNCTION(BlueprintCallable, Category="Celestial Vault|Earth")
    static FTransform GetEarthCenterTransformECEF(double Latitude, double Longitude, double Altitude);

	/** Return the Transformation to apply to a WGS84 Ellipsoid model so that its location in Lat,long,Altitude is tangent to the Origin
	 * It's used to locate the Rotating Celestial Vault for a specific UE Origin
	 * The Transform is expressed in the UE Frame (Location units are UE Units, left-handed Unreal Frame)
	 *
	 * With Geocentric=true, the observer is expected to be at the Earth center, Latitude/longitude are ignored and this function returns an ECEF-Oriented transform. 
	 */
	UFUNCTION(BlueprintCallable, Category="Celestial Vault|Earth")
	static FTransform GetEarthCenterTransformUEFrame(double Latitude, double Longitude, double Altitude, bool Geocentric);

	/** Return the Precession Matrix 
	 *    The Precession matrix transforms coordinates from one equatorial reference epoch to another by accounting for Earth’s long-term, secular drift of its rotation axis (precession of the equinox).
	 *    It's used to compute RA, DEC between the J2000 Epoch and a specific date. (Earth’s rotation axis slowly traces a ~26,000-year cone because of gravitational torques from the Moon and Sun.)
	 */
	UFUNCTION(BlueprintCallable, Category="Celestial Vault|Earth")
	static FMatrix GetPrecessionMatrix(double JulianDate);

	/** Return the Nutation Matrix 
	 *   The Nutation matrix accounts for Nutation, the small, periodic wobble of Earth’s rotation axis – when transforming between “mean” and “true” celestial coordinate frames.
	 */
	UFUNCTION(BlueprintCallable, Category="Celestial Vault|Earth")
	static FMatrix GetNutationMatrix(double JulianDate);

    /* Compute the Aberration of an object located far from the Earth.
	 *   In astronomy, Aberration is an apparent shift in the observed position of a celestial object caused by the finite speed of light combined with the motion of the observer.
     *   It is not an optical distortion — it’s a relativistic effect due to the fact that light takes time to reach you while you are moving.
     */
    UFUNCTION(BlueprintCallable, Category="Celestial Vault|Earth")
	static FVector ComputeAberration(const FVector& TopocentricTargetLocationAU, const FVector& EarthVelocity);

	
#pragma endregion
    
#pragma region Sun & Moon

    /** Compute all Sun Properties for a specific JulianDate */ 
    UFUNCTION(BlueprintCallable, Category="Celestial Vault|Sun")
    static FStellarBody GetSunInformation(double JulianDate, double ObserverLatitude, double ObserverLongitude, bool bGeoCentric);

	/** Compute all Sun Properties for a specific JulianDate
	 *
	 * This override can run faster if we provide it with a cache of the Earth Location and Velocity, because it won't recompute it.
	 * if bGeocentric is true, Latitude and Longitude are ignored. 
	 */ 
	UFUNCTION(BlueprintCallable, Category="Celestial Vault|Sun")
	static FStellarBody GetSunInformation_UsingKnownState(double JulianDate, const FPlanetaryBodyKinematicState& EarthKinematicState, double ObserverLatitude, double ObserverLongitude, bool bGeoCentric);

	
	/** Returns the Moon Phase for a specific Date
	 * 
	 *  This is an approximate computation using a number of lunar cycles with a synodic month equals to 29.53059 days
	 *  Not very precise over more than 1 centuries before of after 2025
	 */
	UFUNCTION(BlueprintCallable, Category="Celestial Vault|Planetary bodies")
	static double GetMoonNormalizedAgeSimple(double JulianDate);
	
#pragma endregion

#pragma region Utilities

    /** Returns the Speed of Light 299 792 458 (m/s) **/
    UFUNCTION(BlueprintCallable, Category="Celestial Vault|Utilities")
    static double GetSpeedOfLight() { return UCelestialMaths::SpeedOfLightMetersPerSeconds; }
    
    /** Convert Astronomical Unit (UA) to meters
     *
     * 1 AU = 149 597 870 700 m 
     */
    UFUNCTION(BlueprintCallable, Category="Celestial Vault|Utilities")
    static double AstronomicalUnitsToMeters(double AU) {return AU * AstronomicalUnitsMeters; };

    /** Convert meters to Astronomical Unit (UA)
     *
     * 1 AU = 149 597 870 700 m
     */
    UFUNCTION(BlueprintCallable, Category="Celestial Vault|Utilities")
    static double MetersToAstronomicalUnits(double Meters) {return Meters / AstronomicalUnitsMeters; };

	/** Convert Parsecs to Astronomical Unit (UA)
	 *
	 * 1 Parsec = 206 264.806 AU
	 */
	UFUNCTION(BlueprintCallable, Category="Celestial Vault|Utilities")
	static double ParsecsToAstronomicalUnits(double Parsecs) {return Parsecs * ParsecAstronomicalUnits; };

	/** Convert Astronomical Unit (UA) to Parsecs
	 *
	 * 1 Parsec = 206 264.806 AU
	 */
	UFUNCTION(BlueprintCallable, Category="Celestial Vault|Utilities")
	static double AstronomicalUnitsToParsec(double AU) {return AU / ParsecAstronomicalUnits; };

    /** Convert Cartesian Coordinates to Polar Coordinates, using a Righ-Handed Frame */
    UFUNCTION(BlueprintCallable, Category="Celestial Vault|Utilities")
    static void XYZToRADEC_RH(FVector XYZ, double& RADegrees, double& DECDegrees, double& Radius);

    /** Convert Polar Coordinates to Cartesian Coordinates, using a Righ-Handed Frame */
    UFUNCTION(BlueprintCallable, Category="Celestial Vault|Utilities")
    static FVector RADECToXYZ_RH(double RADegrees, double DECDegrees, double Radius);

    /** Returns a String displaying a vector with a large number of digits*/
    UFUNCTION(BlueprintCallable, Category="Celestial Vault|Utilities")
    static FString GetPreciseVectorString(FVector Vector, int32 MinimumFractionalDigits = 10);

    /** Celestial Body String Builder */
    UFUNCTION(BlueprintPure, meta = (DisplayName = "To String (CelestialBody)", CompactNodeTitle = "->", BlueprintAutocast), Category = "Celestial|Utilities")
    static FString Conv_CelestialBodyToString(const FCelestialBody& CelestialBody); 

    /** Planetary Body String Builder */
    UFUNCTION(BlueprintPure, meta = (DisplayName = "To String (PlanetaryBody)", CompactNodeTitle = "->", BlueprintAutocast), Category = "Celestial|Utilities")
    static FString Conv_PlanetaryBodyToString(const FPlanetaryBody& PlanetaryBody);

    /** Stellar Body ToString String Builder */
    UFUNCTION(BlueprintPure, meta = (DisplayName = "To String (StellarBody)", CompactNodeTitle = "->", BlueprintAutocast), Category = "Celestial|Utilities")
    static FString Conv_StellarBodyToString(const FStellarBody& StellarBody); 
	
    /** Right Ascension String Builder */ 
    UFUNCTION(BlueprintPure, meta = (DisplayName = "To String (Right Ascension in hours)"), Category = "Celestial|Utilities")
    static FString Conv_RightAscensionToString(const double& RightAscensionHours);

    /** Declination String Builder */
    UFUNCTION(BlueprintPure, meta = (DisplayName = "To String (Declination in Degrees)"), Category = "Celestial|Utilities")
    static FString Conv_DeclinationToString(const double& DecDegrees);

    /** HMS String Builder */
    UFUNCTION(BlueprintPure, meta = (DisplayName = "To String (Angle in Hours Minutes Seconds)"), Category = "Celestial|Utilities")
    static FString Conv_HMSToString(int32 Hours, int32 Minutes, double Seconds);

    /** DMS String Builder */
    UFUNCTION(BlueprintPure, meta = (DisplayName = "To String (Signed Angle in Degrees, Minutes, Seconds )"), Category = "Celestial|Utilities")
    static FString Conv_DMSToString(bool Sign, int32 Degrees, int32 Minutes, double Seconds);

	/** HMS String Builder */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "To HMS String (Angle in Degrees)"), Category = "Celestial|Utilities")
	static FString Conv_DegreesToHMSString(double Degrees);
	
	/** DMS String Builder */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "To DMS String (Angle in Degrees)"), Category = "Celestial|Utilities")
	static FString Conv_DegreesToDMSString(double Degrees);
    
#pragma endregion

#pragma region Private Functions and variables
	
private:

    /** Because the Dates are expressed around Jan 1, 4713 BC, we need a special Floor function to have the right years continuity
     *
     * Special "Math.floor()" function used by dateToJulianDate()
     */
    static int32 FloorForJulianDate(double JulianDate);

	/* Return a Rotation Matrix around X, for Precession/Nutation computations */ 
	static FMatrix GetRotationXMatrix(double AngleRadians);

	/* Return a Rotation Matrix around Y, for Precession/Nutation computations */
	static FMatrix GetRotationYMatrix(double AngleRadians);

	/* Return a Rotation Matrix around Z, for Precession/Nutation computations */
	static FMatrix GetRotationZMatrix(double AngleRadians);
	

private:
    // Static const Members
	static const double SynodicMonthAverage ;
    static const FMatrix VSOPEclipticToFK5Equatorial;
    static const double SpeedOfLightMetersPerSeconds;
    static const double AstronomicalUnitsMeters;
    static const double ParsecAstronomicalUnits;
	static const double NewMoonReferenceJulianDate;

	#pragma endregion
};
