// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"

CELESTIALVAULT_API DECLARE_LOG_CATEGORY_EXTERN(LogCelestialVault, Log, All);

class FCelestialVaultModule : public IModuleInterface
{
public:

	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};
