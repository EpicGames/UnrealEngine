// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "CelestialDataTypes.h"
#include "CelestialInputDataTypes.h"
#include "DaylightSavings.h"
#include "DaySequenceActor.h"
#include "CelestialVaultDaySequenceActor.generated.h"

#ifndef CELESTIAL_VAULT_ENABLE_DRAW_DEBUG
	#define CELESTIAL_VAULT_ENABLE_DRAW_DEBUG !UE_BUILD_SHIPPING
#endif

#if CELESTIAL_VAULT_ENABLE_DRAW_DEBUG
	// class ;
#endif

// Celestial Classes
class UStarsDataTable;

// UE Classes
class USkyAtmosphereComponent;
class USkyLightComponent;
class UVolumetricCloudComponent;
class UDirectionalLightComponent;
class UExponentialHeightFogComponent;
class UPostProcessComponent;
class UStaticMeshComponent;
class UInstancedStaticMeshComponent;
class UMaterialParameterCollection;


UCLASS(Blueprintable, HideCategories=(Tags, Networking, LevelInstance))
class CELESTIALVAULT_API ACelestialVaultDaySequenceActor
	: public ADaySequenceActor
{
	GENERATED_BODY()

public:
	ACelestialVaultDaySequenceActor(const FObjectInitializer& Init);
 
public: // Properties

#pragma region Components 
	
	UPROPERTY(VisibleDefaultsOnly, BlueprintReadOnly, Category= "Celestial Vault", meta = (AllowPrivateAccess = "true"))
	TObjectPtr<USceneComponent> PlanetCenterComponent;

	UPROPERTY(VisibleDefaultsOnly, BlueprintReadOnly, Category= "Celestial Vault", meta = (AllowPrivateAccess = "true"))
	TObjectPtr<USceneComponent> NorthOffsetComponent;

	UPROPERTY(VisibleDefaultsOnly, BlueprintReadOnly, Category= "Celestial Vault", meta = (AllowPrivateAccess = "true"))
	TObjectPtr<UStaticMeshComponent> CompassComponent;

	UPROPERTY(VisibleDefaultsOnly, BlueprintReadOnly, Category= "Celestial Vault", meta = (AllowPrivateAccess = "true"))
	TObjectPtr<USceneComponent> CelestialVaultComponent;

	UPROPERTY(VisibleDefaultsOnly, BlueprintReadOnly, Category= "Celestial Vault", meta = (AllowPrivateAccess = "true"))
	TObjectPtr<USceneComponent> TopocentricVaultComponent;

	UPROPERTY(VisibleDefaultsOnly, BlueprintReadOnly, Category= "Celestial Vault", meta = (AllowPrivateAccess = "true"))
	TObjectPtr<USceneComponent> TopocentricSiderealComponent;
	
	UPROPERTY(VisibleDefaultsOnly, BlueprintReadOnly, Category= "Celestial Vault", meta = (AllowPrivateAccess = "true"))
	TObjectPtr<UDirectionalLightComponent> SunLightComponent;

	UPROPERTY(VisibleDefaultsOnly, BlueprintReadOnly, Category= "Celestial Vault", meta = (AllowPrivateAccess = "true"))
	TObjectPtr<UDirectionalLightComponent> MoonLightComponent;
	
	UPROPERTY(VisibleDefaultsOnly, BlueprintReadOnly, Category = "Celestial Vault", meta = (AllowPrivateAccess = "true"))
	TObjectPtr<USkyAtmosphereComponent> SkyAtmosphereComponent;
	
	UPROPERTY(VisibleDefaultsOnly, BlueprintReadOnly, Category = "Celestial Vault", meta = (AllowPrivateAccess = "true"))
	TObjectPtr<USkyLightComponent> SkyLightComponent;
	
	UPROPERTY(VisibleDefaultsOnly, BlueprintReadOnly, Category = "Celestial Vault", meta = (AllowPrivateAccess = "true"))
	TObjectPtr<UExponentialHeightFogComponent> ExponentialHeightFogComponent;

	UPROPERTY(VisibleDefaultsOnly, BlueprintReadOnly, Category = "Celestial Vault", meta = (AllowPrivateAccess = "true"))
	TObjectPtr<UPostProcessComponent> GlobalPostProcessVolume;
	
	UPROPERTY(VisibleDefaultsOnly, BlueprintReadOnly, Category = "Celestial Vault", meta = (AllowPrivateAccess = "true"))
	TObjectPtr<UVolumetricCloudComponent> VolumetricCloudComponent;

	UPROPERTY(VisibleDefaultsOnly, BlueprintReadOnly, Category = "Celestial Vault", meta = (AllowPrivateAccess = "true"))
	TObjectPtr<UStaticMeshComponent> DeepSkyComponent;

	/* Because the Deep Sky material is set to IsSky to be included in the SkyLight capture, it doesn't write any velocity vector, creating visual artifacts (streaks) when the vault rotates fast.
	 * This component use a simple opaque material to write the velocities and have the background behaving correctly when the vault rotates quickly. */ 
	UPROPERTY(VisibleDefaultsOnly, BlueprintReadOnly, Category = "Celestial Vault", meta = (AllowPrivateAccess = "true"))
	TObjectPtr<UStaticMeshComponent> VelocityVectorsProxyComponent;

	UPROPERTY(VisibleDefaultsOnly, BlueprintReadOnly, Category = "Celestial Vault", meta = (AllowPrivateAccess = "true"))
	TObjectPtr<UStaticMeshComponent> MoonDiscComponent;

	UPROPERTY(VisibleDefaultsOnly, BlueprintReadOnly, Category = "Celestial Vault", meta = (AllowPrivateAccess = "true"))
	TObjectPtr<UInstancedStaticMeshComponent> StarsComponent;

	UPROPERTY(VisibleDefaultsOnly, BlueprintReadOnly, Category = "Celestial Vault", meta = (AllowPrivateAccess = "true"))
	TObjectPtr<UInstancedStaticMeshComponent> PlanetsComponent;

#pragma endregion

#pragma region DateTime related Properties

	/* If true, ignore the Year Month Day value and use the current system Date */ 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Date")
	bool bUseCurrentDate = false;

	/** Current Year*/ 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Date", meta = (EditConditionHides, EditCondition = "!bUseCurrentDate", ClampMin = "1", ClampMax = "9999"))
	int Year = 2026;

	/** Current Month*/ 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Date", meta = (EditConditionHides, EditCondition = "!bUseCurrentDate", ClampMin = "1", ClampMax = "12"))
	EMonth Month = EMonth::January;

	/** Current Day*/ 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Date", meta = (EditConditionHides, EditCondition = "!bUseCurrentDate", ClampMin = "1", ClampMax = "31"))
	int Day = 1;

	/** At runtime, if we cycle over more than one day, we increase this one */
	UPROPERTY(Transient, VisibleAnywhere, BlueprintReadOnly, Category="Date")
	int SequenceLoopCount = 0;
	
#pragma endregion

#pragma region Daylight Saving related Properties
    
	/** Define how the Daylight Savings are taken into consideration */ 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Daylight Savings")
	EDaylightSavingsMode DaylightSavingsMode = EDaylightSavingsMode::Automatic;
	
	/** Rule that applies to determine the Daylight Saving stating day */ 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Daylight Savings", meta=(EditConditionHides, EditCondition = "DaylightSavingsMode!=EDaylightSavingsMode::None", DisplayName = "Starts On"))
	FDaylightSavingsRule DaylightSavingsStart = FDaylightSavingsRule(EDaylightSavingRuleKind::NthDay, 2, EWeekDay::Sunday, 1, EMonth::March);  // Default to 2nd Sunday of March (US/Canada)

	/** Rule that applies to determine the Daylight Saving ending day */ 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Daylight Savings", meta=(EditConditionHides, EditCondition = "DaylightSavingsMode!=EDaylightSavingsMode::None", DisplayName = "End On"))
	FDaylightSavingsRule DaylightSavingsEnd = FDaylightSavingsRule(EDaylightSavingRuleKind::NthDay, 1, EWeekDay::Sunday, 1, EMonth::November); // Default to 1st Sunday of November (US/Canada)

	/** Time of day when the new Daylight savings actually apply */ 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Daylight Savings",  meta = (EditConditionHides, EditCondition = "DaylightSavingsMode!=EDaylightSavingsMode::None", DisplayName = "Switch Hour", ClampMin=0, ClampMax=23))
	int DaylightSavingsSwitchHour = 2;

	/** Return true if the current Date and Time of the Celestial Vault Actor is inside the Daylight savings period */ 
	UFUNCTION(BlueprintCallable, Category = "Daylight Savings")
	bool IsDaylightSavingsNow() const;

	/** Return true if the current Date Celestial Vault Actor with an arbitrary Time of day is inside the Daylight savings period */
	UFUNCTION(BlueprintCallable, Category = "Daylight Savings")
	bool IsDaylightSavingsAtTimeOfDay(double TimeOfDay) const;

#pragma endregion

#pragma region Location related Properties

	/** Current Time Zone*/ 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Location", meta = (ClampMin = "-11", ClampMax = "14"))
	double GMT_TimeZone = -5.0;

	/**
	* if true, the UE origin is located at the Planet Center (ECEF) otherwise,
	* if false, the UE origin is assuming to be defined at one specific point of the planet surface, defined by the Latitude and Longitude properties.
	**/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Location")
	bool bLevelIsGeocentric = false;
	
	/** Latitude of Level Origin on planet */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Location", meta = (ForceUnits="Degrees", ClampMin = "-90", ClampMax = "90", EditConditionHides, EditCondition = "bLevelIsGeocentric==false"))
	double Latitude = 45.0;

	/** Longitude of Level Origin on planet */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Location", meta = (ForceUnits="Degrees", ClampMin = "-180", ClampMax = "180", EditConditionHides, EditCondition = "bLevelIsGeocentric==false"))
	double Longitude = -73.0;

	/**
	 * North Offset
	 *
	 * In most cases, this value is supposed to be 0.
	 * But in some architecture cases, where the input geometry is aligned to the building main directions and not to the East/North axes,
	 * this value helps rotate artificially the celestial vault to accomodate for this offset.  
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Location", meta = (ForceUnits="Degrees", ClampMin = "0", ClampMax = "360"))
	double NorthOffset = 0.0;
	
	// Greenwich Mean Sidereal Time at corresponding to a 0 Time of Day (midnight in the morning) for the selected Date. 
	UPROPERTY(Transient, BlueprintReadOnly, Category="Location")
	double GMST0_Unwrapped = 0.0;

	// Transform to apply to the planet to have it located tangent to the Origin
	UPROPERTY(Transient, BlueprintReadOnly, VisibleAnywhere, Category="Location")
	FTransform PlanetCenterTransform;
	
#pragma endregion

#pragma region Celestial Vault related Properties

	UPROPERTY(EditAnywhere, BlueprintReadOnly, AdvancedDisplay, Category="Celestial Vault")
	TObjectPtr<UMaterialParameterCollection> CelestialVaultMPC;
	
	/** We generate the sky elements the "Platon" way, using a sphere surrounding the Earth. This is the radius of this sphere. Make sure it's not too small to avoid parallax effects */ 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category="Celestial Vault", meta=( ForceUnits="Kilometers", ClampMin = "8000", ClampMax = "500000"))
	double CelestialVaultDistance = 400000.0;

	/** Percentage of the CelestialVaultDistance at which the Stars are created */ 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category="Celestial Vault", meta=( ForceUnits="Percent", ClampMin = "0.1", ClampMax = "100"))
	double StarsVaultPercentage = 99.0;

	/** Percentage of the CelestialVaultDistance at which the Planets are created */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category="Celestial Vault", meta=( ForceUnits="Percent", ClampMin = "0.1", ClampMax = "100"))
	double PlanetsVaultPercentage = 97.0;
	
	/** Percentage of the CelestialVaultDistance at which the Moons are created */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category="Celestial Vault", meta=( ForceUnits="Percent", ClampMin = "0.1", ClampMax = "100"))
	double MoonVaultPercentage = 95.0;

	/** Some effects are based on the Observer location. (SkyAtmosphere Radius, Light Directions, DeepSky Parallax-avoidance Location) 
	 * We don't need to adjust them every frame, so we trigger an update only if the observer has moved by more than this threshold since the last update. 
	 * 
	 * FYI, on earth, a rough order of magnitude says that the ellipsoid radius changes by 2.2 meters each km. 
	 * It's fine to adjust the radius only if we have moved by some dozens of km... 
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category="Celestial Vault", meta=( ForceUnits="Km", ClampMin = "1", ClampMax = "2000"))
	float ObserverBasedEffectsMovementThreshold = 10;
	
	/** Make sure the SkyAtmosphere is dynamically adjusted to the local ellipsoid radius. (Not needed for applications staying close to the Origin).  
	 *
	 * If true, the system will track the current viewport camera location, extract its Latitude, and adjust the radius of the SkyAtmosphere to the expected ellipsoidal radius
	 * This is a trick to have the round SkyAtmosphere fitting with the Earth surface wherever you're located on Earth
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category="Celestial Vault")
	bool bAdjustSkyAtmosphereToLocalRadius = false;

	/** For visualization at high altitudes, using the theoretical ellipsoidal radius below the camera is not sufficient, as we can see other areas of the earth.
	 * Between the AdjustSkyAtmosphereFadeMinAltitude and AdjustSkyAtmosphereFadeMaxAltitude values, we'll progressly scale the SkyAtmosphere radius toward the Ellipsoid major radius to avoid artifacts
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category="Celestial Vault", meta=( ForceUnits="Km", ClampMin = "10", ClampMax = "2000", EditConditionHides, EditCondition = "bAdjustSkyAtmosphereToLocalRadius==true"))
	float AdjustSkyAtmosphereFadeMinAltitude = 100;

	/** For visualization at high altitudes, using the theoretical ellipsoidal radius below the camera is not sufficient, as we can see other areas of the earth.
	 * Between the AdjustSkyAtmosphereFadeMinAltitude and AdjustSkyAtmosphereFadeMaxAltitude values, we'll progressly scale the SkyAtmosphere radius toward the Ellipsoid major radius to avoid artifacts
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category="Celestial Vault", meta=( ForceUnits="Km", ClampMin = "10", ClampMax = "10000", EditConditionHides, EditCondition = "bAdjustSkyAtmosphereToLocalRadius==true"))
	float AdjustSkyAtmosphereFadeMaxAltitude = 300;
	
#pragma endregion

#pragma region Stars related Properties
	
	/** A Datatable containing a Celestial Star Catalog data */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Stars")
	TObjectPtr<UDataTable> CelestialStarCatalog = nullptr;

	/** A Datatable containing a Fictional Star Catalog data */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Stars")
	TObjectPtr<UDataTable> FictionalStarCatalog = nullptr;
	
	/** All stars from the catalog with a Magnitude dimmer than this threshold won't be generated - Usually 6 is the naked eye visibility limit*/ 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Stars", meta = (ClampMin = "-30", ClampMax = "30"))
	float MaxVisibleMagnitude = 6.0f;

	/** If true, the Stars information will be kept in memory and queryable at runtime */ 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Stars")
	bool bKeepStarsInfo = false;
	
	/** Array of the created Stars information - Only populated if KeepStarsInfo is true */ 
	UPROPERTY(Transient, BlueprintReadOnly, Category="Stars")
	TArray<FStellarBody> Stars;
	
#pragma endregion
	
#pragma region Planets related Properties

	/** The Data Catalog containing all Planets data */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Planets")
	TObjectPtr<UDataTable> PlanetsCatalog = nullptr;

	/** Factor to artificially increase the Planetary bodies size */ 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Planets", meta = (ClampMin = "0"))
	float PlanetsScale = 1.0f;

	/** If true, the Stars information will be kept in memory and queryable at runtime */ 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Planets")
	bool bKeepPlanetsInfos = false;
	
	/** Array of the created planetary bodies, with all their computed information - Only populated if KeepPlanetsInfos is true */ 
	UPROPERTY(Transient, BlueprintReadOnly, Category="Celestial Vault|Planets")
	TArray<FPlanetaryBody> Planets;

#pragma endregion

#pragma region Moon / Sun related Properties

	/** Factor to artificially increase the Moon size */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Moons", meta = (ClampMin = "0"))
	float MoonScale = 2.0f;

	/** If true, the moon Age (Phase) and location can be overriden */  
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Moons")
	bool bManualControl = false;
	
	/** Lunar age. 0 = New Moon, 0.25 = First quarter, 0.5 = Full Moon, 1 = Next New Mo */  
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Moons", meta = (EditConditionHides, EditCondition = "bManualControl==true", ClampMin = "0", ClampMax = "1"))
	float MoonAge = 0.2f;
	
	/** When faking the moon location, we need to give a location relative to the sun
	 * This is a way to control this "Horizontally" using an offset in Right Ascension. 
	 * 
	 * Ex: if MoonOffset_RA is 4, the moon will set 4 hours after the sun, so will be visible only the 4 first hours of the night.
	 * Be mindful, it should normally be correlated with the phase.
	 *   (if HoursBehindSun < 12, we are ~1st quarter, so Age should be < 0.5, and if 12< < 24 age should be > 0.5) 
	 * Any combination is possible, but manual control can lead to inconsistencies. 
	 */  
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Moons", meta = (EditConditionHides, DisplayName="Moon Offset to Sun's Right Ascension", EditCondition = "bManualControl==true", ForceUnits="Hours", ClampMin = "0", ClampMax = "24"))
	float MoonOffset_RA = 12.0f;

	/** When faking the moon location, we need to give a location relative to the sun
	 * This is a way to control this "Vertically" using an offset in Declination. */ 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Moons", meta = (EditConditionHides, DisplayName="Moon Offset to Sun's Declination", EditCondition = "bManualControl==true", ForceUnits="Degrees", ClampMin = "-45", ClampMax = "45"))
	float MoonOffset_DEC = 15.0f;

	/** Base Sun Intensity
	 * Typically 120000 Lux
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category="Celestial Vault", meta=( ForceUnits="Lux", ClampMin = "0", ClampMax = "200000"))
	float SunLightIntensity = 120000.0f;
	
	/** Base Moonlight Intensity (for Full Moon)
	 * Typically 0.1 Lux, up to 0.32 Lux when the moon is at its perigee (SuperMoon)
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category="Celestial Vault", meta=( ForceUnits="Lux", ClampMin = "0", ClampMax = "200000"))
	float MoonLightIntensity = 0.1f;
#pragma endregion 

protected: // Functions

	/** BeginPlay and OnConstruction overrides auto-register this actor with the DaySequenceSubsystem. */
	virtual void BeginPlay() override;
	virtual void OnConstruction(const FTransform& Transform) override;
	virtual void Tick(float DeltaTime) override;

	/** Called when the sequence is updated */ 
	virtual void SequencePlayerUpdated(float CurrentTime, float PreviousTime) override;

public:
	virtual void PostRegisterAllComponents() override;
	virtual void UnregisterAllComponents(bool bForReregister = false) override;
	
public: // Functions

	/** Reset the sequence to its initial state: Initial Day + Initial Time of Day */ 
	UFUNCTION(BlueprintCallable, Category="Playback")
	void Stop();
	
	/** Returns the current defined day, without any Time, because the Time will be controlled by the DaySequence Time of day - Uses "Now" or the Year/Month/Day properties */ 
	UFUNCTION(BlueprintCallable, Category = "Celestial Vault|Date")
	FDateTime GetDate() const;

	/** Returns the current defined day, with the Time defined by the DaySequence Time of day*/ 
	UFUNCTION(BlueprintCallable, Category = "Celestial Vault|Date")
	FDateTime GetDateAndTime() const;

	/** Returns the current defined day, with the Time defined by the DaySequence Time of day*/
	UFUNCTION(BlueprintCallable, Category = "Celestial Vault|Date")
	double GetJulianDate() const;
	
	/** Returns the Celestial Info for the Sun, at a specific Julian Date */  
	UFUNCTION(BlueprintCallable, Category = "Celestial Vault|Sun")
	FStellarBody ComputeSunInfo(double JulianDate) const; 

	/** Returns the Celestial Info for the Sun, at current the Date and Time */  
	UFUNCTION(BlueprintCallable, Category = "Celestial Vault|Sun")
	FStellarBody ComputeCurrentSunInfo() const; 

	/** Returns the Celestial Info for the Sun, at a specific Julian Date */  
	UFUNCTION(BlueprintCallable, Category = "Celestial Vault|Sun")
	FStellarBody ComputeSunInfo_UsingKnownState(double JulianDate, const FPlanetaryBodyKinematicState& EarthKinematicState) const;
	
	/** Returns the Celestial Info for the Moon, at a specific Julian Date */  
	UFUNCTION(BlueprintCallable, Category = "Celestial Vault|Moons")
	FPlanetaryBody ComputeMoonInfo(double JulianDate) const;

	/** Returns the Celestial Info for the Moon, at current the Date and Time */  
	UFUNCTION(BlueprintCallable, Category = "Celestial Vault|Moons")
	FPlanetaryBody ComputeCurrentMoonInfo() const;

	/** Returns the Celestial Info for the Moon, at a specific Julian Date */  
	UFUNCTION(BlueprintCallable, Category = "Celestial Vault|Moons")
	FPlanetaryBody ComputeMoonInfo_UsingKnownState(double JulianDate, const FPlanetaryBodyKinematicState& EarthKinematicState) const;
	
	/** Manually set the Moon Age (Phase) */  
	UFUNCTION(BlueprintCallable, Category = "Celestial Vault|Moons")
	void SetMoonDiscAge(float InMoonAge);
	
// Data Queries
	
	/** Return the Celestial Information of the Star closest to a specific direction, within an angle threshold */
	UFUNCTION(BlueprintCallable, Category = "Celestial Vault|Queries")
	bool GetClosestStar(FVector ObserverLocation, FVector LookupDirection, double ThresholdAngleDegree, FStellarBody& FoundStarInfo, FTransform& StarTransform);
	
	/** Return the Celestial Information of the Planetary Body (moon, planet) closest to a specific direction, within an angle threshold */
	UFUNCTION(BlueprintCallable, Category = "Celestial Vault|Queries")
	bool GetClosestPlanetaryBody(FVector StartPosition, FVector LookupDirection, double ThresholdAngleDegree, FPlanetaryBody& FoundPlanetaryBodyInfo, FTransform& BodyTransform);

	/** Return the Celestial Information of a specific Planetary Body (by its orbit type) */
	UFUNCTION(BlueprintCallable, Category = "Celestial Vault|Queries")
	bool GetPlanetaryBodyByVSOP87Type(EVSOP87BodyType VSOP87Type, FPlanetaryBody& FoundPlanetaryBodyInfo, FTransform& BodyTransform);

	/** Returns the Vault Angle for a specific time - This is the GMST angle... */ 
	UFUNCTION(BlueprintCallable, Category = "Celestial Vault|GMST")
	double GetCelestialVaultAngle(double TimeOfDay) const;
	
#if WITH_EDITOR
	
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent) override;
	
#endif

private:
	void InitStarsComponent();
	void InitPlanetsComponent();
	FPlanetaryBody GetPlanetaryBodyInfo_UsingKnownState(double JulianDay, const FPlanetaryBodyInputData& InputPlanetaryBody, const FPlanetaryBodyKinematicState& EarthKinematicState, bool bIgnoreRelativisticEffect = false) const;
	void UpdatePlanetCenter();

	UFUNCTION(BlueprintCallable, CallInEditor, Category= "Celestial Vault")
	void RebuildAll();

	// Special behavior for celestial bodies motion: 
	// The DaySequence system is designed to loop over one single day.
	// In our case, we want to simulate more than one day, and have for instance the moon properly moving in the sky over midnight.
	// So we won't use a procedural sequence for the bodies state, but move the body location on demand, when the DaySequence is updated.
	// To maintain performance, we'll use a local motion cache updated when needed. 

	/** Moves the celestial bodies when the DaySequence is updated
	 *  The motion cache is automatically build when a state is queried, ensuring we have the proper keyframe around the current time of day 
	 * **/ 
	void UpdateBodiesMotion();

	/** Clear the Bodies Motion Cache for the relevant objects - Sun, Moon, Vault **/
	void ResetBodyMotionCache();

	/** Check if we have the proper keyframes around the desired Time of Day. If not, it will add them */ 
	void EnsureKeysAroundTimeOfDay(double TimeOfDay);

	/** Compute and Add a new KeyFrame in the cache */ 
	void AddBodyMotionCacheKey(double CurveKeyTimeHour);

	// Local Interpolable Curves Cache for runtime-moving bodies
	const unsigned int BodyMotionCacheKeyFramesCount = 24;
	double BodyMotionCacheMin = 0.0;
	double BodyMotionCacheMax = 0.0;
	FInterpCurve<FVector>  MoonDiscLocationCurve;
	FInterpCurve<FQuat>  MoonDiscRotationCurve;
	FInterpCurve<float>  MoonAgeCurve;
	FInterpCurve<FVector>  SunLocationCurve;
	
	FVector LastViewportCameraLocation = FVector::ZeroVector;
	
#if CELESTIAL_VAULT_ENABLE_DRAW_DEBUG

	const FString CelestialVaultSF = TEXT("CelestialVault");
	FDelegateHandle DrawDebugDelegateHandle ;
	void DebugDrawCallback(UCanvas* Canvas, APlayerController* PC) const;

#endif
	
};
