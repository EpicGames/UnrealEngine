// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "DaylightSavings.generated.h"


UENUM(BlueprintType)
enum class EDaylightSavingsMode : uint8
{
	// Do not apply daylight saving time (DST).
	None,
	// Automatically determine the Hemisphere from the Latitude sign (>= 0 = Northern, < 0 = Southern). Consider other N/S hemisphere options if your Origin is Geocentric or different from the logic you want to apply. 
	Automatic,
	// Northern Hemisphere rules (DST typically observed during Apr–Oct, i.e., around June).
	NorthernHemisphere,
	// Southern Hemisphere rules (DST typically observed during Oct–Mar, i.e., around December). The following year is used to compute the end date
	SouthernHemisphere    
};

/** Different Daylight Savings rules that apply across the globe*/
UENUM(BlueprintType)
enum class EDaylightSavingRuleKind : uint8
{
	NthDay,
	LastDay,
	FixedDate
};

/** Enumerates the days of a week in 7-days calendars. */
UENUM(BlueprintType)
enum class EWeekDay : uint8
{
	Monday = 0,
	Tuesday = 1,
	Wednesday = 2,
	Thursday = 3,
	Friday = 4,
	Saturday = 5,
	Sunday = 6
};

/** Enumerates the months of the year in 12-month calendars. */
UENUM(BlueprintType)
enum class EMonth: uint8
{
	None = 0 UMETA(Hidden),
	January = 1,
	February = 2,
	March = 3,
	April = 4,
	May = 5,
	June = 6,
	July = 7,
	August = 8,
	September = 9,
	October = 10,
	November = 11,
	December = 12
};


USTRUCT(BlueprintType)
struct CELESTIALVAULT_API FDaylightSavingsRule
{
	GENERATED_BODY()

	/** The rule defining the day to choose for the Daylight Saving Start/End  */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category= "Daylight Savings")
	EDaylightSavingRuleKind Rule = EDaylightSavingRuleKind::NthDay;

	/** Nth Weekday to consider when the rule is NthDay - eg 3 for the 3rd Sunday of January */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category= "Daylight Savings", meta = (ClampMin=1, ClampMax=5, EditCondition = "Rule == EDaylightSavingRuleKind::NthDay", EditConditionHides))
	uint8 Nth = 1;

	/** Reference Weekday for Nth and LastDay rules */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category= "Daylight Savings", meta = (EditCondition = "Rule == EDaylightSavingRuleKind::NthDay || Rule == EDaylightSavingRuleKind::LastDay", EditConditionHides))
	EWeekDay WeekDay = EWeekDay::Sunday;

	/** Number for the Day of the Month for Fixed Date rule (1..31) */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category= "Daylight Savings", meta = (ClampMin=1, ClampMax=31, EditCondition = "Rule == EDaylightSavingRuleKind::FixedDate", EditConditionHides))
	uint8 DayOfMonth = 1;

	/** Month the rule apply to */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category= "Daylight Savings")
	EMonth Month = EMonth::January;

#if WITH_EDITOR
	void ClampToValidDate(int32 Year)
	{
		const int32 MaxDays = FDateTime::DaysInMonth(Year, FMath::Clamp(static_cast<int32>(Month), 1 , 12));
		DayOfMonth = FMath::Clamp(DayOfMonth, 1, MaxDays);
	}
#endif
};

/** Utilities related to Daylight Savings */
UCLASS()
class CELESTIALVAULT_API UDaylightSavings : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/** Return the Date defined by a specific Daylight Saving Rule */ 
	UFUNCTION(BlueprintPure, Category="Daylight Savings")
	static FDateTime ToDate(const FDaylightSavingsRule& DaylightSavingsRule, int Year);

	/** Return the Date String defined by a specific Daylight Saving Rule */ 
	UFUNCTION(BlueprintPure, Category="Daylight Savings")
	static FString ToString(const FDaylightSavingsRule& DaylightSavingsRule);
	
	/** Return the Day for a specific Date */ 
	UFUNCTION(BlueprintPure, Category="Daylight Savings")
	static EWeekDay GetWeekDay(FDateTime Date);

	/** Return the nth Weekday in a Month for a specific Year and Month - Returns (1..31), or 0 if the nth weekday does not exist */ 
	UFUNCTION(BlueprintPure, Category="Daylight Savings")
	static int NthWeekdayInMonth(int Year, EMonth Month, EWeekDay Weekday, int Nth);

	/** Return the Date of the Last Weekday in a Month for a specific Year and Month (1..31) */
	UFUNCTION(BlueprintPure, Category="Daylight Savings")
	static int LastWeekdayInMonth(int Year, EMonth Month, EWeekDay Weekday);

	/** Check if a specific date is inside a Daylight saving range defined by its rules. */
	UFUNCTION(BlueprintPure, Category="Daylight Savings")
	static bool IsDaylightSavings(FDateTime DateTime, EDaylightSavingsMode Mode, double Latitude, FDaylightSavingsRule StartDay, FDaylightSavingsRule EndDay, int SwitchHour);

private:
	static FString GetMonthString(EMonth Month);
	static FString GetDayString(EWeekDay Day);
	static FString GetOrdinalSuffix(int32 Number);
};
