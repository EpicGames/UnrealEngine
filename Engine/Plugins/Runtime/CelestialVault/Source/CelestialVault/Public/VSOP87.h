// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "VSOP87/VSOP87A_Common.h"
#include "VSOP87.generated.h"

/**
  From VSOP87.doc in https://ftp.imcce.fr/pub/ephem/planets/vsop87/

  Here we extracted the VSOP87 coefficients from the FORTRAN Tables, and generated constant arrays in VSOP87/VSOP87A_<Planet>.h
  
  Being given a Julian date JD expressed in dynamical time (TAI+32.184s) and a body (planets, Earth-Moon Barycenter, or Sun) associated to a version of the theory VSOP87 :
	1/ select the file corresponding to the body and the version,
	2/ read sequentially the terms of the series in the records of the file,
	3/ apply for each term the formulae (1) or (2) with T=(JD-2451545)/365250,
	4/ add up the terms so computed for every one coordinate. 
 */
UCLASS()
class CELESTIALVAULT_API UVSOP87 : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	/** Return the Mercury location in the VSOP87 Dynamical Ecliptic Frame J2000 - Note the input time is a specific VSOP87 Time, not Julian Days */ 
	UFUNCTION(BlueprintCallable, Category="Celestial Vault|VSOP87")
	static FVector GetMercuryLocation(double VSOP87Time, EVSOP87Accuracy Accuracy);

	/** Return the Mercury velocity in the VSOP87 Dynamical Ecliptic Frame J2000 - Note the input time is a specific VSOP87 Time, not Julian Days */ 
	UFUNCTION(BlueprintCallable, Category="Celestial Vault|VSOP87")
	static FVector GetMercuryVelocity(double VSOP87Time, EVSOP87Accuracy Accuracy);

	/** Return the Venus location in the VSOP87 Dynamical Ecliptic Frame J2000 - Note the input time is a specific VSOP87 Time, not Julian Days */ 
	UFUNCTION(BlueprintCallable, Category="Celestial Vault|VSOP87")
	static FVector GetVenusLocation(double VSOP87Time, EVSOP87Accuracy Accuracy);

	/** Return the Venus velocity in the VSOP87 Dynamical Ecliptic Frame J2000 - Note the input time is a specific VSOP87 Time, not Julian Days */ 
	UFUNCTION(BlueprintCallable, Category="Celestial Vault|VSOP87")
	static FVector GetVenusVelocity(double VSOP87Time, EVSOP87Accuracy Accuracy);

	/** Return the Earth location in the VSOP87 Dynamical Ecliptic Frame J2000 - Note the input time is a specific VSOP87 Time, not Julian Days */ 
	UFUNCTION(BlueprintCallable, Category="Celestial Vault|VSOP87")
	static FVector GetEarthLocation(double VSOP87Time, EVSOP87Accuracy Accuracy);

	/** Return the Earth velocity in the VSOP87 Dynamical Ecliptic Frame J2000 - Note the input time is a specific VSOP87 Time, not Julian Days */ 
	UFUNCTION(BlueprintCallable, Category="Celestial Vault|VSOP87")
	static FVector GetEarthVelocity(double VSOP87Time, EVSOP87Accuracy Accuracy);

	/** Return the Earth Moon Barycenter location in the VSOP87 Dynamical Ecliptic Frame J2000 - Note the input time is a specific VSOP87 Time, not Julian Days */ 
	UFUNCTION(BlueprintCallable, Category="Celestial Vault|VSOP87")
	static FVector GetEMBLocation(double VSOP87Time, EVSOP87Accuracy Accuracy);

	/** Return the Earth Moon Barycenter velocity in the VSOP87 Dynamical Ecliptic Frame J2000 - Note the input time is a specific VSOP87 Time, not Julian Days */ 
	UFUNCTION(BlueprintCallable, Category="Celestial Vault|VSOP87")
	static FVector GetEMBVelocity(double VSOP87Time, EVSOP87Accuracy Accuracy);

	/** Return the Moon location in the VSOP87 Dynamical Ecliptic Frame J2000 - Note the input time is a specific VSOP87 Time, not Julian Days */ 
	UFUNCTION(BlueprintCallable, Category="Celestial Vault|VSOP87")
	static FVector GetMoonLocation(double VSOP87Time, EVSOP87Accuracy Accuracy);

	/** Return the Moon velocity in the VSOP87 Dynamical Ecliptic Frame J2000 - Note the input time is a specific VSOP87 Time, not Julian Days */ 
	UFUNCTION(BlueprintCallable, Category="Celestial Vault|VSOP87")
	static FVector GetMoonVelocity(double VSOP87Time, EVSOP87Accuracy Accuracy);

	/** Return the Mars location in the VSOP87 Dynamical Ecliptic Frame J2000 - Note the input time is a specific VSOP87 Time, not Julian Days */ 
	UFUNCTION(BlueprintCallable, Category="Celestial Vault|VSOP87")
	static FVector GetMarsLocation(double VSOP87Time, EVSOP87Accuracy Accuracy);

	/** Return the Mars velocity in the VSOP87 Dynamical Ecliptic Frame J2000 - Note the input time is a specific VSOP87 Time, not Julian Days */ 
	UFUNCTION(BlueprintCallable, Category="Celestial Vault|VSOP87")
	static FVector GetMarsVelocity(double VSOP87Time, EVSOP87Accuracy Accuracy);

	/** Return the Jupiter location in the VSOP87 Dynamical Ecliptic Frame J2000 - Note the input time is a specific VSOP87 Time, not Julian Days */ 
	UFUNCTION(BlueprintCallable, Category="Celestial Vault|VSOP87")
	static FVector GetJupiterLocation(double VSOP87Time, EVSOP87Accuracy Accuracy);

	/** Return the Jupiter velocity in the VSOP87 Dynamical Ecliptic Frame J2000 - Note the input time is a specific VSOP87 Time, not Julian Days */ 
	UFUNCTION(BlueprintCallable, Category="Celestial Vault|VSOP87")
	static FVector GetJupiterVelocity(double VSOP87Time, EVSOP87Accuracy Accuracy);

	/** Return the Saturn location in the VSOP87 Dynamical Ecliptic Frame J2000 - Note the input time is a specific VSOP87 Time, not Julian Days */ 
	UFUNCTION(BlueprintCallable, Category="Celestial Vault|VSOP87")
	static FVector GetSaturnLocation(double VSOP87Time, EVSOP87Accuracy Accuracy);

	/** Return the Saturn velocity in the VSOP87 Dynamical Ecliptic Frame J2000 - Note the input time is a specific VSOP87 Time, not Julian Days */ 
	UFUNCTION(BlueprintCallable, Category="Celestial Vault|VSOP87")
	static FVector GetSaturnVelocity(double VSOP87Time, EVSOP87Accuracy Accuracy);

	/** Return the Uranus location in the VSOP87 Dynamical Ecliptic Frame J2000 - Note the input time is a specific VSOP87 Time, not Julian Days */ 
	UFUNCTION(BlueprintCallable, Category="Celestial Vault|VSOP87")
	static FVector GetUranusLocation(double VSOP87Time, EVSOP87Accuracy Accuracy);

	/** Return the Uranus velocity in the VSOP87 Dynamical Ecliptic Frame J2000 - Note the input time is a specific VSOP87 Time, not Julian Days */ 
	UFUNCTION(BlueprintCallable, Category="Celestial Vault|VSOP87")
	static FVector GetUranusVelocity(double VSOP87Time, EVSOP87Accuracy Accuracy);

	/** Return the Neptune location in the VSOP87 Dynamical Ecliptic Frame J2000 - Note the input time is a specific VSOP87 Time, not Julian Days */ 
	UFUNCTION(BlueprintCallable, Category="Celestial Vault|VSOP87")
	static FVector GetNeptuneLocation(double VSOP87Time, EVSOP87Accuracy Accuracy);

	/** Return the Neptune velocity in the VSOP87 Dynamical Ecliptic Frame J2000 - Note the input time is a specific VSOP87 Time, not Julian Days */ 
	UFUNCTION(BlueprintCallable, Category="Celestial Vault|VSOP87")
	static FVector GetNeptuneVelocity(double VSOP87Time, EVSOP87Accuracy Accuracy);
	
private:

	/** Evaluate the VSOP87 Coefficients for the data arrays a of specific body, to compute its Location*/ 
	static double EvaluateCoordinate(const VSOP87Block (&Blocks)[6], const int32 IndexesForAccuracy[6], double VSOP87Time);

	/** Evaluate the VSOP87 Coefficients for the data arrays a of specific body, to compute its Velocity*/
	static double EvaluateVelocity(const VSOP87Block (&Blocks)[6], const int32 IndexesForAccuracy[6], double VSOP87Time);
};

