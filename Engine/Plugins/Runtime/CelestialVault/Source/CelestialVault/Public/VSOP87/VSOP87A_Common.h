// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "VSOP87A_Common.generated.h"

struct VSOP87Term
{
	double A;
	double B;
	double C;
};

struct VSOP87Block
{
	const VSOP87Term* Terms;
	int32 TermsCount;
};

/** Different type of Accuracies for the VSOP87 Computations Full = Higher, Pico = Lower */
UENUM(BlueprintType)
enum class EVSOP87Accuracy : uint8
{
	Full,
	XXLarge,
	XLarge,
	Large,
	Medium,
	Small,
	XSmall,
	Milli,
	Micro,
	Nano,
	Pico
};
