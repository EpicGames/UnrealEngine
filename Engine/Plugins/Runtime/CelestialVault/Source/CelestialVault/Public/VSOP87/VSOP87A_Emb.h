// Copyright Epic Games, Inc. All Rights Reserved.

// Auto-generated VSOP87A header for EMB (rectangular XYZ, J2000)
// Source: IMCCE/BDL VSOP87A.emb file
// Generated by vsop87a_header_gen_imcce.py

#pragma once

#include "VSOP87A_Common.h"

static const VSOP87Term VSOP87A_EMB_X0[793] = {
    { 9.9982927460000004e-01, 1.7534856847500000e+00, 6.2830758499914000e+03 },
    { 8.3525730000000003e-03, 1.7103453945000000e+00, 1.2566151699982800e+04 },
    { 5.6114416100000002e-03, 0.0000000000000000e+00, 0.0000000000000000e+00 },
    { 1.0466628000000000e-04, 1.6672264522300000e+00, 1.8849227549974199e+04 },
    { 2.5524979999999999e-05, 5.8310207301000005e-01, 5.2969096509459996e+02 },
    { 2.1372560000000000e-05, 1.0923518967200001e+00, 1.5773435424478000e+03 },
    { 1.7091030000000002e-05, 4.9540223396999999e-01, 6.2795527316424004e+03 },
    { 1.7078820000000001e-05, 6.1531554748400001e+00, 6.2865989683403996e+03 },
    { 1.4452420000000000e-05, 3.4727278375999999e+00, 2.3528661537717999e+03 },
    { 1.0910060000000000e-05, 3.6898478246500002e+00, 5.2236939198022001e+03 },
    { 9.3442900000000006e-06, 6.0738992258500000e+00, 1.2036460734888200e+04 },
    { 8.9914399999999994e-06, 3.1757195052300000e+00, 1.0213285546211000e+04 },
    { 7.3445499999999998e-06, 4.3550019652999996e+00, 3.9814900340819997e+02 },
    { 6.8441600000000002e-06, 1.3069902122699999e+00, 5.7533848848968000e+03 },
    { 6.8143699999999999e-06, 2.2182153468500001e+00, 4.7057323075435997e+03 },
    { 6.1123800000000004e-06, 5.3847923432300000e+00, 6.8127668150860000e+03 },
    { 5.6651399999999996e-06, 2.1526203401599999e+00, 1.0593819301891999e+03 },
    { 5.4700400000000004e-06, 1.4614665037600001e+00, 1.4143495242430599e+04 },
    { 5.4095700000000003e-06, 7.8677364655000004e-01, 7.7552261132399997e+02 },
    { 5.2048400000000000e-06, 4.4329579997500002e+00, 7.8604193924392002e+03 },
    { 4.5195300000000001e-06, 1.2793372835400001e+00, 6.2567775301915999e+03 },
    { 4.5183600000000004e-06, 6.0876828086800003e+00, 5.8849268465832001e+03 },
    { 4.4951700000000002e-06, 5.3692383171399998e+00, 6.3093741697912001e+03 },
    { 4.0624799999999999e-06, 5.4361367084000001e-01, 6.6812248533995999e+03 },
    { 2.5581999999999999e-06, 2.2655627724600000e+00, 1.2168002696574600e+04 },
    { 2.2789200000000000e-06, 1.2394148280199999e+00, 7.0585984613153996e+03 },
    { 2.2587800000000002e-06, 3.2724430620699998e+00, 4.6940029547076001e+03 },
    { 2.1496000000000001e-06, 4.5021384457300000e+00, 1.1506769769793600e+04 },
    { 2.0902400000000002e-06, 5.8520752807300003e+00, 1.1790629088658799e+04 },
    { 1.9997100000000000e-06, 4.0720993824500002e+00, 1.7789845619784999e+04 },
    { 1.7832500000000000e-06, 6.2437470460200002e+00, 6.2831431602941902e+03 },
    { 1.7832500000000000e-06, 4.0466470868999999e-01, 6.2830085396885997e+03 },
    { 1.7812000000000000e-06, 2.9620542420399998e+00, 7.9629800681639995e+02 },
    { 1.6120500000000001e-06, 1.4733771895600001e+00, 5.4867778431750003e+03 },
    { 1.5548699999999999e-06, 1.6240930952299999e+00, 2.5132303399965600e+04 },
    { 1.5169100000000001e-06, 8.6921639327000000e-01, 2.1329909543799999e+02 },
    { 1.2893300000000001e-06, 5.2169331415000002e+00, 7.0793738568077997e+03 },
    { 1.2809900000000000e-06, 4.8018288222800001e+00, 3.7387614301079998e+03 },
    { 9.4483000000000002e-07, 4.6018762072900001e+00, 1.0977078804699000e+04 },
    { 8.2956999999999998e-07, 6.1925490883999998e+00, 3.3406124266997999e+03 },
    { 7.9200999999999996e-07, 2.4936671651900002e+00, 6.0697767545533998e+03 },
    { 7.7800999999999996e-07, 1.8444479720000000e+00, 8.8273902698748007e+03 },
    { 7.7276999999999995e-07, 3.5839602732500002e+00, 5.5075532386674004e+03 },
    { 7.5601000000000000e-07, 4.1595602378100001e+00, 6.4963749454294002e+03 },
    { 7.3493000000000003e-07, 2.6952239258400001e+00, 1.5890728952838001e+03 },
    { 6.7771000000000005e-07, 4.0341628366900002e+00, 9.4377629348869996e+03 },
    { 6.5359000000000002e-07, 3.6515634797000001e+00, 1.1769853693166400e+04 },
    { 6.3239999999999997e-07, 2.2412413094399999e+00, 6.2623004544989999e+03 },
    { 6.2961000000000004e-07, 4.4072938194100004e+00, 6.3038512454838001e+03 },
    { 6.2119000000000002e-07, 2.0147229891500000e+00, 7.1135470007999997e+00 },
    { 6.0047999999999999e-07, 3.3995180597900001e+00, 4.1369104335162001e+03 },
    { 5.7573999999999999e-07, 2.6788176694999999e+00, 6.2820955289231997e+03 },
    { 5.7573999999999999e-07, 3.9697159031500000e+00, 6.2840561710596003e+03 },
    { 5.7339999999999999e-07, 3.7945997910300000e+00, 3.1546870848956000e+03 },
    { 5.3715000000000002e-07, 1.5622270603299999e+00, 1.1944470102246000e+03 },
    { 5.2308999999999995e-07, 5.3057159987300002e+00, 1.3367972631106601e+04 },
    { 5.1320999999999997e-07, 5.7845002663200003e+00, 1.2352852604544800e+04 },
    { 5.1129000000000002e-07, 1.3125552235000000e+00, 6.2901893969922003e+03 },
    { 5.0528999999999999e-07, 5.3482765064800004e+00, 6.2759623029905997e+03 },
    { 4.9612999999999998e-07, 2.6437007148400000e+00, 6.1276554505572003e+03 },
    { 4.8968999999999999e-07, 4.5418080759000001e-01, 9.2255392732830005e+03 },
    { 4.8103999999999997e-07, 4.7718056693399999e+00, 8.0182093112380005e+02 },
    { 4.8085000000000002e-07, 4.0088619738000002e+00, 6.4384962494255997e+03 },
    { 4.7567999999999998e-07, 3.1612358599900001e+00, 5.8564776591153995e+03 },
    { 4.7378000000000000e-07, 6.1070517556199997e+00, 3.1283887650958000e+03 },
    { 4.7271000000000000e-07, 6.2147119446999999e-01, 2.5443144198834002e+03 },
    { 4.3380999999999999e-07, 3.6450174952199998e+00, 1.7260154654690399e+04 },
    { 4.1148999999999999e-07, 3.3272630218999999e+00, 8.4292412664666008e+03 },
    { 3.9554000000000001e-07, 4.9302239238499999e+00, 5.4812549188676003e+03 },
    { 3.7987000000000003e-07, 3.0410073450400001e+00, 4.2659819087599999e+02 },
    { 3.6898000000000001e-07, 2.9109713418299998e+00, 1.8073704938650200e+04 },
    { 3.2558999999999999e-07, 1.9742786477100001e+00, 2.1461654164751999e+03 },
    { 3.2546000000000000e-07, 3.0179680830700000e+00, 5.0886288397668004e+03 },
    { 3.2520999999999999e-07, 2.8456687133699998e+00, 4.1643119896130002e+03 },
    { 3.2481000000000001e-07, 8.0957263520000000e-01, 3.9302096962196001e+03 },
    { 3.0321999999999998e-07, 1.9934033077300000e+00, 4.5350594369244000e+03 },
    { 2.8514999999999998e-07, 6.1093302127099998e+00, 1.2569674818331800e+04 },
    { 2.8166000000000000e-07, 4.8127325250000003e-01, 1.2562628581633800e+04 },
    { 2.4936000000000000e-07, 2.1793582878700000e+00, 2.6298319799800002e+01 },
    { 2.3250000000000000e-07, 6.1903031140899998e+00, 1.8319536584879599e+04 },
    { 2.2867000000000000e-07, 4.7643739940399996e+00, 8.0310922630584000e+03 },
    { 2.1880000000000000e-07, 5.1263508859000000e+00, 1.1371704689758200e+04 },
    { 2.1687000000000001e-07, 4.3034890144700002e+00, 1.0447387839604400e+04 },
    { 2.1642000000000000e-07, 4.6352415647800003e+00, 1.2559038152982001e+04 },
    { 2.1065000000000000e-07, 3.8002808878700001e+00, 7.4775228602159996e+03 },
    { 2.0480000000000001e-07, 1.4671366020700001e+00, 7.0848967811151997e+03 },
    { 1.9219999999999999e-07, 4.9428010004000000e-01, 1.7298182327326202e+04 },
    { 1.8593000000000000e-07, 3.3841579585999999e+00, 1.7480164130670000e+03 },
    { 1.7889000000000001e-07, 1.5750451508999999e-01, 1.5925960136327999e+03 },
    { 1.7104000000000000e-07, 5.8684867005999997e-01, 4.9332084403325998e+03 },
    { 1.5048999999999999e-07, 3.1208409604999998e+00, 1.6496361396202399e+04 },
    { 1.4420999999999999e-07, 1.4379185141499999e+00, 2.0426571092422000e+04 },
    { 1.4329000000000001e-07, 6.0279195813299999e+00, 7.3424577801805999e+03 },
    { 1.4285000000000001e-07, 1.2952860784000000e+00, 2.9424634232916001e+03 },
    { 1.3897999999999999e-07, 4.5917195861300000e+00, 8.6359420037632008e+03 },
    { 1.3852000000000000e-07, 6.1624302742100001e+00, 7.6329432596502002e+03 },
    { 1.3605000000000000e-07, 5.1972105562699999e+00, 1.3095842665077400e+04 },
    { 1.3246999999999999e-07, 3.6076817465200000e+00, 1.2139553509106800e+04 },
    { 1.2424999999999999e-07, 1.0639656304600000e+00, 1.5110466119866200e+04 },
    { 1.2375000000000001e-07, 3.1094192327300001e+00, 6.2794854213396002e+03 },
    { 1.2366000000000001e-07, 3.5391144643399999e+00, 6.2866662786431998e+03 },
    { 1.1583000000000000e-07, 5.4607602624500000e+00, 5.3313574437407997e+03 },
    { 1.1583000000000000e-07, 4.0525620639300000e+00, 5.7295064471490005e+03 },
    { 1.1129000000000000e-07, 4.1772806317100004e+00, 1.5720838784878400e+04 },
    { 1.1004000000000000e-07, 2.1927672324000000e+00, 1.0988808157535001e+04 },
    { 1.0546000000000001e-07, 4.7950359672799996e+00, 1.3498674096587999e+03 },
    { 1.0507000000000000e-07, 2.6275802505999999e+00, 6.8366452528338004e+03 },
    { 1.0480000000000000e-07, 2.9528890878100000e+00, 5.9992165311261997e+03 },
    { 9.9509999999999996e-08, 1.2573591659600001e+00, 7.2347942562420003e+03 },
    { 9.8200000000000006e-08, 3.6955619424499999e+00, 6.5669351688566003e+03 },
    { 9.7969999999999996e-08, 1.9195182276400000e+00, 2.1228392023545799e+04 },
    { 9.1710000000000005e-08, 3.9221643965299999e+00, 5.6431785636774002e+03 },
    { 9.0600000000000004e-08, 4.6164319475499997e+00, 4.6904798363585996e+03 },
    { 9.0460000000000004e-08, 5.4301664584099996e+00, 9.6236882766912004e+03 },
    { 8.8689999999999995e-08, 5.0469094181300003e+00, 6.1480107699560003e+03 },
    { 8.6140000000000002e-08, 1.6016117691900000e+00, 6.4181409300267997e+03 },
    { 8.5319999999999999e-08, 2.4619838496700002e+00, 1.4712317116458000e+04 },
    { 8.2269999999999995e-08, 3.2929613711000000e-01, 1.0973555686350001e+04 },
    { 7.9100000000000000e-08, 3.1542957484300000e+00, 2.1187638603783998e+03 },
    { 7.7929999999999997e-08, 4.3041186497200004e+00, 2.2003914634869801e+04 },
    { 7.5450000000000004e-08, 1.2360454922399999e+00, 1.2539853380183000e+04 },
    { 7.5339999999999998e-08, 3.5145713501600002e+00, 1.8422629359098191e+04 },
    { 7.4560000000000001e-08, 5.4252264308399996e+00, 6.0403472460173998e+03 },
    { 7.4120000000000005e-08, 5.0319729086900002e+00, 2.3543230504681789e+04 },
    { 7.2540000000000000e-08, 3.5769054986200000e+00, 5.3680451209540001e+02 },
    { 7.2020000000000006e-08, 5.8839154378499998e+00, 6.2450481773556003e+03 },
    { 7.1439999999999996e-08, 7.6464650348999996e-01, 6.3211035226271997e+03 },
    { 7.0179999999999997e-08, 1.2342019982600001e+00, 6.5258044539654002e+03 },
    { 6.9530000000000001e-08, 2.7438454896500000e+00, 1.5542039943419999e+02 },
    { 6.9269999999999998e-08, 7.1517687323000001e-01, 5.2257741809380002e+02 },
    { 6.6250000000000001e-08, 7.0112927897999999e-01, 5.3274761083827998e+03 },
    { 6.5449999999999994e-08, 1.9822946444700000e+00, 1.0309277421860000e+02 },
    { 6.4049999999999995e-08, 2.2533466984700001e+00, 5.6502921106781996e+03 },
    { 6.3930000000000004e-08, 3.9754339144899999e+00, 2.4072921469776400e+04 },
    { 6.3780000000000006e-08, 6.2090384261800002e+00, 9.5171840625059997e+02 },
    { 6.3590000000000002e-08, 5.0354126102099999e+00, 1.9907450170410000e+03 },
    { 6.3460000000000000e-08, 3.5068776907500001e+00, 5.2308074668030004e+03 },
    { 6.2779999999999997e-08, 6.4265846614000000e-01, 5.2165803728013998e+03 },
    { 6.2629999999999999e-08, 4.3952939616300002e+00, 2.6087903141574199e+04 },
    { 6.2509999999999995e-08, 4.4547909084099997e+00, 6.2885987742988000e+03 },
    { 6.2509999999999995e-08, 2.1937426642400002e+00, 6.2775529256840000e+03 },
    { 6.2239999999999994e-08, 2.2441672322900001e+00, 1.8451078546565990e+04 },
    { 6.1599999999999996e-08, 9.4570829908999998e-01, 1.2964300703391000e+04 },
    { 5.8299999999999999e-08, 2.3286051469100002e+00, 6.3989728631399998e+02 },
    { 5.7770000000000000e-08, 6.1016405995999996e+00, 9.5559974160859997e+02 },
    { 5.7620000000000002e-08, 3.4986658347100001e+00, 1.5510452226479999e+03 },
    { 5.7110000000000000e-08, 4.3271449299300002e+00, 6.9158595893046004e+03 },
    { 5.4140000000000000e-08, 2.9996195788000000e+00, 6.1799830757727996e+03 },
    { 5.4140000000000000e-08, 3.6542482069000002e+00, 6.3861686242100004e+03 },
    { 5.3050000000000002e-08, 3.3435917291999999e+00, 2.5158601719765400e+04 },
    { 4.8039999999999999e-08, 2.8282703293200000e+00, 5.7604984318976003e+03 },
    { 4.7370000000000000e-08, 3.7421099778700002e+00, 1.4314168113049800e+04 },
    { 4.6460000000000000e-08, 1.3349210897399999e+00, 5.5356940284240000e+02 },
    { 4.4090000000000000e-08, 3.8056397780300002e+00, 6.8056532680851997e+03 },
    { 4.3490000000000001e-08, 5.6275237605299999e+00, 7.2386755916000002e+03 },
    { 4.2690000000000000e-08, 2.3836799499899999e+00, 7.8756718636242003e+03 },
    { 4.1190000000000000e-08, 3.2966392198199999e+00, 6.7096740408674004e+03 },
    { 4.0670000000000000e-08, 2.4203377612899999e+00, 3.6346210245184002e+03 },
    { 4.0240000000000002e-08, 5.5473527406000001e-01, 1.1015106477334801e+04 },
    { 4.0170000000000002e-08, 3.7858090761200001e+00, 6.2546266625236003e+03 },
    { 3.9930000000000001e-08, 2.8627212424800002e+00, 6.3115250374591997e+03 },
    { 3.9809999999999997e-08, 1.1968519531299999e+00, 1.3341674311306801e+04 },
    { 3.8990000000000000e-08, 5.5369899702899996e+00, 7.4781598567299994e+01 },
    { 3.6979999999999998e-08, 5.2210576402299997e+00, 5.9666839803348003e+03 },
    { 3.6260000000000002e-08, 1.4263216325300001e+00, 6.5994677196479997e+03 },
    { 3.6249999999999997e-08, 3.0019235393799999e+00, 6.0199919266185998e+03 },
    { 3.6239999999999999e-08, 5.6098185699399998e+00, 6.2082942514241004e+03 },
    { 3.5810000000000001e-08, 1.0399523122000001e+00, 6.3578574485586996e+03 },
    { 3.5800000000000003e-08, 4.1968069203800002e+00, 1.1926254413668799e+04 },
    { 3.5320000000000001e-08, 3.9289801329800000e+00, 9.9176968745098002e+03 },
    { 3.4340000000000002e-08, 3.6465688158499998e+00, 6.5461597733642002e+03 },
    { 3.3829999999999999e-08, 1.7474848943700001e+00, 1.4945316173554400e+04 },
    { 3.3479999999999999e-08, 3.5443346912500000e+00, 1.2592450019782600e+04 },
    { 3.3360000000000002e-08, 1.7979337641599999e+00, 1.0575406682941801e+04 },
    { 3.1720000000000002e-08, 5.1914655501600002e+00, 1.0664954771900000e+03 },
    { 3.1489999999999999e-08, 6.2148736887500000e+00, 4.2923308329503998e+03 },
    { 3.0489999999999997e-08, 2.7658937259999999e-02, 4.7320306273433998e+03 },
    { 3.0250000000000003e-08, 3.1862010183099998e+00, 1.8052929543157799e+04 },
    { 3.0139999999999997e-08, 4.7664087717400001e+00, 2.9088811415985001e+04 },
    { 2.9790000000000001e-08, 6.2006757340199998e+00, 1.2566219010285600e+04 },
    { 2.9790000000000001e-08, 3.6159339669000001e-01, 1.2566084389680000e+04 },
    { 2.7929999999999999e-08, 4.1336943224500002e+00, 3.5231183490000002e+00 },
    { 2.7879999999999999e-08, 3.3083572830599999e+00, 1.2721572099417001e+04 },
    { 2.7420000000000000e-08, 5.0666685244499998e+00, 1.3916019109641600e+04 },
    { 2.7160000000000000e-08, 3.0693530501600002e+00, 1.2029347187887401e+04 },
    { 2.6530000000000000e-08, 4.4222075113599999e+00, 5.8635912061161998e+03 },
    { 2.6110000000000000e-08, 2.4326378763499998e+00, 6.7025604938666002e+03 },
    { 2.6020000000000001e-08, 4.2800154911300003e+00, 4.5909101804889997e+03 },
    { 2.5930000000000001e-08, 4.9122571477500001e+00, 1.6730463689595799e+04 },
    { 2.5349999999999999e-08, 1.5831069320100000e+00, 3.1415379249957001e+04 },
    { 2.5290000000000000e-08, 5.9470904057900000e+00, 1.2043574281888999e+04 },
    { 2.4369999999999999e-08, 6.1289601468599999e+00, 1.4956319713459999e+02 },
    { 2.3879999999999999e-08, 1.3031678718000000e-01, 1.1624747044077999e+03 },
    { 2.3779999999999999e-08, 3.6293627994800000e+00, 2.3888940204492001e+03 },
    { 2.3370000000000000e-08, 4.4074541753399998e+00, 1.0021837280099400e+04 },
    { 2.2650000000000001e-08, 5.9570619255699997e+00, 6.0768903015542001e+03 },
    { 2.2500000000000000e-08, 2.3972893059699998e+00, 1.2669244474201399e+04 },
    { 2.2180000000000001e-08, 6.9324011482000003e-01, 6.4892613984285999e+03 },
    { 2.2160000000000001e-08, 5.2663352919699999e+00, 1.3362449706799200e+04 },
    { 2.1670000000000001e-08, 4.7261256356599999e+00, 2.3013539539587200e+04 },
    { 2.1629999999999999e-08, 1.4179794721400001e+00, 6.0372442037620003e+03 },
    { 2.1050000000000000e-08, 4.6399358783500002e+00, 5.7462713378959997e+03 },
    { 2.0520000000000001e-08, 5.2300656830200003e+00, 6.5289074962207997e+03 },
    { 2.0149999999999998e-08, 2.4843224340200001e+00, 6.2627205305926000e+03 },
    { 2.0090000000000000e-08, 4.0975932468499998e+00, 4.7011165017084004e+03 },
    { 2.0050000000000001e-08, 4.1642388910000001e+00, 6.3034311693902000e+03 },
    { 1.9620000000000000e-08, 2.5060934154999998e-01, 4.6868894077067998e+03 },
    { 1.9370000000000001e-08, 2.2112768063999999e-01, 1.3521751441591399e+04 },
    { 1.8930000000000001e-08, 5.6679589236499996e+00, 6.2869571853493999e+03 },
    { 1.8930000000000001e-08, 9.8057464900000002e-01, 6.2791945146334001e+03 },
    { 1.8839999999999999e-08, 5.2701152320400002e+00, 2.4272860397400001e+02 },
    { 1.8189999999999999e-08, 3.1884890782599999e+00, 1.8875525869773999e+04 },
    { 1.7999999999999999e-08, 6.1890801427500000e+00, 3.3019021112204602e+04 },
    { 1.7940000000000001e-08, 5.7417642774399997e+00, 2.5934124331089399e+04 },
    { 1.7739999999999999e-08, 2.0848449734800001e+00, 6.8198803620868002e+03 },
    { 1.7590000000000001e-08, 1.9888470443000000e-01, 1.3517870106233400e+04 },
    { 1.7400000000000000e-08, 5.4871078155599999e+00, 1.9651048481098002e+04 },
    { 1.7269999999999998e-08, 4.5067465969000002e+00, 1.7654780539749601e+04 },
    { 1.6720000000000000e-08, 5.4372622324900002e+00, 1.6200772724501199e+04 },
    { 1.5489999999999998e-08, 7.1612589777000002e-01, 2.0775395492400001e+01 },
    { 1.5430000000000000e-08, 5.9609828783900003e+00, 6.2724391846416001e+03 },
    { 1.5430000000000000e-08, 6.8755069426000004e-01, 6.2937125153411998e+03 },
    { 1.5430000000000000e-08, 4.2623760903800001e+00, 1.8209330263660191e+04 },
    { 1.4920000000000001e-08, 6.2297509056999996e+00, 9.4114646150871995e+03 },
    { 1.4880000000000000e-08, 5.6744398158399996e+00, 7.6266071275599998e+01 },
    { 1.4769999999999999e-08, 4.1400478753699996e+00, 1.2779450795420800e+04 },
    { 1.4710000000000001e-08, 5.7578339175500002e+00, 1.8635928454536199e+04 },
    { 1.4600000000000000e-08, 4.9723366965900002e+00, 2.4583164622940001e+02 },
    { 1.4550000000000000e-08, 5.2953437999200004e+00, 3.8133035637799999e+01 },
    { 1.4460000000000000e-08, 2.2656303858600002e+00, 1.0522683831884001e+03 },
    { 1.4359999999999999e-08, 2.9973483165800001e+00, 1.1020632121940000e+02 },
    { 1.4270000000000000e-08, 1.6885180918100000e+00, 1.3119721102825190e+04 },
    { 1.3949999999999999e-08, 3.2534739253299998e+00, 1.0177257679533601e+04 },
    { 1.3660000000000000e-08, 1.0916044512300001e+00, 2.3791644735716000e+03 },
    { 1.3490000000000000e-08, 5.7509779369900000e+00, 6.0164688082696002e+03 },
    { 1.3119999999999999e-08, 1.1009156964500000e+00, 1.2491370101415499e+04 },
    { 1.2740000000000000e-08, 5.3425352584299999e+00, 6.3278373931320004e+02 },
    { 1.2720000000000001e-08, 8.9738866810999995e-01, 6.5496828917131998e+03 },
    { 1.2680000000000000e-08, 4.6721710947800004e+00, 5.4298794682394000e+03 },
    { 1.2630000000000000e-08, 9.1728971349999999e-01, 1.2573265246983599e+04 },
    { 1.2509999999999999e-08, 1.5004576984800000e+00, 3.8941818295421999e+03 },
    { 1.2280000000000000e-08, 2.8375034525300000e+00, 2.4356780788641601e+04 },
    { 1.1910000000000000e-08, 3.4300803992200000e+00, 6.2901220866894000e+03 },
    { 1.1910000000000000e-08, 3.2184531734299999e+00, 6.2760296132933900e+03 },
    { 1.1560000000000001e-08, 1.3884760901600000e+00, 1.1499656222792801e+04 },
    { 1.1460000000000000e-08, 3.8586500964699999e+00, 2.0618554843720000e+02 },
    { 1.1320000000000000e-08, 9.6198039943000002e-01, 1.7782732072784202e+04 },
    { 1.0920000000000000e-08, 3.9200210550099999e+00, 1.7796959166785800e+04 },
    { 1.0850000000000000e-08, 1.3286214439199999e+00, 3.6949230808424203e+04 },
    { 1.0449999999999999e-08, 4.6337213066999999e-01, 1.5508615123274400e+04 },
    { 1.0280000000000000e-08, 2.5103002388100002e+00, 6.1124029793722002e+03 },
    { 1.0250000000000001e-08, 2.2176840115199998e+00, 1.2545376304490401e+04 },
    { 1.0160000000000000e-08, 7.1504248238000001e-01, 2.2041264243879999e+02 },
    { 1.0000000000000000e-08, 2.4880432000000000e-01, 1.1403676995575001e+04 },
    { 9.9200000000000005e-09, 4.1381771709999997e+00, 6.4537487206105998e+03 },
    { 9.8799999999999998e-09, 9.8006619972999998e-01, 8.2738208670324002e+03 },
    { 9.8600000000000003e-09, 4.6289746953100002e+00, 2.2805735565993600e+04 },
    { 9.7800000000000006e-09, 4.7385276385099999e+00, 6.2470479833139898e+03 },
    { 9.7100000000000006e-09, 1.9100071842599999e+00, 6.3191037166688002e+03 },
    { 9.6199999999999995e-09, 2.6356854485299999e+00, 1.2565171378914600e+04 },
    { 9.6199999999999995e-09, 3.9265836821800000e+00, 1.2567132021051000e+04 },
    { 9.4400000000000005e-09, 7.8473810598000004e-01, 6.3069542877391996e+03 },
    { 9.4400000000000005e-09, 5.8637954666700001e+00, 6.2591974122436004e+03 },
    { 9.4199999999999993e-09, 1.4206262015500000e+00, 6.2678233788063899e+03 },
    { 9.4199999999999993e-09, 5.2279073710999997e+00, 6.2983283211764001e+03 },
    { 9.3100000000000003e-09, 2.2255846942600002e+00, 2.7870430238573999e+03 },
    { 9.0500000000000000e-09, 2.8331454092500001e+00, 6.2815913772831000e+03 },
    { 9.0500000000000000e-09, 3.8153881633900002e+00, 6.2845603226997000e+03 },
    { 8.9500000000000007e-09, 2.4008509864800001e+00, 3.9039113764198000e+03 },
    { 8.8599999999999996e-09, 5.2313243054300003e+00, 6.1728695287720002e+03 },
    { 8.5899999999999995e-09, 1.4813034280299999e+00, 1.6921656695024001e+03 },
    { 8.5299999999999993e-09, 4.4245506831399997e+00, 9.4917560896980001e+02 },
    { 8.5000000000000000e-09, 1.4520428080500001e+00, 6.3932821712107998e+03 },
    { 8.2800000000000004e-09, 1.5406009521399999e+00, 1.0969965257698201e+04 },
    { 8.2599999999999992e-09, 4.7807193829900001e+00, 1.0984192351699799e+04 },
    { 8.2100000000000004e-09, 5.2122150189000005e-01, 1.2416588502848201e+04 },
    { 7.9799999999999993e-09, 2.6250107754299998e+00, 1.2410731300548599e+04 },
    { 7.8299999999999996e-09, 3.4348155769000002e+00, 8.4608283475120004e+02 },
    { 7.8199999999999999e-09, 8.7676753080000003e-01, 4.9073020501456003e+03 },
    { 7.7900000000000006e-09, 3.5833945174299999e+00, 2.6484548254729998e+03 },
    { 7.7799999999999992e-09, 1.5759027807299999e+00, 8.6622403235630009e+03 },
    { 7.6999999999999995e-09, 6.2017375439900002e+00, 4.8042092759269999e+03 },
    { 7.5400000000000000e-09, 3.8289853879200000e+00, 4.0612192153944002e+03 },
    { 7.4300000000000002e-09, 3.7990393342800002e+00, 7.8341210726394002e+03 },
    { 7.1500000000000003e-09, 5.5031265104199996e+00, 1.7256631536341400e+04 },
    { 7.0699999999999998e-09, 3.2853330615899998e+00, 5.8493641121146002e+03 },
    { 6.9600000000000000e-09, 4.8962358204400003e+00, 1.1712955318230799e+04 },
    { 6.7899999999999999e-09, 2.6045226696000001e-01, 6.1335126528567998e+03 },
    { 6.6500000000000000e-09, 2.7508584237600000e+00, 4.0879440504643797e+04 },
    { 6.4899999999999997e-09, 4.9056876879400004e+00, 1.1764330768858999e+04 },
    { 6.4300000000000003e-09, 6.0638258906000004e+00, 2.6735945262213201e+04 },
    { 6.2300000000000002e-09, 2.9473049829400000e+00, 5.3339002410215999e+03 },
    { 6.2300000000000002e-09, 5.5254659875899996e+00, 5.3267866940207996e+03 },
    { 6.2099999999999999e-09, 1.1453348255200000e+00, 5.2175806283148399e+04 },
    { 6.1600000000000002e-09, 5.7047951673900004e+00, 1.5961864422845999e+03 },
    { 6.1099999999999998e-09, 1.3132120045700000e+00, 1.1933367960669601e+04 },
    { 6.1099999999999998e-09, 1.5315567944999999e+00, 8.5319638175199998e+02 },
    { 6.1000000000000000e-09, 3.9432083221999998e+00, 1.0419986283507600e+04 },
    { 5.9900000000000002e-09, 2.1179325099900002e+00, 6.1515338883049999e+03 },
    { 5.9699999999999999e-09, 4.6931603841999996e+00, 9.7791086761253991e+03 },
    { 5.9500000000000003e-09, 1.3054677776800001e+00, 1.2323423096008801e+04 },
    { 5.8200000000000002e-09, 4.5309240753199997e+00, 6.4146178116778001e+03 },
    { 5.6999999999999998e-09, 4.5466596158600003e+00, 1.2586927095475199e+04 },
    { 5.6400000000000004e-09, 3.9202564734699998e+00, 4.1714255366138004e+03 },
    { 5.5899999999999999e-09, 1.7160958941400000e+00, 1.5819593482830001e+03 },
    { 5.5100000000000002e-09, 4.0387443069900000e+00, 6.5034884924301996e+03 },
    { 5.4899999999999999e-09, 6.3911798006999998e-01, 2.2483848574492589e+04 },
    { 5.4500000000000000e-09, 2.6144107581900000e+00, 6.0626632075526004e+03 },
    { 5.4000000000000004e-09, 3.4044070756400000e+00, 6.2797894925736000e+03 },
    { 5.4000000000000004e-09, 3.2441264970099999e+00, 6.2863622074092000e+03 },
    { 5.3599999999999997e-09, 6.0666059095099998e+00, 1.8852750668323199e+04 },
    { 5.3599999999999997e-09, 3.7022496764500001e+00, 7.2322514589612001e+03 },
    { 5.3199999999999998e-09, 4.5876215341200002e+00, 1.8842114002973391e+04 },
    { 5.3199999999999998e-09, 2.2623731981000000e-01, 2.8385931886520001e+02 },
    { 5.3100000000000001e-09, 3.0068454149399999e+00, 3.4960328261340001e+03 },
    { 5.3000000000000003e-09, 4.3491203890000002e-01, 1.8845704431625200e+04 },
    { 5.2800000000000000e-09, 5.6284151405599996e+00, 1.3506508003540000e+02 },
    { 5.2700000000000002e-09, 6.2482731626700003e+00, 2.4602612434870989e+04 },
    { 5.1700000000000001e-09, 2.9309652123099998e+00, 4.3371173787679999e+02 },
    { 5.1499999999999998e-09, 1.6707519331800000e+00, 1.2320320053753399e+04 },
    { 5.1300000000000003e-09, 3.7029160544500002e+00, 1.2455945378763399e+04 },
    { 5.1000000000000002e-09, 4.3819840231500002e+00, 1.1513883316794399e+04 },
    { 5.0499999999999997e-09, 4.6609632759900004e+00, 1.7157061880471800e+04 },
    { 5.0199999999999996e-09, 9.7407194728000002e-01, 1.2489885628707199e+04 },
    { 4.9300000000000001e-09, 6.1192029034999997e+00, 2.9296615389578601e+04 },
    { 4.9099999999999998e-09, 5.9674080350800001e+00, 1.4788665740644001e+03 },
    { 4.8499999999999996e-09, 1.3544419832100001e+00, 1.2528018664345000e+04 },
    { 4.7900000000000002e-09, 1.3358059533900000e+00, 5.5471993364596001e+03 },
    { 4.7799999999999996e-09, 1.8524862207899999e+00, 1.4227094001599999e+01 },
    { 4.7699999999999999e-09, 3.5065795158399999e+00, 9.3880059094152002e+03 },
    { 4.6200000000000002e-09, 1.9661177594399999e+00, 3.1049300594237998e+03 },
    { 4.5100000000000003e-09, 9.1836677580000003e-01, 2.9864334027309000e+04 },
    { 4.4800000000000002e-09, 5.0981503458499997e+00, 4.2745183108323999e+03 },
    { 4.3100000000000002e-09, 5.3114594756000004e+00, 7.0189523635231999e+03 },
    { 4.2100000000000001e-09, 4.8274512082500003e+00, 7.4455505543992003e+03 },
    { 4.1899999999999998e-09, 5.0831890512099998e+00, 1.5671081759406599e+04 },
    { 4.1899999999999998e-09, 6.2417554081000004e-01, 5.6421982426092000e+03 },
    { 4.1700000000000003e-09, 3.7444351282600001e+00, 1.3760598710207400e+04 },
    { 4.1400000000000002e-09, 3.3220434785099999e+00, 2.2779437246193800e+04 },
    { 4.1400000000000002e-09, 4.1940257588999996e+00, 4.4809650200863398e+04 },
    { 4.0499999999999999e-09, 5.3949391859600002e+00, 1.0873986030480401e+04 },
    { 4.0100000000000000e-09, 4.5044300085400000e+00, 1.7267268201691189e+04 },
    { 3.9499999999999998e-09, 5.3942804309600003e+00, 5.9057022420756002e+03 },
    { 3.9499999999999998e-09, 7.7269103310000004e-02, 4.1760413424489998e+03 },
    { 3.9400000000000000e-09, 6.1041105078699998e+00, 4.1571984426121999e+03 },
    { 3.9300000000000003e-09, 4.3718465992300004e+00, 1.4919017853754600e+04 },
    { 3.9300000000000003e-09, 1.5671115341399999e+00, 7.8721487452751999e+03 },
    { 3.9199999999999997e-09, 1.3030590627800001e+00, 4.1948464387519999e+02 },
    { 3.8899999999999996e-09, 3.2180600700899999e+00, 6.3212088856292003e+03 },
    { 3.8899999999999996e-09, 3.4304735025599999e+00, 6.2449428143535997e+03 },
    { 3.8799999999999998e-09, 1.1396322422500000e+00, 6.2973029439929996e+03 },
    { 3.8799999999999998e-09, 5.5089013303999996e+00, 6.2688487559898003e+03 },
    { 3.8700000000000001e-09, 6.8719040069000004e-01, 1.7253041107689591e+04 },
    { 3.8700000000000001e-09, 4.9044025160999999e-01, 9.5628915597059995e+02 },
    { 3.8600000000000003e-09, 2.2904462301900002e+00, 2.6367254726370002e+03 },
    { 3.7900000000000004e-09, 8.1831291068000001e-01, 3.1851920272655998e+03 },
    { 3.7099999999999998e-09, 5.7623382869700004e+00, 1.3625533630172000e+04 },
    { 3.6699999999999999e-09, 1.1872291443400000e+00, 3.0666154958432799e+04 },
    { 3.6500000000000000e-09, 2.1102060785400001e+00, 6.9229731363053997e+03 },
    { 3.6500000000000000e-09, 1.2542529787500001e+00, 6.6604494579071998e+03 },
    { 3.6300000000000001e-09, 4.2906977834899998e+00, 7.3353442331797996e+03 },
    { 3.5800000000000000e-09, 3.1573799973800001e+00, 6.2713464971554004e+03 },
    { 3.5800000000000000e-09, 3.4911535752699998e+00, 6.2948052028273996e+03 },
    { 3.5800000000000000e-09, 8.1490874525000001e-01, 2.1393541969857601e+04 },
    { 3.5100000000000001e-09, 2.2188770728699998e+00, 1.1616976091013001e+04 },
    { 3.5100000000000001e-09, 5.9339545545899997e+00, 1.2345739057544000e+04 },
    { 3.4800000000000000e-09, 4.2756570230900000e+00, 3.9321532631000000e+00 },
    { 3.4400000000000001e-09, 3.5155974533299998e+00, 6.1474505249814001e+03 },
    { 3.3900000000000001e-09, 6.1082891190100002e+00, 6.9239534573736000e+03 },
    { 3.3600000000000000e-09, 5.2352268951500003e+00, 6.2870080032545002e+03 },
    { 3.3600000000000000e-09, 1.4133066775000001e+00, 6.2791436967282998e+03 },
    { 3.3400000000000000e-09, 3.1330310222900000e+00, 6.4187011750013999e+03 },
    { 3.3000000000000002e-09, 3.6811386711299998e+00, 3.8027672635800002e+01 },
    { 3.2900000000000000e-09, 1.1567655897100000e+00, 3.7737360791579999e+02 },
    { 3.2700000000000001e-09, 3.9523222390999999e+00, 2.0995392966449399e+04 },
    { 3.2599999999999999e-09, 4.2470223317300002e+00, 1.6522659716002199e+04 },
    { 3.2500000000000002e-09, 2.4927799822300001e+00, 1.2359966151545599e+04 },
    { 3.2000000000000001e-09, 3.1727575690000002e-02, 1.6858482532933202e+04 },
    { 3.2000000000000001e-09, 2.4979925318300000e+00, 1.0818135286915800e+04 },
    { 3.1599999999999998e-09, 5.5420536673000003e+00, 6.0555997172023999e+03 },
    { 3.1599999999999998e-09, 3.6194124995999999e-01, 2.1954157609397989e+04 },
    { 3.1399999999999999e-09, 4.9935971788900000e+00, 1.7996031168222202e+04 },
    { 3.1099999999999998e-09, 3.2391193749700000e+00, 1.1720068865231600e+04 },
    { 3.0699999999999999e-09, 5.3026103322899996e+00, 5.1206011455835996e+03 },
    { 3.0399999999999998e-09, 1.5986036979000001e-01, 5.7498617665477996e+03 },
    { 3.0399999999999998e-09, 4.9365704073499996e+00, 2.9826306354673201e+04 },
    { 3.0100000000000002e-09, 1.1064800028099999e+00, 6.5105519827804001e+03 },
    { 3.0000000000000000e-09, 3.8708387505099999e+00, 6.4087760738220004e+02 },
    { 2.9800000000000000e-09, 5.1505951878499996e+00, 1.0344295065385801e+04 },
    { 2.9699999999999999e-09, 6.0166488958000004e+00, 2.0597243963041201e+04 },
    { 2.9600000000000001e-09, 7.8995346570000000e-02, 6.1292970395066004e+03 },
    { 2.8900000000000002e-09, 5.5454156938999999e-01, 6.2068097787158003e+03 },
    { 2.8600000000000000e-09, 2.8647339667999999e-01, 6.4368546604761996e+03 },
    { 2.8499999999999999e-09, 2.5049468105299999e+00, 5.4695255660315997e+03 },
    { 2.8299999999999999e-09, 2.5787732395899998e+00, 3.1639186965660002e+02 },
    { 2.7400000000000001e-09, 1.1116207172799999e+00, 2.6709646942413401e+04 },
    { 2.7299999999999999e-09, 4.2964839949800000e+00, 2.8286990484861199e+04 },
    { 2.6900000000000000e-09, 2.0537129126000001e-01, 6.8162899334350004e+03 },
    { 2.6799999999999998e-09, 4.9583513753000004e+00, 1.0454501386605200e+04 },
    { 2.6700000000000001e-09, 3.4200712534200002e+00, 2.4705705209089589e+04 },
    { 2.6500000000000002e-09, 5.1101029199100001e+00, 1.9378918515068799e+04 },
    { 2.6500000000000002e-09, 5.6198583163700002e+00, 4.8739859897082999e+04 },
    { 2.6299999999999998e-09, 6.1131757191200000e+00, 9.3809596727171993e+03 },
    { 2.5899999999999999e-09, 3.4312738259500000e+00, 6.2828320325078903e+03 },
    { 2.5899999999999999e-09, 3.2172597466999999e+00, 6.2833196674748997e+03 },
    { 2.5599999999999998e-09, 2.3014358587500001e+00, 2.2434479570190001e+02 },
    { 2.5500000000000001e-09, 4.9464283187499998e+00, 1.3757737998458001e+03 },
    { 2.5200000000000000e-09, 5.1301411029999999e-01, 4.3776110850510004e+03 },
    { 2.5000000000000001e-09, 4.2418329242799997e+00, 5.9341514295433999e+03 },
    { 2.4600000000000002e-09, 5.8373402886100001e+00, 8.6719698704406001e+03 },
    { 2.4300000000000001e-09, 4.1017288352600003e+00, 6.3104774060882000e+03 },
    { 2.4300000000000001e-09, 2.5468047373900000e+00, 6.2556742938945999e+03 },
    { 2.4300000000000001e-09, 4.1434639034899998e+00, 7.0966261339512002e+03 },
    { 2.4100000000000002e-09, 2.8788321480000001e-01, 8.3901103575338002e+03 },
    { 2.4000000000000000e-09, 5.0960508401600002e+00, 1.5906764126682599e+04 },
    { 2.3899999999999998e-09, 1.3316576364799999e+00, 1.0440274292603601e+04 },
    { 2.3400000000000002e-09, 3.3689391003900000e+00, 4.4878174062704002e+03 },
    { 2.3400000000000002e-09, 2.6119235806800001e+00, 3.4596364654652403e+04 },
    { 2.3300000000000000e-09, 2.0265728774500000e+00, 1.4844727083000000e+00 },
    { 2.2999999999999999e-09, 2.4069234113200002e+00, 6.6320002704394001e+03 },
    { 2.2900000000000002e-09, 1.0138412281800000e+00, 1.2655674786264001e+03 },
    { 2.2499999999999999e-09, 4.0165687908300001e+00, 5.6360650166766000e+03 },
    { 2.2100000000000000e-09, 4.9899806160500004e+00, 2.6660704172179999e+02 },
    { 2.1999999999999998e-09, 7.3345029623000002e-01, 1.1087285125918401e+04 },
    { 2.1700000000000002e-09, 4.5262028457400003e+00, 3.0978838227257902e+03 },
    { 2.1400000000000001e-09, 2.0948771196700000e+00, 2.2218566345969998e+03 },
    { 2.1400000000000001e-09, 4.3411639241600000e+00, 2.4492406113651588e+04 },
    { 2.1299999999999999e-09, 3.7943189664300001e+00, 3.0355997319767801e+04 },
    { 2.1299999999999999e-09, 4.2160996312199996e+00, 1.9804827291582798e+04 },
    { 2.1200000000000001e-09, 1.7547160672000000e+00, 7.1291586847425997e+03 },
    { 2.0700000000000001e-09, 3.0662869439899998e+00, 1.2562561271331000e+04 },
    { 2.0700000000000001e-09, 3.4959821867200001e+00, 1.2569742128634600e+04 },
    { 1.9900000000000000e-09, 9.3488663036999997e-01, 1.1216284290324000e+04 },
    { 1.9300000000000002e-09, 3.5744299435200002e+00, 3.3326578733174203e+04 },
    { 1.9200000000000000e-09, 2.0474865829100000e+00, 1.7271884007526402e+04 },
    { 1.9000000000000001e-09, 6.2691743670799998e+00, 3.9462588505920002e+02 },
    { 1.9000000000000001e-09, 6.6924538276000001e-01, 2.3581258177317599e+04 },
    { 1.8700000000000000e-09, 1.5192842664000000e-01, 1.1609862544012200e+04 },
    { 1.8700000000000000e-09, 5.5882538287000001e+00, 4.0167212175719999e+02 },
    { 1.8700000000000000e-09, 5.7009996264199998e+00, 2.0452869412221800e+04 },
    { 1.8400000000000001e-09, 3.4848335816699998e+00, 5.5400857894587998e+03 },
    { 1.8199999999999999e-09, 5.6921237819600004e+00, 2.8766924424484001e+04 },
    { 1.8199999999999999e-09, 3.6647020934599999e+00, 2.8449187467800002e+01 },
    { 1.7900000000000000e-09, 1.0750315336600000e+00, 1.6460333529524989e+04 },
    { 1.7800000000000001e-09, 3.7257858731800000e+00, 1.2132439962106000e+04 },
    { 1.7700000000000001e-09, 4.7308916814400002e+00, 4.7970957289261996e+03 },
    { 1.7599999999999999e-09, 4.0519945587399997e+00, 1.2012582297140400e+04 },
    { 1.7500000000000000e-09, 5.4117453952299996e+00, 1.1614433293732200e+04 },
    { 1.7500000000000000e-09, 4.5821257930000003e+00, 3.7448273457378000e+03 },
    { 1.7400000000000000e-09, 2.8520790096000002e+00, 2.6308392337279997e+02 },
    { 1.7200000000000001e-09, 5.3897331236700001e+00, 6.3328328754632003e+03 },
    { 1.7200000000000001e-09, 1.2588004489800000e+00, 6.2333188245195997e+03 },
    { 1.6999999999999999e-09, 6.3813817824999997e-01, 6.2720301497275004e+03 },
    { 1.6999999999999999e-09, 6.0103953943999997e+00, 6.2941215502552996e+03 },
    { 1.6999999999999999e-09, 7.6594212114000004e-01, 5.2670069593302600e+04 },
    { 1.6700000000000000e-09, 3.9738803433499998e+00, 5.7142539759640003e+03 },
    { 1.6700000000000000e-09, 5.5407088034900003e+00, 1.7686752845566400e+04 },
    { 1.6700000000000000e-09, 1.2698650871399999e+00, 1.7473453750128399e+04 },
    { 1.6300000000000000e-09, 3.1156098284699998e+00, 6.6123292678099995e+02 },
    { 1.6300000000000000e-09, 4.3584891037700002e+00, 9.9294262273458007e+03 },
    { 1.6200000000000000e-09, 4.7875415568499999e+00, 1.2551924605981199e+04 },
    { 1.6000000000000001e-09, 2.3718729510599998e+00, 7.4299006053259995e+02 },
    { 1.5900000000000001e-09, 3.9024608607100002e+00, 1.3198935439295999e+04 },
    { 1.5700000000000000e-09, 5.4330209813200003e+00, 1.6840670010815189e+04 },
    { 1.5600000000000000e-09, 2.5778895745399999e+00, 4.3075833041353999e+03 },
    { 1.5500000000000000e-09, 2.7598891873999998e-01, 2.0199094959632999e+04 },
    { 1.5500000000000000e-09, 4.0574405637900002e+00, 6.0587310542895002e+03 },
    { 1.5400000000000001e-09, 4.0647592222300002e+00, 3.8526574350871997e+04 },
    { 1.5000000000000000e-09, 9.6205875343000002e-01, 1.9800945956224801e+04 },
    { 1.4800000000000001e-09, 2.6746531884900002e+00, 6.8518977240187996e+03 },
    { 1.4700000000000001e-09, 4.9918838893600004e+00, 1.6737577236596600e+04 },
    { 1.4599999999999999e-09, 2.3801153618000001e-01, 1.7370360975909800e+04 },
    { 1.4599999999999999e-09, 1.5584860534600000e+00, 1.2701216780018200e+04 },
    { 1.4500000000000000e-09, 2.7425003039800000e+00, 6.3607496204194003e+03 },
    { 1.4500000000000000e-09, 3.9060332686699999e+00, 6.2054020795633996e+03 },
    { 1.4300000000000000e-09, 8.5202449839000005e-01, 5.6183198048614004e+03 },
    { 1.4300000000000000e-09, 2.3215191324300002e+00, 5.0103948243641998e+03 },
    { 1.4300000000000000e-09, 2.1356348883799998e+00, 9.5979227217800002e+01 },
    { 1.4300000000000000e-09, 1.5119978147899999e+00, 2.0690068349066000e+03 },
    { 1.4300000000000000e-09, 9.1985752496999995e-01, 1.8772961478698599e+04 },
    { 1.3999999999999999e-09, 1.2226170746700000e+00, 2.0085575391590000e+03 },
    { 1.3800000000000000e-09, 2.0893505142100000e+00, 6.3563729758503996e+03 },
    { 1.3800000000000000e-09, 4.5591830584400004e+00, 6.2097787241324004e+03 },
    { 1.3700000000000000e-09, 2.6013302540400001e+00, 1.1045700263900001e+01 },
    { 1.3700000000000000e-09, 2.5441507816300000e+00, 6.3156084007828003e+03 },
    { 1.3700000000000000e-09, 4.1043827910199999e+00, 6.2505432991999996e+03 },
    { 1.3500000000000001e-09, 5.3492979914200003e+00, 7.3587651353180001e+02 },
    { 1.3500000000000001e-09, 7.8159625159999996e-02, 1.2431086619947400e+04 },
    { 1.3500000000000001e-09, 3.3869868167799999e+00, 3.5315282502998002e+03 },
    { 1.3500000000000001e-09, 5.7059381593000003e-01, 1.8699664352839602e+04 },
    { 1.3399999999999999e-09, 3.0320300487499998e+00, 1.1925274092600601e+04 },
    { 1.3100000000000000e-09, 3.6118529931599999e+00, 1.8415515812097390e+04 },
    { 1.2799999999999999e-09, 5.4133721007800002e+00, 1.2934842235826000e+03 },
    { 1.2600000000000000e-09, 1.1929426623799999e+00, 1.8822929230174399e+04 },
    { 1.2400000000000001e-09, 5.7965091504800004e+00, 6.9478318951213996e+03 },
    { 1.2400000000000001e-09, 1.3739806022100001e+00, 8.9828106693089994e+03 },
    { 1.2300000000000001e-09, 2.3761666624200002e+00, 2.4734154396557398e+04 },
    { 1.2300000000000001e-09, 1.0059441822099999e+00, 6.3058510514421996e+03 },
    { 1.2300000000000001e-09, 5.6425893904400004e+00, 6.2603006485406004e+03 },
    { 1.2199999999999999e-09, 5.6926425552700000e+00, 3.5833410306738001e+03 },
    { 1.2000000000000000e-09, 1.1914090758000000e+00, 1.2808880303956799e+04 },
    { 1.2000000000000000e-09, 5.8407690283199996e+00, 1.2528124027346999e+04 },
    { 1.2000000000000000e-09, 4.9574094579899999e+00, 9.8146041002912007e+03 },
    { 1.2000000000000000e-09, 2.0879841272599999e+00, 1.4158747713615599e+04 },
    { 1.1900000000000000e-09, 1.1969585645700000e+00, 1.9247376553382401e+04 },
    { 1.1900000000000000e-09, 4.2870821661700003e+00, 1.2249759830326200e+04 },
    { 1.1900000000000000e-09, 5.9888261674100001e+00, 1.7892938394003591e+04 },
    { 1.1800000000000001e-09, 1.2392557983600001e+00, 6.3027456108911902e+03 },
    { 1.1800000000000001e-09, 5.4092777742899996e+00, 6.2634060890915998e+03 },
    { 1.1700000000000001e-09, 7.3795877913999997e-01, 1.1610551958374201e+04 },
    { 1.1700000000000001e-09, 5.4481165054899998e+00, 4.2456784047091598e+04 },
    { 1.1599999999999999e-09, 6.0146141700000000e-01, 2.4065807922775592e+04 },
    { 1.1500000000000000e-09, 2.1467623290600000e+00, 1.9402796952816600e+04 },
    { 1.1500000000000000e-09, 3.7822033085500002e+00, 3.6417345715192000e+03 },
    { 1.1500000000000000e-09, 4.6654446906400002e+00, 1.2570083853245900e+04 },
    { 1.1400000000000000e-09, 3.7537209348100000e+00, 6.1550570066540004e+03 },
    { 1.1400000000000000e-09, 2.8948126378399999e+00, 6.4110946933287996e+03 },
    { 1.1400000000000000e-09, 4.0708049872499998e+00, 8.2585683958474001e+03 },
    { 1.1300000000000000e-09, 1.9541356849900000e+00, 2.1116503133776000e+03 },
    { 1.1300000000000000e-09, 2.6550816649800000e+00, 2.4336005393149200e+04 },
    { 1.1300000000000000e-09, 2.0504992251799998e+00, 1.2985636343857999e+04 },
    { 1.1200000000000001e-09, 5.4890513822699996e+00, 1.2188778092066999e+04 },
    { 1.1100000000000001e-09, 2.2744235820799998e+00, 6.3399742249270003e+03 },
    { 1.1100000000000001e-09, 4.3741099905700001e+00, 6.2261774750557997e+03 },
    { 1.0999999999999999e-09, 2.3348062117600001e+00, 1.6062184526116800e+04 },
    { 1.0999999999999999e-09, 4.8866712738000002e+00, 6.2764472910913901e+03 },
    { 1.0999999999999999e-09, 1.7618622988499999e+00, 6.2897044088914099e+03 },
    { 1.0999999999999999e-09, 2.7912025799300002e+00, 6.3125053585273999e+03 },
    { 1.0999999999999999e-09, 3.8573309927200001e+00, 6.2536463414554000e+03 },
    { 1.0999999999999999e-09, 2.1923451020500000e+00, 5.6600279289522201e+04 },
    { 1.0700000000000000e-09, 2.1662217654600000e+00, 1.2463058925764200e+04 },
    { 1.0600000000000001e-09, 1.6765570385199999e+00, 1.1190377900137000e+04 },
    { 1.0600000000000001e-09, 3.2768065201900001e+00, 1.2772337248420001e+04 },
    { 1.0600000000000001e-09, 1.7791989026199999e+00, 2.3539707386332801e+04 },
    { 1.0500000000000001e-09, 2.7747878557100001e+00, 6.4349731310766001e+03 },
    { 1.0500000000000001e-09, 3.8737457169399998e+00, 6.1311785689061999e+03 },
    { 1.0399999999999999e-09, 2.1506104432700002e+00, 1.2560628775675401e+04 },
    { 1.0300000000000000e-09, 2.7580955186499998e+00, 6.3246266409762002e+03 },
    { 1.0300000000000000e-09, 3.8904380540000001e+00, 6.2415250590065998e+03 },
    { 1.0300000000000000e-09, 5.7608342050200001e+00, 2.3937856389740999e+04 },
    { 1.0300000000000000e-09, 4.3774890395700004e+00, 1.8202216716659390e+04 },
    { 1.0200000000000000e-09, 2.8978359345800002e+00, 5.7676119788983997e+03 },
    { 1.0200000000000000e-09, 3.7506976380600001e+00, 6.7985397210844003e+03 },
    { 1.0100000000000000e-09, 3.4237278378100000e+00, 1.9004647949408401e+04 },
    { 1.0100000000000000e-09, 2.3009068890000001e-01, 1.8429742906099000e+04 },
    { 1.0000000000000001e-09, 4.3034403066099998e+00, 1.6627370915377200e+04 },
    { 1.0000000000000001e-09, 5.3779466845600004e+00, 5.3161049725557996e+03 },
    { 9.9000000000000010e-10, 3.6195816158300000e+00, 6.4326390471260002e+03 },
    { 9.9000000000000010e-10, 5.8169736415999995e-01, 6.3139899756264003e+03 },
    { 9.9000000000000010e-10, 6.0668362084899998e+00, 6.2521617243563996e+03 },
    { 9.9000000000000010e-10, 2.4089885462799998e+00, 3.3794543723528601e+04 },
    { 9.9000000000000010e-10, 4.3577814814900000e+00, 5.0175083713650001e+03 },
    { 9.7999999999999992e-10, 5.0767465529100004e+00, 1.6310979045720600e+04 },
    { 9.7999999999999992e-10, 5.5751471024799999e+00, 5.8151108596369904e+03 },
    { 9.7999999999999992e-10, 3.2338128520699998e+00, 6.1266751294890000e+03 },
    { 9.7999999999999992e-10, 3.4147207205800001e+00, 6.4394765704938000e+03 },
    { 9.6999999999999996e-10, 6.0382579114899997e+00, 2.7043502883182799e+04 },
    { 9.6999999999999996e-10, 2.7231106216000001e-01, 1.0557594160823801e+04 },
    { 9.6999999999999996e-10, 4.0215978444299996e+00, 1.0239583866010800e+04 },
    { 9.6999999999999996e-10, 9.2251430622999997e-01, 2.4383079108441401e+04 },
    { 9.6999999999999996e-10, 4.7685700190800002e+00, 9.2747689166822587e+04 },
    { 9.5999999999999999e-10, 5.9758474776100003e+00, 2.6997348193175999e+03 },
    { 9.5999999999999999e-10, 2.6449403409399999e+00, 7.3495713271814002e+03 },
    { 9.5999999999999999e-10, 1.9849785856900000e+00, 5.8884499649321997e+03 },
    { 9.5999999999999999e-10, 4.6635549869600004e+00, 6.6777017350506003e+03 },
    { 9.5000000000000003e-10, 5.6330779861400000e+00, 1.1300584221356399e+04 },
    { 9.5000000000000003e-10, 4.1845241096899999e+00, 7.8263709424722590e+04 },
    { 9.5000000000000003e-10, 6.2289841825000003e-01, 1.1080171578917600e+04 },
    { 9.2999999999999999e-10, 1.3518902832799999e+00, 1.8216443810661000e+04 },
    { 9.0999999999999996e-10, 2.9945747632600002e+00, 1.2829235623355600e+04 },
    { 9.0999999999999996e-10, 2.4752650665500000e+00, 6.6527756659318002e+03 },
    { 9.0999999999999996e-10, 4.1732685061000003e+00, 5.9133760340509998e+03 },
    { 8.9000000000000003e-10, 1.0733860494800000e+00, 6.7510408403457996e+03 },
    { 8.9000000000000003e-10, 5.2024079376800003e+00, 1.7363247428908991e+04 },
    { 8.9000000000000003e-10, 1.9794588786600000e+00, 1.0660686935042400e+04 },
    { 8.7999999999999996e-10, 3.7674595967600002e+00, 1.7053969106253189e+04 },
    { 8.7999999999999996e-10, 3.0039829722200002e+00, 1.5738204240988000e+03 },
    { 8.7999999999999996e-10, 8.7070535179999997e-01, 1.6943762785033799e+04 },
    { 8.7999999999999996e-10, 3.2490126448000001e+00, 1.2537702512514999e+04 },
    { 8.6999999999999999e-10, 2.3218777358199998e+00, 1.5808666607968000e+03 },
    { 8.6999999999999999e-10, 1.7512097743099999e+00, 2.8237233459389401e+04 },
    { 8.6999999999999999e-10, 3.6341876995800000e+00, 1.8106237489441599e+04 },
    { 8.6000000000000003e-10, 1.3414982351300000e+00, 2.3123745860806590e+04 },
    { 8.6000000000000003e-10, 4.3235529156699997e+00, 1.2341806904280900e+04 },
    { 8.4999999999999996e-10, 3.6524153493200000e+00, 1.2850011018847999e+04 },
    { 8.4999999999999996e-10, 2.9098397810600001e+00, 1.2282292381117601e+04 },
    { 8.3999999999999999e-10, 3.1382478200000001e+00, 8.8583149443206003e+03 },
    { 8.3999999999999999e-10, 4.2374540563199998e+00, 6.3790550772092001e+03 },
    { 8.3999999999999999e-10, 2.4110795163300001e+00, 6.1870966227735998e+03 },
    { 8.3000000000000003e-10, 1.6776998389999999e-02, 2.1424466644303400e+04 },
    { 8.1999999999999996e-10, 1.9375634760999999e-01, 5.4832547248259998e+03 },
    { 8.1999999999999996e-10, 3.8270410782400002e+00, 3.9581264411758002e+03 },
    { 8.1999999999999996e-10, 5.4870329261500004e+00, 6.7747391424502002e+03 },
    { 8.0999999999999999e-10, 4.7769423835000002e-01, 3.1441677569756801e+04 },
    { 8.0999999999999999e-10, 1.2705867269500000e+00, 7.2500467274270004e+03 },
    { 8.0000000000000003e-10, 3.5907081322700001e+00, 2.4080035016777190e+04 },
    { 8.0000000000000003e-10, 3.2790653124600002e+00, 6.4853292451654997e+03 },
    { 8.0000000000000003e-10, 3.3694682601900001e+00, 6.0808224548173002e+03 },
    { 8.0000000000000003e-10, 3.6221132052099998e+00, 3.9565080160194002e+03 },
    { 8.0000000000000003e-10, 5.3676000407800002e+00, 2.2380755800274001e+04 },
    { 8.0000000000000003e-10, 7.7942675353000002e-01, 2.3646323278900389e+04 },
    { 7.8999999999999996e-10, 4.5381793558299997e+00, 3.6109382204664587e+04 },
    { 7.8999999999999996e-10, 4.5077713370600003e+00, 5.6218429232103999e+03 },
    { 7.7999999999999999e-10, 4.9433622318500001e+00, 2.3550344051682590e+04 },
    { 7.7999999999999999e-10, 4.9890502703299999e+00, 9.9248104215105996e+03 },
    { 7.7999999999999999e-10, 4.5609405949699999e+00, 1.2564667227274500e+04 },
    { 7.5999999999999996e-10, 1.3365985786600001e+00, 4.0796338922674193e+04 },
    { 7.5999999999999996e-10, 1.9822948559600000e+00, 6.3658877750837601e+04 },
    { 7.5999999999999996e-10, 3.4342312234399999e+00, 5.2379210138037997e+03 },
    { 7.5999999999999996e-10, 2.3488450327899999e+00, 2.3226838635025200e+04 },
    { 7.5000000000000000e-10, 4.4212646714100003e+00, 1.8624882754272301e+04 },
    { 7.5000000000000000e-10, 5.9337478287199996e+00, 3.2217200181080800e+04 },
    { 7.4000000000000003e-10, 5.9812427190899999e+00, 6.3044114904584003e+03 },
    { 7.4000000000000003e-10, 6.6729085355999995e-01, 6.2617402095243997e+03 },
    { 7.4000000000000003e-10, 2.2955600888999999e+00, 2.8313288804660999e+04 },
    { 7.4000000000000003e-10, 3.2340419144700001e+00, 8.2534403620611585e+04 },
    { 7.4000000000000003e-10, 5.9124692900799998e+00, 3.6843780499902000e+03 },
    { 7.2999999999999996e-10, 2.6212933964000000e+00, 6.6954469468657999e+03 },
    { 7.2999999999999996e-10, 4.0272401762500003e+00, 5.8707047531170001e+03 },
    { 7.2999999999999996e-10, 1.2900295197300000e+00, 4.5892730433156990e+04 },
    { 7.2999999999999996e-10, 4.5341442504599998e+00, 6.1957676454515904e+03 },
    { 7.2999999999999996e-10, 2.1143893221900001e+00, 6.3703840545311996e+03 },
    { 7.2999999999999996e-10, 6.1833400872500004e+00, 2.3440137730463201e+04 },
    { 7.2999999999999996e-10, 6.0095073457400003e+00, 3.0348883772767000e+04 },
    { 7.2999999999999996e-10, 4.4054358809999998e+00, 8.0181537466839785e+04 },
    { 7.2000000000000000e-10, 3.2356695637999999e-01, 6.3128972881401996e+03 },
    { 7.2000000000000000e-10, 4.1781309090000003e-02, 6.2532544118425903e+03 },
    { 7.2000000000000000e-10, 4.7686915888200003e+00, 1.5141390794311999e+04 },
    { 7.2000000000000000e-10, 5.4355547964999995e-01, 1.8100003258450000e+04 },
    { 7.2000000000000000e-10, 4.1639825156299999e+00, 4.5980237274898000e+03 },
    { 7.2000000000000000e-10, 1.0540854503399999e+00, 4.5837966334882003e+03 },
    { 7.1000000000000003e-10, 6.1357313510899996e+00, 4.1194487926082402e+04 },
    { 7.1000000000000003e-10, 5.7028166072499999e+00, 1.6207886271502000e+04 },
    { 7.1000000000000003e-10, 3.6189746249299999e+00, 6.0530488985741802e+04 },
    { 6.9999999999999996e-10, 2.0172058818399998e+00, 6.4420193677746001e+03 },
    { 6.9999999999999996e-10, 4.6313276908100001e+00, 6.1241323322081998e+03 },
    { 6.9999999999999996e-10, 2.0467866525499998e+00, 6.1754123260528004e+03 },
    { 6.9999999999999996e-10, 4.6017469201000001e+00, 6.3907393739299996e+03 },
    { 6.9999999999999996e-10, 1.5657932662200000e+00, 1.2299544658261000e+04 },
    { 6.9000000000000000e-10, 2.1407620404499998e+00, 6.9443087767724001e+03 },
    { 6.9000000000000000e-10, 1.8973832508199999e+00, 2.3536116957680992e+04 },
    { 6.8000000000000003e-10, 3.6431243474300001e+00, 4.8971805736551996e+03 },
    { 6.8000000000000003e-10, 2.9081587282600001e+00, 2.8759810877483189e+04 },
    { 6.8000000000000003e-10, 2.7157972823800001e+00, 3.1664164377316001e+03 },
    { 6.8000000000000003e-10, 1.8665475629899999e+00, 3.5371887265976402e+04 },
    { 6.8000000000000003e-10, 6.1903347308800001e+00, 9.6677898863042195e+04 },
    { 6.6999999999999996e-10, 1.7045052996999999e-01, 7.0828969751568002e+03 },
    { 6.6999999999999996e-10, 4.8131257333799997e+00, 6.1718892077037999e+03 },
    { 6.6999999999999996e-10, 1.8354078392700000e+00, 6.3942624922790001e+03 },
    { 6.6999999999999996e-10, 1.3241010748000001e-01, 6.3008883721093998e+03 },
    { 6.6999999999999996e-10, 2.3293815799000001e-01, 6.2652633278734002e+03 },
    { 6.6999999999999996e-10, 1.4386481343499999e+00, 5.9595704333340000e+03 },
    { 6.6999999999999996e-10, 2.6517783092600000e+00, 2.2743409379516401e+04 },
    { 6.6000000000000000e-10, 4.9503939155600003e+00, 3.2350541665740002e+02 },
    { 6.6000000000000000e-10, 3.9527414221799999e+00, 6.2511035441745998e+03 },
    { 6.6000000000000000e-10, 2.6957921504700000e+00, 6.3150481558082001e+03 },
    { 6.6000000000000000e-10, 2.7813613087700002e+00, 1.7576546524346999e+04 },
    { 6.5000000000000003e-10, 6.0890232588900002e+00, 4.7162516354635198e+04 },
    { 6.3999999999999996e-10, 9.0947098130000001e-02, 2.8774037971484799e+04 },
    { 6.3999999999999996e-10, 1.9335615644599999e+00, 2.3006425992586392e+04 },
    { 6.3999999999999996e-10, 5.4243331670500003e+00, 1.4169793562230399e+04 },
    { 6.3999999999999996e-10, 5.1684175432300004e+00, 2.4279107018213592e+04 },
    { 6.3000000000000000e-10, 4.2799542322899997e+00, 1.1823161639450200e+04 },
    { 6.3000000000000000e-10, 4.9100310493899997e+00, 5.4369930152402003e+03 },
    { 6.3000000000000000e-10, 1.2750675802800000e+00, 5.5229243074000003e+00 },
    { 6.3000000000000000e-10, 3.6605637032499998e+00, 5.2197617665390999e+03 },
    { 6.3000000000000000e-10, 5.8993036842000002e-01, 5.2276260730653003e+03 },
    { 6.2000000000000003e-10, 6.7966053290999995e-01, 3.8550336669571998e+03 },
    { 6.2000000000000003e-10, 2.4701714288200001e+00, 1.6193659177500391e+04 },
    { 6.2000000000000003e-10, 4.4684587424700002e+00, 6.9087460423038001e+03 },
    { 6.2000000000000003e-10, 2.1800748301800001e+00, 5.6574056576789999e+03 },
    { 6.0999999999999996e-10, 4.3078256287000000e-01, 3.5050000274475387e+04 },
    { 6.0999999999999996e-10, 2.8896248765100001e+00, 4.0398189919265998e+04 },
    { 6.0999999999999996e-10, 5.1026442885899996e+00, 4.9166329245880002e+02 },
    { 6.0000000000000000e-10, 3.2295075441800001e+00, 1.3088729118076601e+04 },
    { 6.0000000000000000e-10, 4.4721492242699998e+00, 7.5486433286177999e+03 },
    { 6.0000000000000000e-10, 2.2052742348500001e+00, 6.4701083331480004e+02 },
    { 6.0000000000000000e-10, 3.7221954582000000e-01, 1.2171525814923600e+04 },
    { 5.9000000000000003e-10, 4.2392189556999998e-01, 4.1990785932898798e+04 },
    { 5.9000000000000003e-10, 3.9627301187100001e+00, 6.2891417656211997e+03 },
    { 5.9000000000000003e-10, 2.6858034539400002e+00, 6.2770099343616002e+03 },
    { 5.9000000000000003e-10, 2.7378297780300000e+00, 8.5846616659007996e+03 },
    { 5.9000000000000003e-10, 2.1782656838599999e+00, 1.8003144715222999e+04 },
    { 5.9000000000000003e-10, 3.3850494679400001e+00, 6.7589087447057187e+04 },
    { 5.9000000000000003e-10, 2.6002311097900002e+00, 1.5252471184999999e+01 },
    { 5.9000000000000003e-10, 1.0506938762800000e+00, 1.2164479578225601e+04 },
    { 5.9000000000000003e-10, 4.2845613445500002e+00, 3.9814900340820000e+03 },
    { 5.9000000000000003e-10, 1.7243950883600001e+00, 4.4034127589539399e+04 },
    { 5.7999999999999996e-10, 3.1561894095100000e+00, 6.7167875878681998e+03 },
    { 5.7999999999999996e-10, 4.2930851207999998e+00, 3.0775481963643000e+04 },
    { 5.7999999999999996e-10, 2.6535937794300000e+00, 4.0077619573520002e+04 },
    { 5.7999999999999996e-10, 5.4077800823500004e+00, 6.0763751126948000e+03 },
    { 5.7999999999999996e-10, 1.2407534902999999e+00, 6.4897765872880000e+03 },
    { 5.7999999999999996e-10, 3.6325356715000001e-01, 7.4673984228172398e+04 },
    { 5.7000000000000000e-10, 1.2214298239999999e-02, 1.0027903195729201e+04 },
    { 5.7000000000000000e-10, 4.8729936965000000e+00, 7.9277488846740005e+02 },
    { 5.7000000000000000e-10, 3.5580115768399998e+00, 6.7020002488919999e+03 },
    { 5.7000000000000000e-10, 3.0905219958100001e+00, 5.8641514510908000e+03 },
    { 5.6000000000000003e-10, 4.4789505887300001e+00, 1.5265886519300400e+04 },
    { 5.6000000000000003e-10, 1.9699738124899999e+00, 6.3398791121612003e+03 },
    { 5.6000000000000003e-10, 4.6785597601599997e+00, 6.2262725878215997e+03 },
    { 5.6000000000000003e-10, 2.5272872160800000e+00, 1.2303067776610000e+04 },
    { 5.6000000000000003e-10, 4.0245774552000002e+00, 3.1781457905676002e+03 },
    { 5.6000000000000003e-10, 4.1909349986600004e+00, 7.9982112516539996e+02 },
    { 5.6000000000000003e-10, 1.5008902154299999e+00, 2.7707542494294801e+04 },
    { 5.6000000000000003e-10, 6.5498687965000002e-01, 2.8663831650265402e+04 },
    { 5.4999999999999996e-10, 3.8914424086600001e+00, 2.2345260376108199e+04 },
    { 5.4999999999999996e-10, 3.2668259935499999e+00, 3.9302096962196003e+04 },
    { 5.4000000000000000e-10, 3.5656413599799999e+00, 1.5664035522708589e+04 },
    { 5.4000000000000000e-10, 2.7973004147600000e+00, 6.0063300781270000e+03 },
    { 5.4000000000000000e-10, 3.8512331578899999e+00, 6.5598216218558000e+03 },
    { 5.4000000000000000e-10, 1.9568906046200001e+00, 6.5601108437328003e+03 },
    { 5.4000000000000000e-10, 4.6916429680300000e+00, 6.0060408562499997e+03 },
    { 5.3000000000000003e-10, 5.3367755806500004e+00, 1.2146667056107601e+04 },
    { 5.3000000000000003e-10, 3.0252419999700000e+00, 3.8650173506198997e+04 },
    { 5.1999999999999996e-10, 5.6774145948800001e+00, 3.6275074775175999e+03 },
    { 5.1999999999999996e-10, 1.7749513803000001e+00, 6.1551243169567997e+03 },
    { 5.1999999999999996e-10, 4.8735821923499998e+00, 6.4110273830260003e+03 },
    { 5.1999999999999996e-10, 1.8284243193000000e+00, 5.2440492392010001e+03 },
    { 5.1999999999999996e-10, 4.1717972638900003e+00, 1.1919140866668000e+04 },
    { 5.1999999999999996e-10, 7.1926561104999998e-01, 4.6386993743311199e+04 },
    { 5.1999999999999996e-10, 6.2152662313300002e+00, 5.9921029841254003e+03 },
    { 5.1999999999999996e-10, 4.3326734132000000e-01, 6.5740487158573997e+03 },
    { 5.1999999999999996e-10, 4.4116516872700000e+00, 1.2571674624290201e+04 },
    { 5.1999999999999996e-10, 3.5331191429200000e+00, 1.1904918773201800e+04 },
    { 5.1000000000000000e-10, 3.0054076752299999e+00, 7.6689711263276004e+03 },
    { 5.1000000000000000e-10, 2.0646603354000001e-01, 1.8326650131880389e+04 },
    { 5.1000000000000000e-10, 4.5270351278699996e+00, 7.7619424240558001e+03 },
    { 5.1000000000000000e-10, 5.8278536250300004e+00, 8.4111747163059394e+04 },
    { 5.0000000000000003e-10, 1.5389812080000000e+00, 2.5752390943291998e+03 },
    { 5.0000000000000003e-10, 4.6945579992099997e+00, 5.8814037282341997e+03 },
    { 5.0000000000000003e-10, 1.9539755734399999e+00, 6.6847479717486003e+03 },
    { 5.0000000000000003e-10, 3.2167590065599998e+00, 4.3738802345965800e+04 },
    { 5.0000000000000003e-10, 1.9543579072099999e+00, 2.3141558382924599e+04 },
    { 5.0000000000000003e-10, 5.1770955247400003e+00, 1.7583660071347800e+04 },
    { 4.8999999999999996e-10, 5.1656673130000001e+00, 5.4903009615239998e+03 },
    { 4.8999999999999996e-10, 1.9784401172799999e+00, 2.2030212954669601e+04 },
    { 4.8999999999999996e-10, 5.4092989952800004e+00, 2.7511467873537200e+04 },
    { 4.8999999999999996e-10, 3.7301669991700002e+00, 3.2243498500880600e+04 },
    { 4.8999999999999996e-10, 4.5404065807800000e+00, 1.1293470674355600e+04 },
    { 4.8000000000000000e-10, 4.8065627356500000e+00, 3.6463503773543998e+03 },
    { 4.8000000000000000e-10, 2.8057365518599999e+00, 1.7067287061920001e+02 },
    { 4.8000000000000000e-10, 1.4258598274200001e+00, 3.6147409877300401e+04 },
    { 4.8000000000000000e-10, 6.2017389036799999e+00, 3.5579691239569998e+04 },
    { 4.8000000000000000e-10, 2.5026174120500002e+00, 1.2662130927200600e+04 },
    { 4.7000000000000003e-10, 8.2248881138999996e-01, 5.2094668258006004e+03 },
    { 4.7000000000000003e-10, 4.4592425366899997e+00, 4.7082300487091998e+03 },
    { 4.7000000000000003e-10, 1.1215985885799999e+00, 2.4499519660652400e+04 },
    { 4.7000000000000003e-10, 4.5083074409900004e+00, 2.4485292566650791e+04 },
    { 4.6000000000000001e-10, 2.2770325057099998e+00, 4.3847246320517997e+03 },
    { 4.6000000000000001e-10, 4.7241954148499996e+00, 1.6703062133498999e+04 },
    { 4.6000000000000001e-10, 4.0475209793799998e+00, 1.2555105999718900e+04 },
    { 4.5000000000000000e-10, 1.6266196271700000e+00, 4.4807038592695999e+03 },
    { 4.5000000000000000e-10, 4.2295407789999997e+00, 5.3321408344359997e+02 },
    { 4.5000000000000000e-10, 6.2218219561700003e+00, 4.6360695423511403e+04 },
    { 4.5000000000000000e-10, 4.7911430747499999e+00, 7.1519297143276795e+04 },
    { 4.5000000000000000e-10, 4.1231142745699998e+00, 4.4007829269739603e+04 },
    { 4.5000000000000000e-10, 5.5449834944100003e+00, 1.4286139123625818e+05 },
    { 4.3999999999999998e-10, 1.1003369688500000e+00, 2.3015858159093900e+03 },
    { 4.3999999999999998e-10, 8.1389810429999998e-01, 5.1134875985828003e+03 },
    { 4.3999999999999998e-10, 5.3791879186499996e+00, 6.9941953600828987e+04 },
    { 4.3999999999999998e-10, 5.2008749762399997e+00, 6.1408972229552000e+03 },
    { 4.3999999999999998e-10, 1.4476585964099999e+00, 6.4252544770275999e+03 },
    { 4.3000000000000001e-10, 3.1893950364000001e+00, 2.7832038219283200e+04 },
    { 4.3000000000000001e-10, 3.1160810127700000e+00, 8.4018397103698007e+03 },
    { 4.3000000000000001e-10, 1.1330128918500000e+00, 3.0639856638632999e+04 },
    { 4.3000000000000001e-10, 4.6979330697800004e+00, 7.3221024607817999e+03 },
    { 4.3000000000000001e-10, 1.3661549885900000e+00, 7.0260659105240002e+03 },
    { 4.3000000000000001e-10, 5.8143910580100000e+00, 6.3593419212669996e+03 },
    { 4.3000000000000001e-10, 4.4526581549199999e+00, 4.0000040915857790e+04 },
    { 4.2000000000000000e-10, 4.7008304532200000e+00, 6.2317955121289897e+03 },
    { 4.2000000000000000e-10, 1.9477031194300001e+00, 6.3343561878538103e+03 },
    { 4.2000000000000000e-10, 3.7903105190700002e+00, 6.3282170696280000e+03 },
    { 4.2000000000000000e-10, 2.8582230535800002e+00, 6.2379346303548000e+03 },
    { 4.0999999999999998e-10, 4.3156447693000000e-01, 3.7352383117590002e+03 },
    { 4.0999999999999998e-10, 1.6641575531299999e+00, 4.4136951349374001e+04 },
    { 4.0999999999999998e-10, 3.5654140967700001e+00, 6.6475604512999996e+02 },
    { 4.0000000000000001e-10, 5.0004386020000000e-02, 1.8208349942592002e+04 },
    { 4.0000000000000001e-10, 9.0738693746999999e-01, 2.6880319813032602e+04 },
    { 4.0000000000000001e-10, 1.5094792529300001e+00, 7.0758507384588002e+03 },
    { 4.0000000000000001e-10, 7.1563695590000004e-02, 7.1220451377418003e+03 },
    { 4.0000000000000001e-10, 2.9378456988000001e-01, 5.4441065622409997e+03 },
    { 4.0000000000000001e-10, 3.6432828722599999e+00, 3.2151363806432000e+03 },
    { 4.0000000000000001e-10, 7.8396796337999997e-01, 7.1980633574731182e+04 },
    { 3.9000000000000000e-10, 3.8207420659000002e-01, 2.2594054895711990e+04 },
    { 3.9000000000000000e-10, 4.2203327686399996e+00, 2.7278468816440800e+04 },
    { 3.9000000000000000e-10, 2.8892050803700000e+00, 2.0355319398799999e+01 },
    { 3.7999999999999998e-10, 2.2452724794900001e+00, 4.4933249356190398e+04 },
    { 3.7000000000000001e-10, 5.3998280011000004e+00, 2.3020653086587990e+04 },
    { 3.7000000000000001e-10, 9.1168136529999999e-02, 6.8507944877217997e+03 },
    { 3.7000000000000001e-10, 2.7418012893999999e-01, 5.7153572122610003e+03 },
    { 3.7000000000000001e-10, 1.9065394281500001e+00, 1.6723350142595002e+04 },
    { 3.6000000000000000e-10, 2.3520459255300001e+00, 2.0760427033191401e+04 },
};

static const VSOP87Term VSOP87A_EMB_X1[478] = {
    { 1.2340304600000000e-03, 0.0000000000000000e+00, 0.0000000000000000e+00 },
    { 5.1500156000000000e-04, 6.0026626720399996e+00, 1.2566151699982800e+04 },
    { 1.2907260000000000e-05, 5.9594312458300003e+00, 1.8849227549974199e+04 },
    { 1.0686269999999999e-05, 2.0155417655100001e+00, 6.2830758499914000e+03 },
    { 2.1268900000000001e-06, 1.7338019049100000e+00, 6.2795527316424004e+03 },
    { 2.1251499999999999e-06, 4.9148937103300003e+00, 6.2865989683403996e+03 },
    { 6.2259999999999998e-07, 3.6239798177999999e-01, 4.7057323075435997e+03 },
    { 5.9821999999999997e-07, 3.8119536987100000e+00, 6.2567775301915999e+03 },
    { 5.9513999999999997e-07, 2.8363416015000000e+00, 6.3093741697912001e+03 },
    { 4.8841000000000002e-07, 5.2141938933500001e+00, 7.7552261132399997e+02 },
    { 4.6286000000000000e-07, 1.8394941030000000e-02, 7.8604193924392002e+03 },
    { 4.2883000000000001e-07, 4.3789776558999999e-01, 1.0593819301891999e+03 },
    { 3.6061000000000000e-07, 2.1600220107100001e+00, 5.7533848848968000e+03 },
    { 3.5675000000000001e-07, 1.4527932726399999e+00, 5.8849268465832001e+03 },
    { 3.5367000000000002e-07, 4.4724382009500001e+00, 6.8127668150860000e+03 },
    { 3.2137000000000002e-07, 5.1958985189300000e+00, 6.6812248533995999e+03 },
    { 2.8762999999999999e-07, 5.9161898951199996e+00, 2.5132303399965600e+04 },
    { 2.8447000000000001e-07, 1.1497625380700001e+00, 6.1276554505572003e+03 },
    { 2.7572999999999999e-07, 5.5011910468299998e+00, 6.4384962494255997e+03 },
    { 2.4815000000000001e-07, 2.9220490981200000e+00, 5.4867778431750003e+03 },
    { 2.0611000000000001e-07, 3.7179088096799999e+00, 7.0793738568077997e+03 },
    { 1.9565000000000001e-07, 2.8935192446900002e+00, 5.5075532386674004e+03 },
    { 1.8308000000000000e-07, 1.4695431499200000e+00, 1.1790629088658799e+04 },
    { 1.6757000000000001e-07, 3.8193501581199998e+00, 7.0585984613153996e+03 },
    { 1.6471000000000000e-07, 6.2268263929199996e+00, 1.1506769769793600e+04 },
    { 1.4555000000000000e-07, 5.9755482353099998e+00, 6.2901893969922003e+03 },
    { 1.4387999999999999e-07, 6.8157599309000005e-01, 6.2759623029905997e+03 },
    { 1.3920999999999999e-07, 1.4415617240900001e+00, 7.9629800681639995e+02 },
    { 1.2573000000000000e-07, 3.0429978571999999e-01, 7.1135470007999997e+00 },
    { 1.1845000000000001e-07, 4.1524650362299997e+00, 4.6940029547076001e+03 },
    { 1.0425000000000000e-07, 4.2661081014800004e+00, 6.2820955289231997e+03 },
    { 1.0425000000000000e-07, 2.3824254711699999e+00, 6.2840561710596003e+03 },
    { 1.0073000000000000e-07, 3.2894017182800002e+00, 3.7387614301079998e+03 },
    { 9.5650000000000000e-08, 1.8931504433699999e+00, 4.1369104335162001e+03 },
    { 9.5130000000000005e-08, 4.6265711853199996e+00, 6.0697767545533998e+03 },
    { 9.0439999999999995e-08, 2.0097742068200000e+00, 6.4963749454294002e+03 },
    { 8.5430000000000005e-08, 5.5638792100000001e-02, 1.1944470102246000e+03 },
    { 8.2300000000000002e-08, 1.2520431827900000e+00, 1.5890728952838001e+03 },
    { 7.3119999999999996e-08, 4.9039562572000001e-01, 4.5350594369244000e+03 },
    { 7.0739999999999997e-08, 4.3944715518299997e+00, 5.0886288397668004e+03 },
    { 6.2520000000000007e-08, 4.7556051212500003e+00, 8.4292412664666008e+03 },
    { 6.1809999999999996e-08, 3.3623565660299999e+00, 8.8273902698748007e+03 },
    { 5.6069999999999998e-08, 2.5580058873599998e+00, 5.7295064471490005e+03 },
    { 5.5250000000000002e-08, 5.3696025959800000e+00, 4.9332084403325998e+03 },
    { 5.3389999999999997e-08, 5.6316859375300004e+00, 3.1546870848956000e+03 },
    { 5.2619999999999997e-08, 6.1947132513199996e+00, 8.0310922630584000e+03 },
    { 5.1679999999999997e-08, 5.1570058649000003e+00, 1.1769853693166400e+04 },
    { 5.1000000000000000e-08, 4.1086244953700000e+00, 6.8366452528338004e+03 },
    { 5.0750000000000001e-08, 2.3058291747099999e+00, 7.4775228602159996e+03 },
    { 4.9850000000000000e-08, 4.6277761691099997e+00, 1.2569674818331800e+04 },
    { 4.9019999999999998e-08, 4.1459917868699998e+00, 3.9302096962196001e+03 },
    { 4.8779999999999997e-08, 6.3279675324999995e-01, 6.0403472460173998e+03 },
    { 4.7090000000000000e-08, 3.9640148290399999e+00, 5.3313574437407997e+03 },
    { 4.6010000000000000e-08, 6.0237608286300004e+00, 6.5258044539654002e+03 },
    { 4.3940000000000002e-08, 4.7092838759999998e-02, 1.2559038152982001e+04 },
    { 4.3310000000000002e-08, 1.3299908152400000e+00, 7.6329432596502002e+03 },
    { 4.2939999999999999e-08, 4.9375582541599998e+00, 1.5925960136327999e+03 },
    { 3.9680000000000002e-08, 2.7269616978100002e+00, 7.2347942562420003e+03 },
    { 3.4949999999999999e-08, 3.7357104956999998e+00, 4.1643119896130002e+03 },
    { 3.4079999999999998e-08, 2.4059316487800002e+00, 3.1283887650958000e+03 },
    { 3.3899999999999999e-08, 4.2383651589499998e+00, 1.5542039943419999e+02 },
    { 3.3659999999999998e-08, 2.8947921379000002e-01, 1.1371704689758200e+04 },
    { 3.3409999999999999e-08, 1.6498805663000000e+00, 5.2236939198022001e+03 },
    { 3.3130000000000000e-08, 1.9201652605699999e+00, 8.0182093112380005e+02 },
    { 3.2870000000000003e-08, 5.4101350110400004e+00, 1.0977078804699000e+04 },
    { 2.9970000000000003e-08, 4.8653381965499998e+00, 1.7480164130670000e+03 },
    { 2.9910000000000001e-08, 2.2643358065300001e+00, 8.6359420037632008e+03 },
    { 2.7780000000000001e-08, 1.4615134875000000e+00, 5.4812549188676003e+03 },
    { 2.7500000000000001e-08, 5.2442962367200003e+00, 7.3424577801805999e+03 },
    { 2.6689999999999999e-08, 3.4283609001199999e+00, 2.1461654164751999e+03 },
    { 2.6560000000000001e-08, 5.7140574600800003e+00, 2.6298319799800002e+01 },
    { 2.5290000000000000e-08, 9.7555673900000008e-03, 1.3498674096587999e+03 },
    { 2.4050000000000000e-08, 5.9619525156500002e+00, 4.6904798363585996e+03 },
    { 2.3840000000000000e-08, 5.7483632518599999e+00, 7.0848967811151997e+03 },
    { 2.2880000000000000e-08, 4.5108894657000000e+00, 1.7260154654690399e+04 },
    { 2.1349999999999999e-08, 1.8902034547000000e+00, 5.3680451209540001e+02 },
    { 2.0520000000000001e-08, 1.4201267604100001e+00, 9.5171840625059997e+02 },
    { 2.0490000000000000e-08, 3.5363495412899999e+00, 1.9907450170410000e+03 },
    { 2.0129999999999999e-08, 2.4017221026300000e+00, 5.2257741809380002e+02 },
    { 1.9040000000000000e-08, 4.2326311972299999e+00, 9.4377629348869996e+03 },
    { 1.8800000000000000e-08, 1.7487254463099999e+00, 1.0973555686350001e+04 },
    { 1.8769999999999999e-08, 2.8284656348300001e+00, 5.5356940284240000e+02 },
    { 1.8580000000000002e-08, 7.4657310663999998e-01, 3.9814900340819997e+02 },
    { 1.8570000000000000e-08, 2.3400917763400000e+00, 5.2165803728013998e+03 },
    { 1.8530000000000001e-08, 1.8401219043499999e+00, 5.2308074668030004e+03 },
    { 1.7789999999999999e-08, 1.8081758190499999e+00, 1.2562628581633800e+04 },
    { 1.7080000000000001e-08, 2.9065876129000001e-01, 1.0988808157535001e+04 },
    { 1.5580000000000001e-08, 1.1002474406000000e+00, 4.2659819087599999e+02 },
    { 1.5550000000000000e-08, 3.6700420511399998e+00, 1.3095842665077400e+04 },
    { 1.5430000000000000e-08, 5.1451286594400001e+00, 1.0447387839604400e+04 },
    { 1.4500000000000001e-08, 9.9962470221999999e-01, 6.2627205305926000e+03 },
    { 1.4500000000000001e-08, 5.6489088704299997e+00, 6.3034311693902000e+03 },
    { 1.3730000000000000e-08, 4.8387368642300004e+00, 1.2721572099417001e+04 },
    { 1.3720000000000000e-08, 8.9279228105999997e-01, 7.8756718636242003e+03 },
    { 1.2470000000000000e-08, 1.8631842570099999e+00, 2.1187638603783998e+03 },
    { 1.2310000000000001e-08, 4.1027704417599997e+00, 1.8319536584879599e+04 },
    { 1.1650000000000000e-08, 5.9077983126199998e+00, 1.5720838784878400e+04 },
    { 1.1400000000000000e-08, 3.4925596254800002e+00, 1.0664954771900000e+03 },
    { 1.1360000000000000e-08, 2.8338934176900001e+00, 2.9424634232916001e+03 },
    { 1.1269999999999999e-08, 6.0484356460799997e+00, 5.6431785636774002e+03 },
    { 1.0710000000000000e-08, 3.7751142244000002e+00, 2.4272860397400001e+02 },
    { 1.0420000000000000e-08, 2.3073842493900001e+00, 5.2969096509459996e+02 },
    { 1.0330000000000001e-08, 1.0479354954400000e+00, 5.7604984318976003e+03 },
    { 1.0179999999999999e-08, 4.2337939874600004e+00, 1.2539853380183000e+04 },
    { 1.0169999999999999e-08, 5.5917264986199999e+00, 6.8056532680851997e+03 },
    { 9.9300000000000002e-09, 3.2082237906500000e+00, 1.0575406682941801e+04 },
    { 9.7399999999999999e-09, 2.4120093775000001e-01, 6.3989728631399998e+02 },
    { 9.7300000000000001e-09, 1.3052626747300000e+00, 4.2923308329503998e+03 },
    { 9.6400000000000006e-09, 2.1352164938999998e+00, 2.3888940204492001e+03 },
    { 9.3900000000000000e-09, 5.7826873538500001e+00, 2.0426571092422000e+04 },
    { 9.2900000000000008e-09, 1.0624124936399999e+00, 1.6496361396202399e+04 },
    { 8.9800000000000000e-09, 2.2635297734000002e+00, 1.3367972631106601e+04 },
    { 8.8900000000000005e-09, 6.5812396960000005e-02, 6.2885987742988000e+03 },
    { 8.8900000000000005e-09, 2.9953586851000003e-01, 6.2775529256840000e+03 },
    { 8.0100000000000003e-09, 5.3642376777800003e+00, 1.4314168113049800e+04 },
    { 8.0000000000000005e-09, 4.7544567572499998e+00, 1.2029347187887401e+04 },
    { 7.9500000000000001e-09, 4.9703146110300001e+00, 6.2797894925736000e+03 },
    { 7.9500000000000001e-09, 1.6782189616200001e+00, 6.2863622074092000e+03 },
    { 7.4400000000000000e-09, 4.2582061714200004e+00, 1.2043574281888999e+04 },
    { 7.2300000000000001e-09, 3.8771486847499999e+00, 9.6236882766912004e+03 },
    { 7.1600000000000001e-09, 4.0715759304799999e+00, 1.6730463689595799e+04 },
    { 6.9800000000000003e-09, 2.3124375738000000e-01, 6.1480107699560003e+03 },
    { 6.8800000000000002e-09, 4.1160312809999999e-01, 1.3916019109641600e+04 },
    { 6.7800000000000002e-09, 1.3408273301000001e-01, 6.4181409300267997e+03 },
    { 6.7500000000000001e-09, 5.5613623573400002e+00, 1.2964300703391000e+04 },
    { 6.7100000000000002e-09, 1.2894050461500000e+00, 6.7096740408674004e+03 },
    { 6.6599999999999997e-09, 6.1866433111200001e+00, 5.7462713378959997e+03 },
    { 6.6599999999999997e-09, 3.2994694227400001e+00, 3.6346210245184002e+03 },
    { 6.5100000000000001e-09, 5.3187666427600000e+00, 5.8564776591153995e+03 },
    { 6.5000000000000003e-09, 3.9608207485800002e+00, 1.4712317116458000e+04 },
    { 6.4300000000000003e-09, 6.1457578812199998e+00, 2.3528661537717999e+03 },
    { 6.2700000000000001e-09, 1.8532385318300000e+00, 4.6868894077067998e+03 },
    { 6.2000000000000001e-09, 5.8739016769500001e+00, 3.1415379249957001e+04 },
    { 6.1000000000000000e-09, 4.8716946879000000e-01, 6.8198803620868002e+03 },
    { 5.8800000000000004e-09, 3.2444211687800002e+00, 1.3119721102825190e+04 },
    { 5.8500000000000003e-09, 1.4860338726000000e-01, 1.2036460734888200e+04 },
    { 5.8399999999999997e-09, 1.7842056159699999e+00, 1.3517870106233400e+04 },
    { 5.5299999999999997e-09, 2.3607414893300001e+00, 6.3069542877391996e+03 },
    { 5.5299999999999997e-09, 4.2877920833200003e+00, 6.2591974122436004e+03 },
    { 5.2700000000000002e-09, 4.6584525672500003e+00, 1.0177257679533601e+04 },
    { 5.2300000000000003e-09, 5.5599182599199999e+00, 1.2139553509106800e+04 },
    { 5.1400000000000000e-09, 2.3034367895900001e+00, 4.7011165017084004e+03 },
    { 5.1199999999999997e-09, 1.8836581821900000e+00, 1.1015106477334801e+04 },
    { 4.9900000000000003e-09, 1.0874697828700000e+00, 1.2410731300548599e+04 },
    { 4.9499999999999997e-09, 3.5806704158099998e+00, 1.3362449706799200e+04 },
    { 4.8600000000000002e-09, 3.0561843254599999e+00, 1.2592450019782600e+04 },
    { 4.8099999999999997e-09, 4.1419769651399996e+00, 6.2869571853493999e+03 },
    { 4.8099999999999997e-09, 2.5065566075099999e+00, 6.2791945146334001e+03 },
    { 4.6500000000000003e-09, 8.1901657744000000e-01, 1.4143495242430599e+04 },
    { 4.5699999999999997e-09, 4.1920044074699998e+00, 6.0768903015542001e+03 },
    { 4.5699999999999997e-09, 2.4565291651800001e+00, 6.4892613984285999e+03 },
    { 4.5399999999999996e-09, 7.2878821246000003e-01, 2.7870430238573999e+03 },
    { 4.5200000000000001e-09, 5.5540987098799999e+00, 1.2573265246983599e+04 },
    { 4.3999999999999997e-09, 2.9197512692899998e+00, 3.8941818295421999e+03 },
    { 4.3999999999999997e-09, 2.1490252864200001e+00, 2.4072921469776400e+04 },
    { 4.3500000000000001e-09, 4.7694552878999996e+00, 9.9176968745098002e+03 },
    { 4.3100000000000002e-09, 3.0996885404699999e+00, 1.1499656222792801e+04 },
    { 4.1899999999999998e-09, 5.6221999925599997e+00, 5.3274761083827998e+03 },
    { 3.9799999999999999e-09, 5.7718116851900003e+00, 8.2738208670324002e+03 },
    { 3.9799999999999999e-09, 1.2276624389000000e-01, 1.5510452226479999e+03 },
    { 3.8300000000000002e-09, 4.0420536338000002e-01, 6.3191037166688002e+03 },
    { 3.8300000000000002e-09, 6.2443282092699999e+00, 6.2470479833139898e+03 },
    { 3.7900000000000004e-09, 4.6246450298199999e+00, 6.3861686242100004e+03 },
    { 3.7900000000000004e-09, 2.0238885428300000e+00, 6.1799830757727996e+03 },
    { 3.7799999999999998e-09, 2.4707702500900002e+00, 2.1329909543799999e+02 },
    { 3.7700000000000000e-09, 5.9035542220800004e+00, 7.8721487452751999e+03 },
    { 3.6699999999999999e-09, 2.7510259163600002e+00, 1.0021837280099400e+04 },
    { 3.6399999999999998e-09, 2.2507071170100001e+00, 6.3211035226271997e+03 },
    { 3.6399999999999998e-09, 4.3978264556399997e+00, 6.2450481773556003e+03 },
    { 3.6000000000000000e-09, 4.1011609151000004e+00, 1.0522683831884001e+03 },
    { 3.4900000000000001e-09, 9.5962542266999995e-01, 1.7789845619784999e+04 },
    { 3.4400000000000001e-09, 2.8667648401000001e-01, 1.8451078546565990e+04 },
    { 3.4299999999999999e-09, 2.9355052995099999e+00, 6.2678233788063899e+03 },
    { 3.4299999999999999e-09, 3.7130282731399999e+00, 6.2983283211764001e+03 },
    { 3.4200000000000002e-09, 6.1001964048600001e+00, 6.3038512454838001e+03 },
    { 3.4200000000000002e-09, 5.4833716778999997e-01, 6.2623004544989999e+03 },
    { 3.3999999999999998e-09, 2.7506844423399999e+00, 1.2323423096008801e+04 },
    { 3.3700000000000001e-09, 2.7124426779499999e+00, 1.7782732072784202e+04 },
    { 3.3099999999999999e-09, 9.4287498279000004e-01, 7.2386755916000002e+03 },
    { 3.2099999999999999e-09, 2.2371961934600000e+00, 1.7796959166785800e+04 },
    { 3.1800000000000002e-09, 1.3235034154600001e+00, 6.3115250374591997e+03 },
    { 3.1800000000000002e-09, 5.3250301571899996e+00, 6.2546266625236003e+03 },
    { 3.1599999999999998e-09, 1.4807870150999999e-01, 1.1926254413668799e+04 },
    { 3.0600000000000002e-09, 3.2052325953400000e+00, 5.8635912061161998e+03 },
    { 3.0399999999999998e-09, 2.0097054829099998e+00, 1.2779450795420800e+04 },
    { 2.9699999999999999e-09, 5.1546903036000002e-01, 5.4298794682394000e+03 },
    { 2.9499999999999999e-09, 4.0585021284799998e+00, 6.7025604938666002e+03 },
    { 2.8999999999999999e-09, 5.3116332184699999e+00, 2.2041264243879999e+02 },
    { 2.8200000000000002e-09, 4.5026049448999999e+00, 1.3341674311306801e+04 },
    { 2.7999999999999998e-09, 5.7187185148699999e+00, 2.0618554843720000e+02 },
    { 2.7999999999999998e-09, 1.9108297605700000e+00, 5.6421982426092000e+03 },
    { 2.7499999999999998e-09, 3.6715087581099999e+00, 1.4945316173554400e+04 },
    { 2.6900000000000000e-09, 6.1095763487400001e+00, 9.7791086761253991e+03 },
    { 2.6799999999999998e-09, 4.3657303879600002e+00, 2.5443144198834002e+03 },
    { 2.6299999999999998e-09, 2.1688850003600000e+00, 1.2168002696574600e+04 },
    { 2.6000000000000001e-09, 3.1656418546900000e+00, 1.0969965257698201e+04 },
    { 2.5700000000000000e-09, 1.0474973374200001e+00, 9.5559974160859997e+02 },
    { 2.5300000000000002e-09, 2.2291808025100002e+00, 1.8073704938650200e+04 },
    { 2.5000000000000001e-09, 4.6159590229200003e+00, 6.9239534573736000e+03 },
    { 2.3400000000000002e-09, 4.0739971595700002e+00, 1.5961864422845999e+03 },
    { 2.3300000000000000e-09, 3.0133962184400001e+00, 1.0984192351699799e+04 },
    { 2.2999999999999999e-09, 2.2377419510899998e+00, 1.2567132021051000e+04 },
    { 2.2699999999999998e-09, 4.9566686616800002e+00, 5.8493641121146002e+03 },
    { 2.2600000000000001e-09, 4.4453496692499996e+00, 3.4960328261340001e+03 },
    { 2.1600000000000000e-09, 5.6056164687200001e+00, 3.1851920272655998e+03 },
    { 2.1100000000000000e-09, 1.2211137934700000e+00, 1.9651048481098002e+04 },
    { 2.0299999999999998e-09, 4.3949682847000000e+00, 6.0626632075526004e+03 },
    { 2.0299999999999998e-09, 2.2535652879499999e+00, 6.5034884924301996e+03 },
    { 1.9900000000000000e-09, 2.8410323906500001e+00, 3.5231183490000002e+00 },
    { 1.9399999999999999e-09, 2.3763077038899998e+00, 6.4087760738220004e+02 },
    { 1.9200000000000000e-09, 2.9957829074700002e+00, 4.7320306273433998e+03 },
    { 1.8700000000000000e-09, 4.5916768874200002e+00, 6.2760296132933900e+03 },
    { 1.8700000000000000e-09, 2.0568566852300001e+00, 6.2901220866894000e+03 },
    { 1.8400000000000001e-09, 1.3859812924000001e-01, 1.8209330263660191e+04 },
    { 1.8400000000000001e-09, 4.2095514947900003e+00, 1.2566219010285600e+04 },
    { 1.8400000000000001e-09, 4.6536544646399998e+00, 1.2566084389680000e+04 },
    { 1.8400000000000001e-09, 3.3804991495599999e+00, 6.0372442037620003e+03 },
    { 1.8000000000000000e-09, 5.7679924859399998e+00, 8.5319638175199998e+02 },
    { 1.7500000000000000e-09, 3.2677150731200002e+00, 6.5289074962207997e+03 },
    { 1.7200000000000001e-09, 8.8734623011000002e-01, 1.5773435424478000e+03 },
    { 1.6600000000000001e-09, 1.1947787105800001e+00, 4.3371173787679999e+02 },
    { 1.6600000000000001e-09, 4.5131356084799998e+00, 1.2352852604544800e+04 },
    { 1.6500000000000001e-09, 1.3430738091000001e+00, 1.1020632121940000e+02 },
    { 1.5600000000000000e-09, 2.3445793654400000e+00, 2.6484548254729998e+03 },
    { 1.5400000000000001e-09, 9.7127667707999998e-01, 6.1124029793722002e+03 },
    { 1.5400000000000001e-09, 5.6772568955600002e+00, 6.4537487206105998e+03 },
    { 1.4599999999999999e-09, 4.0566480275599996e+00, 1.4956319713459999e+02 },
    { 1.4599999999999999e-09, 1.9916476143299999e+00, 1.3760598710207400e+04 },
    { 1.4400000000000000e-09, 3.7617969078900000e+00, 1.8635928454536199e+04 },
    { 1.4200000000000001e-09, 1.3453336994400000e+00, 4.1571984426121999e+03 },
    { 1.3999999999999999e-09, 1.2545169581400000e+00, 9.3809596727171993e+03 },
    { 1.3600000000000001e-09, 7.5192359085000005e-01, 1.1712955318230799e+04 },
    { 1.3200000000000000e-09, 4.9419682791700001e+00, 1.8875525869773999e+04 },
    { 1.2900000000000001e-09, 2.5618715043300000e+00, 1.1513883316794399e+04 },
    { 1.2400000000000001e-09, 9.9880484927000002e-01, 1.7256631536341400e+04 },
    { 1.2300000000000001e-09, 2.6259126301000002e-01, 6.1728695287720002e+03 },
    { 1.2300000000000001e-09, 1.0275700246000000e-01, 6.3932821712107998e+03 },
    { 1.2300000000000001e-09, 4.4283939350199999e+00, 1.2565171378914600e+04 },
    { 1.2100000000000000e-09, 5.4008221826499998e+00, 2.0775395492400001e+01 },
    { 1.2100000000000000e-09, 2.2778180385100000e+00, 1.0419986283507600e+04 },
    { 1.2100000000000000e-09, 5.3369720539999999e-02, 1.3625533630172000e+04 },
    { 1.2000000000000000e-09, 2.2918427974000002e+00, 1.7253041107689591e+04 },
    { 1.1900000000000000e-09, 2.4456263546000001e-01, 1.0213285546211000e+04 },
    { 1.1800000000000001e-09, 5.9679970137299998e+00, 1.7654780539749601e+04 },
    { 1.1700000000000001e-09, 4.4771171305099999e+00, 1.8852750668323199e+04 },
    { 1.1400000000000000e-09, 2.2169534368099999e+00, 4.1714255366138004e+03 },
    { 1.1200000000000001e-09, 5.8654969541000002e-01, 6.3278373931320004e+02 },
    { 1.1100000000000001e-09, 3.6625623758899999e+00, 1.5819593482830001e+03 },
    { 1.0800000000000000e-09, 2.4963921163900000e+00, 2.3543230504681789e+04 },
    { 1.0600000000000001e-09, 5.1428170561200002e+00, 1.1216284290324000e+04 },
    { 1.0600000000000001e-09, 6.0047126449499997e+00, 3.0978838227257902e+03 },
    { 1.0500000000000001e-09, 5.2679926700199999e+00, 2.3013539539587200e+04 },
    { 1.0399999999999999e-09, 5.4210525338600002e+00, 5.6502921106781996e+03 },
    { 1.0300000000000000e-09, 2.4702432230999999e-01, 6.2713464971554004e+03 },
    { 1.0300000000000000e-09, 1.1832394316000000e-01, 6.2948052028273996e+03 },
    { 1.0200000000000000e-09, 2.6980361958299999e+00, 1.7267268201691189e+04 },
    { 1.0200000000000000e-09, 5.6704202732700004e+00, 6.5496828917131998e+03 },
    { 1.0200000000000000e-09, 9.7811329938000002e-01, 6.0164688082696002e+03 },
    { 1.0100000000000000e-09, 4.4210043023600001e+00, 4.8042092759269999e+03 },
    { 1.0100000000000000e-09, 2.8619181196999999e+00, 3.1049300594237998e+03 },
    { 9.5000000000000003e-10, 1.3435855969199999e+00, 1.2345739057544000e+04 },
    { 9.5000000000000003e-10, 2.5004643971400000e+00, 1.2012582297140400e+04 },
    { 9.2000000000000003e-10, 8.7789019102999999e-01, 1.0818135286915800e+04 },
    { 8.9999999999999999e-10, 3.6882803985699999e+00, 7.6266071275599998e+01 },
    { 8.9999999999999999e-10, 6.0229439773499998e+00, 6.1957676454515904e+03 },
    { 8.9999999999999999e-10, 6.2558959530000002e-01, 6.3703840545311996e+03 },
    { 8.6999999999999999e-10, 5.0975511116399996e+00, 3.9039113764198000e+03 },
    { 8.6000000000000003e-10, 5.9394975196799997e+00, 1.2808880303956799e+04 },
    { 8.6000000000000003e-10, 8.1273058912999996e-01, 1.2359966151545599e+04 },
    { 8.4999999999999996e-10, 5.7953687772500002e+00, 5.6360650166766000e+03 },
    { 8.3999999999999999e-10, 1.1475526202699999e+00, 6.2556742938945999e+03 },
    { 8.3999999999999999e-10, 5.5009809523800000e+00, 6.3104774060882000e+03 },
    { 8.3999999999999999e-10, 3.0189126879099999e+00, 1.0440274292603601e+04 },
    { 8.3999999999999999e-10, 4.3552150116200004e+00, 2.3791644735716000e+03 },
    { 8.3000000000000003e-10, 3.7090203950000000e+00, 5.1206011455835996e+03 },
    { 8.3000000000000003e-10, 5.3199491685900000e+00, 1.8415515812097390e+04 },
    { 8.1999999999999996e-10, 1.9767948099999999e-01, 1.8842114002973391e+04 },
    { 8.0999999999999999e-10, 3.8477281482099999e+00, 1.1614433293732200e+04 },
    { 8.0000000000000003e-10, 3.4711727403999998e-01, 6.9229731363053997e+03 },
    { 8.0000000000000003e-10, 4.1970265026100000e+00, 3.5833410306738001e+03 },
    { 8.0000000000000003e-10, 1.4193931248400000e+00, 1.3521751441591399e+04 },
    { 7.8999999999999996e-10, 6.2056347130000002e-02, 2.2805735565993600e+04 },
    { 7.7999999999999999e-10, 4.3163874186099997e+00, 8.6719698704406001e+03 },
    { 7.7000000000000003e-10, 2.0545564037799999e+00, 1.6858482532933202e+04 },
    { 7.5999999999999996e-10, 3.9266593534999998e+00, 1.7298182327326202e+04 },
    { 7.5999999999999996e-10, 4.8348295987199998e+00, 1.8429742906099000e+04 },
    { 7.5000000000000000e-10, 1.6283392257700000e+00, 3.0355997319767801e+04 },
    { 7.5000000000000000e-10, 2.1413683863699999e+00, 2.3581258177317599e+04 },
    { 7.4000000000000003e-10, 2.3375831430399998e+00, 3.3406124266997999e+03 },
    { 7.2999999999999996e-10, 6.0959442471500003e+00, 9.4917560896980001e+02 },
    { 7.2999999999999996e-10, 4.4952751023899999e+00, 9.3880059094152002e+03 },
    { 7.2000000000000000e-10, 5.8729642567400004e+00, 6.5105519827804001e+03 },
    { 7.2000000000000000e-10, 7.7556931591000000e-01, 6.0555997172023999e+03 },
    { 7.1000000000000003e-10, 2.3083621208999999e-01, 1.2545376304490401e+04 },
    { 6.9000000000000000e-10, 3.2466235414200000e+00, 1.0454501386605200e+04 },
    { 6.5000000000000003e-10, 5.8704516122900001e+00, 7.3353442331797996e+03 },
    { 6.5000000000000003e-10, 2.3393849680200001e+00, 1.4919017853754600e+04 },
    { 6.3000000000000000e-10, 9.9541849433999996e-01, 2.4356780788641601e+04 },
    { 6.2000000000000003e-10, 4.0634280558800002e+00, 1.1925274092600601e+04 },
    { 6.0999999999999996e-10, 3.7027175634899998e+00, 1.6737577236596600e+04 },
    { 6.0999999999999996e-10, 1.5194461973000001e-01, 2.8237233459389401e+04 },
    { 6.0999999999999996e-10, 5.3770518153699998e+00, 6.2536463414554000e+03 },
    { 6.0999999999999996e-10, 1.2714817572799999e+00, 6.3125053585273999e+03 },
    { 6.0000000000000000e-10, 3.5154808847400001e+00, 3.4570066334852600e+04 },
    { 5.7999999999999996e-10, 8.9406841197999998e-01, 1.7996031168222202e+04 },
    { 5.7999999999999996e-10, 2.4654315041800001e+00, 1.5141390794311999e+04 },
    { 5.7000000000000000e-10, 2.4891408410000002e+00, 4.1948464387519999e+02 },
    { 5.6000000000000003e-10, 5.3339380951699997e+00, 1.2455945378763399e+04 },
    { 5.6000000000000003e-10, 2.1796599381799999e+00, 1.1764330768858999e+04 },
    { 5.4999999999999996e-10, 1.1860370357000001e+00, 2.6997348193175999e+03 },
    { 5.4999999999999996e-10, 5.9407360489599998e+00, 1.5671081759406599e+04 },
    { 5.4000000000000000e-10, 3.9746744240000000e-02, 7.8341210726394002e+03 },
    { 5.3000000000000003e-10, 1.7773957851200000e+00, 2.6735945262213201e+04 },
    { 5.3000000000000003e-10, 5.2854935972200003e+00, 6.3578574485586996e+03 },
    { 5.3000000000000003e-10, 1.3630399754300000e+00, 6.2082942514241004e+03 },
    { 5.1999999999999996e-10, 2.6563539971000000e-01, 5.3267866940207996e+03 },
    { 5.1000000000000000e-10, 7.6407742617999996e-01, 1.6522659716002199e+04 },
    { 5.1000000000000000e-10, 3.4068329421699999e+00, 2.3006425992586392e+04 },
    { 5.0000000000000003e-10, 7.5232394679999995e-02, 2.6084021806216198e+04 },
    { 5.0000000000000003e-10, 4.4944184855399998e+00, 2.4602612434870989e+04 },
    { 4.8999999999999996e-10, 4.1873978101500002e+00, 3.5579691239569998e+04 },
    { 4.8999999999999996e-10, 4.9267447966600004e+00, 2.1393541969857601e+04 },
    { 4.8999999999999996e-10, 6.0525485887999997e-01, 7.0189523635231999e+03 },
    { 4.8999999999999996e-10, 6.0432787137700004e+00, 5.5471993364596001e+03 },
    { 4.8999999999999996e-10, 2.5912370156700000e+00, 1.2416588502848201e+04 },
    { 4.8999999999999996e-10, 1.6684678473900001e+00, 5.7498617665477996e+03 },
    { 4.8999999999999996e-10, 4.9800657252600002e+00, 6.8162899334350004e+03 },
    { 4.8999999999999996e-10, 1.4570159910800000e+00, 8.9828106693089994e+03 },
    { 4.8999999999999996e-10, 2.9591850596899998e+00, 1.6460333529524989e+04 },
    { 4.8999999999999996e-10, 4.5148635679400000e+00, 2.9303728936579399e+04 },
    { 4.8999999999999996e-10, 4.9897874344600002e+00, 2.9289501842577800e+04 },
    { 4.8999999999999996e-10, 5.7816930761099998e+00, 1.8216443810661000e+04 },
    { 4.8000000000000000e-10, 5.4698544398799998e+00, 1.2132439962106000e+04 },
    { 4.8000000000000000e-10, 4.8819052160899998e+00, 1.6200772724501199e+04 },
    { 4.7000000000000003e-10, 9.7667389279000005e-01, 2.4279107018213592e+04 },
    { 4.7000000000000003e-10, 2.0930017402900001e+00, 2.4065807922775592e+04 },
    { 4.6000000000000001e-10, 3.4513453780400001e+00, 1.9378918515068799e+04 },
    { 4.6000000000000001e-10, 5.7379745055200004e+00, 1.0435161256629678e+05 },
    { 4.5000000000000000e-10, 1.2799634054000000e-01, 2.4491425792583399e+04 },
    { 4.5000000000000000e-10, 2.1314764695099999e+00, 5.4247413135750401e+04 },
    { 4.3999999999999998e-10, 3.9272674118799999e+00, 8.6622403235630009e+03 },
    { 4.3999999999999998e-10, 3.6376473715900000e+00, 6.1754123260528004e+03 },
    { 4.3999999999999998e-10, 3.0108862010599999e+00, 6.3907393739299996e+03 },
    { 4.3999999999999998e-10, 4.9301121797400000e+00, 2.8230187222691391e+04 },
    { 4.3999999999999998e-10, 2.7901717992499999e+00, 3.9814900340820000e+03 },
    { 4.3000000000000001e-10, 3.3467714268800002e+00, 5.2440492392010001e+03 },
    { 4.2000000000000000e-10, 1.4730574311200000e+00, 2.4705705209089589e+04 },
    { 4.0999999999999998e-10, 4.7869665685599996e+00, 1.7370360975909800e+04 },
    { 4.0999999999999998e-10, 4.9961086921400000e+00, 2.2483848574492589e+04 },
    { 4.0999999999999998e-10, 3.6260330324500001e+00, 2.3020653086587990e+04 },
    { 4.0999999999999998e-10, 5.0592200383600003e+00, 2.6482170809624400e+04 },
    { 4.0999999999999998e-10, 4.9531184578399996e+00, 1.5508615123274400e+04 },
    { 4.0999999999999998e-10, 1.3549652737200000e+00, 2.9088811415985001e+04 },
    { 4.0999999999999998e-10, 4.0910190222700002e+00, 2.1202093703745999e+04 },
    { 4.0000000000000001e-10, 3.0039322963199999e+00, 3.2243498500880600e+04 },
    { 4.0000000000000001e-10, 4.5946989377000003e-01, 2.4336005393149200e+04 },
    { 4.0000000000000001e-10, 2.5493760600000001e-03, 8.4018397103698007e+03 },
    { 4.0000000000000001e-10, 3.4322354808800002e+00, 2.7511467873537200e+04 },
    { 3.9000000000000000e-10, 5.5487205210599999e+00, 4.5585172812187397e+04 },
    { 3.9000000000000000e-10, 3.3151178088100002e+00, 2.9826306354673201e+04 },
    { 3.7999999999999998e-10, 3.9356155874700001e+00, 6.2688487559898003e+03 },
    { 3.7999999999999998e-10, 2.7129179851799998e+00, 6.2973029439929996e+03 },
    { 3.7999999999999998e-10, 1.0976793876000000e-01, 6.1718892077037999e+03 },
    { 3.7999999999999998e-10, 2.5558032671000003e-01, 6.3942624922790001e+03 },
    { 3.7999999999999998e-10, 4.0395205190699999e+00, 1.6062184526116800e+04 },
    { 3.7999999999999998e-10, 6.1626821742800004e+00, 1.8202216716659390e+04 },
    { 3.7999999999999998e-10, 6.1492044225100004e+00, 2.6709646942413401e+04 },
    { 3.7999999999999998e-10, 4.9558261452399996e+00, 1.9004647949408401e+04 },
    { 3.7999999999999998e-10, 5.1222420975600000e+00, 3.1570799649391200e+04 },
    { 3.7000000000000001e-10, 2.0758731684199998e+00, 2.0452869412221800e+04 },
    { 3.7000000000000001e-10, 6.0574477769200001e+00, 2.1150813365883601e+04 },
    { 3.7000000000000001e-10, 1.2159768795300001e+00, 7.3495713271814002e+03 },
    { 3.7000000000000001e-10, 3.4191284324900000e+00, 2.9296615389578601e+04 },
    { 3.6000000000000000e-10, 3.8431980185999999e+00, 3.9302096962196003e+04 },
    { 3.6000000000000000e-10, 6.2804970858400004e+00, 9.9248104215105996e+03 },
    { 3.6000000000000000e-10, 4.8121707777599996e+00, 1.5664035522708589e+04 },
    { 3.4999999999999998e-10, 4.6673213445100004e+00, 4.0612192153944002e+03 },
    { 3.4999999999999998e-10, 3.2066169046600002e+00, 7.3221024607817999e+03 },
    { 3.4999999999999998e-10, 2.6926124448300000e+00, 1.1919140866668000e+04 },
    { 3.4999999999999998e-10, 8.3074174405000001e-01, 2.9062513096185201e+04 },
    { 3.4999999999999998e-10, 4.8373687692500003e+00, 2.1228392023545799e+04 },
    { 3.4000000000000001e-10, 1.3880102255800000e+00, 6.1474505249814001e+03 },
    { 3.4000000000000001e-10, 5.2605233470700004e+00, 6.4187011750013999e+03 },
    { 3.4000000000000001e-10, 1.9796747896200000e+00, 1.7271884007526402e+04 },
    { 3.4000000000000001e-10, 1.9050948430600001e+00, 2.9026485229507791e+04 },
    { 3.4000000000000001e-10, 1.2171918407200000e+00, 2.1548962369291799e+04 },
    { 3.4000000000000001e-10, 4.9512849953100000e+00, 3.8500276031072201e+04 },
    { 3.4000000000000001e-10, 7.8574341965000005e-01, 5.9057022420756002e+03 },
    { 3.4000000000000001e-10, 5.8627901529999997e+00, 6.6604494579071998e+03 },
    { 3.3000000000000000e-10, 3.2286720182200002e+00, 3.0666154958432799e+04 },
    { 3.3000000000000000e-10, 1.7092174135200000e+00, 3.4520309309380798e+04 },
    { 3.3000000000000000e-10, 3.4572238252799998e+00, 2.8628336226099600e+04 },
    { 3.1999999999999998e-10, 4.3830238950200000e+00, 1.7686752845566400e+04 },
    { 3.1999999999999998e-10, 6.2616480216600001e+00, 6.2317955121289897e+03 },
    { 3.1999999999999998e-10, 3.8688555098999999e-01, 6.3343561878538103e+03 },
    { 3.1999999999999998e-10, 4.4788919628000001e-01, 3.1172650645983002e+04 },
    { 3.1000000000000002e-10, 2.8544969416999999e-01, 2.2003914634869801e+04 },
    { 3.1000000000000002e-10, 1.9462514992999999e-01, 2.7832038219283200e+04 },
    { 3.1000000000000002e-10, 2.6997030124600001e+00, 3.3019021112204602e+04 },
    { 3.1000000000000002e-10, 3.1902348653799999e+00, 5.0175083713650001e+03 },
    { 3.0000000000000000e-10, 2.9601249792700002e+00, 1.2489885628707199e+04 },
    { 3.0000000000000000e-10, 2.3240571936599999e+00, 1.2586927095475199e+04 },
    { 2.8999999999999998e-10, 7.0432070922000001e-01, 4.9515382508406998e+04 },
    { 2.8999999999999998e-10, 2.2625647009200001e+00, 2.7278468816440800e+04 },
    { 2.8999999999999998e-10, 1.7746825116000001e-01, 6.5923541726472004e+03 },
    { 2.8999999999999998e-10, 1.8788001431000001e-01, 5.9737975273355996e+03 },
    { 2.8999999999999998e-10, 9.3434607314999996e-01, 1.1933367960669601e+04 },
    { 2.8000000000000002e-10, 1.8784914370700001e+00, 2.4080035016777190e+04 },
    { 2.8000000000000002e-10, 5.5477280446100004e+00, 3.4911412076091001e+04 },
    { 2.8000000000000002e-10, 2.8994461510699998e+00, 5.8953145443294001e+04 },
    { 2.8000000000000002e-10, 3.8033147782099999e+00, 1.0239583866010800e+04 },
    { 2.8000000000000002e-10, 2.8082572986900001e+00, 2.2476735027491792e+04 },
    { 2.8000000000000002e-10, 2.4185500310100001e+00, 4.6386993743311199e+04 },
    { 2.8000000000000002e-10, 6.1091859581600003e+00, 1.8052929543157799e+04 },
    { 2.7000000000000000e-10, 3.7602531546000000e+00, 1.9402796952816600e+04 },
    { 2.7000000000000000e-10, 5.3007933074600002e+00, 4.3232306658415597e+04 },
    { 2.7000000000000000e-10, 6.1718914020900000e+00, 1.6921656695024001e+03 },
    { 2.7000000000000000e-10, 6.5637418130000003e-01, 6.3607496204194003e+03 },
    { 2.7000000000000000e-10, 5.9921593913500004e+00, 6.2054020795633996e+03 },
    { 2.7000000000000000e-10, 2.3204922897600002e+00, 3.2367097656207599e+04 },
    { 2.7000000000000000e-10, 1.7711481395699999e+00, 4.4878174062704002e+03 },
    { 2.7000000000000000e-10, 6.4682460544999998e-01, 2.9424634232916000e+04 },
    { 2.7000000000000000e-10, 5.9193598548799997e+00, 5.2175806283148399e+04 },
    { 2.7000000000000000e-10, 2.7248218780000000e-02, 2.5158601719765400e+04 },
    { 2.7000000000000000e-10, 6.7002293804000002e-01, 3.4513263072682799e+04 },
    { 2.5999999999999998e-10, 3.4241373743999999e+00, 2.4383079108441401e+04 },
    { 2.5999999999999998e-10, 1.9818248906600000e+00, 1.9804827291582798e+04 },
    { 2.5999999999999998e-10, 3.6912120822000000e-01, 6.3399742249270003e+03 },
    { 2.5999999999999998e-10, 6.2794123644299997e+00, 6.2261774750557997e+03 },
    { 2.5999999999999998e-10, 6.0141687086699998e+00, 4.7970957289261996e+03 },
    { 2.5999999999999998e-10, 3.9357609701200000e+00, 3.6949230808424203e+04 },
    { 2.5999999999999998e-10, 2.5405955204000001e+00, 9.9105833275090008e+03 },
    { 2.5999999999999998e-10, 4.4434782440099996e+00, 1.5110466119866200e+04 },
    { 2.5999999999999998e-10, 3.5764552808799999e+00, 8.5846616659007996e+03 },
    { 2.5999999999999998e-10, 3.4105463803200000e+00, 1.8422629359098191e+04 },
    { 2.5000000000000002e-10, 2.7849106535499999e+00, 6.3027456108911902e+03 },
    { 2.5000000000000002e-10, 3.8636229191000000e+00, 6.2634060890915998e+03 },
    { 2.5000000000000002e-10, 8.1555419575999999e-01, 1.4155224595266600e+04 },
    { 2.5000000000000002e-10, 2.4488007694000000e+00, 2.0199094959632999e+04 },
    { 2.5000000000000002e-10, 3.5321105510400002e+00, 5.5798458358398399e+04 },
    { 2.5000000000000002e-10, 3.8024667868000001e-01, 9.2255392732830005e+03 },
    { 2.4000000000000000e-10, 1.2262950160199999e+00, 2.1947111372700001e+04 },
    { 2.4000000000000000e-10, 5.5027097678000003e-01, 1.1616976091013001e+04 },
    { 2.4000000000000000e-10, 3.7345843125800000e+00, 3.1968948652799401e+04 },
    { 2.4000000000000000e-10, 3.2126006822200002e+00, 2.3550344051682590e+04 },
    { 2.4000000000000000e-10, 1.2085688989900001e+00, 9.4114646150871995e+03 },
    { 2.3000000000000001e-10, 1.9924237928799999e+00, 3.0774501642574800e+04 },
    { 2.3000000000000001e-10, 6.2347300028999997e-01, 1.7892938394003591e+04 },
    { 2.3000000000000001e-10, 1.3606353453600000e+00, 6.7167875878681998e+03 },
    { 2.3000000000000001e-10, 3.0628394518499999e+00, 6.2720301497275004e+03 },
    { 2.3000000000000001e-10, 3.5856941207999999e+00, 6.2941215502552996e+03 },
    { 2.3000000000000001e-10, 2.2509630608700002e+00, 3.4115114069274598e+04 },
    { 2.1999999999999999e-10, 3.9974179196200001e+00, 7.4455505543992003e+03 },
    { 2.1999999999999999e-10, 6.1069489569500002e+00, 1.5265886519300400e+04 },
    { 2.1999999999999999e-10, 4.6799093655900004e+00, 3.4596364654652403e+04 },
    { 2.1999999999999999e-10, 1.3826202442400000e+00, 4.3796390374902003e+03 },
    { 2.1999999999999999e-10, 4.5222692510800000e+00, 2.8759810877483189e+04 },
    { 2.1999999999999999e-10, 4.4447042267799999e+00, 1.8326650131880389e+04 },
    { 2.1999999999999999e-10, 4.4650514363499996e+00, 6.3328328754632003e+03 },
    { 2.1999999999999999e-10, 2.1834821362999999e+00, 6.2333188245195997e+03 },
    { 2.1000000000000000e-10, 5.7825354050400000e+00, 2.2779437246193800e+04 },
    { 2.1000000000000000e-10, 4.0857740732799996e+00, 3.0220932239732399e+04 },
    { 2.0000000000000001e-10, 4.7093224248999999e-01, 4.7162516354635198e+04 },
    { 2.0000000000000001e-10, 5.3135596916300001e+00, 4.0879440504643797e+04 },
    { 2.0000000000000001e-10, 5.0915271489400000e+00, 8.1865126624925997e+03 },
    { 2.0000000000000001e-10, 1.7123870040400000e+00, 2.7433889215874991e+04 },
    { 2.0000000000000001e-10, 1.6529245175900000e+00, 6.3790550772092001e+03 },
    { 2.0000000000000001e-10, 4.9956090550600001e+00, 6.1870966227735998e+03 },
    { 2.0000000000000001e-10, 2.9417850313399998e+00, 1.9800945956224801e+04 },
    { 2.0000000000000001e-10, 4.3234957110699996e+00, 6.2883355139513602e+04 },
    { 2.0000000000000001e-10, 2.8539072028399999e+00, 9.0701188738487999e+03 },
    { 1.8999999999999999e-10, 4.7861006475999996e+00, 2.8313288804660999e+04 },
    { 1.8999999999999999e-10, 4.3784032998500004e+00, 1.6207886271502000e+04 },
    { 1.8999999999999999e-10, 1.3656987542700001e+00, 2.3539707386332801e+04 },
    { 1.8999999999999999e-10, 2.3645575268200001e+00, 5.6183198048614004e+03 },
    { 1.8999999999999999e-10, 4.2839760458300002e+00, 6.9478318951213996e+03 },
    { 1.8000000000000000e-10, 4.6181706244300003e+00, 6.5939166488598003e+03 },
    { 1.8000000000000000e-10, 2.0303629482200001e+00, 5.9722350511229997e+03 },
};

static const VSOP87Term VSOP87A_EMB_X2[185] = {
    { 4.1432169999999997e-05, 3.1415926535900001e+00, 0.0000000000000000e+00 },
    { 2.1756949999999998e-05, 4.3999984957200002e+00, 1.2566151699982800e+04 },
    { 9.9523300000000000e-06, 2.0790847155000000e-01, 6.2830758499914000e+03 },
    { 9.2658999999999999e-07, 4.1928547100999998e+00, 1.8849227549974199e+04 },
    { 1.3679000000000000e-07, 3.3783364206300002e+00, 6.2865989683403996e+03 },
    { 1.3668000000000000e-07, 3.2727149201899999e+00, 6.2795527316424004e+03 },
    { 8.1899999999999999e-08, 4.4192802203000001e+00, 6.2567775301915999e+03 },
    { 8.1450000000000005e-08, 2.2292509340800000e+00, 6.3093741697912001e+03 },
    { 8.0039999999999994e-08, 5.9228819732800000e+00, 6.1276554505572003e+03 },
    { 7.7459999999999993e-08, 7.2469358509000004e-01, 6.4384962494255997e+03 },
    { 3.7370000000000003e-08, 2.4872442076399999e+00, 5.5075532386674004e+03 },
    { 3.2030000000000003e-08, 4.1427508374400004e+00, 7.0585984613153996e+03 },
    { 2.9370000000000001e-08, 4.0816081324500004e+00, 2.5132303399965600e+04 },
    { 2.8930000000000002e-08, 4.9497269745999999e+00, 4.7057323075435997e+03 },
    { 2.2539999999999999e-08, 3.5195721352099998e+00, 7.7552261132399997e+02 },
    { 2.1699999999999999e-08, 1.7049573901699999e+00, 7.8604193924392002e+03 },
    { 2.0310000000000001e-08, 2.2936343897000002e+00, 6.2759623029905997e+03 },
    { 2.0310000000000001e-08, 4.3548991829499997e+00, 6.2901893969922003e+03 },
    { 1.9040000000000000e-08, 4.4666896183400002e+00, 5.4867778431750003e+03 },
    { 1.7199999999999999e-08, 5.2314209428199998e+00, 1.0593819301891999e+03 },
    { 1.5630000000000002e-08, 2.1810148769700000e+00, 7.0793738568077997e+03 },
    { 1.5480000000000000e-08, 2.1290674574900001e+00, 6.0403472460173998e+03 },
    { 1.4699999999999999e-08, 4.5165954939799997e+00, 6.5258044539654002e+03 },
    { 1.4559999999999999e-08, 3.5377082359899998e+00, 5.7533848848968000e+03 },
    { 1.4289999999999999e-08, 3.0199811855199998e+00, 5.8849268465832001e+03 },
    { 1.4240000000000000e-08, 3.1625098455899998e+00, 6.8127668150860000e+03 },
    { 1.3380000000000000e-08, 1.0444570518500000e+00, 5.7295064471490005e+03 },
    { 1.2880000000000000e-08, 3.6289228869699999e+00, 6.6812248533995999e+03 },
    { 1.2120000000000000e-08, 5.6171317017200000e+00, 6.8366452528338004e+03 },
    { 1.0390000000000001e-08, 7.9543243401999997e-01, 6.2840561710596003e+03 },
    { 1.0390000000000001e-08, 5.8531011386299996e+00, 6.2820955289231997e+03 },
    { 9.0400000000000002e-09, 2.4353519967300001e+00, 5.3313574437407997e+03 },
    { 8.7899999999999996e-09, 3.0613393009799998e+00, 1.1790629088658799e+04 },
    { 8.7099999999999999e-09, 3.8481600044299999e+00, 4.9332084403325998e+03 },
    { 8.5400000000000007e-09, 5.2432864297600004e+00, 4.5350594369244000e+03 },
    { 8.3400000000000006e-09, 5.7511897578199997e+00, 1.5542039943419999e+02 },
    { 7.8500000000000008e-09, 5.8877629947300001e+00, 5.0886288397668004e+03 },
    { 7.6899999999999997e-09, 4.2281214694199996e+00, 7.2347942562420003e+03 },
    { 7.4400000000000000e-09, 3.4956564055000000e-01, 4.1369104335162001e+03 },
    { 6.8599999999999999e-09, 1.9013582392699999e+00, 8.0182093112380005e+02 },
    { 6.7700000000000004e-09, 2.7196522639100000e+00, 3.1283887650958000e+03 },
    { 6.7500000000000001e-09, 2.8258744687799999e+00, 7.6329432596502002e+03 },
    { 6.6500000000000000e-09, 4.8174949944699996e+00, 1.1944470102246000e+03 },
    { 6.5400000000000002e-09, 1.4032244196100001e+00, 1.1506769769793600e+04 },
    { 5.9200000000000002e-09, 1.3943396224500000e+00, 8.0310922630584000e+03 },
    { 5.6200000000000000e-09, 7.7105088911999997e-01, 7.4775228602159996e+03 },
    { 5.4800000000000001e-09, 6.1212896586400003e+00, 7.9629800681639995e+02 },
    { 5.0700000000000001e-09, 5.4240802929100003e+00, 4.6940029547076001e+03 },
    { 5.0300000000000002e-09, 3.4145662237100001e+00, 1.5925960136327999e+03 },
    { 4.9900000000000003e-09, 3.0275223585000002e+00, 1.2569674818331800e+04 },
    { 4.9799999999999998e-09, 6.1900010349899999e+00, 1.5890728952838001e+03 },
    { 4.8000000000000000e-09, 1.3270442520000000e-02, 8.4292412664666008e+03 },
    { 4.1499999999999999e-09, 5.5896245000000002e-03, 6.0697767545533998e+03 },
    { 4.1499999999999999e-09, 3.5131226280000000e-01, 6.4963749454294002e+03 },
    { 4.0199999999999998e-09, 1.7217712843999999e+00, 3.7387614301079998e+03 },
    { 3.9000000000000002e-09, 5.3671121972100000e+00, 2.6298319799800002e+01 },
    { 3.7200000000000000e-09, 4.3515358580200001e+00, 5.5356940284240000e+02 },
    { 3.6800000000000001e-09, 3.9191093141200000e+00, 9.4377629348869996e+03 },
    { 3.6600000000000002e-09, 1.0449761483000000e-01, 1.2721572099417001e+04 },
    { 3.4699999999999998e-09, 1.3424894863000001e+00, 5.4812549188676003e+03 },
    { 3.3499999999999998e-09, 1.1260833153500001e+00, 4.6904798363585996e+03 },
    { 3.2500000000000002e-09, 2.9494829301099998e+00, 9.5171840625059997e+02 },
    { 3.2099999999999999e-09, 2.0179825163700000e+00, 1.9907450170410000e+03 },
    { 3.0699999999999999e-09, 1.5681733015999999e+00, 1.3498674096587999e+03 },
    { 2.8699999999999998e-09, 2.2647642622299999e+00, 2.4272860397400001e+02 },
    { 2.5500000000000001e-09, 1.8387611731400000e+00, 1.1371704689758200e+04 },
    { 2.4199999999999999e-09, 4.9232700130999998e+00, 8.8273902698748007e+03 },
    { 2.4199999999999999e-09, 1.2263961387000000e-01, 1.7480164130670000e+03 },
    { 2.3300000000000000e-09, 5.2302362537300002e+00, 7.0848967811151997e+03 },
    { 2.2999999999999999e-09, 1.0587019292299999e+00, 3.1546870848956000e+03 },
    { 2.2200000000000002e-09, 4.8885196016799997e+00, 4.1643119896130002e+03 },
    { 2.0700000000000001e-09, 2.2858725691699999e+00, 1.3367972631106601e+04 },
    { 2.0400000000000000e-09, 4.4240387853000002e-01, 1.1769853693166400e+04 },
    { 2.0200000000000001e-09, 3.2144520198099999e+00, 1.0973555686350001e+04 },
    { 1.8899999999999999e-09, 6.1654841816999995e-01, 2.3888940204492001e+03 },
    { 1.8800000000000001e-09, 5.6588899347000003e+00, 7.8756718636242003e+03 },
    { 1.8300000000000001e-09, 4.1012823686699997e+00, 7.3424577801805999e+03 },
    { 1.7300000000000000e-09, 5.2511015365000002e-01, 1.0977078804699000e+04 },
    { 1.5900000000000001e-09, 2.7229443253399999e+00, 4.2923308329503998e+03 },
    { 1.5500000000000000e-09, 5.8141282749399998e+00, 1.2410731300548599e+04 },
    { 1.4500000000000000e-09, 5.4588258924500002e+00, 3.9814900340819997e+02 },
    { 1.3399999999999999e-09, 4.5629697911099996e+00, 1.0575406682941801e+04 },
    { 1.3399999999999999e-09, 6.2520166725799999e+00, 3.9302096962196001e+03 },
    { 1.2700000000000000e-09, 4.8461256755999997e+00, 1.3119721102825190e+04 },
    { 1.2500000000000000e-09, 3.8781428333600001e+00, 1.2562628581633800e+04 },
    { 1.2500000000000000e-09, 1.7913722961000000e-01, 8.6359420037632008e+03 },
    { 1.2199999999999999e-09, 1.0501917611000000e+00, 1.2592450019782600e+04 },
    { 1.1500000000000000e-09, 4.9653944804499996e+00, 2.1461654164751999e+03 },
    { 1.1100000000000001e-09, 2.1576583523999999e+00, 1.5773435424478000e+03 },
    { 1.0999999999999999e-09, 3.6376895807000000e+00, 5.2236939198022001e+03 },
    { 1.0900000000000000e-09, 3.9972675343100001e+00, 1.2539853380183000e+04 },
    { 1.0700000000000000e-09, 3.4360188442899999e+00, 1.3517870106233400e+04 },
    { 1.0500000000000001e-09, 3.5384551079600000e+00, 5.6421982426092000e+03 },
    { 1.0200000000000000e-09, 4.3712514426700002e+00, 1.0988808157535001e+04 },
    { 1.0000000000000001e-09, 2.3496909300200000e+00, 1.3095842665077400e+04 },
    { 9.5000000000000003e-10, 5.9865694172400001e+00, 1.7260154654690399e+04 },
    { 9.2000000000000003e-10, 1.9824056447000000e+00, 1.3916019109641600e+04 },
    { 9.0999999999999996e-10, 3.1096044468100001e+00, 6.9239534573736000e+03 },
    { 8.4999999999999996e-10, 4.3251208985899998e+00, 3.8941818295421999e+03 },
    { 8.1999999999999996e-10, 5.9091429814199996e+00, 1.0177257679533601e+04 },
    { 8.1999999999999996e-10, 4.0021849058200001e+00, 3.1415379249957001e+04 },
    { 7.8999999999999996e-10, 5.2258660952599998e+00, 1.6730463689595799e+04 },
    { 7.8999999999999996e-10, 8.4977013970000001e-02, 1.0447387839604400e+04 },
    { 7.5999999999999996e-10, 4.1184637433000004e+00, 1.2323423096008801e+04 },
    { 7.5000000000000000e-10, 5.7306731922600003e+00, 6.7096740408674004e+03 },
    { 7.5000000000000000e-10, 9.1786038038999995e-01, 5.8564776591153995e+03 },
    { 7.1000000000000003e-10, 5.9970099558000001e-01, 1.4314168113049800e+04 },
    { 6.6000000000000000e-10, 5.4958589890500003e+00, 2.7870430238573999e+03 },
    { 6.3999999999999996e-10, 3.3578152644100001e+00, 4.7320306273433998e+03 },
    { 6.0999999999999996e-10, 4.2398429122800003e+00, 8.2738208670324002e+03 },
    { 5.7000000000000000e-10, 4.7139448487099997e+00, 6.3989728631399998e+02 },
    { 5.6000000000000003e-10, 1.3110624269700000e+00, 1.5720838784878400e+04 },
    { 5.4999999999999996e-10, 1.4142909142800000e+00, 5.6431785636774002e+03 },
    { 5.4999999999999996e-10, 3.8354854418700000e+00, 1.3341674311306801e+04 },
    { 5.4000000000000000e-10, 5.6315310173700004e+00, 1.6496361396202399e+04 },
    { 5.3000000000000003e-10, 4.1237513486699999e+00, 1.4919017853754600e+04 },
    { 5.1999999999999996e-10, 4.3983925142400002e+00, 3.6346210245184002e+03 },
    { 5.1000000000000000e-10, 1.2657361840700001e+00, 6.1957676454515904e+03 },
    { 5.1000000000000000e-10, 5.3827973885800002e+00, 6.3703840545311996e+03 },
    { 5.0000000000000003e-10, 4.1871626220699998e+00, 1.2573265246983599e+04 },
    { 4.8999999999999996e-10, 2.4829667604300001e+00, 1.4143495242430599e+04 },
    { 4.8000000000000000e-10, 2.0235300553700002e+00, 1.3362449706799200e+04 },
    { 4.8000000000000000e-10, 5.9989438773000003e-01, 2.1187638603783998e+03 },
    { 4.7000000000000003e-10, 3.8295904833300001e+00, 1.2964300703391000e+04 },
    { 4.3999999999999998e-10, 1.6710334238100000e+00, 8.6622403235630009e+03 },
    { 4.2000000000000000e-10, 3.8069707842000000e-01, 2.4072921469776400e+04 },
    { 4.0999999999999998e-10, 2.4973195101100001e+00, 1.8319536584879599e+04 },
    { 4.0999999999999998e-10, 4.0932816877300002e+00, 3.1851920272655998e+03 },
    { 4.0999999999999998e-10, 5.6287578016199999e+00, 1.7789845619784999e+04 },
    { 4.0000000000000001e-10, 6.0503961740900003e+00, 3.4960328261340001e+03 },
    { 3.9000000000000000e-10, 4.2764192983599996e+00, 2.0426571092422000e+04 },
    { 3.4999999999999998e-10, 3.6080463027100000e+00, 1.8875525869773999e+04 },
    { 3.4999999999999998e-10, 6.0907712815300004e+00, 1.0419986283507600e+04 },
    { 3.4999999999999998e-10, 9.1150627653000005e-01, 9.7791086761253991e+03 },
    { 3.3000000000000000e-10, 5.3088739877000002e-01, 1.0213285546211000e+04 },
    { 3.1000000000000002e-10, 6.4652215196999996e-01, 2.3013539539587200e+04 },
    { 3.1000000000000002e-10, 5.5305342823199997e+00, 1.4712317116458000e+04 },
    { 3.1000000000000002e-10, 7.6672410297000004e-01, 2.2779437246193800e+04 },
    { 3.0000000000000000e-10, 4.3971570054100004e+00, 1.2808880303956799e+04 },
    { 2.8000000000000002e-10, 2.3448135181500001e+00, 5.4298794682394000e+03 },
    { 2.8000000000000002e-10, 3.9108671544900000e+00, 1.1015106477334801e+04 },
    { 2.7000000000000000e-10, 4.6896113258799996e+00, 1.6200772724501199e+04 },
    { 2.5999999999999998e-10, 2.2939398067000001e+00, 9.6236882766912004e+03 },
    { 2.5999999999999998e-10, 4.3545937659499998e+00, 2.9424634232916001e+03 },
    { 2.5999999999999998e-10, 1.7523420832600001e+00, 1.8209330263660191e+04 },
    { 2.5999999999999998e-10, 9.0493478768000002e-01, 1.2012582297140400e+04 },
    { 2.5000000000000002e-10, 6.0460696125800002e+00, 9.9176968745098002e+03 },
    { 2.5000000000000002e-10, 2.6881321521400001e+00, 3.5833410306738001e+03 },
    { 2.3000000000000001e-10, 2.2978858922700001e+00, 9.3809596727171993e+03 },
    { 2.3000000000000001e-10, 2.4916800583100001e+00, 1.2559038152982001e+04 },
    { 2.3000000000000001e-10, 2.2672722650999999e+00, 2.6709646942413401e+04 },
    { 2.3000000000000001e-10, 2.5341022448100001e+00, 1.9651048481098002e+04 },
    { 2.1999999999999999e-10, 3.7638209444899999e+00, 1.7298182327326202e+04 },
    { 2.1000000000000000e-10, 2.3833494517900000e+00, 1.7256631536341400e+04 },
    { 2.1000000000000000e-10, 3.7736808169999998e+00, 1.6858482532933202e+04 },
    { 2.1000000000000000e-10, 4.4704781938000000e+00, 1.2036460734888200e+04 },
    { 1.8999999999999999e-10, 1.9305865844900001e+00, 1.1926254413668799e+04 },
    { 1.8999999999999999e-10, 3.6610066232600000e+00, 3.0639856638632999e+04 },
    { 1.8999999999999999e-10, 5.3909692528499997e+00, 1.6460333529524989e+04 },
    { 1.8999999999999999e-10, 2.8141964817499998e+00, 8.6719698704406001e+03 },
    { 1.8000000000000000e-10, 2.2825579999399999e+00, 1.1614433293732200e+04 },
    { 1.8000000000000000e-10, 3.1993649600100000e+00, 1.8073704938650200e+04 },
    { 1.7000000000000001e-10, 3.9570302499999999e+00, 8.9828106693089994e+03 },
    { 1.5999999999999999e-10, 4.1160329501999998e-01, 2.3543230504681789e+04 },
    { 1.5000000000000000e-10, 6.5562803693000005e-01, 1.6062184526116800e+04 },
    { 1.5000000000000000e-10, 4.2184950008300000e+00, 2.0199094959632999e+04 },
    { 1.5000000000000000e-10, 5.7625721168999999e-01, 1.2567132021051000e+04 },
    { 1.4000000000000001e-10, 4.1292793327800004e+00, 2.3528661537717999e+03 },
    { 1.4000000000000001e-10, 1.5947511180000001e-01, 2.2483848574492589e+04 },
    { 1.4000000000000001e-10, 1.9331444470500001e+00, 2.1228392023545799e+04 },
    { 1.2000000000000000e-10, 2.7099088900599999e+00, 1.7996031168222202e+04 },
    { 1.2000000000000000e-10, 7.2222975089999997e-01, 1.9402796952816600e+04 },
    { 1.2000000000000000e-10, 4.3788072861599998e+00, 1.0818135286915800e+04 },
    { 1.0999999999999999e-10, 1.7847283078300000e+00, 2.8766924424484001e+04 },
    { 1.0999999999999999e-10, 4.9332789666399997e+00, 1.8451078546565990e+04 },
    { 1.0999999999999999e-10, 5.4358799225999999e+00, 1.4945316173554400e+04 },
    { 1.0999999999999999e-10, 2.2460750862999999e+00, 3.5371887265976402e+04 },
    { 1.0999999999999999e-10, 5.0222890173200003e+00, 2.2805735565993600e+04 },
    { 1.0999999999999999e-10, 4.0547309070499997e+00, 2.3581258177317599e+04 },
    { 1.0999999999999999e-10, 3.9656469738000000e-01, 2.5158601719765400e+04 },
    { 1.0000000000000000e-10, 2.1693038332099999e+00, 1.5664035522708589e+04 },
    { 1.0000000000000000e-10, 6.0531059589199998e+00, 2.3141558382924599e+04 },
    { 1.0000000000000000e-10, 2.5564528269800002e+00, 2.2345260376108199e+04 },
    { 1.0000000000000000e-10, 1.8358815991999999e-01, 1.2779450795420800e+04 },
    { 8.9999999999999999e-11, 1.4915517948200001e+00, 2.9826306354673201e+04 },
};

static const VSOP87Term VSOP87A_EMB_X3[18] = {
    { 1.7521299999999999e-06, 3.1415926535900001e+00, 0.0000000000000000e+00 },
    { 7.2337000000000005e-07, 2.8930395247599998e+00, 1.2566151699982800e+04 },
    { 8.3640000000000000e-08, 3.8550095409599998e+00, 6.2830758499914000e+03 },
    { 5.0400000000000001e-08, 2.5315298978600000e+00, 1.8849227549974199e+04 },
    { 1.4810000000000000e-08, 2.2367223523400002e+00, 6.4384962494255997e+03 },
    { 1.4810000000000000e-08, 4.4118112203099997e+00, 6.1276554505572003e+03 },
    { 6.4400000000000001e-09, 4.8856790478000001e+00, 6.2795527316424004e+03 },
    { 6.4400000000000001e-09, 1.7628545248500000e+00, 6.2865989683403996e+03 },
    { 3.1800000000000002e-09, 3.6384031018299998e+00, 6.0403472460173998e+03 },
    { 3.1800000000000002e-09, 3.0101304708200001e+00, 6.5258044539654002e+03 },
    { 2.1999999999999998e-09, 2.3416694531200002e+00, 2.5132303399965600e+04 },
    { 2.0099999999999999e-09, 8.2714760809999999e-01, 6.8366452528338004e+03 },
    { 2.0099999999999999e-09, 5.8213859645500001e+00, 5.7295064471490005e+03 },
    { 3.6000000000000000e-10, 1.4120277045400000e+00, 1.2569674818331800e+04 },
    { 3.3000000000000000e-10, 4.2355458561099999e+00, 1.2410731300548599e+04 },
    { 1.8000000000000000e-10, 2.3270948775500000e+00, 1.2721572099417001e+04 },
    { 1.0999999999999999e-10, 6.2731398189099998e+00, 1.2592450019782600e+04 },
    { 6.0000000000000000e-11, 2.5602885985400001e+00, 2.4272860397400001e+02 },
};

static const VSOP87Term VSOP87A_EMB_X4[10] = {
    { 4.0219999999999999e-08, 0.0000000000000000e+00, 0.0000000000000000e+00 },
    { 1.9270000000000000e-08, 1.1874623345299999e+00, 1.2566151699982800e+04 },
    { 4.9099999999999998e-09, 5.2989024449100004e+00, 6.2830758499914000e+03 },
    { 2.2900000000000002e-09, 8.8072621187000000e-01, 1.8849227549974199e+04 },
    { 2.1000000000000002e-09, 3.7431953004800000e+00, 6.4384962494255997e+03 },
    { 2.1000000000000002e-09, 2.9053382721699998e+00, 6.1276554505572003e+03 },
    { 2.1000000000000000e-10, 1.8394116300999999e-01, 6.2795527316424004e+03 },
    { 2.1000000000000000e-10, 1.8140710246000000e-01, 6.2865989683403996e+03 },
    { 1.0999999999999999e-10, 6.5552931189999997e-01, 2.5132303399965600e+04 },
    { 5.0000000000000002e-11, 2.6891553870800000e+00, 1.2410731300548599e+04 },
};

static const VSOP87Term VSOP87A_EMB_X5[6] = {
    { 1.0800000000000000e-09, 0.0000000000000000e+00, 0.0000000000000000e+00 },
    { 4.3000000000000001e-10, 6.0878965205800002e+00, 1.2566151699982800e+04 },
    { 2.5000000000000002e-10, 5.3377966534599999e+00, 6.4384962494255997e+03 },
    { 2.5000000000000002e-10, 1.3107369191900000e+00, 6.1276554505572003e+03 },
    { 2.0000000000000001e-10, 9.4333339827999996e-01, 6.2830758499914000e+03 },
    { 7.9999999999999995e-11, 5.5258806114199999e+00, 1.8849227549974199e+04 },
};

static const VSOP87Term VSOP87A_EMB_Y0[804] = {
    { 9.9989209644999999e-01, 1.8265890456000000e-01, 6.2830758499914000e+03 },
    { 2.4426988409999999e-02, 3.1415926535900001e+00, 0.0000000000000000e+00 },
    { 8.3529231399999994e-03, 1.3952878991000001e-01, 1.2566151699982800e+04 },
    { 1.0466965000000000e-04, 9.6416905580000004e-02, 1.8849227549974199e+04 },
    { 2.5703380000000001e-05, 5.3010397335999997e+00, 5.2969096509459996e+02 },
    { 2.1474730000000002e-05, 2.6625353890499999e+00, 1.5773435424478000e+03 },
    { 1.7092190000000001e-05, 5.2078040107100003e+00, 6.2795527316424004e+03 },
    { 1.7079870000000000e-05, 4.5823285876600002e+00, 6.2865989683403996e+03 },
    { 1.4402650000000000e-05, 1.9006816466400001e+00, 2.3528661537717999e+03 },
    { 1.1350920000000000e-05, 5.2731341521999999e+00, 5.2236939198022001e+03 },
    { 9.3453899999999996e-06, 4.5030120184399998e+00, 1.2036460734888200e+04 },
    { 9.0056500000000008e-06, 1.6056328812000000e+00, 1.0213285546211000e+04 },
    { 7.4493199999999997e-06, 2.8072887188600002e+00, 3.9814900340819997e+02 },
    { 6.8132399999999997e-06, 6.4729627496999997e-01, 4.7057323075435997e+03 },
    { 6.3931599999999998e-06, 6.0292391501699996e+00, 5.7533848848968000e+03 },
    { 6.1134699999999996e-06, 3.8138149528600001e+00, 6.8127668150860000e+03 },
    { 5.6712600000000000e-06, 5.8142248753000003e-01, 1.0593819301891999e+03 },
    { 5.5138999999999998e-06, 3.9612524936900000e+00, 5.5075532386674004e+03 },
    { 5.4635999999999997e-06, 6.1731113178500001e+00, 1.4143495242430599e+04 },
    { 5.4127300000000003e-06, 5.4990280591699996e+00, 7.7552261132399997e+02 },
    { 5.0708400000000001e-06, 2.8702519338100001e+00, 7.8604193924392002e+03 },
    { 4.5201799999999998e-06, 5.9916724270700001e+00, 6.2567775301915999e+03 },
    { 4.5043500000000003e-06, 4.5278557248900002e+00, 5.8849268465832001e+03 },
    { 4.4996799999999998e-06, 3.7988037559499999e+00, 6.3093741697912001e+03 },
    { 4.0633400000000000e-06, 5.2561626802700001e+00, 6.6812248533995999e+03 },
    { 2.5584500000000000e-06, 6.9454231563000002e-01, 1.2168002696574600e+04 },
    { 2.2793700000000002e-06, 5.9517924881399997e+00, 7.0585984613153996e+03 },
    { 2.2779200000000000e-06, 4.8454707473300003e+00, 4.6940029547076001e+03 },
    { 2.1950400000000000e-06, 2.9521613956800001e+00, 1.1506769769793600e+04 },
    { 2.0625700000000000e-06, 4.2836672888200003e+00, 1.1790629088658799e+04 },
    { 2.0000500000000000e-06, 2.5014408812000002e+00, 1.7789845619784999e+04 },
    { 1.7924199999999999e-06, 1.4000344602100001e+00, 7.9629800681639995e+02 },
    { 1.7827999999999999e-06, 5.1171755223100002e+00, 6.2830085396885997e+03 },
    { 1.7827999999999999e-06, 4.6730725524599999e+00, 6.2831431602941902e+03 },
    { 1.5545399999999999e-06, 5.3405254340000000e-02, 2.5132303399965600e+04 },
    { 1.5279000000000000e-06, 5.5812080044999997e+00, 2.1329909543799999e+02 },
    { 1.4976900000000000e-06, 6.0742902327800001e+00, 5.4867778431750003e+03 },
    { 1.2900600000000001e-06, 3.6462370863400002e+00, 7.0793738568077997e+03 },
    { 1.2821100000000000e-06, 3.2325482138099999e+00, 3.7387614301079998e+03 },
    { 1.1872499999999999e-06, 5.4536149048800002e+00, 9.4377629348869996e+03 },
    { 8.3511000000000002e-07, 4.6322352729800000e+00, 3.3406124266997999e+03 },
    { 8.0739999999999999e-07, 9.2225805400000005e-01, 6.0697767545533998e+03 },
    { 7.8418999999999998e-07, 2.8253093913999999e-01, 8.8273902698748007e+03 },
    { 7.5626000000000002e-07, 2.5886072612200000e+00, 6.4963749454294002e+03 },
    { 7.3513000000000005e-07, 1.1243337188000000e+00, 1.5890728952838001e+03 },
    { 6.5693999999999995e-07, 2.0889858705900002e+00, 1.1769853693166400e+04 },
    { 6.3616000000000000e-07, 4.9038985445000000e-01, 7.1135470007999997e+00 },
    { 6.3239999999999997e-07, 6.7044491569999998e-01, 6.2623004544989999e+03 },
    { 6.2961000000000004e-07, 2.8364975599600002e+00, 6.3038512454838001e+03 },
    { 6.1132999999999996e-07, 5.3433572124399999e+00, 3.1546870848956000e+03 },
    { 6.0039000000000004e-07, 1.8287888871300000e+00, 4.1369104335162001e+03 },
    { 5.8088999999999996e-07, 5.5165509744999996e-01, 1.0977078804699000e+04 },
    { 5.7573999999999999e-07, 1.1080213427000001e+00, 6.2820955289231997e+03 },
    { 5.7573999999999999e-07, 2.3989195763600000e+00, 6.2840561710596003e+03 },
    { 5.3746999999999995e-07, 6.2755394841600003e+00, 1.1944470102246000e+03 },
    { 5.1292999999999997e-07, 4.2136783455800000e+00, 1.2352852604544800e+04 },
    { 5.1141000000000005e-07, 6.0245804676599999e+00, 6.2901893969922003e+03 },
    { 5.0679000000000000e-07, 3.7713377989199999e+00, 6.2759623029905997e+03 },
    { 4.9612999999999998e-07, 1.0729039012699999e+00, 6.1276554505572003e+03 },
    { 4.9347000000000004e-07, 2.1574011399200002e+00, 2.5443144198834002e+03 },
    { 4.8912999999999999e-07, 5.1653104903999996e+00, 9.2255392732830005e+03 },
    { 4.8424999999999996e-07, 5.8331837099999999e-02, 8.0182093112380005e+02 },
    { 4.8378999999999997e-07, 2.5856696780999999e-01, 7.0848967811151997e+03 },
    { 4.7933999999999995e-07, 2.4317695223800002e+00, 6.4384962494255997e+03 },
    { 4.7358999999999999e-07, 4.5360794531100002e+00, 3.1283887650958000e+03 },
    { 4.2135000000000000e-07, 2.0653256004000000e+00, 1.7260154654690399e+04 },
    { 3.9728000000000002e-07, 4.7004886345400001e+00, 5.8564776591153995e+03 },
    { 3.9555000000000002e-07, 3.3594328417599999e+00, 5.4812549188676003e+03 },
    { 3.7954000000000002e-07, 1.4700564010799999e+00, 4.2659819087599999e+02 },
    { 3.7538999999999998e-07, 1.6004060360900001e+00, 8.4292412664666008e+03 },
    { 3.6367999999999998e-07, 1.3341649001800000e+00, 1.8073704938650200e+04 },
    { 3.2603000000000002e-07, 4.4180662616599999e+00, 4.1643119896130002e+03 },
    { 3.1140000000000000e-07, 3.6157956873299999e+00, 2.1461654164751999e+03 },
    { 3.0329000000000001e-07, 4.2324614390999998e-01, 4.5350594369244000e+03 },
    { 2.8514999999999998e-07, 4.5385338896400000e+00, 1.2569674818331800e+04 },
    { 2.8166000000000000e-07, 5.1936622291200001e+00, 1.2562628581633800e+04 },
    { 2.6740999999999998e-07, 5.9717288216200002e+00, 3.9302096962196001e+03 },
    { 2.5701000000000000e-07, 7.5895390817999997e-01, 1.3367972631106601e+04 },
    { 2.5031999999999998e-07, 3.7498202136100001e+00, 2.6298319799800002e+01 },
    { 2.4843000000000001e-07, 1.0026532601300000e+00, 5.0886288397668004e+03 },
    { 2.3246000000000001e-07, 4.6191003820500001e+00, 1.8319536584879599e+04 },
    { 2.1642000000000000e-07, 3.0644452647799998e+00, 1.2559038152982001e+04 },
    { 2.1070000000000000e-07, 2.2295947713099999e+00, 7.4775228602159996e+03 },
    { 2.0498000000000000e-07, 3.4443538453600002e+00, 1.1371704689758200e+04 },
    { 2.0024000000000000e-07, 2.9682318099799998e+00, 8.0310922630584000e+03 },
    { 1.7985000000000001e-07, 2.0774121130799998e+00, 1.7298182327326202e+04 },
    { 1.7888000000000000e-07, 5.0150419449800001e+00, 1.7480164130670000e+03 },
    { 1.7861000000000001e-07, 4.8680957028299998e+00, 1.5925960136327999e+03 },
    { 1.7512000000000000e-07, 6.1619258929100003e+00, 1.0447387839604400e+04 },
    { 1.7077000000000001e-07, 5.2972221627699998e+00, 4.9332084403325998e+03 },
    { 1.5078000000000000e-07, 1.5513222473199999e+00, 1.6496361396202399e+04 },
    { 1.4385999999999999e-07, 6.1489873293299997e+00, 2.0426571092422000e+04 },
    { 1.4331999999999999e-07, 4.4567869372900004e+00, 7.3424577801805999e+03 },
    { 1.4095000000000000e-07, 6.0370710353800003e+00, 2.9424634232916001e+03 },
    { 1.3831000000000000e-07, 3.0245052862400001e+00, 8.6359420037632008e+03 },
    { 1.3605000000000000e-07, 3.6264114503000000e+00, 1.3095842665077400e+04 },
    { 1.3558000000000000e-07, 1.4466633150299999e+00, 1.6730463689595799e+04 },
    { 1.3376000000000000e-07, 2.0361688707100001e+00, 1.2139553509106800e+04 },
    { 1.2555999999999999e-07, 2.5919480853799999e+00, 1.5720838784878400e+04 },
    { 1.2445999999999999e-07, 5.7809491852899999e+00, 1.5110466119866200e+04 },
    { 1.2375000000000001e-07, 1.5386228947700000e+00, 6.2794854213396002e+03 },
    { 1.2366000000000001e-07, 1.9683181487200001e+00, 6.2866662786431998e+03 },
    { 1.2227000000000000e-07, 4.3835942373499996e+00, 7.6329432596502002e+03 },
    { 1.1583000000000000e-07, 3.8897133360799998e+00, 5.3313574437407997e+03 },
    { 1.1583000000000000e-07, 2.4816580984300001e+00, 5.7295064471490005e+03 },
    { 1.0989000000000000e-07, 6.2040343661999997e-01, 1.0988808157535001e+04 },
    { 1.0480000000000000e-07, 1.3820930192900001e+00, 5.9992165311261997e+03 },
    { 1.0219000000000000e-07, 1.3075179540000001e-01, 1.3498674096587999e+03 },
    { 1.0125999999999999e-07, 9.9630008630000000e-01, 6.8366452528338004e+03 },
    { 9.8200000000000006e-08, 2.1247665514400000e+00, 6.5669351688566003e+03 },
    { 9.5399999999999994e-08, 3.4961327170600001e+00, 2.1228392023545799e+04 },
    { 9.1209999999999994e-08, 5.8300604714700004e+00, 7.2347942562420003e+03 },
    { 9.0229999999999995e-08, 3.8565502949899999e+00, 9.6236882766912004e+03 },
    { 8.8689999999999995e-08, 3.4761126528999999e+00, 6.1480107699560003e+03 },
    { 8.6140000000000002e-08, 3.0815893029999999e-02, 6.4181409300267997e+03 },
    { 8.6019999999999999e-08, 5.4395357531000004e+00, 5.6431785636774002e+03 },
    { 8.5440000000000003e-08, 1.8282930185699999e+00, 1.1015106477334801e+04 },
    { 8.2930000000000002e-08, 8.5407021371000003e-01, 1.4712317116458000e+04 },
    { 7.9109999999999998e-08, 1.5834823835900000e+00, 2.1187638603783998e+03 },
    { 7.5450000000000004e-08, 5.9484344443000001e+00, 1.2539853380183000e+04 },
    { 7.5269999999999998e-08, 1.9439212707400000e+00, 1.8422629359098191e+04 },
    { 7.4589999999999995e-08, 2.7113389887000001e+00, 2.2003914634869801e+04 },
    { 7.4140000000000001e-08, 3.4672043362100000e+00, 2.3543230504681789e+04 },
    { 7.3529999999999997e-08, 3.8322030732600001e+00, 6.0403472460173998e+03 },
    { 7.3169999999999999e-08, 2.0166447782099999e+00, 5.3680451209540001e+02 },
    { 7.2020000000000006e-08, 4.3131189893600004e+00, 6.2450481773556003e+03 },
    { 7.1439999999999996e-08, 5.4770356083599996e+00, 6.3211035226271997e+03 },
    { 7.0179999999999997e-08, 5.9465918051899997e+00, 6.5258044539654002e+03 },
    { 7.0039999999999997e-08, 3.5749283024200000e+00, 1.0309277421860000e+02 },
    { 6.9759999999999997e-08, 4.7662636148799997e+00, 1.0973555686350001e+04 },
    { 6.9160000000000005e-08, 5.3965807167800000e+00, 5.2257741809380002e+02 },
    { 6.8740000000000005e-08, 4.5750351246100003e+00, 7.2386755916000002e+03 },
    { 6.8519999999999994e-08, 4.3378127440599998e+00, 1.5542039943419999e+02 },
    { 6.6269999999999997e-08, 5.4137310277799999e+00, 5.3274761083827998e+03 },
    { 6.4869999999999998e-08, 2.2213587427200001e+00, 5.2165803728013998e+03 },
    { 6.3889999999999998e-08, 2.4058007146099998e+00, 2.4072921469776400e+04 },
    { 6.3520000000000002e-08, 3.4646130439800000e+00, 1.9907450170410000e+03 },
    { 6.3320000000000001e-08, 2.8372509870800000e+00, 2.6087903141574199e+04 },
    { 6.2509999999999995e-08, 1.5286089833300001e+00, 9.5171840625059997e+02 },
    { 6.2509999999999995e-08, 2.8839945816100001e+00, 6.2885987742988000e+03 },
    { 6.2509999999999995e-08, 6.2294633745000005e-01, 6.2775529256840000e+03 },
    { 6.2219999999999997e-08, 6.7296135144000002e-01, 1.8451078546565990e+04 },
    { 6.2099999999999994e-08, 8.1866451118000005e-01, 5.6502921106781996e+03 },
    { 6.1580000000000000e-08, 5.6585267523300002e+00, 1.2964300703391000e+04 },
    { 6.0640000000000006e-08, 5.1970877987000001e+00, 5.2308074668030004e+03 },
    { 5.8360000000000001e-08, 7.5921238848000006e-01, 6.3989728631399998e+02 },
    { 5.8280000000000003e-08, 1.3755538306399999e+00, 9.5559974160859997e+02 },
    { 5.7110000000000000e-08, 2.7563037492600002e+00, 6.9158595893046004e+03 },
    { 5.6050000000000002e-08, 1.9215070048100000e+00, 1.5510452226479999e+03 },
    { 5.4140000000000000e-08, 1.4288232519999999e+00, 6.1799830757727996e+03 },
    { 5.4130000000000002e-08, 2.0832598344700002e+00, 6.3861686242100004e+03 },
    { 5.3760000000000000e-08, 2.3496850550000001e+00, 1.4314168113049800e+04 },
    { 5.3370000000000001e-08, 1.9712781367100001e+00, 4.6904798363585996e+03 },
    { 5.2740000000000001e-08, 4.9168703669099996e+00, 2.5158601719765400e+04 },
    { 5.0160000000000000e-08, 1.4600685831300000e+00, 4.7320306273433998e+03 },
    { 4.8580000000000002e-08, 1.2791437467500000e+00, 5.7604984318976003e+03 },
    { 4.5960000000000003e-08, 2.9293659319500001e+00, 5.5356940284240000e+02 },
    { 4.4099999999999998e-08, 2.2353626421000001e+00, 6.8056532680851997e+03 },
    { 4.2699999999999999e-08, 8.1269903423000001e-01, 7.8756718636242003e+03 },
    { 4.1199999999999998e-08, 1.7256852147499999e+00, 6.7096740408674004e+03 },
    { 4.0709999999999999e-08, 3.9915915886300000e+00, 3.6346210245184002e+03 },
    { 4.0170000000000002e-08, 2.2150126590900001e+00, 6.2546266625236003e+03 },
    { 3.9930000000000001e-08, 1.2919250062500001e+00, 6.3115250374591997e+03 },
    { 3.9799999999999999e-08, 5.9091721062399998e+00, 1.3341674311306801e+04 },
    { 3.8850000000000000e-08, 3.9436014982300001e+00, 7.4781598567299994e+01 },
    { 3.7450000000000001e-08, 3.6502371210400000e+00, 5.9666839803348003e+03 },
    { 3.6260000000000002e-08, 6.1387106129099998e+00, 6.5994677196479997e+03 },
    { 3.6249999999999997e-08, 1.4311264105299999e+00, 6.0199919266185998e+03 },
    { 3.5920000000000000e-08, 4.0384380428500002e+00, 6.2082942514241004e+03 },
    { 3.5810000000000001e-08, 5.7523412925799997e+00, 6.3578574485586996e+03 },
    { 3.4340000000000002e-08, 2.0757733329699999e+00, 6.5461597733642002e+03 },
    { 3.3850000000000002e-08, 3.8107663710100002e+00, 1.3916019109641600e+04 },
    { 3.2899999999999997e-08, 9.5752698537000003e-01, 1.6200772724501199e+04 },
    { 3.1739999999999998e-08, 3.6203013150099999e+00, 1.0664954771900000e+03 },
    { 3.0829999999999999e-08, 5.6873190977899997e+00, 9.9176968745098002e+03 },
    { 3.0640000000000002e-08, 2.4660556147800001e+00, 5.8635912061161998e+03 },
    { 3.0600000000000003e-08, 1.6314239849700001e+00, 1.8052929543157799e+04 },
    { 3.0489999999999997e-08, 5.2288733079999997e-02, 2.9088811415985001e+04 },
    { 2.9790000000000001e-08, 4.6298794072299998e+00, 1.2566219010285600e+04 },
    { 2.9790000000000001e-08, 5.0739823770800001e+00, 1.2566084389680000e+04 },
    { 2.9270000000000000e-08, 1.7920182214200000e+00, 1.2721572099417001e+04 },
    { 2.7840000000000000e-08, 3.3795416517099999e+00, 1.4945316173554400e+04 },
    { 2.7160000000000000e-08, 1.4985584072800000e+00, 1.2029347187887401e+04 },
    { 2.6810000000000000e-08, 5.8562003955800002e+00, 4.5909101804889997e+03 },
    { 2.6700000000000001e-08, 2.6437088720899999e+00, 3.5231183490000002e+00 },
    { 2.6110000000000000e-08, 8.6180315522999995e-01, 6.7025604938666002e+03 },
    { 2.6000000000000001e-08, 3.9310981964699998e+00, 1.9651048481098002e+04 },
    { 2.5889999999999999e-08, 2.2755964082100002e+00, 1.2592450019782600e+04 },
    { 2.5580000000000001e-08, 6.0815132964699998e+00, 1.0575406682941801e+04 },
    { 2.5349999999999999e-08, 1.2310788880000000e-02, 3.1415379249957001e+04 },
    { 2.5290000000000000e-08, 4.3762958991399996e+00, 1.2043574281888999e+04 },
    { 2.4400000000000000e-08, 4.5584474698499999e+00, 1.4956319713459999e+02 },
    { 2.4019999999999999e-08, 4.8450325327000003e+00, 1.1624747044077999e+03 },
    { 2.3779999999999999e-08, 2.0565151903599999e+00, 2.3888940204492001e+03 },
    { 2.3269999999999999e-08, 2.8347272650400002e+00, 1.0021837280099400e+04 },
    { 2.2840000000000001e-08, 4.3814681454700004e+00, 6.0768903015542001e+03 },
    { 2.2500000000000000e-08, 8.2649282050999995e-01, 1.2669244474201399e+04 },
    { 2.2180000000000001e-08, 5.4056290952100001e+00, 6.4892613984285999e+03 },
    { 2.2150000000000000e-08, 3.6957432130900001e+00, 1.3362449706799200e+04 },
    { 2.1970000000000001e-08, 5.2241162812099997e+00, 1.3517870106233400e+04 },
    { 2.1629999999999999e-08, 6.1303676556399997e+00, 6.0372442037620003e+03 },
    { 2.0520000000000001e-08, 3.6592701647500001e+00, 6.5289074962207997e+03 },
    { 2.0149999999999998e-08, 9.1352603582000003e-01, 6.2627205305926000e+03 },
    { 2.0109999999999999e-08, 5.6982889142599999e+00, 4.7011165017084004e+03 },
    { 2.0050000000000001e-08, 2.5934426376999999e+00, 6.3034311693902000e+03 },
    { 1.9659999999999999e-08, 1.8222492818500000e+00, 4.6868894077067998e+03 },
    { 1.9519999999999999e-08, 3.0964355084599999e+00, 2.3013539539587200e+04 },
    { 1.8930000000000001e-08, 4.0971625968599996e+00, 6.2869571853493999e+03 },
    { 1.8930000000000001e-08, 5.6929636293800003e+00, 6.2791945146334001e+03 },
    { 1.8880000000000001e-08, 2.8785080538700001e+00, 8.6622403235630009e+03 },
    { 1.8410000000000001e-08, 3.6794737318999999e+00, 2.4272860397400001e+02 },
    { 1.7949999999999999e-08, 1.4716188616899999e+00, 3.3019021112204602e+04 },
    { 1.7780000000000001e-08, 2.7911989320199999e+00, 4.2923308329503998e+03 },
    { 1.7739999999999999e-08, 5.1400448882000005e-01, 6.8198803620868002e+03 },
    { 1.7560000000000000e-08, 2.9648288655399999e+00, 5.7462713378959997e+03 },
    { 1.6720000000000000e-08, 3.3097080655000000e-01, 1.3119721102825190e+04 },
    { 1.6479999999999999e-08, 3.9813541619400001e+00, 6.3278373931320004e+02 },
    { 1.6269999999999999e-08, 4.7744573786800002e+00, 1.3521751441591399e+04 },
    { 1.5780000000000000e-08, 4.1004638540700000e+00, 2.5934124331089399e+04 },
    { 1.5580000000000001e-08, 2.7967995167100002e+00, 1.7654780539749601e+04 },
    { 1.5570000000000000e-08, 4.7861483577600001e+00, 1.8875525869773999e+04 },
    { 1.5440000000000001e-08, 1.3391106147400000e+00, 1.1020632121940000e+02 },
    { 1.5430000000000000e-08, 4.3901865515900003e+00, 6.2724391846416001e+03 },
    { 1.5430000000000000e-08, 5.3999396746499997e+00, 6.2937125153411998e+03 },
    { 1.5049999999999999e-08, 2.7083712990100000e+00, 1.8209330263660191e+04 },
    { 1.4909999999999999e-08, 4.6589364594599996e+00, 9.4114646150871995e+03 },
    { 1.4880000000000000e-08, 4.1037744073100004e+00, 7.6266071275599998e+01 },
    { 1.4839999999999999e-08, 2.1530197683700001e+00, 2.0775395492400001e+01 },
    { 1.4769999999999999e-08, 2.5692512848100000e+00, 1.2779450795420800e+04 },
    { 1.4710000000000001e-08, 4.1869545652999998e+00, 1.8635928454536199e+04 },
    { 1.4640000000000001e-08, 3.7300016599200001e+00, 3.8133035637799999e+01 },
    { 1.4600000000000000e-08, 3.4015697397700002e+00, 2.4583164622940001e+02 },
    { 1.4400000000000000e-08, 6.9014829975000003e-01, 1.0522683831884001e+03 },
    { 1.3850000000000000e-08, 2.6577672770300000e+00, 2.3791644735716000e+03 },
    { 1.3580000000000000e-08, 3.3781842225999997e-01, 5.1206011455835996e+03 },
    { 1.3490000000000000e-08, 4.1801807281799999e+00, 6.0164688082696002e+03 },
    { 1.3119999999999999e-08, 5.8133049646000003e+00, 1.2491370101415499e+04 },
    { 1.2720000000000001e-08, 5.6097785736099999e+00, 6.5496828917131998e+03 },
    { 1.2630000000000000e-08, 5.6296786938799999e+00, 1.2573265246983599e+04 },
    { 1.2520000000000001e-08, 6.2683019832099998e+00, 1.1499656222792801e+04 },
    { 1.2420000000000000e-08, 1.2735917709800000e+00, 2.4356780788641601e+04 },
    { 1.2340000000000000e-08, 2.5042775659899998e+00, 2.2483848574492589e+04 },
    { 1.2229999999999999e-08, 6.2269472353699999e+00, 5.4298794682394000e+03 },
    { 1.1910000000000000e-08, 1.8592840724199999e+00, 6.2901220866894000e+03 },
    { 1.1910000000000000e-08, 1.6476568466399999e+00, 6.2760296132933900e+03 },
    { 1.1339999999999999e-08, 5.7628783167700002e+00, 1.7782732072784202e+04 },
    { 1.0920000000000000e-08, 2.3504865769499999e+00, 1.7796959166785800e+04 },
    { 1.0810000000000001e-08, 2.8924597952600002e+00, 3.6949230808424203e+04 },
    { 1.0449999999999999e-08, 5.1729289950600004e+00, 1.5508615123274400e+04 },
    { 1.0280000000000000e-08, 9.3950338023000002e-01, 6.1124029793722002e+03 },
    { 1.0250000000000001e-08, 6.4688768472000002e-01, 1.2545376304490401e+04 },
    { 1.0230000000000000e-08, 5.4281324618899998e+00, 2.2041264243879999e+02 },
    { 1.0140000000000000e-08, 4.3607292662999999e+00, 1.7256631536341400e+04 },
    { 9.9499999999999998e-09, 1.1179441301299999e+00, 1.0177257679533601e+04 },
    { 9.9200000000000005e-09, 2.5673813915700001e+00, 6.4537487206105998e+03 },
    { 9.8999999999999993e-09, 4.9721077826100002e+00, 1.1403676995575001e+04 },
    { 9.8799999999999998e-09, 5.6917145338399999e+00, 8.2738208670324002e+03 },
    { 9.7800000000000006e-09, 3.1677311982199998e+00, 6.2470479833139898e+03 },
    { 9.7300000000000001e-09, 6.1935882442099999e+00, 2.2805735565993600e+04 },
    { 9.7100000000000006e-09, 3.3921097187999999e-01, 6.3191037166688002e+03 },
    { 9.6199999999999995e-09, 1.0648891217300001e+00, 1.2565171378914600e+04 },
    { 9.6199999999999995e-09, 2.3557873553899999e+00, 1.2567132021051000e+04 },
    { 9.4400000000000005e-09, 5.4971270863599999e+00, 6.3069542877391996e+03 },
    { 9.4400000000000005e-09, 4.2929991398800000e+00, 6.2591974122436004e+03 },
    { 9.4199999999999993e-09, 6.1330151819300003e+00, 6.2678233788063899e+03 },
    { 9.4199999999999993e-09, 3.6571110443000001e+00, 6.2983283211764001e+03 },
    { 9.2900000000000008e-09, 6.4994792212999997e-01, 2.7870430238573999e+03 },
    { 9.0500000000000000e-09, 1.2623490824600001e+00, 6.2815913772831000e+03 },
    { 9.0500000000000000e-09, 2.2445918366000002e+00, 6.2845603226997000e+03 },
    { 8.9600000000000005e-09, 8.3013691513999999e-01, 3.9039113764198000e+03 },
    { 8.9399999999999993e-09, 2.9468168362000000e+00, 2.0618554843720000e+02 },
    { 8.7299999999999994e-09, 2.8676630680100001e+00, 9.4917560896980001e+02 },
    { 8.6900000000000004e-09, 1.7478566353700000e+00, 6.1335126528567998e+03 },
    { 8.5999999999999993e-09, 6.1932775500300004e+00, 1.6921656695024001e+03 },
    { 8.5600000000000002e-09, 1.3618199394299999e+00, 4.8042092759269999e+03 },
    { 8.5000000000000000e-09, 6.1644317884399999e+00, 6.3932821712107998e+03 },
    { 8.1699999999999997e-09, 3.5885557322600001e+00, 6.1728695287720002e+03 },
    { 8.1400000000000004e-09, 3.8200424648800002e+00, 3.8941818295421999e+03 },
    { 8.1300000000000007e-09, 5.2328337218299996e+00, 1.2416588502848201e+04 },
    { 7.9799999999999993e-09, 1.0542139358800000e+00, 1.2410731300548599e+04 },
    { 7.8800000000000001e-09, 1.8674637941900001e+00, 8.4608283475120004e+02 },
    { 7.8500000000000008e-09, 2.4491866413599999e+00, 4.9073020501456003e+03 },
    { 7.7900000000000006e-09, 2.0126028155200002e+00, 2.6484548254729998e+03 },
    { 7.5599999999999995e-09, 5.4006311673100003e+00, 4.0612192153944002e+03 },
    { 7.4300000000000002e-09, 2.2282843037600002e+00, 7.8341210726394002e+03 },
    { 7.3900000000000003e-09, 3.3989619816099998e+00, 1.1926254413668799e+04 },
    { 7.3700000000000000e-09, 5.4135830231200002e+00, 2.3581258177317599e+04 },
    { 6.6400000000000002e-09, 4.3135801248399996e+00, 4.0879440504643797e+04 },
    { 6.4899999999999997e-09, 3.3348885603700000e+00, 1.1764330768858999e+04 },
    { 6.4599999999999996e-09, 5.9111694284800000e+00, 5.2175806283148399e+04 },
    { 6.2900000000000004e-09, 1.3445581604100001e+00, 2.6735945262213201e+04 },
    { 6.2000000000000001e-09, 8.3937590497000003e-01, 5.3267866940207996e+03 },
    { 6.1600000000000002e-09, 4.1338975577000001e+00, 1.5961864422845999e+03 },
    { 6.1200000000000004e-09, 5.8918440607799996e+00, 1.6858482532933202e+04 },
    { 6.1099999999999998e-09, 6.2438794373500004e+00, 8.5319638175199998e+02 },
    { 6.0799999999999997e-09, 2.3758880767199999e+00, 1.0419986283507600e+04 },
    { 6.0699999999999999e-09, 4.7984991631199998e+00, 5.8493641121146002e+03 },
    { 5.9900000000000002e-09, 5.4713576844000000e-01, 6.1515338883049999e+03 },
    { 5.9600000000000001e-09, 9.9065349155000004e-01, 6.0626632075526004e+03 },
    { 5.8999999999999999e-09, 4.7736597307000000e-01, 1.5671081759406599e+04 },
    { 5.8200000000000002e-09, 2.9601281957599999e+00, 6.4146178116778001e+03 },
    { 5.6999999999999998e-09, 2.9758632890599999e+00, 1.2586927095475199e+04 },
    { 5.6500000000000001e-09, 5.4923826198199999e+00, 4.1714255366138004e+03 },
    { 5.6100000000000003e-09, 5.8581424659600003e+00, 1.2323423096008801e+04 },
    { 5.5999999999999997e-09, 1.4432555287000001e-01, 1.5819593482830001e+03 },
    { 5.5500000000000001e-09, 2.8669830179300000e+00, 1.1513883316794399e+04 },
    { 5.5200000000000000e-09, 6.0826094468300003e+00, 1.1933367960669601e+04 },
    { 5.5100000000000002e-09, 2.4679479801999999e+00, 6.5034884924301996e+03 },
    { 5.4000000000000004e-09, 1.8336107488400000e+00, 6.2797894925736000e+03 },
    { 5.4000000000000004e-09, 1.6733301702200001e+00, 6.2863622074092000e+03 },
    { 5.3800000000000000e-09, 1.3348111551599999e+00, 5.3339002410215999e+03 },
    { 5.3599999999999997e-09, 4.4958095827199998e+00, 1.8852750668323199e+04 },
    { 5.3599999999999997e-09, 2.1314533496600001e+00, 7.2322514589612001e+03 },
    { 5.3199999999999998e-09, 3.0168257139199999e+00, 1.8842114002973391e+04 },
    { 5.3000000000000003e-09, 5.1473010192800004e+00, 1.8845704431625200e+04 },
    { 5.2700000000000002e-09, 4.6774732704700002e+00, 2.4602612434870989e+04 },
    { 5.1600000000000004e-09, 1.3579770924600001e+00, 4.3371173787679999e+02 },
    { 5.1499999999999998e-09, 9.9957036400000004e-02, 1.2320320053753399e+04 },
    { 5.1300000000000003e-09, 2.1321197276600001e+00, 1.2455945378763399e+04 },
    { 5.0300000000000002e-09, 4.0285706717099998e+00, 1.3506508003540000e+02 },
    { 5.0199999999999996e-09, 5.6864613664599997e+00, 1.2489885628707199e+04 },
    { 5.0099999999999999e-09, 3.0835569085299999e+00, 1.7157061880471800e+04 },
    { 4.9200000000000004e-09, 4.3975718932100003e+00, 1.4788665740644001e+03 },
    { 4.8499999999999996e-09, 6.0668309636000002e+00, 1.2528018664345000e+04 },
    { 4.8399999999999998e-09, 3.1541423563999998e-01, 1.4227094001599999e+01 },
    { 4.7900000000000002e-09, 6.0475590766299998e+00, 5.5471993364596001e+03 },
    { 4.7799999999999996e-09, 4.5827555012000003e+00, 2.9296615389578601e+04 },
    { 4.7200000000000002e-09, 3.8118180473000001e+00, 1.0969965257698201e+04 },
    { 4.6200000000000002e-09, 3.5372912882800001e+00, 3.1049300594237998e+03 },
    { 4.4999999999999998e-09, 3.8613288324000000e-01, 4.2745183108323999e+03 },
    { 4.3200000000000000e-09, 5.2227753308500002e+00, 9.3880059094152002e+03 },
    { 4.3100000000000002e-09, 3.7406631487999999e+00, 7.0189523635231999e+03 },
    { 4.2299999999999997e-09, 2.4528137660199998e+00, 9.7791086761253991e+03 },
    { 4.2100000000000001e-09, 3.2565413572900002e+00, 7.4455505543992003e+03 },
    { 4.2000000000000004e-09, 1.7685120155200000e+00, 2.2779437246193800e+04 },
    { 4.1599999999999997e-09, 2.1739998797400002e+00, 1.3760598710207400e+04 },
    { 4.1400000000000002e-09, 5.7345848292500001e+00, 4.4809650200863398e+04 },
    { 4.0899999999999997e-09, 6.1457756437900004e+00, 1.0984192351699799e+04 },
    { 4.0899999999999997e-09, 2.0394629862600002e+00, 2.1954157609397989e+04 },
    { 4.0800000000000000e-09, 4.9863653855200001e+00, 3.4960328261340001e+03 },
    { 3.9499999999999998e-09, 1.3922562525300000e+00, 4.1571984426121999e+03 },
    { 3.9499999999999998e-09, 3.8234829317000001e+00, 5.9057022420756002e+03 },
    { 3.9499999999999998e-09, 4.7895592951400001e+00, 4.1760413424489998e+03 },
    { 3.9300000000000003e-09, 6.2788495113699998e+00, 7.8721487452751999e+03 },
    { 3.8899999999999996e-09, 1.6472637432999999e+00, 6.3212088856292003e+03 },
    { 3.8899999999999996e-09, 1.8596771757599999e+00, 6.2449428143535997e+03 },
    { 3.8799999999999998e-09, 5.8520212226300004e+00, 6.2973029439929996e+03 },
    { 3.8799999999999998e-09, 3.9381050036100000e+00, 6.2688487559898003e+03 },
    { 3.8799999999999998e-09, 5.2035494333500001e+00, 9.5628915597059995e+02 },
    { 3.8600000000000003e-09, 7.1941423742999999e-01, 2.6367254726370002e+03 },
    { 3.7900000000000004e-09, 5.5265043052999996e+00, 3.1851920272655998e+03 },
    { 3.7399999999999999e-09, 5.1569753225400001e+00, 5.6421982426092000e+03 },
    { 3.7099999999999998e-09, 4.1915389751700003e+00, 1.3625533630172000e+04 },
    { 3.7000000000000000e-09, 2.7962302552399998e+00, 1.4919017853754600e+04 },
    { 3.6899999999999999e-09, 3.3632740572499999e+00, 5.4369930152402003e+03 },
    { 3.6500000000000000e-09, 5.3940975175000005e-01, 6.9229731363053997e+03 },
    { 3.6500000000000000e-09, 5.9666432293899998e+00, 6.6604494579071998e+03 },
    { 3.6300000000000001e-09, 2.7199014567000002e+00, 7.3353442331797996e+03 },
    { 3.6100000000000001e-09, 5.5353807716599999e+00, 2.1393541969857601e+04 },
    { 3.5800000000000000e-09, 1.5865836705800000e+00, 6.2713464971554004e+03 },
    { 3.5800000000000000e-09, 1.9203572484800000e+00, 6.2948052028273996e+03 },
    { 3.5699999999999999e-09, 1.1215375572299999e+00, 1.6460333529524989e+04 },
    { 3.5699999999999999e-09, 2.7465414942500002e+00, 3.0666154958432799e+04 },
    { 3.5400000000000002e-09, 2.9658085778599999e+00, 1.7267268201691189e+04 },
    { 3.5100000000000001e-09, 6.4808322426999998e-01, 1.1616976091013001e+04 },
    { 3.5100000000000001e-09, 4.3631590078600002e+00, 1.2345739057544000e+04 },
    { 3.4900000000000001e-09, 1.0407727082700000e+00, 3.1639186965660002e+02 },
    { 3.4699999999999998e-09, 5.2704006846900002e+00, 1.7253041107689591e+04 },
    { 3.4400000000000001e-09, 1.9448007040899999e+00, 6.1474505249814001e+03 },
    { 3.3900000000000001e-09, 4.5374952069600001e+00, 6.9239534573736000e+03 },
    { 3.3600000000000000e-09, 3.6644305683499998e+00, 6.2870080032545002e+03 },
    { 3.3600000000000000e-09, 6.1256956578899997e+00, 6.2791436967282998e+03 },
    { 3.3400000000000000e-09, 1.5622351361200000e+00, 6.4187011750013999e+03 },
    { 3.2900000000000000e-09, 5.8796113401000003e+00, 3.9321532631000000e+00 },
    { 3.2900000000000000e-09, 5.8692878809800000e+00, 3.7737360791579999e+02 },
    { 3.2500000000000002e-09, 9.2198442482999998e-01, 1.2359966151545599e+04 },
    { 3.2000000000000001e-09, 9.2923666608000000e-01, 1.0818135286915800e+04 },
    { 3.1899999999999999e-09, 5.3061370952100004e+00, 3.8027672635800002e+01 },
    { 3.1599999999999998e-09, 3.9712566227899999e+00, 6.0555997172023999e+03 },
    { 3.1500000000000001e-09, 3.4767889422499998e+00, 1.0873986030480401e+04 },
    { 3.1200000000000000e-09, 1.6115099263499999e+00, 1.1720068865231600e+04 },
    { 3.0899999999999999e-09, 5.7495592500000003e-01, 2.7511467873537200e+04 },
    { 3.0800000000000001e-09, 2.5433760642099998e+00, 2.0995392966449399e+04 },
    { 3.0800000000000001e-09, 5.4006948274399997e+00, 2.9864334027309000e+04 },
    { 3.0399999999999998e-09, 4.8722475787399997e+00, 5.7498617665477996e+03 },
    { 3.0399999999999998e-09, 3.3689399570599998e+00, 2.9826306354673201e+04 },
    { 3.0100000000000002e-09, 5.8188697364099999e+00, 6.5105519827804001e+03 },
    { 2.9800000000000000e-09, 2.2844335769700002e+00, 6.4087760738220004e+02 },
    { 2.9600000000000001e-09, 4.7913838497399999e+00, 6.1292970395066004e+03 },
    { 2.9400000000000002e-09, 2.5993551294700001e+00, 2.8286990484861199e+04 },
    { 2.8600000000000000e-09, 4.9988628775299997e+00, 6.4368546604761996e+03 },
    { 2.8499999999999999e-09, 9.3414827012000001e-01, 5.4695255660315997e+03 },
    { 2.8499999999999999e-09, 5.8510453256600004e+00, 1.6522659716002199e+04 },
    { 2.8200000000000002e-09, 3.6450585333699999e+00, 2.6308392337279997e+02 },
    { 2.7799999999999999e-09, 3.8633587480900000e+00, 2.0597243963041201e+04 },
    { 2.7299999999999999e-09, 5.7910943210500001e+00, 2.6709646942413401e+04 },
    { 2.6900000000000000e-09, 4.9177622679799997e+00, 6.8162899334350004e+03 },
    { 2.6700000000000001e-09, 1.8493625272600001e+00, 2.4705705209089589e+04 },
    { 2.6500000000000002e-09, 3.5393069587200001e+00, 1.9378918515068799e+04 },
    { 2.6299999999999998e-09, 6.0495386314099999e+00, 1.1712955318230799e+04 },
    { 2.5899999999999999e-09, 1.8604774991499999e+00, 6.2828320325078903e+03 },
    { 2.5899999999999999e-09, 1.6464634199099999e+00, 6.2833196674748997e+03 },
    { 2.5700000000000000e-09, 8.7147328094999998e-01, 4.8739859897082999e+04 },
    { 2.5599999999999998e-09, 7.3074058890000004e-01, 2.2434479570190001e+02 },
    { 2.5500000000000001e-09, 3.3752477257900000e+00, 1.3757737998458001e+03 },
    { 2.5300000000000002e-09, 2.0849029302100002e+00, 4.3776110850510004e+03 },
    { 2.5000000000000001e-09, 2.6710353758099998e+00, 5.9341514295433999e+03 },
    { 2.4600000000000002e-09, 4.2649507616499998e+00, 8.6719698704406001e+03 },
    { 2.4300000000000001e-09, 2.5309325084599998e+00, 6.3104774060882000e+03 },
    { 2.4300000000000001e-09, 9.7600841059999999e-01, 6.2556742938945999e+03 },
    { 2.4300000000000001e-09, 2.5726701577700002e+00, 7.0966261339512002e+03 },
    { 2.4100000000000002e-09, 3.5216135983400001e+00, 1.5906764126682599e+04 },
    { 2.4100000000000002e-09, 5.0004045511899999e+00, 8.3901103575338002e+03 },
    { 2.4000000000000000e-09, 4.5238664562999997e-01, 1.4844727083000000e+00 },
    { 2.3499999999999999e-09, 4.9414307413299996e+00, 4.4878174062704002e+03 },
    { 2.3400000000000002e-09, 3.3992064306000003e-01, 5.0175083713650001e+03 },
    { 2.2999999999999999e-09, 5.7289896006300003e+00, 1.2655674786264001e+03 },
    { 2.2999999999999999e-09, 8.3612842797999998e-01, 6.6320002704394001e+03 },
    { 2.2699999999999998e-09, 4.1681139727799996e+00, 3.4596364654652403e+04 },
    { 2.2499999999999999e-09, 3.4388365482799998e+00, 2.6660704172179999e+02 },
    { 2.2400000000000001e-09, 5.5868038852400002e+00, 5.6360650166766000e+03 },
    { 2.2100000000000000e-09, 4.6575792324199998e+00, 5.5400857894587998e+03 },
    { 2.2100000000000000e-09, 2.0722429253099999e+00, 6.2068097787158003e+03 },
    { 2.1999999999999998e-09, 6.0035645970199996e+00, 2.0199094959632999e+04 },
    { 2.1400000000000001e-09, 5.2396088269999996e-01, 2.2218566345969998e+03 },
    { 2.1400000000000001e-09, 2.7723780133500000e+00, 2.4492406113651588e+04 },
    { 2.1299999999999999e-09, 2.2236523701899999e+00, 3.0355997319767801e+04 },
    { 2.1200000000000001e-09, 1.8391974040000000e-01, 7.1291586847425997e+03 },
    { 2.0900000000000000e-09, 3.7113909976100001e+00, 4.1948464387519999e+02 },
    { 2.0799999999999998e-09, 3.1089767949899998e+00, 1.0440274292603601e+04 },
    { 2.0700000000000001e-09, 1.4954906172000000e+00, 1.2562561271331000e+04 },
    { 2.0700000000000001e-09, 1.9251858599300000e+00, 1.2569742128634600e+04 },
    { 2.0500000000000002e-09, 2.5653347334999999e+00, 1.6627370915377200e+04 },
    { 2.0099999999999999e-09, 6.2305985760800002e+00, 3.0978838227257902e+03 },
    { 1.9900000000000000e-09, 5.6480704669700001e+00, 1.1216284290324000e+04 },
    { 1.9900000000000000e-09, 5.1562082202699999e+00, 3.3326578733174203e+04 },
    { 1.9599999999999998e-09, 5.4185103242699997e+00, 1.1087285125918401e+04 },
    { 1.9399999999999999e-09, 2.1371272933299998e+00, 2.8449187467800002e+01 },
    { 1.9300000000000002e-09, 4.7150654713500000e+00, 3.9462588505920002e+02 },
    { 1.9200000000000000e-09, 4.7333599880999999e-01, 1.7271884007526402e+04 },
    { 1.8899999999999999e-09, 2.5291930238800000e+00, 1.6062184526116800e+04 },
    { 1.8899999999999999e-09, 4.0341187245799999e+00, 4.0167212175719999e+02 },
    { 1.8500000000000000e-09, 1.5138687010500000e+00, 1.9800945956224801e+04 },
    { 1.8400000000000001e-09, 3.8348736972499999e+00, 9.3809596727171993e+03 },
    { 1.7800000000000001e-09, 1.8079259830000000e-02, 4.7970957289261996e+03 },
    { 1.7599999999999999e-09, 2.4811932999400002e+00, 1.2012582297140400e+04 },
    { 1.7500000000000000e-09, 3.8409362824000000e+00, 1.1614433293732200e+04 },
    { 1.7500000000000000e-09, 6.1531364652200002e+00, 3.7448273457378000e+03 },
    { 1.7200000000000001e-09, 3.8189367968800001e+00, 6.3328328754632003e+03 },
    { 1.7200000000000001e-09, 5.9711894293599999e+00, 6.2333188245195997e+03 },
    { 1.6999999999999999e-09, 5.3505271586400003e+00, 6.2720301497275004e+03 },
    { 1.6999999999999999e-09, 4.4395990675999997e+00, 6.2941215502552996e+03 },
    { 1.6900000000000000e-09, 8.0098424793000000e-01, 7.4299006053259995e+02 },
    { 1.6800000000000000e-09, 3.9761735742800002e+00, 1.7686752845566400e+04 },
    { 1.6700000000000000e-09, 2.4030822831399998e+00, 5.7142539759640003e+03 },
    { 1.6700000000000000e-09, 1.1666095816800000e+00, 1.0344295065385801e+04 },
    { 1.6700000000000000e-09, 5.9837137112400001e+00, 1.7473453750128399e+04 },
    { 1.6500000000000001e-09, 3.9341190294000000e-01, 1.0454501386605200e+04 },
    { 1.6300000000000000e-09, 1.5443216104499999e+00, 6.6123292678099995e+02 },
    { 1.6300000000000000e-09, 2.7876964477900001e+00, 9.9294262273458007e+03 },
    { 1.6200000000000000e-09, 3.2167452300599999e+00, 1.2551924605981199e+04 },
    { 1.6100000000000000e-09, 2.1845566684200000e+00, 1.2132439962106000e+04 },
    { 1.6100000000000000e-09, 1.7502579842099999e+00, 3.1441677569756801e+04 },
    { 1.6100000000000000e-09, 1.0289880800100000e+00, 2.0452869412221800e+04 },
    { 1.6100000000000000e-09, 2.2899481679500000e+00, 5.2670069593302600e+04 },
    { 1.5900000000000001e-09, 2.3316645339200002e+00, 1.3198935439295999e+04 },
    { 1.5600000000000000e-09, 1.0070891232000001e+00, 4.3075833041353999e+03 },
    { 1.5500000000000000e-09, 5.6282623945800001e+00, 6.0587310542895002e+03 },
    { 1.5100000000000000e-09, 4.7272957530399999e+00, 1.6723350142595002e+04 },
    { 1.4800000000000001e-09, 1.1038588177699999e+00, 6.8518977240187996e+03 },
    { 1.4800000000000001e-09, 3.9365790577599999e+00, 2.8766924424484001e+04 },
    { 1.4800000000000001e-09, 3.5590974078600000e+00, 2.8237233459389401e+04 },
    { 1.4599999999999999e-09, 4.9520184954799999e+00, 1.7370360975909800e+04 },
    { 1.4599999999999999e-09, 6.2708750338500003e+00, 1.2701216780018200e+04 },
    { 1.4500000000000000e-09, 3.7096789011800002e+00, 9.5979227217800002e+01 },
    { 1.4500000000000000e-09, 1.1717039771800000e+00, 6.3607496204194003e+03 },
    { 1.4500000000000000e-09, 2.3352369418799999e+00, 6.2054020795633996e+03 },
    { 1.4400000000000000e-09, 2.0392999457599998e+00, 1.6737577236596600e+04 },
    { 1.4300000000000000e-09, 5.5644114722699998e+00, 5.6183198048614004e+03 },
    { 1.4300000000000000e-09, 3.8936537464200001e+00, 5.0103948243641998e+03 },
    { 1.4300000000000000e-09, 6.2248881738900002e+00, 2.0690068349066000e+03 },
    { 1.4300000000000000e-09, 5.6322160475000000e+00, 1.8772961478698599e+04 },
    { 1.3999999999999999e-09, 5.9346032807100002e+00, 2.0085575391590000e+03 },
    { 1.3999999999999999e-09, 5.6275623038799996e+00, 3.8526574350871997e+04 },
    { 1.3900000000000000e-09, 4.2082969143200000e+00, 1.1609862544012200e+04 },
    { 1.3800000000000000e-09, 5.1855418741000003e-01, 6.3563729758503996e+03 },
    { 1.3800000000000000e-09, 2.9883867316499999e+00, 6.2097787241324004e+03 },
    { 1.3700000000000000e-09, 1.0298976813500000e+00, 1.1045700263900001e+01 },
    { 1.3700000000000000e-09, 2.5335864642299999e+00, 6.2505432991999996e+03 },
    { 1.3700000000000000e-09, 9.7335445483000005e-01, 6.3156084007828003e+03 },
    { 1.3500000000000001e-09, 4.7905486055399997e+00, 1.2431086619947400e+04 },
    { 1.3500000000000001e-09, 4.9583891446999999e+00, 3.5315282502998002e+03 },
    { 1.3500000000000001e-09, 2.8038765026800001e+00, 1.9804827291582798e+04 },
    { 1.3500000000000001e-09, 5.2829251214599999e+00, 1.8699664352839602e+04 },
    { 1.3100000000000000e-09, 2.0412239549400000e+00, 1.8415515812097390e+04 },
    { 1.3100000000000000e-09, 5.9378205027200002e+00, 5.7914125575325997e+03 },
    { 1.2600000000000000e-09, 5.9053316427600002e+00, 1.8822929230174399e+04 },
    { 1.2600000000000000e-09, 7.0432668892000005e-01, 1.2934842235826000e+03 },
    { 1.2400000000000001e-09, 4.2257151346899997e+00, 6.9478318951213996e+03 },
    { 1.2400000000000001e-09, 5.5065777592099998e+00, 1.0239583866010800e+04 },
    { 1.2400000000000001e-09, 3.1264742101700000e+00, 1.9402796952816600e+04 },
    { 1.2300000000000001e-09, 8.0538262186999998e-01, 2.4734154396557398e+04 },
    { 1.2300000000000001e-09, 5.7183331625899996e+00, 6.3058510514421996e+03 },
    { 1.2300000000000001e-09, 4.0717930636500004e+00, 6.2603006485406004e+03 },
    { 1.2199999999999999e-09, 4.1218407758100000e+00, 3.5833410306738001e+03 },
    { 1.2100000000000000e-09, 3.5343103342000002e+00, 1.7996031168222202e+04 },
    { 1.2000000000000000e-09, 5.9037980561800003e+00, 1.2808880303956799e+04 },
    { 1.2000000000000000e-09, 4.2699727015200004e+00, 1.2528124027346999e+04 },
    { 1.2000000000000000e-09, 4.4236531152899996e+00, 1.7892938394003591e+04 },
    { 1.2000000000000000e-09, 5.1717377639999995e-01, 1.4158747713615599e+04 },
    { 1.1900000000000000e-09, 5.9093471929700003e+00, 1.9247376553382401e+04 },
    { 1.1900000000000000e-09, 2.7162858393799998e+00, 1.2249759830326200e+04 },
    { 1.1900000000000000e-09, 2.6217017763000000e-01, 9.8146041002912007e+03 },
    { 1.1800000000000001e-09, 5.9516447787500004e+00, 6.3027456108911902e+03 },
    { 1.1800000000000001e-09, 3.8384814474900000e+00, 6.2634060890915998e+03 },
    { 1.1700000000000001e-09, 5.4503477291899998e+00, 1.1610551958374201e+04 },
    { 1.1700000000000001e-09, 1.1111413126000000e+00, 2.4336005393149200e+04 },
    { 1.1700000000000001e-09, 2.4183104268500002e+00, 2.4383079108441401e+04 },
    { 1.1500000000000000e-09, 5.3551120748400001e+00, 3.6417345715192000e+03 },
    { 1.1500000000000000e-09, 3.0946483638500002e+00, 1.2570083853245900e+04 },
    { 1.1400000000000000e-09, 2.1829246080200000e+00, 6.1550570066540004e+03 },
    { 1.1400000000000000e-09, 1.3240163110400001e+00, 6.4110946933287996e+03 },
    { 1.1400000000000000e-09, 2.5000143309300000e+00, 8.2585683958474001e+03 },
    { 1.1300000000000000e-09, 3.8311301541999998e-01, 2.1116503133776000e+03 },
    { 1.1300000000000000e-09, 3.0678944145200000e+00, 1.0557594160823801e+04 },
    { 1.1300000000000000e-09, 4.7970289838000002e-01, 1.2985636343857999e+04 },
    { 1.1200000000000001e-09, 3.8813622226500000e+00, 1.6840670010815189e+04 },
    { 1.1200000000000001e-09, 3.9182550554800000e+00, 1.2188778092066999e+04 },
    { 1.1100000000000001e-09, 7.0362725529000003e-01, 6.3399742249270003e+03 },
    { 1.1100000000000001e-09, 2.8033136637700000e+00, 6.2261774750557997e+03 },
    { 1.0999999999999999e-09, 3.3158749470100002e+00, 6.2764472910913901e+03 },
    { 1.0999999999999999e-09, 1.9106597205000000e-01, 6.2897044088914099e+03 },
    { 1.0999999999999999e-09, 1.2204062531400000e+00, 6.3125053585273999e+03 },
    { 1.0999999999999999e-09, 2.2865346659200001e+00, 6.2536463414554000e+03 },
    { 1.0999999999999999e-09, 1.9980394718100001e+00, 1.6097679950282600e+04 },
    { 1.0700000000000000e-09, 5.9542543866999997e-01, 1.2463058925764200e+04 },
    { 1.0600000000000001e-09, 2.1883541651499998e+00, 1.2341806904280900e+04 },
    { 1.0600000000000001e-09, 1.7060101934000000e+00, 1.2772337248420001e+04 },
    { 1.0500000000000001e-09, 1.2039915289200001e+00, 6.4349731310766001e+03 },
    { 1.0500000000000001e-09, 2.3029493901400002e+00, 6.1311785689061999e+03 },
    { 1.0399999999999999e-09, 5.7981411648000003e-01, 1.2560628775675401e+04 },
    { 1.0300000000000000e-09, 1.1872991918600000e+00, 6.3246266409762002e+03 },
    { 1.0300000000000000e-09, 2.3196417272000001e+00, 6.2415250590065998e+03 },
    { 1.0300000000000000e-09, 2.8066622549200000e+00, 1.8202216716659390e+04 },
    { 1.0300000000000000e-09, 3.7100801092600002e+00, 5.6600279289522201e+04 },
    { 1.0200000000000000e-09, 1.3270396077900000e+00, 5.7676119788983997e+03 },
    { 1.0200000000000000e-09, 2.1799013112700001e+00, 6.7985397210844003e+03 },
    { 1.0100000000000000e-09, 4.9427050184799999e+00, 1.8429742906099000e+04 },
    { 1.0000000000000001e-09, 3.0790895157399998e+00, 5.5229243074000003e+00 },
    { 1.0000000000000001e-09, 3.8071475239299999e+00, 5.3161049725557996e+03 },
    { 9.9000000000000010e-10, 2.6845537797899999e+00, 7.8263709424722590e+04 },
    { 9.9000000000000010e-10, 2.0487852890300000e+00, 6.4326390471260002e+03 },
    { 9.9000000000000010e-10, 5.2940863445500002e+00, 6.3139899756264003e+03 },
    { 9.9000000000000010e-10, 4.4960398816899998e+00, 6.2521617243563996e+03 },
    { 9.7999999999999992e-10, 4.0043493350799997e+00, 5.8151108596369904e+03 },
    { 9.7999999999999992e-10, 1.6630165252800000e+00, 6.1266751294890000e+03 },
    { 9.7999999999999992e-10, 1.8439243937800001e+00, 6.4394765704938000e+03 },
    { 9.5999999999999999e-10, 3.4779312356699998e+00, 2.2743409379516401e+04 },
    { 9.5999999999999999e-10, 1.0741440141500000e+00, 7.3495713271814002e+03 },
    { 9.5999999999999999e-10, 4.1418225889999999e-01, 5.8884499649321997e+03 },
    { 9.5999999999999999e-10, 3.0927586601599999e+00, 6.6777017350506003e+03 },
    { 9.5000000000000003e-10, 4.0622855240800000e+00, 1.1300584221356399e+04 },
    { 9.5000000000000003e-10, 8.1797133474999995e-01, 1.1925274092600601e+04 },
    { 9.4000000000000006e-10, 1.3065202435000001e+00, 2.6997348193175999e+03 },
    { 9.4000000000000006e-10, 5.5410652659900004e+00, 2.3539707386332801e+04 },
    { 9.2999999999999999e-10, 3.4809743975000002e-01, 1.1190377900137000e+04 },
    { 9.2999999999999999e-10, 3.1068065750699998e+00, 3.5371887265976402e+04 },
    { 9.2999999999999999e-10, 4.6616336788100003e+00, 2.3937856389740999e+04 },
    { 9.2999999999999999e-10, 2.2422436514999999e-01, 2.8385931886520001e+02 },
    { 9.2000000000000003e-10, 1.7588610974000001e+00, 2.1424466644303400e+04 },
    { 9.2000000000000003e-10, 6.0739627378599996e+00, 1.8216443810661000e+04 },
    { 9.0999999999999996e-10, 1.4237784364699999e+00, 1.2829235623355600e+04 },
    { 9.0999999999999996e-10, 9.0446873975999997e-01, 6.6527756659318002e+03 },
    { 9.0999999999999996e-10, 2.6024721792999999e+00, 5.9133760340509998e+03 },
    { 8.9999999999999999e-10, 4.0501633370699999e+00, 1.5664035522708589e+04 },
    { 8.9000000000000003e-10, 5.7857766209800001e+00, 6.7510408403457996e+03 },
    { 8.7999999999999996e-10, 1.2768507476999999e-01, 1.5141390794311999e+04 },
    { 8.7999999999999996e-10, 2.1966328121099998e+00, 1.7053969106253189e+04 },
    { 8.7999999999999996e-10, 4.5749691372400001e+00, 1.5738204240988000e+03 },
    { 8.7999999999999996e-10, 5.5830638743299996e+00, 1.6943762785033799e+04 },
    { 8.7999999999999996e-10, 1.6782163180000000e+00, 1.2537702512514999e+04 },
    { 8.7999999999999996e-10, 3.5912254279800000e+00, 1.0660686935042400e+04 },
    { 8.6999999999999999e-10, 1.6353534437199999e+00, 2.3141558382924599e+04 },
    { 8.6999999999999999e-10, 3.8928641332699998e+00, 1.5808666607968000e+03 },
    { 8.6999999999999999e-10, 2.0633609149300001e+00, 1.8106237489441599e+04 },
    { 8.6000000000000003e-10, 6.0538567576600002e+00, 2.3123745860806590e+04 },
    { 8.6000000000000003e-10, 4.1843948684000001e+00, 1.6193659177500391e+04 },
    { 8.4999999999999996e-10, 2.0816190225200000e+00, 1.2850011018847999e+04 },
    { 8.4999999999999996e-10, 1.3390434542600000e+00, 1.2282292381117601e+04 },
    { 8.3999999999999999e-10, 2.6666577295299998e+00, 6.3790550772092001e+03 },
    { 8.3999999999999999e-10, 8.4028318953000003e-01, 6.1870966227735998e+03 },
    { 8.3999999999999999e-10, 3.8219641330999998e+00, 3.2217200181080800e+04 },
    { 8.3000000000000003e-10, 4.7152308415700004e+00, 8.8583149443206003e+03 },
    { 8.1999999999999996e-10, 4.9061425643099996e+00, 5.4832547248259998e+03 },
    { 8.1999999999999996e-10, 5.4027872385700002e+00, 3.9581264411758002e+03 },
    { 8.1999999999999996e-10, 3.9162365993499999e+00, 6.7747391424502002e+03 },
    { 8.0999999999999999e-10, 5.1914910566800003e+00, 3.9565080160194002e+03 },
    { 8.0999999999999999e-10, 5.9829791890699999e+00, 7.2500467274270004e+03 },
    { 8.0000000000000003e-10, 2.0198825145199999e+00, 2.4080035016777190e+04 },
    { 8.0000000000000003e-10, 1.7082689856700000e+00, 6.4853292451654997e+03 },
    { 8.0000000000000003e-10, 1.7986719333900001e+00, 6.0808224548173002e+03 },
    { 8.0000000000000003e-10, 3.7967732561299998e+00, 2.2380755800274001e+04 },
    { 8.0000000000000003e-10, 5.4917852760599999e+00, 2.3646323278900389e+04 },
    { 7.8999999999999996e-10, 6.1611473857999997e+00, 4.0796338922674193e+04 },
    { 7.8999999999999996e-10, 2.9673540893200001e+00, 3.6109382204664587e+04 },
    { 7.8999999999999996e-10, 4.7636216092200003e+00, 4.1194487926082402e+04 },
    { 7.8999999999999996e-10, 2.9369730136999999e+00, 5.6218429232103999e+03 },
    { 7.8999999999999996e-10, 5.6633740371799997e+00, 2.4065807922775592e+04 },
    { 7.8999999999999996e-10, 2.4947675727199998e+00, 2.2476735027491792e+04 },
    { 7.7999999999999999e-10, 3.3828338352700000e+00, 2.3550344051682590e+04 },
    { 7.7999999999999999e-10, 8.3740696354999999e-01, 4.2456784047091598e+04 },
    { 7.7999999999999999e-10, 2.9901442681799999e+00, 1.2564667227274500e+04 },
    { 7.7000000000000003e-10, 5.0758643030400004e+00, 2.2345260376108199e+04 },
    { 7.7000000000000003e-10, 5.0065602920399996e+00, 5.2379210138037997e+03 },
    { 7.7000000000000003e-10, 2.9137018416999999e-01, 9.9248104215105996e+03 },
    { 7.7000000000000003e-10, 2.1416512449899998e+00, 4.6386993743311199e+04 },
    { 7.5999999999999996e-10, 4.1146256256000002e-01, 6.3658877750837601e+04 },
    { 7.5999999999999996e-10, 7.7801824813999998e-01, 2.3226838635025200e+04 },
    { 7.5000000000000000e-10, 3.6454724383200001e+00, 2.7832038219283200e+04 },
    { 7.5000000000000000e-10, 2.8504378867600000e+00, 1.8624882754272301e+04 },
    { 7.4000000000000003e-10, 4.4104463922899999e+00, 6.3044114904584003e+03 },
    { 7.4000000000000003e-10, 5.3796798339500000e+00, 6.2617402095243997e+03 },
    { 7.4000000000000003e-10, 5.6194558971099999e+00, 1.5265886519300400e+04 },
    { 7.4000000000000003e-10, 1.5661470906699999e+00, 2.7043502883182799e+04 },
    { 7.4000000000000003e-10, 1.1992890573399999e+00, 3.6843780499902000e+03 },
    { 7.2999999999999996e-10, 1.0504970696100000e+00, 6.6954469468657999e+03 },
    { 7.2999999999999996e-10, 2.4564438494499998e+00, 5.8707047531170001e+03 },
    { 7.2999999999999996e-10, 1.6176561689600000e+00, 1.6207886271502000e+04 },
    { 7.2999999999999996e-10, 2.9633479236700002e+00, 6.1957676454515904e+03 },
    { 7.2999999999999996e-10, 5.4359299538999994e-01, 6.3703840545311996e+03 },
    { 7.2999999999999996e-10, 4.6125160237699996e+00, 2.3440137730463201e+04 },
    { 7.2999999999999996e-10, 4.7396321743499996e+00, 7.3587651353180001e+02 },
    { 7.2999999999999996e-10, 4.4386805611000000e+00, 3.0348883772767000e+04 },
    { 7.2000000000000000e-10, 5.0359559367699998e+00, 6.3128972881401996e+03 },
    { 7.2000000000000000e-10, 4.7541702894700002e+00, 6.2532544118425903e+03 },
    { 7.2000000000000000e-10, 2.1109684346300002e+00, 1.8100003258450000e+04 },
    { 7.2000000000000000e-10, 4.8753460180300001e+00, 8.9828106693089994e+03 },
    { 7.2000000000000000e-10, 5.7365394822800004e+00, 4.5980237274898000e+03 },
    { 7.2000000000000000e-10, 2.6270273625299998e+00, 4.5837966334882003e+03 },
    { 7.1000000000000003e-10, 3.5716697893000000e+00, 1.7363247428908991e+04 },
    { 6.9999999999999996e-10, 4.4640955505000002e-01, 6.4420193677746001e+03 },
    { 6.9999999999999996e-10, 3.0605313640100000e+00, 6.1241323322081998e+03 },
    { 6.9999999999999996e-10, 4.7599032576000000e-01, 6.1754123260528004e+03 },
    { 6.9999999999999996e-10, 3.0309505933000001e+00, 6.3907393739299996e+03 },
    { 6.9999999999999996e-10, 6.2781822466000001e+00, 1.2299544658261000e+04 },
    { 6.9999999999999996e-10, 7.2171736979999998e-02, 1.6833556463814391e+04 },
    { 6.9000000000000000e-10, 5.6996801146999998e-01, 6.9443087767724001e+03 },
    { 6.9000000000000000e-10, 3.3788710386999998e-01, 2.3536116957680992e+04 },
    { 6.8000000000000003e-10, 2.0723241164299999e+00, 4.8971805736551996e+03 },
    { 6.8000000000000003e-10, 5.2464381854999997e-01, 1.8003144715222999e+04 },
    { 6.8000000000000003e-10, 1.3373319436200000e+00, 2.8759810877483189e+04 },
    { 6.8000000000000003e-10, 1.1449933872300000e+00, 3.1664164377316001e+03 },
    { 6.8000000000000003e-10, 4.6195079462299997e+00, 9.6677898863042195e+04 },
    { 6.6999999999999996e-10, 4.8828427992799996e+00, 7.0828969751568002e+03 },
    { 6.6999999999999996e-10, 3.2423294065900001e+00, 6.1718892077037999e+03 },
    { 6.6999999999999996e-10, 2.6461151246999998e-01, 6.3942624922790001e+03 },
    { 6.6999999999999996e-10, 4.8447990878600002e+00, 6.3008883721093998e+03 },
    { 6.6999999999999996e-10, 4.9453271383799997e+00, 6.2652633278734002e+03 },
    { 6.6000000000000000e-10, 3.0143018198600000e+00, 5.9595704333340000e+03 },
    { 6.6000000000000000e-10, 2.3819450953799999e+00, 6.2511035441745998e+03 },
    { 6.6000000000000000e-10, 1.1249958236800000e+00, 6.3150481558082001e+03 },
    { 6.6000000000000000e-10, 1.2105345241200001e+00, 1.7576546524346999e+04 },
    { 6.6000000000000000e-10, 8.0721495514999997e-01, 5.2201708014531996e+03 },
    { 6.6000000000000000e-10, 5.1300369361299998e+00, 6.0530488985741802e+04 },
    { 6.3999999999999996e-10, 3.5073875518600000e+00, 3.2350541665740002e+02 },
    { 6.3999999999999996e-10, 4.8033056206699998e+00, 2.8774037971484799e+04 },
    { 6.3000000000000000e-10, 2.7091626894499998e+00, 1.1823161639450200e+04 },
    { 6.3000000000000000e-10, 2.1622466842199999e+00, 5.2276260730653003e+03 },
    { 6.3000000000000000e-10, 5.2328771816100002e+00, 5.2197617665390999e+03 },
    { 6.2000000000000003e-10, 2.2504671782000001e+00, 3.8550336669571998e+03 },
    { 6.2000000000000003e-10, 2.8976624156800002e+00, 6.9087460423038001e+03 },
    { 6.2000000000000003e-10, 6.0927850338000000e-01, 5.6574056576789999e+03 },
    { 6.0999999999999996e-10, 3.2516380082900000e+00, 2.7707542494294801e+04 },
    { 6.0999999999999996e-10, 3.2874526397200001e+00, 4.4034127589539399e+04 },
    { 6.0999999999999996e-10, 3.5351068144000002e+00, 4.9166329245880002e+02 },
    { 6.0000000000000000e-10, 5.8300801726599998e+00, 1.8606498946000200e+04 },
    { 6.0000000000000000e-10, 1.6587112173900000e+00, 1.3088729118076601e+04 },
    { 6.0000000000000000e-10, 2.9013528974799998e+00, 7.5486433286177999e+03 },
    { 6.0000000000000000e-10, 6.3449806942999998e-01, 6.4701083331480004e+02 },
    { 6.0000000000000000e-10, 5.0846108845800000e+00, 1.2171525814923600e+04 },
    { 6.0000000000000000e-10, 4.8342399630999999e-01, 5.2440492392010001e+03 },
    { 6.0000000000000000e-10, 3.9206621409600002e+00, 2.8313288804660999e+04 },
    { 6.0000000000000000e-10, 3.5128391181000000e+00, 5.0317203439530800e+04 },
    { 5.9000000000000003e-10, 2.4333062640800001e+00, 5.1134875985828003e+03 },
    { 5.9000000000000003e-10, 5.1362804181000001e+00, 4.1990785932898798e+04 },
    { 5.9000000000000003e-10, 2.3919337919200001e+00, 6.2891417656211997e+03 },
    { 5.9000000000000003e-10, 1.1150071271399999e+00, 6.2770099343616002e+03 },
    { 5.9000000000000003e-10, 1.8142178210300000e+00, 6.7589087447057187e+04 },
    { 5.9000000000000003e-10, 1.0396830169100000e+00, 1.5252471184999999e+01 },
    { 5.9000000000000003e-10, 5.7630852432099999e+00, 1.2164479578225601e+04 },
    { 5.9000000000000003e-10, 2.7137600873099998e+00, 3.9814900340820000e+03 },
    { 5.9000000000000003e-10, 6.8886289824000002e-01, 6.2883355139513602e+04 },
    { 5.7999999999999996e-10, 1.5853930827100000e+00, 6.7167875878681998e+03 },
    { 5.7999999999999996e-10, 2.7222613217200000e+00, 3.0775481963643000e+04 },
    { 5.7999999999999996e-10, 7.4658643142000003e-01, 1.4169793562230399e+04 },
    { 5.7999999999999996e-10, 3.8369837555499999e+00, 6.0763751126948000e+03 },
    { 5.7999999999999996e-10, 5.9531424706899996e+00, 6.4897765872880000e+03 },
    { 5.7000000000000000e-10, 3.3091109134200001e+00, 7.9277488846740005e+02 },
    { 5.7000000000000000e-10, 1.6129051122400000e+00, 1.0027903195729201e+04 },
    { 5.7000000000000000e-10, 1.9872152500400000e+00, 6.7020002488919999e+03 },
    { 5.7000000000000000e-10, 1.5197256690200001e+00, 5.8641514510908000e+03 },
    { 5.7000000000000000e-10, 2.6270497142100000e+00, 7.9982112516539996e+02 },
    { 5.6000000000000003e-10, 5.2400284066199996e+00, 3.5050000274475387e+04 },
    { 5.6000000000000003e-10, 3.9917748570000000e-01, 6.3398791121612003e+03 },
    { 5.6000000000000003e-10, 3.1077634333600002e+00, 6.2262725878215997e+03 },
    { 5.6000000000000003e-10, 9.5649088929000003e-01, 1.2303067776610000e+04 },
    { 5.6000000000000003e-10, 2.4537758737900002e+00, 3.1781457905676002e+03 },
    { 5.6000000000000003e-10, 5.6744210970999998e+00, 8.8041956859279002e+04 },
    { 5.6000000000000003e-10, 5.3673454021799998e+00, 2.8663831650265402e+04 },
    { 5.6000000000000003e-10, 1.6041880978600001e+00, 8.2534403620611585e+04 },
    { 5.4999999999999996e-10, 5.9787620670699999e+00, 8.0181537466839785e+04 },
    { 5.4000000000000000e-10, 4.9620130586900002e+00, 3.6147409877300401e+04 },
    { 5.4000000000000000e-10, 1.1567692898699999e+00, 1.8208349942592002e+04 },
    { 5.4000000000000000e-10, 7.7163765121000005e-01, 4.1654963115967803e+04 },
    { 5.4000000000000000e-10, 4.5080379290800003e+00, 3.9302096962196003e+04 },
    { 5.4000000000000000e-10, 2.2804368310899998e+00, 6.5598216218558000e+03 },
    { 5.4000000000000000e-10, 1.2265040879700000e+00, 6.0063300781270000e+03 },
    { 5.4000000000000000e-10, 3.6237138967600000e+00, 2.4279107018213592e+04 },
    { 5.4000000000000000e-10, 3.8609427782000000e-01, 6.5601108437328003e+03 },
    { 5.4000000000000000e-10, 3.1208466412400000e+00, 6.0060408562499997e+03 },
    { 5.3000000000000003e-10, 5.7525885671600001e+00, 3.7724753419748202e+04 },
    { 5.3000000000000003e-10, 1.9012233667200000e+00, 1.1080171578917600e+04 },
    { 5.1999999999999996e-10, 9.6546586881999996e-01, 3.6275074775175999e+03 },
    { 5.1999999999999996e-10, 2.0415505350999999e-01, 6.1551243169567997e+03 },
    { 5.1999999999999996e-10, 3.3027858655500002e+00, 6.4110273830260003e+03 },
    { 5.1999999999999996e-10, 5.8029130096900001e+00, 3.0639856638632999e+04 },
    { 5.1999999999999996e-10, 5.1456563217099998e+00, 6.5740487158573997e+03 },
    { 5.1999999999999996e-10, 4.6444699045300002e+00, 5.9921029841254003e+03 },
    { 5.1999999999999996e-10, 1.1500026011800000e+00, 4.0398189919265998e+04 },
    { 5.1999999999999996e-10, 2.8408553604700000e+00, 1.2571674624290201e+04 },
    { 5.1999999999999996e-10, 1.9623228161299999e+00, 1.1904918773201800e+04 },
    { 5.1000000000000000e-10, 1.4346165985499999e+00, 7.6689711263276004e+03 },
    { 5.1000000000000000e-10, 1.1602840542499999e+00, 3.8650173506198997e+04 },
    { 5.1000000000000000e-10, 6.0090491423300003e+00, 3.4284354760811998e+03 },
    { 5.1000000000000000e-10, 4.9188765106999996e+00, 1.8326650131880389e+04 },
    { 5.1000000000000000e-10, 2.9562388010800000e+00, 7.7619424240558001e+03 },
    { 5.0000000000000003e-10, 3.1101115495800000e+00, 2.5752390943291998e+03 },
    { 5.0000000000000003e-10, 3.1237616724200001e+00, 5.8814037282341997e+03 },
    { 5.0000000000000003e-10, 3.8317924664000003e-01, 6.6847479717486003e+03 },
    { 5.0000000000000003e-10, 1.6459322219100001e+00, 4.3738802345965800e+04 },
    { 5.0000000000000003e-10, 3.6062687400900000e+00, 1.7583660071347800e+04 },
    { 5.0000000000000003e-10, 3.5452929555099999e+00, 2.2030212954669601e+04 },
    { 5.0000000000000003e-10, 3.1387397956399998e+00, 9.2747689166822587e+04 },
    { 5.0000000000000003e-10, 3.5643037555000001e-01, 2.1947111372700001e+04 },
    { 4.8999999999999996e-10, 3.5948688092699999e+00, 5.4903009615239998e+03 },
    { 4.8999999999999996e-10, 2.1182495931899998e+00, 6.6813564835733196e+04 },
    { 4.8000000000000000e-10, 5.4467433792500000e+00, 2.5685872802808000e+04 },
    { 4.8000000000000000e-10, 3.2357601175599999e+00, 3.6463503773543998e+03 },
    { 4.8000000000000000e-10, 4.6416451477300003e+00, 3.5579691239569998e+04 },
    { 4.8000000000000000e-10, 2.3947799306099999e+00, 5.2094668258006004e+03 },
    { 4.8000000000000000e-10, 9.3182108525999996e-01, 1.2662130927200600e+04 },
    { 4.7000000000000003e-10, 4.4337850401800001e+00, 1.7067287061920001e+02 },
    { 4.7000000000000003e-10, 2.1702318178400000e+00, 4.5585172812187397e+04 },
    { 4.7000000000000003e-10, 6.0333863099000000e+00, 4.7082300487091998e+03 },
    { 4.7000000000000003e-10, 5.8339571111200001e+00, 2.4499519660652400e+04 },
    { 4.7000000000000003e-10, 2.5721628494900002e+00, 1.1919140866668000e+04 },
    { 4.7000000000000003e-10, 2.9374806563400000e+00, 2.4485292566650791e+04 },
    { 4.6000000000000001e-10, 8.3996053249000002e-01, 2.5287723799399799e+04 },
    { 4.6000000000000001e-10, 3.1965124527200000e+00, 4.4807038592695999e+03 },
    { 4.6000000000000001e-10, 3.8496547918399999e+00, 4.3847246320517997e+03 },
    { 4.6000000000000001e-10, 3.1533940618399998e+00, 1.6703062133498999e+04 },
    { 4.6000000000000001e-10, 2.4767246525800002e+00, 1.2555105999718900e+04 },
    { 4.6000000000000001e-10, 3.4558153892200001e+00, 1.2146667056107601e+04 },
    { 4.5000000000000000e-10, 5.8136395376400003e+00, 3.3318816062458201e+04 },
    { 4.5000000000000000e-10, 2.9195078507000000e-01, 4.4136951349374001e+04 },
    { 4.5000000000000000e-10, 4.6511300296300000e+00, 4.6360695423511403e+04 },
    { 4.5000000000000000e-10, 3.2203119391400001e+00, 7.1519297143276795e+04 },
    { 4.5000000000000000e-10, 8.3262497187999995e-01, 1.4286139123625818e+05 },
    { 4.3999999999999998e-10, 2.7127465031100000e+00, 2.3015858159093900e+03 },
    { 4.3999999999999998e-10, 3.8083611340100001e+00, 6.9941953600828987e+04 },
    { 4.3999999999999998e-10, 2.6193999117700000e+00, 5.3321408344359997e+02 },
    { 4.3999999999999998e-10, 3.6300786494400001e+00, 6.1408972229552000e+03 },
    { 4.3999999999999998e-10, 6.1600475768000003e+00, 6.4252544770275999e+03 },
    { 4.3999999999999998e-10, 3.5775767661600000e+00, 4.9515382508406998e+04 },
    { 4.3000000000000001e-10, 1.2077547124200001e+00, 2.0894775679208789e+04 },
    { 4.3000000000000001e-10, 1.5452818399599999e+00, 8.4018397103698007e+03 },
    { 4.3000000000000001e-10, 2.7546757621400002e+00, 2.7278468816440800e+04 },
    { 4.3000000000000001e-10, 3.1271410996100002e+00, 7.3221024607817999e+03 },
    { 4.3000000000000001e-10, 4.4544101960900004e+00, 1.9004647949408401e+04 },
    { 4.3000000000000001e-10, 6.0785439689800000e+00, 7.0260659105240002e+03 },
    { 4.3000000000000001e-10, 4.2435947312200000e+00, 6.3593419212669996e+03 },
    { 4.2000000000000000e-10, 5.1305417441300003e+00, 2.7433889215874991e+04 },
    { 4.2000000000000000e-10, 1.3209381808300000e+00, 2.1548962369291799e+04 },
    { 4.2000000000000000e-10, 2.6835347857800000e+00, 2.8230187222691391e+04 },
    { 4.2000000000000000e-10, 3.6639213671599999e+00, 2.6084021806216198e+04 },
    { 4.2000000000000000e-10, 3.1300341264200000e+00, 6.2317955121289897e+03 },
    { 4.2000000000000000e-10, 3.7690679263999999e-01, 6.3343561878538103e+03 },
    { 4.2000000000000000e-10, 2.2195141922700001e+00, 6.3282170696280000e+03 },
    { 4.2000000000000000e-10, 1.2874267267899999e+00, 6.2379346303548000e+03 },
    { 4.0999999999999998e-10, 5.1439484757100002e+00, 3.7352383117590002e+03 },
    { 4.0999999999999998e-10, 4.9521565654200002e+00, 3.7057577492566197e+04 },
    { 4.0999999999999998e-10, 1.9868925525900001e+00, 6.6475604512999996e+02 },
    { 4.0000000000000001e-10, 1.0784239315299999e+00, 2.8628336226099600e+04 },
    { 4.0000000000000001e-10, 6.2218727638600004e+00, 7.0758507384588002e+03 },
    { 4.0000000000000001e-10, 4.7839526759700002e+00, 7.1220451377418003e+03 },
    { 4.0000000000000001e-10, 5.0061735502699998e+00, 5.4441065622409997e+03 },
    { 4.0000000000000001e-10, 5.2144707245799999e+00, 3.2151363806432000e+03 },
    { 3.9000000000000000e-10, 7.1682305361999998e-01, 3.4570066334852600e+04 },
    { 3.9000000000000000e-10, 5.0944327291200002e+00, 2.2594054895711990e+04 },
    { 3.9000000000000000e-10, 2.9713633200500000e+00, 2.1150813365883601e+04 },
    { 3.9000000000000000e-10, 3.8512255808900000e+00, 3.0774501642574800e+04 },
    { 3.7999999999999998e-10, 6.7444569484000005e-01, 4.4933249356190398e+04 },
    { 3.7000000000000001e-10, 4.8035571169100004e+00, 6.8507944877217997e+03 },
    { 3.7000000000000001e-10, 4.9865691093300004e+00, 5.7153572122610003e+03 },
};

static const VSOP87Term VSOP87A_EMB_Y1[482] = {
    { 9.3046317000000005e-04, 0.0000000000000000e+00, 0.0000000000000000e+00 },
    { 5.1506608999999997e-04, 4.4318049928600001e+00, 1.2566151699982800e+04 },
    { 1.2908000000000001e-05, 4.3886054854000003e+00, 1.8849227549974199e+04 },
    { 4.6454999999999998e-06, 5.8272991295200001e+00, 6.2830758499914000e+03 },
    { 2.1268900000000001e-06, 1.6300556918000000e-01, 6.2795527316424004e+03 },
    { 2.1253300000000002e-06, 3.3440059540700000e+00, 6.2865989683403996e+03 },
    { 6.2345000000000004e-07, 5.0737735482700002e+00, 4.7057323075435997e+03 },
    { 5.9793999999999997e-07, 2.2410090727199998e+00, 6.2567775301915999e+03 },
    { 5.9441000000000004e-07, 1.2661999062600000e+00, 6.3093741697912001e+03 },
    { 4.8986999999999998e-07, 3.6416657783500002e+00, 7.7552261132399997e+02 },
    { 4.6642000000000002e-07, 4.7157521579999999e+00, 7.8604193924392002e+03 },
    { 4.2745999999999999e-07, 5.1517742582399997e+00, 1.0593819301891999e+03 },
    { 3.7273999999999999e-07, 6.6041323803999996e-01, 5.7533848848968000e+03 },
    { 3.5624999999999998e-07, 6.1546284696600004e+00, 5.8849268465832001e+03 },
    { 3.5372999999999998e-07, 2.9005201215200000e+00, 6.8127668150860000e+03 },
    { 3.2156999999999999e-07, 3.6266970201500000e+00, 6.6812248533995999e+03 },
    { 2.8762999999999999e-07, 4.3453935562100003e+00, 2.5132303399965600e+04 },
    { 2.8447000000000001e-07, 5.8621510321399999e+00, 6.1276554505572003e+03 },
    { 2.7501999999999998e-07, 3.9264163254200000e+00, 6.4384962494255997e+03 },
    { 2.4830000000000002e-07, 1.3519011595299999e+00, 5.4867778431750003e+03 },
    { 2.0618000000000001e-07, 2.1477239664700001e+00, 7.0793738568077997e+03 },
    { 1.9530999999999999e-07, 1.2995831059400000e+00, 5.5075532386674004e+03 },
    { 1.8447000000000001e-07, 6.1733053265100004e+00, 1.1790629088658799e+04 },
    { 1.6754000000000000e-07, 2.2486222028200000e+00, 7.0585984613153996e+03 },
    { 1.6520000000000000e-07, 4.6332803948899999e+00, 1.1506769769793600e+04 },
    { 1.4560000000000001e-07, 4.4042576136099996e+00, 6.2901893969922003e+03 },
    { 1.4426000000000000e-07, 5.3922633085699996e+00, 6.2759623029905997e+03 },
    { 1.4186000000000000e-07, 6.1854721941899999e+00, 7.9629800681639995e+02 },
    { 1.2966000000000001e-07, 5.1033893582400003e+00, 7.1135470007999997e+00 },
    { 1.2666000000000000e-07, 5.7323139375199998e+00, 4.6940029547076001e+03 },
    { 1.0425000000000000e-07, 2.6953117746799999e+00, 6.2820955289231997e+03 },
    { 1.0425000000000000e-07, 8.1162914438000000e-01, 6.2840561710596003e+03 },
    { 1.0100000000000000e-07, 1.7246451359299999e+00, 3.7387614301079998e+03 },
    { 9.5760000000000005e-08, 3.2459396887999997e-01, 4.1369104335162001e+03 },
    { 9.4269999999999996e-08, 3.0811574304399998e+00, 6.0697767545533998e+03 },
    { 9.0429999999999996e-08, 4.3888880403000002e-01, 6.4963749454294002e+03 },
    { 8.5450000000000001e-08, 4.7713749764099997e+00, 1.1944470102246000e+03 },
    { 8.1989999999999995e-08, 5.9683831635800004e+00, 1.5890728952838001e+03 },
    { 7.3160000000000001e-08, 5.2040435540500001e+00, 4.5350594369244000e+03 },
    { 6.4609999999999994e-08, 2.6784058610899999e+00, 5.0886288397668004e+03 },
    { 6.2909999999999999e-08, 3.1831201917300000e+00, 8.4292412664666008e+03 },
    { 6.1329999999999994e-08, 1.7803329035500000e+00, 8.8273902698748007e+03 },
    { 5.6069999999999998e-08, 9.8720797401000004e-01, 5.7295064471490005e+03 },
    { 5.5239999999999997e-08, 3.7984213521700001e+00, 4.9332084403325998e+03 },
    { 5.2700000000000002e-08, 9.4095218348999998e-01, 3.1546870848956000e+03 },
    { 5.1459999999999999e-08, 3.5714687051100000e+00, 1.1769853693166400e+04 },
    { 5.0769999999999997e-08, 7.3549620961999995e-01, 7.4775228602159996e+03 },
    { 5.0530000000000003e-08, 4.5516294208100003e+00, 8.0310922630584000e+03 },
    { 4.9850000000000000e-08, 3.0569798423100001e+00, 1.2569674818331800e+04 },
    { 4.9740000000000001e-08, 2.5002914740100000e+00, 6.8366452528338004e+03 },
    { 4.8289999999999998e-08, 5.3287653849999996e+00, 6.0403472460173998e+03 },
    { 4.7090000000000000e-08, 2.3929209945099998e+00, 5.3313574437407997e+03 },
    { 4.6010000000000000e-08, 4.4529653271000003e+00, 6.5258044539654002e+03 },
    { 4.3940000000000002e-08, 4.7594818484000001e+00, 1.2559038152982001e+04 },
    { 4.2900000000000000e-08, 3.3673673929799999e+00, 1.5925960136327999e+03 },
    { 4.0809999999999999e-08, 5.9419174359699998e+00, 7.6329432596502002e+03 },
    { 3.7790000000000003e-08, 1.0738517125300000e+00, 7.2347942562420003e+03 },
    { 3.6139999999999998e-08, 2.5245529914799998e+00, 3.9302096962196001e+03 },
    { 3.5759999999999997e-08, 5.3021979948100002e+00, 4.1643119896130002e+03 },
    { 3.4819999999999997e-08, 3.8265219880100001e+00, 1.0977078804699000e+04 },
    { 3.4060000000000002e-08, 8.3525752780999996e-01, 3.1283887650958000e+03 },
    { 3.3759999999999999e-08, 5.0021314525099996e+00, 1.1371704689758200e+04 },
    { 3.3339999999999999e-08, 3.4882399311799999e+00, 8.0182093112380005e+02 },
    { 3.3279999999999998e-08, 5.8381254499499997e+00, 1.5542039943419999e+02 },
    { 3.1240000000000000e-08, 3.1714398298400002e+00, 7.0848967811151997e+03 },
    { 3.0080000000000002e-08, 6.7203387303999995e-01, 8.6359420037632008e+03 },
    { 2.8200000000000001e-08, 2.3146914320000000e-01, 5.2236939198022001e+03 },
    { 2.7780000000000001e-08, 2.6817898615000002e-01, 1.7480164130670000e+03 },
    { 2.7780000000000001e-08, 6.1739240593400000e+00, 5.4812549188676003e+03 },
    { 2.7520000000000001e-08, 3.6738962148900001e+00, 7.3424577801805999e+03 },
    { 2.6939999999999998e-08, 4.7098913814800003e+00, 5.2969096509459996e+02 },
    { 2.6700000000000001e-08, 1.0015174787900001e+00, 2.6298319799800002e+01 },
    { 2.4299999999999999e-08, 1.6409026341899999e+00, 1.3498674096587999e+03 },
    { 2.3479999999999999e-08, 5.1966566416499997e+00, 2.1461654164751999e+03 },
    { 2.3070000000000001e-08, 2.9893014234600002e+00, 1.7260154654690399e+04 },
    { 2.1620000000000001e-08, 3.3268106562999999e-01, 5.3680451209540001e+02 },
    { 2.0459999999999999e-08, 1.9660597147200001e+00, 1.9907450170410000e+03 },
    { 2.0330000000000001e-08, 8.0949219977999998e-01, 5.2257741809380002e+02 },
    { 2.0000000000000000e-08, 3.0312426621899999e+00, 9.5171840625059997e+02 },
    { 1.9399999999999998e-08, 3.9108527362899999e+00, 5.2165803728013998e+03 },
    { 1.9020000000000001e-08, 2.6656722257299998e+00, 9.4377629348869996e+03 },
    { 1.8299999999999998e-08, 4.4380642354399997e+00, 5.5356940284240000e+02 },
    { 1.7900000000000001e-08, 3.4865556248599998e+00, 5.2308074668030004e+03 },
    { 1.7789999999999999e-08, 2.3737949224999999e-01, 1.2562628581633800e+04 },
    { 1.7680000000000000e-08, 8.8265005770000002e-02, 1.0973555686350001e+04 },
    { 1.7620000000000002e-08, 3.8493118561699999e+00, 4.6904798363585996e+03 },
    { 1.7620000000000002e-08, 2.3954283584499998e+00, 3.9814900340819997e+02 },
    { 1.7080000000000001e-08, 5.0041787838199996e+00, 1.0988808157535001e+04 },
    { 1.5550000000000000e-08, 2.0992441768500001e+00, 1.3095842665077400e+04 },
    { 1.5239999999999999e-08, 5.8137022728399996e+00, 4.2659819087599999e+02 },
    { 1.4500000000000001e-08, 5.7120136826000003e+00, 6.2627205305926000e+03 },
    { 1.4500000000000001e-08, 4.0781125436399996e+00, 6.3034311693902000e+03 },
    { 1.4279999999999999e-08, 3.2944277748299999e+00, 1.2721572099417001e+04 },
    { 1.3730000000000000e-08, 5.6052318537300003e+00, 7.8756718636242003e+03 },
    { 1.2470000000000000e-08, 2.9212365805000001e-01, 2.1187638603783998e+03 },
    { 1.2299999999999999e-08, 2.5313185080300000e+00, 1.8319536584879599e+04 },
    { 1.1409999999999999e-08, 1.9214745267100000e+00, 1.0664954771900000e+03 },
    { 1.1400000000000000e-08, 1.2141080668299999e+00, 2.9424634232916001e+03 },
    { 1.0880000000000000e-08, 5.7765685668600000e+00, 5.7604984318976003e+03 },
    { 1.0640000000000000e-08, 4.4056664348799996e+00, 1.5720838784878400e+04 },
    { 1.0449999999999999e-08, 2.1753572127999998e+00, 2.4272860397400001e+02 },
    { 1.0179999999999999e-08, 2.6629976606699999e+00, 1.2539853380183000e+04 },
    { 1.0169999999999999e-08, 4.0209301718299999e+00, 6.8056532680851997e+03 },
    { 9.7300000000000001e-09, 4.9543764998400004e+00, 6.3989728631399998e+02 },
    { 9.6899999999999994e-09, 1.2940504573400000e+00, 5.6431785636774002e+03 },
    { 9.6199999999999995e-09, 5.5758587779000002e-01, 2.3888940204492001e+03 },
    { 9.4300000000000007e-09, 4.2206243702200004e+00, 2.0426571092422000e+04 },
    { 9.0699999999999995e-09, 7.0119368825999995e-01, 1.3367972631106601e+04 },
    { 8.8900000000000005e-09, 4.7782013773400003e+00, 6.2885987742988000e+03 },
    { 8.8900000000000005e-09, 5.0119248488899997e+00, 6.2775529256840000e+03 },
    { 8.7399999999999992e-09, 1.4303695888800001e+00, 1.0575406682941801e+04 },
    { 8.6900000000000004e-09, 5.8086061524300003e+00, 1.6496361396202399e+04 },
    { 8.0000000000000005e-09, 3.1836621859099998e+00, 1.2029347187887401e+04 },
    { 7.9500000000000001e-09, 3.3995182842400000e+00, 6.2797894925736000e+03 },
    { 7.9500000000000001e-09, 1.0742263482000000e-01, 6.2863622074092000e+03 },
    { 7.9400000000000003e-09, 3.7932907993999998e+00, 1.4314168113049800e+04 },
    { 7.5900000000000005e-09, 4.9830174909400000e+00, 1.5773435424478000e+03 },
    { 7.5699999999999993e-09, 3.8175407327199999e+00, 5.8564776591153995e+03 },
    { 7.5400000000000000e-09, 5.2465544527099999e+00, 1.3916019109641600e+04 },
    { 7.4400000000000000e-09, 2.6874116078000001e+00, 1.2043574281888999e+04 },
    { 7.1600000000000001e-09, 2.2868726951600000e+00, 9.6236882766912004e+03 },
    { 6.9800000000000003e-09, 4.9436322996900000e+00, 6.1480107699560003e+03 },
    { 6.9600000000000000e-09, 2.5387712923799999e+00, 1.6730463689595799e+04 },
    { 6.7800000000000002e-09, 4.8464721630399996e+00, 6.4181409300267997e+03 },
    { 6.7299999999999997e-09, 2.4532292806200000e+00, 1.4712317116458000e+04 },
    { 6.7200000000000000e-09, 3.9915394140300000e+00, 1.2964300703391000e+04 },
    { 6.7100000000000002e-09, 6.0017940265399998e+00, 6.7096740408674004e+03 },
    { 6.7100000000000002e-09, 4.0543909772999998e-01, 1.3517870106233400e+04 },
    { 6.6700000000000003e-09, 4.8715137291000001e+00, 3.6346210245184002e+03 },
    { 6.6199999999999999e-09, 4.0403110364000003e+00, 1.1015106477334801e+04 },
    { 6.5700000000000003e-09, 1.8016359634700001e+00, 1.3119721102825190e+04 },
    { 6.2900000000000004e-09, 3.4249372559400002e+00, 4.6868894077067998e+03 },
    { 6.2000000000000001e-09, 4.3031053501600001e+00, 3.1415379249957001e+04 },
    { 6.1000000000000000e-09, 5.1995584491800004e+00, 6.8198803620868002e+03 },
    { 6.0900000000000003e-09, 4.5754404399900004e+00, 5.7462713378959997e+03 },
    { 6.0399999999999998e-09, 5.3946212537599996e+00, 1.4143495242430599e+04 },
    { 5.9600000000000001e-09, 1.0803804369500001e+00, 1.0447387839604400e+04 },
    { 5.7500000000000002e-09, 1.3941025229099999e+00, 1.2592450019782600e+04 },
    { 5.6200000000000000e-09, 4.7614566601800004e+00, 4.2923308329503998e+03 },
    { 5.5299999999999997e-09, 7.8994516254000002e-01, 6.3069542877391996e+03 },
    { 5.5299999999999997e-09, 2.7169957565199998e+00, 6.2591974122436004e+03 },
    { 5.5299999999999997e-09, 4.8156839521699997e+00, 1.2036460734888200e+04 },
    { 5.2400000000000001e-09, 3.9654394589900002e+00, 1.2139553509106800e+04 },
    { 5.1400000000000000e-09, 3.8779314842899999e+00, 4.7011165017084004e+03 },
    { 4.9900000000000003e-09, 5.7998586152299998e+00, 1.2410731300548599e+04 },
    { 4.9499999999999997e-09, 2.0098739563699999e+00, 1.3362449706799200e+04 },
    { 4.8099999999999997e-09, 2.5711806383500000e+00, 6.2869571853493999e+03 },
    { 4.8099999999999997e-09, 9.3576028070999995e-01, 6.2791945146334001e+03 },
    { 4.5699999999999997e-09, 2.6212080806700002e+00, 6.0768903015542001e+03 },
    { 4.5699999999999997e-09, 8.8573283839000005e-01, 6.4892613984285999e+03 },
    { 4.5299999999999999e-09, 5.4351375847199996e+00, 2.7870430238573999e+03 },
    { 4.5200000000000001e-09, 3.9833023830799998e+00, 1.2573265246983599e+04 },
    { 4.4500000000000001e-09, 1.6895928991100000e+00, 1.1499656222792801e+04 },
    { 4.3999999999999997e-09, 5.8121786394999997e-01, 2.4072921469776400e+04 },
    { 4.1899999999999998e-09, 4.0514011686700000e+00, 5.3274761083827998e+03 },
    { 4.1800000000000000e-09, 2.7606936800400002e+00, 1.0177257679533601e+04 },
    { 4.0400000000000001e-09, 5.6451183178999997e-01, 5.8635912061161998e+03 },
    { 3.9799999999999999e-09, 4.2003518911900004e+00, 8.2738208670324002e+03 },
    { 3.9799999999999999e-09, 4.8355850840499999e+00, 1.5510452226479999e+03 },
    { 3.8300000000000002e-09, 5.1165943437700001e+00, 6.3191037166688002e+03 },
    { 3.8300000000000002e-09, 4.6735318824699998e+00, 6.2470479833139898e+03 },
    { 3.7900000000000004e-09, 3.0538487030199999e+00, 6.3861686242100004e+03 },
    { 3.7900000000000004e-09, 4.5309221604000000e-01, 6.1799830757727996e+03 },
    { 3.7799999999999998e-09, 5.8086110190199998e+00, 7.2386755916000002e+03 },
    { 3.7700000000000000e-09, 4.3326929093100004e+00, 7.8721487452751999e+03 },
    { 3.6899999999999999e-09, 4.7823159786999998e+00, 4.7320306273433998e+03 },
    { 3.6399999999999998e-09, 6.7991079022000001e-01, 6.3211035226271997e+03 },
    { 3.6399999999999998e-09, 2.8270301288400002e+00, 6.2450481773556003e+03 },
    { 3.6199999999999999e-09, 1.1689588744199999e+00, 1.0021837280099400e+04 },
    { 3.5800000000000000e-09, 2.5271397215500002e+00, 1.0522683831884001e+03 },
    { 3.4400000000000001e-09, 4.9990734942500001e+00, 1.8451078546565990e+04 },
    { 3.4299999999999999e-09, 1.3647089727099999e+00, 6.2678233788063899e+03 },
    { 3.4299999999999999e-09, 2.1422319463499999e+00, 6.2983283211764001e+03 },
    { 3.4200000000000002e-09, 4.5294000780700001e+00, 6.3038512454838001e+03 },
    { 3.4200000000000002e-09, 5.2607261481699998e+00, 6.2623004544989999e+03 },
    { 3.3400000000000000e-09, 5.6215917989999997e-01, 1.8073704938650200e+04 },
    { 3.3400000000000000e-09, 1.1641672984200000e+00, 1.7782732072784202e+04 },
    { 3.3200000000000001e-09, 2.1379155177100002e+00, 2.1329909543799999e+02 },
    { 3.2099999999999999e-09, 6.6776723306999997e-01, 1.7796959166785800e+04 },
    { 3.1899999999999999e-09, 4.8317315324900001e+00, 1.1926254413668799e+04 },
    { 3.1800000000000002e-09, 6.0358923958500004e+00, 6.3115250374591997e+03 },
    { 3.1800000000000002e-09, 3.7542338303900000e+00, 6.2546266625236003e+03 },
    { 3.1500000000000001e-09, 1.0995091895700000e+00, 1.2323423096008801e+04 },
    { 3.1099999999999998e-09, 5.9407447245300000e+00, 2.3528661537717999e+03 },
    { 3.1000000000000000e-09, 4.0247718201699998e+00, 1.0213285546211000e+04 },
    { 3.0500000000000000e-09, 5.7265325630999997e-01, 5.6421982426092000e+03 },
    { 3.0399999999999998e-09, 4.3890915611999998e-01, 1.2779450795420800e+04 },
    { 2.9699999999999999e-09, 2.0797453646699999e+00, 5.4298794682394000e+03 },
    { 2.9499999999999999e-09, 2.4877058016900002e+00, 6.7025604938666002e+03 },
    { 2.8800000000000000e-09, 3.7305553273999998e+00, 2.2041264243879999e+02 },
    { 2.8699999999999998e-09, 3.1572653335000000e-01, 9.9176968745098002e+03 },
    { 2.8200000000000002e-09, 2.9318118487399998e+00, 1.3341674311306801e+04 },
    { 2.7600000000000000e-09, 5.9107321564000004e-01, 1.2168002696574600e+04 },
    { 2.6299999999999998e-09, 2.6090984732100000e+00, 9.5559974160859997e+02 },
    { 2.6200000000000001e-09, 5.1540912275900004e+00, 1.4945316173554400e+04 },
    { 2.6099999999999999e-09, 1.9378221626000001e+00, 1.6200772724501199e+04 },
    { 2.5899999999999999e-09, 5.5311232459899999e+00, 3.8941818295421999e+03 },
    { 2.5800000000000002e-09, 4.3810357931299997e+00, 2.0618554843720000e+02 },
    { 2.5000000000000001e-09, 3.0451651160400002e+00, 6.9239534573736000e+03 },
    { 2.4800000000000001e-09, 5.6205090089800001e+00, 1.7789845619784999e+04 },
    { 2.3400000000000002e-09, 2.5030752933099998e+00, 1.5961864422845999e+03 },
    { 2.2999999999999999e-09, 6.6694562430000004e-01, 1.2567132021051000e+04 },
    { 2.1700000000000002e-09, 2.7459781302600001e+00, 2.5443144198834002e+03 },
    { 2.1600000000000000e-09, 4.0299160435100001e+00, 3.1851920272655998e+03 },
    { 2.1299999999999999e-09, 4.0995955485200000e+00, 9.7791086761253991e+03 },
    { 2.0299999999999998e-09, 2.8241719579000000e+00, 6.0626632075526004e+03 },
    { 2.0299999999999998e-09, 6.8276896115999997e-01, 6.5034884924301996e+03 },
    { 1.9300000000000002e-09, 7.8779752820000004e-01, 6.4087760738220004e+02 },
    { 1.8800000000000001e-09, 4.8329872543399999e+00, 1.8209330263660191e+04 },
    { 1.8700000000000000e-09, 3.0208805606300002e+00, 6.2760296132933900e+03 },
    { 1.8700000000000000e-09, 4.8606035843000001e-01, 6.2901220866894000e+03 },
    { 1.8400000000000001e-09, 2.6387551679999999e+00, 1.2566219010285600e+04 },
    { 1.8400000000000001e-09, 3.0828581378400002e+00, 1.2566084389680000e+04 },
    { 1.8400000000000001e-09, 1.8097020271099999e+00, 6.0372442037620003e+03 },
    { 1.8199999999999999e-09, 1.9927362554000000e-01, 5.8493641121146002e+03 },
    { 1.8000000000000000e-09, 4.1973848321300000e+00, 8.5319638175199998e+02 },
    { 1.7900000000000000e-09, 1.2762245539099999e+00, 3.5231183490000002e+00 },
    { 1.7500000000000000e-09, 1.6969195646899999e+00, 6.5289074962207997e+03 },
    { 1.6800000000000000e-09, 6.0215873567399996e+00, 1.1020632121940000e+02 },
    { 1.6600000000000001e-09, 5.9072517295000004e+00, 4.3371173787679999e+02 },
    { 1.6300000000000000e-09, 2.9616756706500000e+00, 1.2352852604544800e+04 },
    { 1.6200000000000000e-09, 4.0509792795999999e-01, 1.8875525869773999e+04 },
    { 1.6100000000000000e-09, 6.0663109917900000e+00, 8.6622403235630009e+03 },
    { 1.5900000000000001e-09, 3.1775301913999998e-01, 3.4960328261340001e+03 },
    { 1.5600000000000000e-09, 7.7329171048000001e-01, 2.6484548254729998e+03 },
    { 1.5400000000000001e-09, 5.6836656574699997e+00, 6.1124029793722002e+03 },
    { 1.5400000000000001e-09, 4.1064605687700002e+00, 6.4537487206105998e+03 },
    { 1.5000000000000000e-09, 4.3792028492100004e+00, 3.3406124266997999e+03 },
    { 1.4700000000000001e-09, 4.6061263170100002e+00, 1.7654780539749601e+04 },
    { 1.4599999999999999e-09, 2.4871461717900001e+00, 1.4956319713459999e+02 },
    { 1.4599999999999999e-09, 4.2084370396999998e-01, 1.3760598710207400e+04 },
    { 1.4400000000000000e-09, 2.1909838592400002e+00, 1.8635928454536199e+04 },
    { 1.4300000000000000e-09, 6.2720299820000006e-02, 1.9651048481098002e+04 },
    { 1.4200000000000001e-09, 2.9167743972100002e+00, 4.1571984426121999e+03 },
    { 1.4100000000000001e-09, 5.6357474907000000e+00, 1.0969965257698201e+04 },
    { 1.3500000000000001e-09, 4.5921339009900004e+00, 1.0984192351699799e+04 },
    { 1.2500000000000000e-09, 7.8764990757999997e-01, 2.0775395492400001e+01 },
    { 1.2300000000000001e-09, 4.9749802433900001e+00, 6.1728695287720002e+03 },
    { 1.2300000000000001e-09, 4.8151459828499998e+00, 6.3932821712107998e+03 },
    { 1.2300000000000001e-09, 2.8575976082299999e+00, 1.2565171378914600e+04 },
    { 1.2100000000000000e-09, 4.7657578287300000e+00, 1.3625533630172000e+04 },
    { 1.1900000000000000e-09, 7.1211331874999995e-01, 1.0419986283507600e+04 },
    { 1.1800000000000001e-09, 5.6765082041600001e+00, 1.7256631536341400e+04 },
    { 1.1700000000000001e-09, 2.9063208037099999e+00, 1.8852750668323199e+04 },
    { 1.1400000000000000e-09, 3.7888534418200002e+00, 4.1714255366138004e+03 },
    { 1.1300000000000000e-09, 4.2998848042700004e+00, 2.3013539539587200e+04 },
    { 1.1300000000000000e-09, 9.7983808350000001e-01, 1.6858482532933202e+04 },
    { 1.1200000000000001e-09, 6.1034141916999995e-01, 1.7253041107689591e+04 },
    { 1.1100000000000001e-09, 2.0912832059899999e+00, 1.5819593482830001e+03 },
    { 1.1100000000000001e-09, 8.5102422911999998e-01, 1.1513883316794399e+04 },
    { 1.0700000000000000e-09, 5.4694261251500000e+00, 9.3809596727171993e+03 },
    { 1.0600000000000001e-09, 8.7687085687999999e-01, 2.3543230504681789e+04 },
    { 1.0600000000000001e-09, 3.5719993706099999e+00, 1.1216284290324000e+04 },
    { 1.0500000000000001e-09, 5.1325422305300004e+00, 6.3278373931320004e+02 },
    { 1.0399999999999999e-09, 7.0746530545999997e-01, 5.6502921106781996e+03 },
    { 1.0300000000000000e-09, 4.9594133026999998e+00, 6.2713464971554004e+03 },
    { 1.0300000000000000e-09, 4.8307129235400001e+00, 6.2948052028273996e+03 },
    { 1.0200000000000000e-09, 5.9936023099700000e+00, 4.8042092759269999e+03 },
    { 1.0200000000000000e-09, 4.0996239464800004e+00, 6.5496828917131998e+03 },
    { 1.0200000000000000e-09, 5.6905022797599996e+00, 6.0164688082696002e+03 },
    { 1.0200000000000000e-09, 4.4338431589100002e+00, 3.1049300594237998e+03 },
    { 9.5999999999999999e-10, 1.4720567702800000e+00, 3.0978838227257902e+03 },
    { 9.5000000000000003e-10, 6.0559745772999998e+00, 1.2345739057544000e+04 },
    { 9.5000000000000003e-10, 9.2966303480000001e-01, 1.2012582297140400e+04 },
    { 9.2999999999999999e-10, 2.5314064011099999e+00, 1.6460333529524989e+04 },
    { 9.2000000000000003e-10, 1.8415803250900000e+00, 2.2805735565993600e+04 },
    { 9.0999999999999996e-10, 5.5931668291600003e+00, 1.0818135286915800e+04 },
    { 9.0999999999999996e-10, 1.4361272765999999e+00, 1.5671081759406599e+04 },
    { 8.9999999999999999e-10, 2.1177404735900001e+00, 7.6266071275599998e+01 },
    { 8.9999999999999999e-10, 4.4521476505499997e+00, 6.1957676454515904e+03 },
    { 8.9999999999999999e-10, 5.3379785756900002e+00, 6.3703840545311996e+03 },
    { 8.9000000000000003e-10, 6.2598526540600004e+00, 1.6723350142595002e+04 },
    { 8.7999999999999996e-10, 1.1896734578199999e+00, 1.7298182327326202e+04 },
    { 8.6999999999999999e-10, 1.1294152530199999e+00, 1.7267268201691189e+04 },
    { 8.6999999999999999e-10, 1.3653441165600000e+00, 8.9828106693089994e+03 },
    { 8.6999999999999999e-10, 3.5267504157700000e+00, 3.9039113764198000e+03 },
    { 8.6000000000000003e-10, 4.3687011928899997e+00, 1.2808880303956799e+04 },
    { 8.6000000000000003e-10, 5.5251195695100002e+00, 1.2359966151545599e+04 },
    { 8.4999999999999996e-10, 1.0819876434200000e+00, 5.6360650166766000e+03 },
    { 8.3999999999999999e-10, 5.8599416006600000e+00, 6.2556742938945999e+03 },
    { 8.3999999999999999e-10, 3.9301846255799999e+00, 6.3104774060882000e+03 },
    { 8.3999999999999999e-10, 4.6404020120300000e+00, 1.0440274292603601e+04 },
    { 8.3999999999999999e-10, 5.9251402920100000e+00, 2.3791644735716000e+03 },
    { 8.3000000000000003e-10, 3.7491136336199999e+00, 1.8415515812097390e+04 },
    { 8.1999999999999996e-10, 4.9100684613799999e+00, 1.8842114002973391e+04 },
    { 8.0999999999999999e-10, 2.2769187652600000e+00, 1.1614433293732200e+04 },
    { 8.0000000000000003e-10, 5.0595062544299996e+00, 6.9229731363053997e+03 },
    { 8.0000000000000003e-10, 2.6262247457000001e+00, 3.5833410306738001e+03 },
    { 8.0000000000000003e-10, 6.0320549978300004e+00, 1.3521751441591399e+04 },
    { 7.7999999999999999e-10, 2.7456076255199999e+00, 8.6719698704406001e+03 },
    { 7.5999999999999996e-10, 2.8540968755199998e+00, 1.1925274092600601e+04 },
    { 7.5999999999999996e-10, 3.2639940138600001e+00, 1.8429742906099000e+04 },
    { 7.5000000000000000e-10, 5.7523430469999998e-02, 3.0355997319767801e+04 },
    { 7.4000000000000003e-10, 5.9005543819800002e+00, 5.1206011455835996e+03 },
    { 7.2999999999999996e-10, 4.5297682374499999e+00, 9.4917560896980001e+02 },
    { 7.2999999999999996e-10, 6.0837970005499997e+00, 9.3880059094152002e+03 },
    { 7.2000000000000000e-10, 4.3021679299500004e+00, 6.5105519827804001e+03 },
    { 7.2000000000000000e-10, 5.4879582962900004e+00, 6.0555997172023999e+03 },
    { 7.1000000000000003e-10, 4.9432251924799999e+00, 1.2545376304490401e+04 },
    { 6.6999999999999996e-10, 4.0053000195699999e+00, 1.6062184526116800e+04 },
    { 6.6999999999999996e-10, 4.2074281153099999e+00, 1.1933367960669601e+04 },
    { 6.6000000000000000e-10, 4.8163406413000001e+00, 1.0454501386605200e+04 },
    { 6.6000000000000000e-10, 7.9633834660000002e-01, 1.4919017853754600e+04 },
    { 6.5000000000000003e-10, 4.2996552855000001e+00, 7.3353442331797996e+03 },
    { 6.3000000000000000e-10, 5.7077216748500001e+00, 2.4356780788641601e+04 },
    { 6.0999999999999996e-10, 4.8642799986699998e+00, 2.8237233459389401e+04 },
    { 6.0999999999999996e-10, 3.8062554885800002e+00, 6.2536463414554000e+03 },
    { 6.0999999999999996e-10, 5.9838707376600002e+00, 6.3125053585273999e+03 },
    { 6.0000000000000000e-10, 1.9447015357299999e+00, 3.4570066334852600e+04 },
    { 5.7999999999999996e-10, 6.2471467735699999e+00, 1.6737577236596600e+04 },
    { 5.7999999999999996e-10, 4.0362404712100002e+00, 1.5141390794311999e+04 },
    { 5.6000000000000003e-10, 3.7631417683800001e+00, 1.2455945378763399e+04 },
    { 5.6000000000000003e-10, 6.0887110989000004e-01, 1.1764330768858999e+04 },
    { 5.4999999999999996e-10, 1.3212770393300000e+00, 2.3539707386332801e+04 },
    { 5.4999999999999996e-10, 5.6068094168400000e+00, 1.5664035522708589e+04 },
    { 5.4000000000000000e-10, 4.7521450654599997e+00, 7.8341210726394002e+03 },
    { 5.3000000000000003e-10, 5.6446603445100001e+00, 1.7996031168222202e+04 },
    { 5.3000000000000003e-10, 2.8100532560399998e+00, 2.6997348193175999e+03 },
    { 5.3000000000000003e-10, 3.7146972704199999e+00, 6.3578574485586996e+03 },
    { 5.3000000000000003e-10, 6.0754289558199996e+00, 6.2082942514241004e+03 },
    { 5.1999999999999996e-10, 1.8379610884199999e+00, 5.3267866940207996e+03 },
    { 5.1999999999999996e-10, 6.1376591764999999e-01, 4.5585172812187397e+04 },
    { 5.1000000000000000e-10, 2.3351265911799999e+00, 1.6522659716002199e+04 },
    { 5.1000000000000000e-10, 1.8359836380200001e+00, 2.3006425992586392e+04 },
    { 5.0000000000000003e-10, 4.7875636028799997e+00, 2.6084021806216198e+04 },
    { 5.0000000000000003e-10, 5.5219473110899999e+00, 2.1424466644303400e+04 },
    { 5.0000000000000003e-10, 2.9236613577199999e+00, 2.4602612434870989e+04 },
    { 4.8999999999999996e-10, 2.6165673567500001e+00, 3.5579691239569998e+04 },
    { 4.8999999999999996e-10, 3.3559168355100000e+00, 2.1393541969857601e+04 },
    { 4.8999999999999996e-10, 5.3176438392699996e+00, 7.0189523635231999e+03 },
    { 4.8999999999999996e-10, 4.4724823869700003e+00, 5.5471993364596001e+03 },
    { 4.8999999999999996e-10, 1.0204406888799999e+00, 1.2416588502848201e+04 },
    { 4.8999999999999996e-10, 9.7671520590000005e-02, 5.7498617665477996e+03 },
    { 4.8999999999999996e-10, 3.4092693984700002e+00, 6.8162899334350004e+03 },
    { 4.8999999999999996e-10, 2.9440367832900001e+00, 2.9303728936579399e+04 },
    { 4.8999999999999996e-10, 3.4189606498099998e+00, 2.9289501842577800e+04 },
    { 4.8999999999999996e-10, 4.2108662914600004e+00, 1.8216443810661000e+04 },
    { 4.8000000000000000e-10, 1.9540870768400000e+00, 5.2440492392010001e+03 },
    { 4.7000000000000003e-10, 3.5596007409100001e+00, 2.2743409379516401e+04 },
    { 4.7000000000000003e-10, 5.6890170244900000e+00, 2.4279107018213592e+04 },
    { 4.7000000000000003e-10, 5.2217551706999998e-01, 2.4065807922775592e+04 },
    { 4.7000000000000003e-10, 5.8167758607500000e+00, 1.8208349942592002e+04 },
    { 4.7000000000000003e-10, 1.1741888100800000e+00, 5.8953145443294001e+04 },
    { 4.6000000000000001e-10, 1.8805490512500000e+00, 1.9378918515068799e+04 },
    { 4.6000000000000001e-10, 4.1671622623099998e+00, 1.0435161256629678e+05 },
    { 4.5000000000000000e-10, 3.7022866350400001e+00, 5.4247413135750401e+04 },
    { 4.3999999999999998e-10, 2.0668510447999999e+00, 6.1754123260528004e+03 },
    { 4.3999999999999998e-10, 1.4400898742599999e+00, 6.3907393739299996e+03 },
    { 4.3999999999999998e-10, 3.2091427368800001e+00, 2.6735945262213201e+04 },
    { 4.3999999999999998e-10, 1.2193705574700000e+00, 3.9814900340820000e+03 },
    { 4.3000000000000001e-10, 3.9385731587100001e+00, 1.2132439962106000e+04 },
    { 4.3000000000000001e-10, 6.0171130597299998e+00, 5.5022935747074400e+04 },
    { 4.2000000000000000e-10, 6.1854190947100003e+00, 2.4705705209089589e+04 },
    { 4.0999999999999998e-10, 3.2161397839100001e+00, 1.7370360975909800e+04 },
    { 4.0999999999999998e-10, 2.0551892676899999e+00, 2.3020653086587990e+04 },
    { 4.0999999999999998e-10, 2.6965596734099999e+00, 1.6207886271502000e+04 },
    { 4.0999999999999998e-10, 3.3823477781600002e+00, 1.5508615123274400e+04 },
    { 4.0999999999999998e-10, 2.5202112543799999e+00, 2.1202093703745999e+04 },
    { 4.0999999999999998e-10, 4.3984326356099999e+00, 1.8052929543157799e+04 },
    { 4.0000000000000001e-10, 4.5747412752200001e+00, 3.2243498500880600e+04 },
    { 4.0000000000000001e-10, 5.1718276955800002e+00, 2.4336005393149200e+04 },
    { 4.0000000000000001e-10, 3.6281647758600002e+00, 1.9804827291582798e+04 },
    { 4.0000000000000001e-10, 4.7149383564500003e+00, 8.4018397103698007e+03 },
    { 4.0000000000000001e-10, 2.0281843473099999e+00, 4.9515382508406998e+04 },
    { 3.9000000000000000e-10, 1.1346259412799999e+00, 1.8606498946000200e+04 },
    { 3.9000000000000000e-10, 1.7519880472000000e+00, 2.9826306354673201e+04 },
    { 3.9000000000000000e-10, 8.8019393819000002e-01, 2.4889574795991601e+04 },
    { 3.9000000000000000e-10, 4.6387056839499996e+00, 2.6709646942413401e+04 },
    { 3.7999999999999998e-10, 5.5650742954299997e+00, 2.9088811415985001e+04 },
    { 3.7999999999999998e-10, 2.3648192606700000e+00, 6.2688487559898003e+03 },
    { 3.7999999999999998e-10, 1.1421216583900000e+00, 6.2973029439929996e+03 },
    { 3.7999999999999998e-10, 4.8221569191500002e+00, 6.1718892077037999e+03 },
    { 3.7999999999999998e-10, 4.9679693070899997e+00, 6.3942624922790001e+03 },
    { 3.7999999999999998e-10, 4.5918553896400001e+00, 1.8202216716659390e+04 },
    { 3.7999999999999998e-10, 5.0127754289100004e+00, 9.2255392732830005e+03 },
    { 3.7999999999999998e-10, 2.6704143574899999e+00, 1.5110466119866200e+04 },
    { 3.7999999999999998e-10, 3.5514153129100001e+00, 3.1570799649391200e+04 },
    { 3.7000000000000001e-10, 3.6477686193499999e+00, 2.0452869412221800e+04 },
    { 3.7000000000000001e-10, 6.1857705527300002e+00, 4.0612192153944002e+03 },
    { 3.7000000000000001e-10, 5.9283658599100004e+00, 7.3495713271814002e+03 },
    { 3.7000000000000001e-10, 1.8482755164800000e+00, 2.9296615389578601e+04 },
    { 3.6000000000000000e-10, 1.5681201404800000e+00, 9.9248104215105996e+03 },
    { 3.6000000000000000e-10, 6.0706972051800001e+00, 2.5287723799399799e+04 },
    { 3.4999999999999998e-10, 3.1692971406799999e+00, 2.6482170809624400e+04 },
    { 3.4999999999999998e-10, 1.6358249530200000e+00, 7.3221024607817999e+03 },
    { 3.4999999999999998e-10, 3.0016092433499999e+00, 2.1954157609397989e+04 },
    { 3.4999999999999998e-10, 2.4944506014200001e+00, 8.5846616659007996e+03 },
    { 3.4999999999999998e-10, 4.2634208971699996e+00, 1.1919140866668000e+04 },
    { 3.4999999999999998e-10, 2.5948226266200001e+00, 6.2883355139513602e+04 },
    { 3.4999999999999998e-10, 5.5431307244400001e+00, 2.9062513096185201e+04 },
    { 3.4999999999999998e-10, 5.9688820043100002e+00, 2.5685872802808000e+04 },
    { 3.4000000000000001e-10, 6.1003992059599996e+00, 6.1474505249814001e+03 },
    { 3.4000000000000001e-10, 3.6897270202799999e+00, 6.4187011750013999e+03 },
    { 3.4000000000000001e-10, 2.5476088664900001e+00, 2.1228392023545799e+04 },
    { 3.4000000000000001e-10, 6.8922440987000000e-01, 3.3019021112204602e+04 },
    { 3.4000000000000001e-10, 4.0887359373999999e-01, 1.7271884007526402e+04 },
    { 3.4000000000000001e-10, 3.3424154418000002e-01, 2.9026485229507791e+04 },
    { 3.4000000000000001e-10, 3.3805206880799998e+00, 3.8500276031072201e+04 },
    { 3.4000000000000001e-10, 3.8238717878299999e+00, 2.2483848574492589e+04 },
    { 3.4000000000000001e-10, 5.4981324000300003e+00, 5.9057022420756002e+03 },
    { 3.4000000000000001e-10, 4.2919938262099997e+00, 6.6604494579071998e+03 },
    { 3.3000000000000000e-10, 1.3837047156000001e-01, 3.4520309309380798e+04 },
    { 3.3000000000000000e-10, 4.7104178182799998e+00, 5.0175083713650001e+03 },
    { 3.1999999999999998e-10, 2.8121971103700001e+00, 1.7686752845566400e+04 },
    { 3.1999999999999998e-10, 4.6908516948700001e+00, 6.2317955121289897e+03 },
    { 3.1999999999999998e-10, 5.0992745313699999e+00, 6.3343561878538103e+03 },
    { 3.1999999999999998e-10, 5.0471192505699998e+00, 2.7511467873537200e+04 },
    { 3.1000000000000002e-10, 2.7949892618100001e+00, 2.8230187222691391e+04 },
    { 3.1000000000000002e-10, 7.7685387250000004e-01, 1.5265886519300400e+04 },
    { 3.1000000000000002e-10, 2.3703481321400002e+00, 2.4491425792583399e+04 },
    { 3.0000000000000000e-10, 1.3893286524799999e+00, 1.2489885628707199e+04 },
    { 3.0000000000000000e-10, 7.5326086687000005e-01, 1.2586927095475199e+04 },
    { 3.0000000000000000e-10, 4.3213210158999997e+00, 5.2175806283148399e+04 },
    { 2.8999999999999998e-10, 6.9173791628000003e-01, 2.7278468816440800e+04 },
    { 2.8999999999999998e-10, 4.0087061749500004e+00, 4.6386993743311199e+04 },
    { 2.8999999999999998e-10, 4.8898572315499997e+00, 6.5923541726472004e+03 },
    { 2.8999999999999998e-10, 4.9002689946900002e+00, 5.9737975273355996e+03 },
    { 2.8999999999999998e-10, 4.6274169269799996e+00, 1.9402796952816600e+04 },
    { 2.8000000000000002e-10, 3.0766379270999999e-01, 2.4080035016777190e+04 },
    { 2.8000000000000002e-10, 1.8020070345899999e+00, 2.1150813365883601e+04 },
    { 2.8000000000000002e-10, 3.9769012599700000e+00, 3.4911412076091001e+04 },
    { 2.8000000000000002e-10, 5.7994617282499998e+00, 1.6193659177500391e+04 },
    { 2.8000000000000002e-10, 5.3741198416799998e+00, 1.0239583866010800e+04 },
    { 2.7000000000000000e-10, 2.0342161428500001e+00, 3.6949230808424203e+04 },
    { 2.7000000000000000e-10, 4.9966533387399998e+00, 2.4383079108441401e+04 },
    { 2.7000000000000000e-10, 4.6010904926300000e+00, 1.6921656695024001e+03 },
    { 2.7000000000000000e-10, 3.0146551073299999e+00, 2.3581258177317599e+04 },
    { 2.7000000000000000e-10, 5.3687631616899996e+00, 6.3607496204194003e+03 },
    { 2.7000000000000000e-10, 4.4213630645500004e+00, 6.2054020795633996e+03 },
    { 2.7000000000000000e-10, 7.4966550511999996e-01, 3.2367097656207599e+04 },
    { 2.7000000000000000e-10, 3.3432676815900000e+00, 4.4878174062704002e+03 },
    { 2.7000000000000000e-10, 2.7981391134900000e+00, 1.9800945956224801e+04 },
    { 2.7000000000000000e-10, 1.5682380163300000e+00, 2.8628336226099600e+04 },
    { 2.7000000000000000e-10, 5.3823814605700004e+00, 3.4513263072682799e+04 },
    { 2.5999999999999998e-10, 1.3028592937400001e+00, 4.7970957289261996e+03 },
    { 2.5999999999999998e-10, 5.0815101886000003e+00, 6.3399742249270003e+03 },
    { 2.5999999999999998e-10, 4.7086160376399997e+00, 6.2261774750557997e+03 },
    { 2.5999999999999998e-10, 3.8092001546800001e+00, 2.5158601719765400e+04 },
    { 2.5999999999999998e-10, 4.1235375708699999e+00, 9.9105833275090008e+03 },
    { 2.5999999999999998e-10, 4.6444351363200003e+00, 3.0666154958432799e+04 },
    { 2.5999999999999998e-10, 1.8397195956700001e+00, 1.8422629359098191e+04 },
    { 2.5000000000000002e-10, 1.2141143267500001e+00, 6.3027456108911902e+03 },
    { 2.5000000000000002e-10, 2.2928265923100000e+00, 6.2634060890915998e+03 },
    { 2.5000000000000002e-10, 5.5279431761500000e+00, 1.4155224595266600e+04 },
    { 2.5000000000000002e-10, 1.9612904846300001e+00, 5.5798458358398399e+04 },
    { 2.5000000000000002e-10, 4.0627160124000001e+00, 2.9864334027309000e+04 },
    { 2.4000000000000000e-10, 5.2626599571600003e+00, 1.1616976091013001e+04 },
    { 2.4000000000000000e-10, 4.6838821564500002e+00, 3.1172650645983002e+04 },
    { 2.4000000000000000e-10, 2.1637321784700001e+00, 3.1968948652799401e+04 },
    { 2.4000000000000000e-10, 6.1208540916300000e+00, 2.0597243963041201e+04 },
    { 2.4000000000000000e-10, 1.6515541182800000e+00, 2.3550344051682590e+04 },
    { 2.4000000000000000e-10, 3.8390436425100001e+00, 8.1865126624925997e+03 },
    { 2.4000000000000000e-10, 5.9208685680300004e+00, 9.4114646150871995e+03 },
    { 2.3000000000000001e-10, 5.3358315228300004e+00, 1.7892938394003591e+04 },
    { 2.3000000000000001e-10, 5.0538302233300003e+00, 2.9424634232916000e+04 },
    { 2.3000000000000001e-10, 6.0730243257399996e+00, 6.7167875878681998e+03 },
    { 2.3000000000000001e-10, 1.4920431250499999e+00, 6.2720301497275004e+03 },
    { 2.3000000000000001e-10, 2.0148977940099999e+00, 6.2941215502552996e+03 },
    { 2.1999999999999999e-10, 2.4266215928300001e+00, 7.4455505543992003e+03 },
    { 2.1999999999999999e-10, 3.3957046657399999e+00, 4.0879440504643797e+04 },
    { 2.1999999999999999e-10, 6.0950092246300001e+00, 4.3796390374902003e+03 },
    { 2.1999999999999999e-10, 2.9514424664300001e+00, 2.8759810877483189e+04 },
    { 2.1999999999999999e-10, 1.0628497184000001e+00, 2.7707542494294801e+04 },
    { 2.1999999999999999e-10, 2.8739355167000000e+00, 1.8326650131880389e+04 },
    { 2.1999999999999999e-10, 2.8942551095500000e+00, 6.3328328754632003e+03 },
    { 2.1999999999999999e-10, 6.1268580950999996e-01, 6.2333188245195997e+03 },
    { 2.1000000000000000e-10, 4.2117071884100001e+00, 2.2779437246193800e+04 },
    { 2.1000000000000000e-10, 2.5149472886300002e+00, 3.0220932239732399e+04 },
    { 2.1000000000000000e-10, 5.4308840451500000e+00, 5.0317203439530800e+04 },
    { 2.0000000000000001e-10, 7.5971546170000001e-02, 2.8313288804660999e+04 },
    { 2.0000000000000001e-10, 8.2128190790000000e-02, 6.3790550772092001e+03 },
    { 2.0000000000000001e-10, 3.4248127282700001e+00, 6.1870966227735998e+03 },
    { 2.0000000000000001e-10, 4.2678430916299996e+00, 2.1548962369291799e+04 },
    { 2.0000000000000001e-10, 3.8470757996999999e-01, 2.2345260376108199e+04 },
    { 2.0000000000000001e-10, 2.6025108372100001e+00, 4.2456784047091598e+04 },
    { 2.0000000000000001e-10, 1.2831324094900001e+00, 9.0701188738487999e+03 },
    { 1.8999999999999999e-10, 7.5805525916000005e-01, 1.1712955318230799e+04 },
    { 1.8999999999999999e-10, 7.9376120002999995e-01, 5.6183198048614004e+03 },
    { 1.8999999999999999e-10, 2.7131797190300002e+00, 6.9478318951213996e+03 },
    { 1.8000000000000000e-10, 3.0473742976399998e+00, 6.5939166488598003e+03 },
    { 1.8000000000000000e-10, 4.5956662141999999e-01, 5.9722350511229997e+03 },
    { 1.8000000000000000e-10, 2.8017726157000000e-01, 3.4115114069274598e+04 },
};

static const VSOP87Term VSOP87A_EMB_Y2[184] = {
    { 5.0802079999999998e-05, 0.0000000000000000e+00, 0.0000000000000000e+00 },
    { 2.1780159999999999e-05, 2.8295754423499999e+00, 1.2566151699982800e+04 },
    { 1.0204870000000001e-05, 4.6374671859800003e+00, 6.2830758499914000e+03 },
    { 9.2688000000000005e-07, 2.6221874841999999e+00, 1.8849227549974199e+04 },
    { 1.3680000000000001e-07, 1.8073689688400001e+00, 6.2865989683403996e+03 },
    { 1.3668000000000000e-07, 1.7019185933900001e+00, 6.2795527316424004e+03 },
    { 8.1899999999999999e-08, 2.8484838090200002e+00, 6.2567775301915999e+03 },
    { 8.1450000000000005e-08, 6.5845469207999996e-01, 6.3093741697912001e+03 },
    { 8.0039999999999994e-08, 4.3520851613299998e+00, 6.1276554505572003e+03 },
    { 7.7540000000000005e-08, 5.4389383249599996e+00, 6.4384962494255997e+03 },
    { 3.7300000000000003e-08, 9.1975073802999996e-01, 5.5075532386674004e+03 },
    { 3.2030000000000003e-08, 2.5719373153200000e+00, 7.0585984613153996e+03 },
    { 2.9370000000000001e-08, 2.5108118056600000e+00, 2.5132303399965600e+04 },
    { 2.8950000000000001e-08, 3.3791892467300002e+00, 4.7057323075435997e+03 },
    { 2.2679999999999999e-08, 1.9491768667700000e+00, 7.7552261132399997e+02 },
    { 2.1480000000000001e-08, 1.4152116438000001e-01, 7.8604193924392002e+03 },
    { 2.0310000000000001e-08, 7.2283806290999997e-01, 6.2759623029905997e+03 },
    { 2.0310000000000001e-08, 2.7841028561500001e+00, 6.2901893969922003e+03 },
    { 1.9040000000000000e-08, 2.8958905459800000e+00, 5.4867778431750003e+03 },
    { 1.7280000000000000e-08, 3.6613296073699999e+00, 1.0593819301891999e+03 },
    { 1.5630000000000002e-08, 6.1022183613000003e-01, 7.0793738568077997e+03 },
    { 1.5530000000000001e-08, 5.6386962805999996e-01, 6.0403472460173998e+03 },
    { 1.4699999999999999e-08, 2.9457999971800000e+00, 6.5258044539654002e+03 },
    { 1.4590000000000000e-08, 1.8825731029599999e+00, 5.7533848848968000e+03 },
    { 1.4359999999999999e-08, 1.4535716837799999e+00, 5.8849268465832001e+03 },
    { 1.4250000000000000e-08, 1.5914785976200001e+00, 6.8127668150860000e+03 },
    { 1.3380000000000000e-08, 5.7568444452799996e+00, 5.7295064471490005e+03 },
    { 1.2880000000000000e-08, 2.0581281914899998e+00, 6.6812248533995999e+03 },
    { 1.1910000000000000e-08, 4.0191707424200001e+00, 6.8366452528338004e+03 },
    { 1.0390000000000001e-08, 5.5078214144000004e+00, 6.2840561710596003e+03 },
    { 1.0390000000000001e-08, 4.2823048118399996e+00, 6.2820955289231997e+03 },
    { 9.0400000000000002e-09, 8.6455323243000004e-01, 5.3313574437407997e+03 },
    { 8.7700000000000001e-09, 1.4946982901500001e+00, 1.1790629088658799e+04 },
    { 8.7099999999999999e-09, 2.2768897869100000e+00, 4.9332084403325998e+03 },
    { 8.5500000000000005e-09, 3.6724106426400001e+00, 4.5350594369244000e+03 },
    { 8.1400000000000004e-09, 1.0752025917200001e+00, 1.5542039943419999e+02 },
    { 7.7799999999999992e-09, 4.3274808799500004e+00, 5.0886288397668004e+03 },
    { 7.7400000000000002e-09, 2.6816390103200001e+00, 7.2347942562420003e+03 },
    { 7.4600000000000003e-09, 5.0650554848500002e+00, 4.1369104335162001e+03 },
    { 6.8599999999999999e-09, 3.4713100018000000e+00, 8.0182093112380005e+02 },
    { 6.7700000000000004e-09, 1.1489114878100000e+00, 3.1283887650958000e+03 },
    { 6.6400000000000002e-09, 3.2197812690999998e+00, 1.1944470102246000e+03 },
    { 6.5300000000000004e-09, 6.1415985483500002e+00, 1.1506769769793600e+04 },
    { 6.5300000000000004e-09, 1.2004299117299999e+00, 7.6329432596502002e+03 },
    { 5.9799999999999996e-09, 6.1270125669600004e+00, 8.0310922630584000e+03 },
    { 5.7900000000000001e-09, 4.6469091322100002e+00, 7.9629800681639995e+02 },
    { 5.6200000000000000e-09, 5.4834452037999997e+00, 7.4775228602159996e+03 },
    { 5.1099999999999999e-09, 7.1257437268000001e-01, 4.6940029547076001e+03 },
    { 5.0000000000000001e-09, 1.8374633980999999e+00, 1.5925960136327999e+03 },
    { 4.9900000000000003e-09, 1.4567260317099999e+00, 1.2569674818331800e+04 },
    { 4.9900000000000003e-09, 4.6187781806999997e+00, 1.5890728952838001e+03 },
    { 4.8000000000000000e-09, 4.7256652264800003e+00, 8.4292412664666008e+03 },
    { 4.1499999999999999e-09, 5.0637012431799997e+00, 6.4963749454294002e+03 },
    { 4.1400000000000002e-09, 4.7348953573900001e+00, 6.0697767545533998e+03 },
    { 4.0400000000000001e-09, 1.5585991312000000e-01, 3.7387614301079998e+03 },
    { 3.9199999999999997e-09, 6.4094689718999998e-01, 2.6298319799800002e+01 },
    { 3.6699999999999999e-09, 2.3550839132600001e+00, 9.4377629348869996e+03 },
    { 3.6600000000000002e-09, 5.9325568885499997e+00, 5.5356940284240000e+02 },
    { 3.6100000000000001e-09, 3.7813991491399999e+00, 7.0848967811151997e+03 },
    { 3.5600000000000001e-09, 4.7876415708900000e+00, 1.2721572099417001e+04 },
    { 3.4699999999999998e-09, 6.0548761950900003e+00, 5.4812549188676003e+03 },
    { 3.2000000000000001e-09, 4.3684109891000000e-01, 1.9907450170410000e+03 },
    { 3.1800000000000002e-09, 4.5387236749200000e+00, 9.5171840625059997e+02 },
    { 2.8800000000000000e-09, 3.1652145204100002e+00, 1.3498674096587999e+03 },
    { 2.7900000000000001e-09, 5.5153176727100002e+00, 4.6904798363585996e+03 },
    { 2.7900000000000001e-09, 6.5072513622999995e-01, 2.4272860397400001e+02 },
    { 2.6000000000000001e-09, 9.8725902771999996e-01, 5.2236939198022001e+03 },
    { 2.5500000000000001e-09, 2.6797013701000000e-01, 1.1371704689758200e+04 },
    { 2.4500000000000000e-09, 2.6027019681599999e+00, 3.1546870848956000e+03 },
    { 2.4199999999999999e-09, 3.3524800661500000e+00, 8.8273902698748007e+03 },
    { 2.2200000000000002e-09, 1.7857635291999999e-01, 4.1643119896130002e+03 },
    { 2.2100000000000000e-09, 1.8368312461400000e+00, 1.7480164130670000e+03 },
    { 2.0700000000000001e-09, 7.1423535949000005e-01, 1.3367972631106601e+04 },
    { 2.0599999999999999e-09, 1.6811872765500000e+00, 1.0973555686350001e+04 },
    { 2.0400000000000000e-09, 5.1547963918299997e+00, 1.1769853693166400e+04 },
    { 2.0400000000000000e-09, 5.1724632225100002e+00, 1.0977078804699000e+04 },
    { 1.9599999999999998e-09, 4.8130603143600004e+00, 3.9302096962196001e+03 },
    { 1.8800000000000001e-09, 4.0881020062799998e+00, 7.8756718636242003e+03 },
    { 1.8800000000000001e-09, 5.3202267915299997e+00, 2.3888940204492001e+03 },
    { 1.8300000000000001e-09, 2.5302040700499999e+00, 7.3424577801805999e+03 },
    { 1.5500000000000000e-09, 4.2433318282599997e+00, 1.2410731300548599e+04 },
    { 1.4400000000000000e-09, 3.1211954451700001e+00, 1.0575406682941801e+04 },
    { 1.4200000000000001e-09, 3.4376902395900002e+00, 5.2969096509459996e+02 },
    { 1.3500000000000001e-09, 5.6246049535099996e+00, 1.2592450019782600e+04 },
    { 1.2500000000000000e-09, 2.3073465065600001e+00, 1.2562628581633800e+04 },
    { 1.2400000000000001e-09, 4.8871698069200002e+00, 8.6359420037632008e+03 },
    { 1.1800000000000001e-09, 3.1591180263799998e+00, 1.3119721102825190e+04 },
    { 1.1100000000000001e-09, 8.0973565730999997e-01, 3.9814900340819997e+02 },
    { 1.0900000000000000e-09, 2.4264712075100001e+00, 1.2539853380183000e+04 },
    { 1.0500000000000001e-09, 1.9676566688599999e+00, 5.6421982426092000e+03 },
    { 1.0200000000000000e-09, 2.8004279255100002e+00, 1.0988808157535001e+04 },
    { 1.0100000000000000e-09, 2.6697958405000000e-01, 4.2923308329503998e+03 },
    { 1.0000000000000001e-09, 7.7889460323000004e-01, 1.3095842665077400e+04 },
    { 9.9000000000000010e-10, 1.7490457978900000e+00, 1.3517870106233400e+04 },
    { 9.6999999999999996e-10, 4.0851988367900001e+00, 1.7260154654690399e+04 },
    { 9.5000000000000003e-10, 5.8410858338000005e-01, 2.1461654164751999e+03 },
    { 9.4000000000000006e-10, 4.5691361541899997e+00, 1.0177257679533601e+04 },
    { 9.2999999999999999e-10, 4.8772965408699998e+00, 1.5773435424478000e+03 },
    { 9.2000000000000003e-10, 4.1159879069999999e-01, 1.3916019109641600e+04 },
    { 9.0999999999999996e-10, 1.5388105379399999e+00, 6.9239534573736000e+03 },
    { 8.1999999999999996e-10, 3.6508209323499998e+00, 1.6730463689595799e+04 },
    { 8.1999999999999996e-10, 2.4313885790300001e+00, 3.1415379249957001e+04 },
    { 8.0000000000000003e-10, 2.6763620293499999e+00, 1.2323423096008801e+04 },
    { 7.5000000000000000e-10, 4.1598768654600002e+00, 6.7096740408674004e+03 },
    { 7.5000000000000000e-10, 5.6302493607799997e+00, 5.8564776591153995e+03 },
    { 7.1000000000000003e-10, 5.3120756770600002e+00, 1.4314168113049800e+04 },
    { 6.6000000000000000e-10, 3.9250563742000000e+00, 2.7870430238573999e+03 },
    { 6.5000000000000003e-10, 4.9263361879199996e+00, 4.7320306273433998e+03 },
    { 6.0999999999999996e-10, 2.6690586221000001e+00, 8.2738208670324002e+03 },
    { 5.6000000000000003e-10, 3.1434358452800000e+00, 6.3989728631399998e+02 },
    { 5.4999999999999996e-10, 2.9844019398100001e+00, 5.6431785636774002e+03 },
    { 5.4999999999999996e-10, 2.2646906240500000e+00, 1.3341674311306801e+04 },
    { 5.4000000000000000e-10, 4.0607102831699997e+00, 1.6496361396202399e+04 },
    { 5.3000000000000003e-10, 2.5529377326399998e+00, 1.4919017853754600e+04 },
    { 5.1999999999999996e-10, 5.9708568948299998e+00, 3.6346210245184002e+03 },
    { 5.1999999999999996e-10, 6.1967230054000000e+00, 1.5720838784878400e+04 },
    { 5.1000000000000000e-10, 5.9781251644499998e+00, 6.1957676454515904e+03 },
    { 5.1000000000000000e-10, 3.8120010617900002e+00, 6.3703840545311996e+03 },
    { 5.0000000000000003e-10, 2.6163662952800002e+00, 1.2573265246983599e+04 },
    { 4.8000000000000000e-10, 4.5273646609000001e-01, 1.3362449706799200e+04 },
    { 4.8000000000000000e-10, 5.3122782969699998e+00, 2.1187638603783998e+03 },
    { 4.7000000000000003e-10, 2.2587965183900001e+00, 1.2964300703391000e+04 },
    { 4.7000000000000003e-10, 1.0689512961000001e+00, 3.8941818295421999e+03 },
    { 4.5000000000000000e-10, 6.0460993038800002e+00, 9.7791086761253991e+03 },
    { 4.3999999999999998e-10, 3.2418412625599999e+00, 8.6622403235630009e+03 },
    { 4.2000000000000000e-10, 5.0930054901500004e+00, 2.4072921469776400e+04 },
    { 4.0999999999999998e-10, 9.2655469135000001e-01, 1.8319536584879599e+04 },
    { 4.0999999999999998e-10, 2.5224794661000001e+00, 3.1851920272655998e+03 },
    { 4.0000000000000001e-10, 4.0292674400499999e+00, 1.7789845619784999e+04 },
    { 3.9000000000000000e-10, 2.7055990439199999e+00, 2.0426571092422000e+04 },
    { 3.4999999999999998e-10, 5.1788550255599999e+00, 1.8875525869773999e+04 },
    { 3.4999999999999998e-10, 4.5199513750799998e+00, 1.0419986283507600e+04 },
    { 3.3000000000000000e-10, 1.6776493193699999e+00, 3.4960328261340001e+03 },
    { 3.1000000000000002e-10, 5.3588642680999996e+00, 2.3013539539587200e+04 },
    { 3.1000000000000002e-10, 3.9597188454099999e+00, 1.4712317116458000e+04 },
    { 3.1000000000000002e-10, 5.4791130833599997e+00, 2.2779437246193800e+04 },
    { 3.0000000000000000e-10, 2.8263606786100000e+00, 1.2808880303956799e+04 },
    { 3.0000000000000000e-10, 2.3168220348699999e+00, 1.0447387839604400e+04 },
    { 3.0000000000000000e-10, 1.1755591314400000e+00, 9.3809596727171993e+03 },
    { 3.0000000000000000e-10, 1.8376248773399999e+00, 1.0213285546211000e+04 },
    { 2.8000000000000002e-10, 2.1187643592700001e+00, 1.4143495242430599e+04 },
    { 2.8000000000000002e-10, 3.9147895497700000e+00, 5.4298794682394000e+03 },
    { 2.5999999999999998e-10, 7.2314347990000005e-01, 9.6236882766912004e+03 },
    { 2.5999999999999998e-10, 2.7837974391600002e+00, 2.9424634232916001e+03 },
    { 2.5999999999999998e-10, 1.8151529860999999e-01, 1.8209330263660191e+04 },
    { 2.5999999999999998e-10, 5.6173237680600003e+00, 1.2012582297140400e+04 },
    { 2.5000000000000002e-10, 1.1173303928600000e+00, 3.5833410306738001e+03 },
    { 2.4000000000000000e-10, 1.3677521601200000e+00, 9.9176968745098002e+03 },
    { 2.3000000000000001e-10, 9.2088373151000003e-01, 1.2559038152982001e+04 },
    { 2.3000000000000001e-10, 5.5232502821200002e+00, 1.1015106477334801e+04 },
    { 2.3000000000000001e-10, 6.9647593830999999e-01, 2.6709646942413401e+04 },
    { 2.1999999999999999e-10, 2.1929941598400000e+00, 1.7298182327326202e+04 },
    { 2.1000000000000000e-10, 8.1252266713999999e-01, 1.7256631536341400e+04 },
    { 2.1000000000000000e-10, 2.2028540323499999e+00, 1.6858482532933202e+04 },
    { 2.1000000000000000e-10, 2.8996818670100000e+00, 1.2036460734888200e+04 },
    { 1.8999999999999999e-10, 3.5979025769000000e-01, 1.1926254413668799e+04 },
    { 1.8999999999999999e-10, 4.8978319714699996e+00, 1.8073704938650200e+04 },
    { 1.8999999999999999e-10, 2.0902102964700000e+00, 3.0639856638632999e+04 },
    { 1.8999999999999999e-10, 1.2325173291800000e+00, 1.9651048481098002e+04 },
    { 1.8999999999999999e-10, 1.2434168806000001e+00, 8.6719698704406001e+03 },
    { 1.8000000000000000e-10, 7.1176167314000005e-01, 1.1614433293732200e+04 },
    { 1.7000000000000001e-10, 2.3862420400400000e+00, 8.9828106693089994e+03 },
    { 1.5999999999999999e-10, 5.1239428121200001e+00, 2.3543230504681789e+04 },
    { 1.5000000000000000e-10, 2.6476682161800000e+00, 2.0199094959632999e+04 },
    { 1.5000000000000000e-10, 5.2886461920799999e+00, 1.2567132021051000e+04 },
    { 1.4000000000000001e-10, 3.3056659977499998e+00, 1.6460333529524989e+04 },
    { 1.4000000000000001e-10, 4.8718114427800003e+00, 2.2483848574492589e+04 },
    { 1.4000000000000001e-10, 3.6237106346999998e-01, 2.1228392023545799e+04 },
    { 1.2000000000000000e-10, 1.1390821054100000e+00, 1.7996031168222202e+04 },
    { 1.2000000000000000e-10, 5.4345882734300002e+00, 1.9402796952816600e+04 },
    { 1.2000000000000000e-10, 2.8079805015099999e+00, 1.0818135286915800e+04 },
    { 1.0999999999999999e-10, 2.1390152319000000e-01, 2.8766924424484001e+04 },
    { 1.0999999999999999e-10, 6.8624711980999997e-01, 1.4945316173554400e+04 },
    { 1.0999999999999999e-10, 3.3624826398400001e+00, 1.8451078546565990e+04 },
    { 1.0999999999999999e-10, 4.6982643669700002e+00, 2.3528661537717999e+03 },
    { 1.0999999999999999e-10, 3.0993049479000001e-01, 2.2805735565993600e+04 },
    { 1.0999999999999999e-10, 6.7524830165000005e-01, 3.5371887265976402e+04 },
    { 1.0999999999999999e-10, 5.1089232199100003e+00, 2.5158601719765400e+04 },
    { 1.0000000000000000e-10, 4.4822791742700003e+00, 2.3141558382924599e+04 },
    { 1.0000000000000000e-10, 9.8562604233999995e-01, 2.2345260376108199e+04 },
    { 1.0000000000000000e-10, 4.8959771403100003e+00, 1.2779450795420800e+04 },
    { 1.0000000000000000e-10, 4.5306288748399997e+00, 2.1329909543799999e+02 },
    { 8.9999999999999999e-11, 6.2038754135599996e+00, 2.9826306354673201e+04 },
    { 8.9999999999999999e-11, 2.8965918625599998e+00, 2.3581258177317599e+04 },
};

static const VSOP87Term VSOP87A_EMB_Y3[17] = {
    { 1.2811600000000001e-06, 3.1415926535900001e+00, 0.0000000000000000e+00 },
    { 7.2366000000000001e-07, 1.3201931441300001e+00, 1.2566151699982800e+04 },
    { 1.3847000000000000e-07, 5.3744470163300004e+00, 6.2830758499914000e+03 },
    { 5.0440000000000000e-08, 9.6001931081000003e-01, 1.8849227549974199e+04 },
    { 1.4810000000000000e-08, 6.6592602555000002e-01, 6.4384962494255997e+03 },
    { 1.4810000000000000e-08, 2.8410148935100001e+00, 6.1276554505572003e+03 },
    { 6.4400000000000001e-09, 3.3148827210000000e+00, 6.2795527316424004e+03 },
    { 6.4400000000000001e-09, 1.9205819805999999e-01, 6.2865989683403996e+03 },
    { 3.1800000000000002e-09, 2.0676067750299998e+00, 6.0403472460173998e+03 },
    { 3.1800000000000002e-09, 1.4393341440300000e+00, 6.5258044539654002e+03 },
    { 2.1999999999999998e-09, 7.7087312633000005e-01, 2.5132303399965600e+04 },
    { 2.0099999999999999e-09, 5.5395365884799999e+00, 6.8366452528338004e+03 },
    { 2.0099999999999999e-09, 4.2505896377600001e+00, 5.7295064471490005e+03 },
    { 3.6000000000000000e-10, 6.1244166849299999e+00, 1.2569674818331800e+04 },
    { 3.3000000000000000e-10, 2.6647495293199999e+00, 1.2410731300548599e+04 },
    { 1.8000000000000000e-10, 7.5629855074999996e-01, 1.2721572099417001e+04 },
    { 1.0999999999999999e-10, 4.7023434921099998e+00, 1.2592450019782600e+04 },
};

static const VSOP87Term VSOP87A_EMB_Y4[10] = {
    { 4.1869999999999997e-08, 3.1415926535900001e+00, 0.0000000000000000e+00 },
    { 1.9309999999999999e-08, 5.8988383799999999e+00, 1.2566151699982800e+04 },
    { 5.9500000000000003e-09, 5.1679590840000000e-01, 6.2830758499914000e+03 },
    { 2.2900000000000002e-09, 5.5931151922600000e+00, 1.8849227549974199e+04 },
    { 2.1000000000000002e-09, 2.1723989736800000e+00, 6.4384962494255997e+03 },
    { 2.1000000000000002e-09, 1.3345419453800000e+00, 6.1276554505572003e+03 },
    { 2.1000000000000000e-10, 4.8963301434000002e+00, 6.2795527316424004e+03 },
    { 2.1000000000000000e-10, 4.8937960828399998e+00, 6.2865989683403996e+03 },
    { 1.0999999999999999e-10, 5.3679182922799997e+00, 2.5132303399965600e+04 },
    { 5.0000000000000002e-11, 1.1183590602900000e+00, 1.2410731300548599e+04 },
};

static const VSOP87Term VSOP87A_EMB_Y5[6] = {
    { 7.2000000000000000e-10, 0.0000000000000000e+00, 0.0000000000000000e+00 },
    { 4.3000000000000001e-10, 4.5171001937900002e+00, 1.2566151699982800e+04 },
    { 2.5000000000000002e-10, 3.7670003266599998e+00, 6.4384962494255997e+03 },
    { 2.5000000000000002e-10, 6.0231258995800001e+00, 6.1276554505572003e+03 },
    { 2.1000000000000000e-10, 2.5688259100400002e+00, 6.2830758499914000e+03 },
    { 7.9999999999999995e-11, 3.9550842846299998e+00, 1.8849227549974199e+04 },
};

static const VSOP87Term VSOP87A_EMB_Z0[154] = {
    { 1.0162500000000000e-06, 5.4224811059700002e+00, 5.5075532386674004e+03 },
    { 8.0460999999999995e-07, 3.8802715791400000e+00, 5.2236939198022001e+03 },
    { 4.3855000000000002e-07, 3.7043034772299999e+00, 2.3528661537717999e+03 },
    { 3.1852999999999998e-07, 3.9999784098600002e+00, 1.5773435424478000e+03 },
    { 1.8150000000000000e-07, 4.9847961332099997e+00, 6.2830758499914000e+03 },
    { 1.6395000000000001e-07, 3.5645616252300001e+00, 5.8564776591153995e+03 },
    { 1.4476999999999999e-07, 3.7025842346500002e+00, 9.4377629348869996e+03 },
    { 1.4324999999999999e-07, 3.4102024693100002e+00, 1.0213285546211000e+04 },
    { 1.1204000000000000e-07, 4.8302149909800001e+00, 1.4143495242430599e+04 },
    { 1.0896000000000000e-07, 2.0802370832100001e+00, 6.8127668150860000e+03 },
    { 9.7160000000000004e-08, 3.4756020506400001e+00, 4.6940029547076001e+03 },
    { 9.2570000000000001e-08, 1.1418264782000001e+00, 6.6208901131878001e+03 },
    { 8.7919999999999996e-08, 4.4489072928000004e+00, 5.7533848848968000e+03 },
    { 8.3369999999999998e-08, 4.9916770604799998e+00, 7.0848967811151997e+03 },
    { 7.2050000000000000e-08, 3.6244159937800000e+00, 5.2969096509459996e+02 },
    { 6.9209999999999996e-08, 4.3255905407300004e+00, 6.2759623029905997e+03 },
    { 5.2560000000000002e-08, 4.4244574452299998e+00, 7.8604193924392002e+03 },
    { 5.1860000000000002e-08, 6.2538453312600000e+00, 1.8073704938650200e+04 },
    { 5.0269999999999999e-08, 2.4972791074899998e+00, 4.7057323075435997e+03 },
    { 4.7540000000000001e-08, 7.1100829533999998e-01, 5.8849268465832001e+03 },
    { 4.5289999999999997e-08, 2.3382774735599998e+00, 6.3093741697912001e+03 },
    { 4.2920000000000003e-08, 1.1003401903600001e+00, 6.6812248533995999e+03 },
    { 4.0609999999999998e-08, 6.0292398984900002e+00, 3.9302096962196001e+03 },
    { 4.0250000000000000e-08, 5.1199028561800004e+00, 1.3367972631106601e+04 },
    { 3.8549999999999998e-08, 1.8233147776000000e+00, 5.4867778431750003e+03 },
    { 3.7980000000000000e-08, 4.4370219832000002e-01, 3.1546870848956000e+03 },
    { 3.4240000000000001e-08, 5.4206519169999998e+00, 6.0697767545533998e+03 },
    { 2.9329999999999999e-08, 5.1242813536299998e+00, 1.0593819301891999e+03 },
    { 2.9119999999999999e-08, 1.9268808770200001e+00, 1.0977078804699000e+04 },
    { 2.6799999999999998e-08, 1.3949735928700000e+00, 2.2003914634869801e+04 },
    { 2.5060000000000001e-08, 6.0484952637000000e-01, 6.4963749454294002e+03 },
    { 2.2350000000000002e-08, 8.1363184041000003e-01, 8.6359420037632008e+03 },
    { 2.2300000000000001e-08, 2.7640782244600000e+00, 1.2036460734888200e+04 },
    { 2.1200000000000001e-08, 4.3069100028499996e+00, 5.6431785636774002e+03 },
    { 1.9449999999999999e-08, 5.6994978917700001e+00, 1.1790629088658799e+04 },
    { 1.8189999999999999e-08, 3.7574800310700001e+00, 3.3406124266997999e+03 },
    { 1.5880000000000001e-08, 2.5704624784000002e-01, 1.7298182327326202e+04 },
    { 1.5810000000000001e-08, 3.1997623094800001e+00, 5.0886288397668004e+03 },
    { 1.5290000000000000e-08, 1.9863934812199999e+00, 3.9814900340819997e+02 },
    { 1.4780000000000001e-08, 2.8180820756900000e+00, 2.5934124331089399e+04 },
    { 1.4240000000000000e-08, 6.2611647231300003e+00, 2.5443144198834002e+03 },
    { 1.4050000000000000e-08, 4.6939487348100002e+00, 7.0585984613153996e+03 },
    { 1.1230000000000000e-08, 2.3860528593599999e+00, 3.7387614301079998e+03 },
    { 1.0859999999999999e-08, 3.1415926535900001e+00, 0.0000000000000000e+00 },
    { 9.8999999999999993e-09, 4.2960069915599997e+00, 9.2255392732830005e+03 },
    { 9.8400000000000008e-09, 5.9678422539199998e+00, 7.0793738568077997e+03 },
    { 9.2300000000000006e-09, 3.0719573643100002e+00, 4.1643119896130002e+03 },
    { 9.0799999999999993e-09, 1.2102885474000000e-01, 1.1015106477334801e+04 },
    { 8.6300000000000002e-09, 5.5119493097000005e-01, 8.4292412664666008e+03 },
    { 8.5000000000000000e-09, 4.2412001609500001e+00, 2.9864334027309000e+04 },
    { 8.2900000000000001e-09, 3.5133697816699998e+00, 1.1506769769793600e+04 },
    { 8.2399999999999997e-09, 1.5005005514700001e+00, 1.0447387839604400e+04 },
    { 8.0999999999999997e-09, 5.1396149859899998e+00, 2.9424634232916001e+03 },
    { 7.6600000000000004e-09, 6.1397570576000005e-01, 7.9629800681639995e+02 },
    { 7.6199999999999997e-09, 2.9002803489200000e+00, 4.7320306273433998e+03 },
    { 7.5100000000000007e-09, 1.6747985016600000e+00, 2.1228392023545799e+04 },
    { 7.4199999999999996e-09, 1.9915913928100000e+00, 2.6087903141574199e+04 },
    { 7.1099999999999996e-09, 1.3859615172399999e+00, 2.1461654164751999e+03 },
    { 7.0800000000000004e-09, 1.9140654236200001e+00, 8.0310922630584000e+03 },
    { 5.9200000000000002e-09, 4.5948150431899997e+00, 4.6904798363585996e+03 },
    { 5.8900000000000001e-09, 2.0219331604700002e+00, 7.7552261132399997e+02 },
    { 5.4999999999999996e-09, 1.6111929896299999e+00, 8.0182093112380005e+02 },
    { 5.1899999999999997e-09, 5.7292503911399999e+00, 8.8273902698748007e+03 },
    { 5.1300000000000003e-09, 2.6073465112799998e+00, 1.2566151699982800e+04 },
    { 5.0400000000000000e-09, 3.2620766915999999e+00, 7.6329432596502002e+03 },
    { 5.0300000000000002e-09, 5.6643313711200003e+00, 3.3794543723528601e+04 },
    { 5.0300000000000002e-09, 4.8580244413400004e+00, 6.2901893969922003e+03 },
    { 4.7600000000000001e-09, 3.6964047244799998e+00, 2.1329909543799999e+02 },
    { 4.7600000000000001e-09, 3.5281047242099999e+00, 1.6496361396202399e+04 },
    { 4.6299999999999999e-09, 1.2498724048300001e+00, 1.5720838784878400e+04 },
    { 4.2000000000000004e-09, 9.7821135132000003e-01, 4.1369104335162001e+03 },
    { 4.1800000000000000e-09, 3.7575999444599999e+00, 5.2308074668030004e+03 },
    { 4.1700000000000003e-09, 8.1920713533000000e-01, 5.2165803728013998e+03 },
    { 4.1100000000000001e-09, 1.0769444525000001e-01, 3.1283887650958000e+03 },
    { 3.9600000000000004e-09, 1.2250771235400000e+00, 6.4384962494255997e+03 },
    { 3.9400000000000000e-09, 5.8149602115599999e+00, 4.2659819087599999e+02 },
    { 3.8700000000000001e-09, 3.0914506141799998e+00, 2.5158601719765400e+04 },
    { 3.8099999999999999e-09, 2.6189074982900000e+00, 7.3424577801805999e+03 },
    { 3.7200000000000000e-09, 1.7225610793799999e+00, 1.4945316173554400e+04 },
    { 3.6300000000000001e-09, 5.7001572045200000e+00, 1.5890728952838001e+03 },
    { 3.6300000000000001e-09, 1.2837643657900000e+00, 6.2068097787158003e+03 },
    { 3.5300000000000000e-09, 4.7005913311000000e+00, 7.2347942562420003e+03 },
    { 3.4900000000000001e-09, 2.9484072007300002e+00, 9.6236882766912004e+03 },
    { 3.3499999999999998e-09, 6.2205796731999996e+00, 1.2168002696574600e+04 },
    { 3.2299999999999998e-09, 5.5025480847099999e+00, 1.1944470102246000e+03 },
    { 3.2200000000000000e-09, 1.2116217880500000e+00, 8.6622403235630009e+03 },
    { 3.0500000000000000e-09, 8.0429352049000002e-01, 3.7724753419748202e+04 },
    { 3.0199999999999999e-09, 2.8113904507199998e+00, 1.7480164130670000e+03 },
    { 2.7600000000000000e-09, 2.3339976792299999e+00, 1.9651048481098002e+04 },
    { 2.6599999999999999e-09, 2.6292628235400000e+00, 7.2386755916000002e+03 },
    { 2.6500000000000002e-09, 6.1035850767099999e+00, 6.8366452528338004e+03 },
    { 2.6299999999999998e-09, 6.2208950123699998e+00, 6.1335126528567998e+03 },
    { 2.6200000000000001e-09, 1.3429726910499999e+00, 1.1769853693166400e+04 },
    { 2.5000000000000001e-09, 4.5645289554700001e+00, 7.4775228602159996e+03 },
    { 2.3199999999999998e-09, 4.1007498211600000e+00, 3.3781426319640002e+02 },
    { 2.2999999999999999e-09, 3.6165162947000001e-01, 5.8635912061161998e+03 },
    { 2.2600000000000001e-09, 2.4234641587299999e+00, 1.1371704689758200e+04 },
    { 2.2499999999999999e-09, 3.1833965260500001e+00, 1.8875525869773999e+04 },
    { 2.1600000000000000e-09, 3.6872127518500002e+00, 5.8493641121146002e+03 },
    { 2.1000000000000002e-09, 4.5069190914400004e+00, 2.9088811415985001e+04 },
    { 2.0599999999999999e-09, 3.7055602798199998e+00, 1.0973555686350001e+04 },
    { 2.0000000000000001e-09, 5.8607315905900004e+00, 4.5350594369244000e+03 },
    { 1.9900000000000000e-09, 5.0518662255500004e+00, 5.4298794682394000e+03 },
    { 1.9200000000000000e-09, 3.8967894386499999e+00, 1.7789845619784999e+04 },
    { 1.8800000000000001e-09, 2.2274612859600000e+00, 4.1654963115967803e+04 },
    { 1.8400000000000001e-09, 3.7546507236900002e+00, 2.3581258177317599e+04 },
    { 1.8300000000000001e-09, 3.1983658440100000e+00, 1.6730463689595799e+04 },
    { 1.8100000000000000e-09, 6.0029478312700002e+00, 4.2923308329503998e+03 },
    { 1.6700000000000000e-09, 6.2734837081299997e+00, 1.0988808157535001e+04 },
    { 1.6000000000000001e-09, 5.2020779518899998e+00, 5.4812549188676003e+03 },
    { 1.5900000000000001e-09, 5.3259084647500003e+00, 7.1135470007999997e+00 },
    { 1.4900000000000000e-09, 1.5530683221699999e+00, 1.0021837280099400e+04 },
    { 1.4700000000000001e-09, 4.6337197140799997e+00, 2.2805735565993600e+04 },
    { 1.4400000000000000e-09, 5.1948637472600003e+00, 2.0426571092422000e+04 },
    { 1.2700000000000000e-09, 5.3106823351500001e+00, 6.3989728631399998e+02 },
    { 1.2700000000000000e-09, 4.7459657420900001e+00, 4.9515382508406998e+04 },
    { 1.2700000000000000e-09, 2.7076699730699998e+00, 1.2352852604544800e+04 },
    { 1.2600000000000000e-09, 1.6944695818400000e+00, 1.5110466119866200e+04 },
    { 1.2400000000000001e-09, 2.9363842184000002e+00, 3.2217200181080800e+04 },
    { 1.2300000000000001e-09, 2.5536108714600001e+00, 1.2592450019782600e+04 },
    { 1.2300000000000001e-09, 5.1599279293400002e+00, 2.7511467873537200e+04 },
    { 1.2100000000000000e-09, 3.1374878278299998e+00, 1.4712317116458000e+04 },
    { 1.2000000000000000e-09, 1.0421149978499999e+00, 6.9158595893046004e+03 },
    { 1.2000000000000000e-09, 5.6063881184600000e+00, 5.6502921106781996e+03 },
    { 1.1900000000000000e-09, 5.9211045898499997e+00, 3.3019021112204602e+04 },
    { 1.1800000000000001e-09, 2.7326811406200000e+00, 1.8849227549974199e+04 },
    { 1.1700000000000001e-09, 3.6506527164000002e+00, 4.5585172812187397e+04 },
    { 1.1300000000000000e-09, 6.0717877672600000e+00, 6.7096740408674004e+03 },
    { 1.1300000000000000e-09, 1.2071149890499999e+00, 1.4919017853754600e+04 },
    { 1.0999999999999999e-09, 3.5120337926300000e+00, 1.8052929543157799e+04 },
    { 1.0800000000000000e-09, 3.7635988075300002e+00, 1.4314168113049800e+04 },
    { 1.0300000000000000e-09, 4.6351984173299998e+00, 1.3498674096587999e+03 },
    { 1.0200000000000000e-09, 3.9738652217100001e+00, 1.0818135286915800e+04 },
    { 1.0000000000000001e-09, 6.0773309710200003e+00, 2.6735945262213201e+04 },
    { 1.0000000000000001e-09, 1.3800278711900000e+00, 2.8286990484861199e+04 },
    { 9.9000000000000010e-10, 4.4577468173200003e+00, 4.9332084403325998e+03 },
    { 9.6999999999999996e-10, 1.0790872479400000e+00, 9.9176968745098002e+03 },
    { 9.5999999999999999e-10, 6.1603810648500001e+00, 2.4356780788641601e+04 },
    { 9.4000000000000006e-10, 4.3387397280000002e+00, 3.6147409877300401e+04 },
    { 9.2999999999999999e-10, 4.3190062025399998e+00, 4.4809650200863398e+04 },
    { 8.6999999999999999e-10, 3.9363781295000000e+00, 1.7654780539749601e+04 },
    { 8.6999999999999999e-10, 2.6630214763999999e-01, 3.1441677569756801e+04 },
    { 8.6000000000000003e-10, 5.2602963825000000e+00, 2.8313288804660999e+04 },
    { 8.3999999999999999e-10, 4.2124195888499996e+00, 2.2779437246193800e+04 },
    { 8.1999999999999996e-10, 4.9520266455500002e+00, 1.0575406682941801e+04 },
    { 8.1999999999999996e-10, 1.6964764707500000e+00, 1.9907450170410000e+03 },
    { 8.0999999999999999e-10, 4.0165524042999996e+00, 1.5925960136327999e+03 },
    { 8.0999999999999999e-10, 2.1642055232200001e+00, 1.3095842665077400e+04 },
    { 7.7999999999999999e-10, 7.2462427282999997e-01, 1.2779450795420800e+04 },
    { 7.5999999999999996e-10, 2.8389755442000002e+00, 1.7260154654690399e+04 },
    { 7.5999999999999996e-10, 6.2274340593500002e+00, 1.3521751441591399e+04 },
    { 7.5000000000000000e-10, 2.2983609564399998e+00, 3.6346210245184002e+03 },
    { 7.5000000000000000e-10, 2.6636787655699998e+00, 1.6200772724501199e+04 },
    { 7.2000000000000000e-10, 1.5582059774699999e+00, 6.2567775301915999e+03 },
};

static const VSOP87Term VSOP87A_EMB_Z1[113] = {
    { 2.2782244200000002e-03, 3.4137250427799999e+00, 6.2830758499914000e+03 },
    { 5.4292819999999998e-05, 0.0000000000000000e+00, 0.0000000000000000e+00 },
    { 1.9031829999999998e-05, 3.3706127096400000e+00, 1.2566151699982800e+04 },
    { 2.3859000000000002e-07, 3.3283626197800000e+00, 1.8849227549974199e+04 },
    { 8.6710000000000000e-08, 1.8228992503500001e+00, 5.2236939198022001e+03 },
    { 7.7669999999999993e-08, 3.8878743583199999e+00, 5.5075532386674004e+03 },
    { 7.0630000000000004e-08, 5.1930316937900001e+00, 2.3528661537717999e+03 },
    { 5.3620000000000000e-08, 2.2242922168899999e+00, 5.2969096509459996e+02 },
    { 3.8940000000000003e-08, 2.1556851717800001e+00, 6.2795527316424004e+03 },
    { 3.8920000000000000e-08, 1.5302106490400000e+00, 6.2865989683403996e+03 },
    { 3.2850000000000000e-08, 4.8804758785200004e+00, 1.0213285546211000e+04 },
    { 2.1649999999999999e-08, 6.2322022354700000e+00, 7.8604193924392002e+03 },
    { 2.0840000000000000e-08, 5.8468206701799996e+00, 1.5773435424478000e+03 },
    { 1.9589999999999999e-08, 1.4227091606200000e+00, 1.2036460734888200e+04 },
    { 1.7129999999999999e-08, 6.0466112349300003e+00, 3.9814900340819997e+02 },
    { 1.5840000000000002e-08, 1.8118525308600000e+00, 5.8564776591153995e+03 },
    { 1.5729999999999999e-08, 2.2007499951799998e+00, 9.4377629348869996e+03 },
    { 1.5030000000000000e-08, 3.9464890271500002e+00, 4.7057323075435997e+03 },
    { 1.4929999999999999e-08, 1.7569518245700000e+00, 5.8849268465832001e+03 },
    { 1.3960000000000001e-08, 5.0057569258000001e-01, 6.3093741697912001e+03 },
    { 1.2979999999999999e-08, 3.7159869370399998e+00, 1.0593819301891999e+03 },
    { 1.1710000000000000e-08, 7.4619691354999995e-01, 6.8127668150860000e+03 },
    { 1.1500000000000000e-08, 2.4394039643499998e+00, 7.7552261132399997e+02 },
    { 1.0410000000000000e-08, 2.9481526353400000e+00, 6.2567775301915999e+03 },
    { 9.8299999999999993e-09, 2.2486586310200001e+00, 6.6812248533995999e+03 },
    { 8.8100000000000008e-09, 1.8610344312400000e+00, 3.9302096962196001e+03 },
    { 7.3200000000000004e-09, 1.6034889896300000e+00, 4.6940029547076001e+03 },
    { 6.9600000000000000e-09, 1.4589695468999999e+00, 1.1790629088658799e+04 },
    { 6.3199999999999997e-09, 4.3420792590800001e+00, 5.7533848848968000e+03 },
    { 5.8299999999999999e-09, 3.9256080257699999e+00, 1.2168002696574600e+04 },
    { 4.8699999999999999e-09, 4.2821167180000001e-02, 1.1506769769793600e+04 },
    { 4.8300000000000001e-09, 5.7498041987899997e+00, 1.7789845619784999e+04 },
    { 4.1700000000000003e-09, 3.6363106462300001e+00, 1.3367972631106601e+04 },
    { 4.0899999999999997e-09, 4.6304770018500001e+00, 7.9629800681639995e+02 },
    { 4.0499999999999999e-09, 1.9755828687200001e+00, 6.2830085396885997e+03 },
    { 4.0499999999999999e-09, 1.5314798988700000e+00, 6.2831431602941902e+03 },
    { 3.6699999999999999e-09, 1.5508138738399999e+00, 1.8073704938650200e+04 },
    { 3.5400000000000002e-09, 3.2849046960899999e+00, 2.5132303399965600e+04 },
    { 3.5300000000000000e-09, 2.4250312120599999e+00, 2.1329909543799999e+02 },
    { 3.3799999999999999e-09, 2.5527444058600000e+00, 7.0585984613153996e+03 },
    { 2.9499999999999999e-09, 2.7539917219399999e+00, 1.4143495242430599e+04 },
    { 2.8800000000000000e-09, 2.9840052109599999e+00, 6.2759623029905997e+03 },
    { 2.8400000000000001e-09, 1.5061211477000000e-01, 3.7387614301079998e+03 },
    { 2.5099999999999998e-09, 2.8131568444799999e+00, 6.2901893969922003e+03 },
    { 2.5000000000000001e-09, 4.2640065085000001e-01, 7.0793738568077997e+03 },
    { 2.2999999999999999e-09, 2.5418212233099999e+00, 7.0848967811151997e+03 },
    { 2.1600000000000000e-09, 2.9505400317900001e+00, 2.2003914634869801e+04 },
    { 2.1200000000000001e-09, 4.7177804680800000e+00, 6.4963749454294002e+03 },
    { 1.8899999999999999e-09, 5.0953473949300001e+00, 1.7298182327326202e+04 },
    { 1.8800000000000001e-09, 3.1740385556100001e+00, 5.4867778431750003e+03 },
    { 1.8400000000000001e-09, 4.2924595834700003e+00, 1.5890728952838001e+03 },
    { 1.6500000000000001e-09, 1.1940716551099999e+00, 5.6431785636774002e+03 },
    { 1.6300000000000000e-09, 2.0062760058200002e+00, 5.2308074668030004e+03 },
    { 1.6200000000000000e-09, 2.9410034299199999e+00, 1.5720838784878400e+04 },
    { 1.6100000000000000e-09, 3.3242199969100001e+00, 6.2833196674748997e+03 },
    { 1.5000000000000000e-09, 1.6529144401600000e+00, 3.1283887650958000e+03 },
    { 1.4900000000000000e-09, 5.2305785241800002e+00, 1.1769853693166400e+04 },
    { 1.4400000000000000e-09, 3.8120375692900001e+00, 6.2623004544989999e+03 },
    { 1.4300000000000000e-09, 5.4836279550000002e+00, 1.0977078804699000e+04 },
    { 1.4300000000000000e-09, 5.9780902135499998e+00, 6.3038512454838001e+03 },
    { 1.3900000000000000e-09, 8.4923405784999995e-01, 6.0697767545533998e+03 },
    { 1.3600000000000001e-09, 4.9703815407200000e+00, 4.1369104335162001e+03 },
    { 1.3500000000000001e-09, 3.6802095001200001e+00, 7.1135470007999997e+00 },
    { 1.3100000000000000e-09, 4.2496139962899999e+00, 6.2820955289231997e+03 },
    { 1.3100000000000000e-09, 5.5405122299500000e+00, 6.2840561710596003e+03 },
    { 1.3100000000000000e-09, 2.2916740462299998e+00, 3.1546870848956000e+03 },
    { 1.3000000000000001e-09, 4.4142806193400004e+00, 6.1276554505572003e+03 },
    { 1.2799999999999999e-09, 4.3548987336499998e+00, 2.5934124331089399e+04 },
    { 1.2300000000000001e-09, 9.3547502050999998e-01, 3.3406124266997999e+03 },
    { 1.2199999999999999e-09, 3.1339468305699998e+00, 1.1944470102246000e+03 },
    { 1.1599999999999999e-09, 1.0720856919899999e+00, 1.2352852604544800e+04 },
    { 1.1300000000000000e-09, 4.4257886267000002e+00, 1.9651048481098002e+04 },
    { 1.1200000000000001e-09, 5.2989937935100002e+00, 2.5443144198834002e+03 },
    { 1.0900000000000000e-09, 3.2193228982100002e+00, 8.0182093112380005e+02 },
    { 1.0700000000000000e-09, 2.9744734352000002e+00, 8.8273902698748007e+03 },
    { 1.0700000000000000e-09, 4.1198538606000001e-01, 5.4812549188676003e+03 },
    { 9.6999999999999996e-10, 5.5095969236500002e+00, 2.9864334027309000e+04 },
    { 9.5999999999999999e-10, 5.2069182539899996e+00, 1.7260154654690399e+04 },
    { 9.4000000000000006e-10, 2.5007575444000002e-01, 2.1228392023545799e+04 },
    { 8.6000000000000003e-10, 2.7871456064500002e+00, 5.2165803728013998e+03 },
    { 8.4999999999999996e-10, 4.7419986896799999e+00, 8.4292412664666008e+03 },
    { 8.1999999999999996e-10, 3.0199455564100002e+00, 2.0426571092422000e+04 },
    { 7.5000000000000000e-10, 7.3893922793000000e-01, 9.2255392732830005e+03 },
    { 7.4000000000000003e-10, 4.5117058459599999e+00, 8.6359420037632008e+03 },
    { 7.4000000000000003e-10, 1.2764736080700001e+00, 4.1643119896130002e+03 },
    { 7.4000000000000003e-10, 2.2018482889500000e+00, 3.7724753419748202e+04 },
    { 7.1000000000000003e-10, 4.7420303374000000e-01, 2.1461654164751999e+03 },
    { 6.9000000000000000e-10, 3.5648387975000002e+00, 4.5350594369244000e+03 },
    { 6.5000000000000003e-10, 1.3969412360500000e+00, 1.2569674818331800e+04 },
    { 6.3999999999999996e-10, 2.0520695755300000e+00, 1.2562628581633800e+04 },
    { 6.0999999999999996e-10, 6.3918772258000001e-01, 3.3794543723528601e+04 },
    { 5.7999999999999996e-10, 5.3673639333600001e+00, 1.1015106477334801e+04 },
    { 5.7999999999999996e-10, 1.6820128443000000e+00, 2.5158601719765400e+04 },
    { 5.6000000000000003e-10, 4.1442459137199998e+00, 5.0886288397668004e+03 },
    { 5.4999999999999996e-10, 1.3724265802800000e+00, 7.3424577801805999e+03 },
    { 5.3000000000000003e-10, 1.4775077284600000e+00, 1.8319536584879599e+04 },
    { 5.1000000000000000e-10, 5.0360061872699999e+00, 5.8635912061161998e+03 },
    { 5.0000000000000003e-10, 4.3058231275200001e+00, 4.2659819087599999e+02 },
    { 5.0000000000000003e-10, 7.7635440261999999e-01, 2.6298319799800002e+01 },
    { 5.0000000000000003e-10, 3.6274083509600001e+00, 4.1654963115967803e+04 },
    { 5.0000000000000003e-10, 1.2043698145999999e-01, 2.9088811415985001e+04 },
    { 4.8999999999999996e-10, 6.2060379183699999e+00, 1.2559038152982001e+04 },
    { 4.8000000000000000e-10, 5.3711874248999996e+00, 7.4775228602159996e+03 },
    { 4.7000000000000003e-10, 3.0276119176999999e-01, 1.1371704689758200e+04 },
    { 4.5000000000000000e-10, 5.1833299801399999e+00, 1.6496361396202399e+04 },
    { 4.5000000000000000e-10, 1.6957701045800000e+00, 6.7025604938666002e+03 },
    { 4.5000000000000000e-10, 6.1098244635699999e+00, 8.0310922630584000e+03 },
    { 4.2000000000000000e-10, 1.1693102263000001e-01, 2.3543230504681789e+04 },
    { 4.0999999999999998e-10, 1.8734492913900000e+00, 1.7480164130670000e+03 },
    { 4.0999999999999998e-10, 5.9186114492400002e+00, 2.3581258177317599e+04 },
    { 4.0000000000000001e-10, 3.0203332393200002e+00, 1.0447387839604400e+04 },
    { 3.9000000000000000e-10, 2.1556295091800002e+00, 4.9332084403325998e+03 },
    { 3.4999999999999998e-10, 3.5201053468899999e+00, 2.6087903141574199e+04 },
};

static const VSOP87Term VSOP87A_EMB_Z2[46] = {
    { 9.7219890000000000e-05, 5.1523372591500003e+00, 6.2830758499914000e+03 },
    { 3.4950099999999998e-06, 3.1415926535900001e+00, 0.0000000000000000e+00 },
    { 6.7136000000000000e-07, 6.4403888586000002e-01, 1.2566151699982800e+04 },
    { 2.1690000000000001e-08, 1.0731442973600001e+00, 1.8849227549974199e+04 },
    { 6.3499999999999998e-09, 3.5198533865599999e+00, 6.2795527316424004e+03 },
    { 3.4699999999999998e-09, 6.1612798486499996e+00, 5.2236939198022001e+03 },
    { 3.2200000000000000e-09, 3.7660897388999998e-01, 6.2865989683403996e+03 },
    { 3.1399999999999999e-09, 2.3708987170500002e+00, 5.5075532386674004e+03 },
    { 2.4300000000000001e-09, 3.9674196139100002e+00, 5.2969096509459996e+02 },
    { 2.4199999999999999e-09, 1.6481007116099999e+00, 7.8604193924392002e+03 },
    { 2.2900000000000002e-09, 5.0869157074000004e-01, 2.3528661537717999e+03 },
    { 1.7400000000000000e-09, 1.3496987899999999e-01, 1.0213285546211000e+04 },
    { 1.6999999999999999e-09, 5.2866829052300002e+00, 6.2567775301915999e+03 },
    { 1.2500000000000000e-09, 3.1379989928100001e+00, 5.8849268465832001e+03 },
    { 1.2300000000000001e-09, 4.1710253062499998e+00, 6.3093741697912001e+03 },
    { 1.1800000000000001e-09, 3.8546605534400000e+00, 1.5773435424478000e+03 },
    { 8.9999999999999999e-10, 4.1333330229799996e+00, 5.7533848848968000e+03 },
    { 8.9000000000000003e-10, 1.6507475780700001e+00, 4.7057323075435997e+03 },
    { 8.7999999999999996e-10, 2.9323065722599999e+00, 1.2036460734888200e+04 },
    { 6.9000000000000000e-10, 1.2134093117100000e+00, 3.9814900340819997e+02 },
    { 6.6999999999999996e-10, 3.0879908580999998e-01, 7.7552261132399997e+02 },
    { 6.6999999999999996e-10, 1.1513104565100001e+00, 2.5132303399965600e+04 },
    { 6.5000000000000003e-10, 6.0261990550700002e+00, 6.8127668150860000e+03 },
    { 6.5000000000000003e-10, 2.7205583785499998e+00, 6.1276554505572003e+03 },
    { 6.2000000000000003e-10, 7.8482367183000001e-01, 6.4384962494255997e+03 },
    { 5.7000000000000000e-10, 1.4901755826100000e+00, 1.1506769769793600e+04 },
    { 5.6000000000000003e-10, 4.5592121475500003e+00, 5.4867778431750003e+03 },
    { 4.7000000000000003e-10, 5.2893166200600001e+00, 7.0793738568077997e+03 },
    { 4.3999999999999998e-10, 3.0969726853499999e+00, 1.1790629088658799e+04 },
    { 4.0000000000000001e-10, 1.7407386137400001e+00, 1.0593819301891999e+03 },
    { 4.0000000000000001e-10, 2.7006998316900002e+00, 1.0988808157535001e+04 },
    { 3.6000000000000000e-10, 3.2483893578000000e-01, 6.6812248533995999e+03 },
    { 3.3000000000000000e-10, 1.2626649600199999e+00, 6.2901893969922003e+03 },
    { 3.3000000000000000e-10, 2.2506706549800000e+00, 6.2759623029905997e+03 },
    { 3.1999999999999998e-10, 2.9059568198600001e+00, 7.9629800681639995e+02 },
    { 2.8999999999999998e-10, 1.9617967046500000e+00, 7.1135470007999997e+00 },
    { 2.5000000000000002e-10, 5.5950308510399998e+00, 7.0585984613153996e+03 },
    { 2.4000000000000000e-10, 5.8369044282699996e+00, 6.2820955289231997e+03 },
    { 2.4000000000000000e-10, 3.9532217979700000e+00, 6.2840561710596003e+03 },
    { 2.4000000000000000e-10, 5.4071554260500001e+00, 1.2168002696574600e+04 },
    { 2.3000000000000001e-10, 4.8662377895200004e+00, 3.7387614301079998e+03 },
    { 2.1999999999999999e-10, 3.4661866224600000e+00, 4.1369104335162001e+03 },
    { 2.1000000000000000e-10, 6.2227500840300003e+00, 6.0697767545533998e+03 },
    { 2.1000000000000000e-10, 3.5804814576199999e+00, 6.4963749454294002e+03 },
    { 1.8999999999999999e-10, 1.6297823228199999e+00, 1.1944470102246000e+03 },
    { 1.8999999999999999e-10, 2.8267905099899999e+00, 1.5890728952838001e+03 },
};

static const VSOP87Term VSOP87A_EMB_Z3[10] = {
    { 2.7607699999999999e-06, 5.9413258729999996e-01, 6.2830758499914000e+03 },
    { 2.5550999999999999e-07, 3.1415926535900001e+00, 0.0000000000000000e+00 },
    { 1.8100000000000000e-08, 1.1612262117000000e-01, 1.2566151699982800e+04 },
    { 1.1300000000000000e-09, 5.6615284722499997e+00, 1.8849227549974199e+04 },
    { 5.6000000000000003e-10, 5.0276555483500003e+00, 6.2795527316424004e+03 },
    { 1.8999999999999999e-10, 5.9900764626100003e+00, 6.2567775301915999e+03 },
    { 1.8000000000000000e-10, 3.8000473456699999e+00, 6.3093741697912001e+03 },
    { 1.8000000000000000e-10, 1.2104925077399999e+00, 6.1276554505572003e+03 },
    { 1.8000000000000000e-10, 2.2973456713700000e+00, 6.4384962494255997e+03 },
    { 1.5000000000000000e-10, 4.7288146726300004e+00, 6.2865989683403996e+03 },
};

static const VSOP87Term VSOP87A_EMB_Z4[4] = {
    { 5.7509999999999997e-08, 2.2706909089199998e+00, 6.2830758499914000e+03 },
    { 1.3049999999999999e-08, 0.0000000000000000e+00, 0.0000000000000000e+00 },
    { 6.0000000000000000e-10, 4.2671205084599997e+00, 1.2566151699982800e+04 },
    { 6.0000000000000000e-11, 4.1391401195300004e+00, 1.8849227549974199e+04 },
};

static const VSOP87Term VSOP87A_EMB_Z5[2] = {
    { 1.1400000000000000e-09, 4.3116915692299997e+00, 6.2830758499914000e+03 },
    { 3.4999999999999998e-10, 0.0000000000000000e+00, 0.0000000000000000e+00 },
};

static const VSOP87Block VSOP87A_EMB_X[6] = {
    { VSOP87A_EMB_X0, 793 },
    { VSOP87A_EMB_X1, 478 },
    { VSOP87A_EMB_X2, 185 },
    { VSOP87A_EMB_X3, 18 },
    { VSOP87A_EMB_X4, 10 },
    { VSOP87A_EMB_X5, 6 },
};

static const VSOP87Block VSOP87A_EMB_Y[6] = {
    { VSOP87A_EMB_Y0, 804 },
    { VSOP87A_EMB_Y1, 482 },
    { VSOP87A_EMB_Y2, 184 },
    { VSOP87A_EMB_Y3, 17 },
    { VSOP87A_EMB_Y4, 10 },
    { VSOP87A_EMB_Y5, 6 },
};

static const VSOP87Block VSOP87A_EMB_Z[6] = {
    { VSOP87A_EMB_Z0, 154 },
    { VSOP87A_EMB_Z1, 113 },
    { VSOP87A_EMB_Z2, 46 },
    { VSOP87A_EMB_Z3, 10 },
    { VSOP87A_EMB_Z4, 4 },
    { VSOP87A_EMB_Z5, 2 },
};

static const int32 VSOP87A_EMB_TRUNC_COUNT[11][3][6] = {
    {
        { 793, 478, 185, 18, 10, 6 },
        { 804, 482, 184, 17, 10, 6 },
        { 154, 113, 46, 10, 4, 2 },
    },
    {
        { 254, 105, 31, 6, 2, 0 },
        { 253, 103, 31, 6, 2, 0 },
        { 44, 24, 4, 3, 2, 0 },
    },
    {
        { 150, 49, 10, 4, 0, 0 },
        { 155, 48, 10, 4, 0, 0 },
        { 19, 8, 3, 2, 1, 0 },
    },
    {
        { 108, 33, 6, 2, 0, 0 },
        { 109, 33, 6, 3, 0, 0 },
        { 10, 4, 3, 2, 0, 0 },
    },
    {
        { 59, 9, 4, 2, 0, 0 },
        { 58, 9, 4, 2, 0, 0 },
        { 2, 3, 3, 1, 0, 0 },
    },
    {
        { 38, 6, 3, 1, 0, 0 },
        { 40, 6, 3, 1, 0, 0 },
        { 1, 3, 2, 1, 0, 0 },
    },
    {
        { 20, 4, 3, 0, 0, 0 },
        { 21, 3, 3, 0, 0, 0 },
        { 0, 3, 1, 0, 0, 0 },
    },
    {
        { 10, 4, 2, 0, 0, 0 },
        { 10, 3, 3, 0, 0, 0 },
        { 0, 3, 1, 0, 0, 0 },
    },
    {
        { 4, 2, 0, 0, 0, 0 },
        { 4, 2, 1, 0, 0, 0 },
        { 0, 2, 1, 0, 0, 0 },
    },
    {
        { 4, 2, 0, 0, 0, 0 },
        { 4, 2, 0, 0, 0, 0 },
        { 0, 1, 0, 0, 0, 0 },
    },
    {
        { 3, 1, 0, 0, 0, 0 },
        { 3, 0, 0, 0, 0, 0 },
        { 0, 1, 0, 0, 0, 0 },
    },
};

