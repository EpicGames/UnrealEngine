// Copyright Epic Games, Inc. All Rights Reserved.

// Auto-generated VSOP87A header for JUPITER (rectangular XYZ, J2000)
// Source: IMCCE/BDL VSOP87A.jupiter file
// Generated by vsop87a_header_gen_imcce.py

#pragma once

#include "VSOP87A_Common.h"

static const VSOP87Term VSOP87A_JUPITER_X0[1055] = {
    { 5.1966347011399998e+00, 5.9945082355000001e-01, 5.2969096509459996e+02 },
    { 3.6662642320000000e-01, 3.1415926535900001e+00, 0.0000000000000000e+00 },
    { 1.2593937922000001e-01, 9.4911583701000002e-01, 1.0593819301891999e+03 },
    { 1.5006720560000001e-02, 7.3175134610000003e-01, 5.2257741809380002e+02 },
    { 1.4762245780000001e-02, 3.6173692112200002e+00, 5.3680451209540001e+02 },
    { 4.5775273599999999e-03, 1.2988370075500000e+00, 1.5890728952838001e+03 },
    { 3.8597537500000001e-03, 2.0122991068700000e+00, 1.0309277421860000e+02 },
    { 3.0168979800000001e-03, 5.1737255114799998e+00, 7.1135470007999997e+00 },
    { 1.9402540500000000e-03, 5.0258036399600003e+00, 4.2659819087599999e+02 },
    { 1.5067879300000001e-03, 6.1200302773899997e+00, 1.1020632121940000e+02 },
    { 1.4486764100000000e-03, 5.5598057707999997e+00, 6.3278373931320004e+02 },
    { 1.3422699600000000e-03, 8.7648567011000000e-01, 2.1329909543799999e+02 },
    { 1.1420156200000000e-03, 1.5670842689999999e-02, 1.1624747044077999e+03 },
    { 1.0349464100000000e-03, 6.1932476911999998e+00, 1.0522683831884001e+03 },
    { 7.2095575000000005e-04, 3.9611743064299998e+00, 1.0664954771900000e+03 },
    { 6.8284021000000002e-04, 3.4405112263099999e+00, 8.4608283475120004e+02 },
    { 5.9486083000000005e-04, 4.4576937435800001e+00, 9.4917560896980001e+02 },
    { 4.7092251000000000e-04, 1.4461263645100000e+00, 4.1948464387519999e+02 },
    { 3.0623416999999998e-04, 2.9913232142699999e+00, 2.0618554843720000e+02 },
    { 2.6613458999999998e-04, 4.8516990649400000e+00, 3.2350541665740002e+02 },
    { 1.9727456999999999e-04, 1.6489162621300000e+00, 2.1187638603783998e+03 },
    { 1.6481594000000000e-04, 1.9515005656800000e+00, 3.1639186965660002e+02 },
    { 1.6101974000000001e-04, 8.7973155979999995e-01, 5.1546387109299997e+02 },
    { 1.5192516000000000e-04, 6.2582012790599997e+00, 7.3587651353180001e+02 },
    { 1.4209486999999999e-04, 2.0776962141299999e+00, 7.4299006053259995e+02 },
    { 1.2155285000000000e-04, 3.7522992499900001e+00, 5.2575881183150000e+02 },
    { 1.1996271000000000e-04, 5.8568573728999995e-01, 5.3362311835770004e+02 },
    { 1.1423199000000000e-04, 3.4814610892900002e+00, 5.4391805909619995e+02 },
    { 8.6941239999999996e-05, 3.8262009410999998e-01, 1.6921656695024001e+03 },
    { 8.4685560000000005e-05, 3.4724875173899998e+00, 6.3989728631399998e+02 },
    { 8.2233019999999998e-05, 5.5668044714300002e+00, 1.4788665740644001e+03 },
    { 7.6558670000000001e-05, 1.4178789085999999e-01, 1.5819593482830001e+03 },
    { 7.5164699999999995e-05, 9.2896448411999999e-01, 1.2655674786264001e+03 },
    { 7.4275170000000001e-05, 5.9838075119600003e+00, 9.5628915597059995e+02 },
    { 5.7773109999999998e-05, 5.0372616562800001e+00, 1.4227094001599999e+01 },
    { 5.3187910000000000e-05, 1.1049401634900000e+00, 5.2650957135689998e+02 },
    { 5.2184919999999997e-05, 3.2323512922400002e+00, 5.3287235883230005e+02 },
    { 4.6226849999999999e-05, 3.7581708609900000e+00, 1.3757737998458001e+03 },
    { 4.5694439999999997e-05, 2.1508728170999998e+00, 9.5979227217800002e+01 },
    { 3.9398640000000002e-05, 4.3089268751100001e+00, 1.5961864422845999e+03 },
    { 2.9527119999999999e-05, 3.8598848394699998e+00, 3.0927832265580003e+02 },
    { 2.8579349999999999e-05, 6.0111847373899998e+00, 1.1731986822019999e+02 },
    { 2.6751120000000000e-05, 3.1872344909399999e+00, 1.1695882514085999e+03 },
    { 2.4400940000000002e-05, 4.2399576570199997e+00, 4.3371173787679999e+02 },
    { 2.4382570000000001e-05, 3.8880846382200001e+00, 2.2041264243879999e+02 },
    { 2.3864250000000001e-05, 5.9635499432400003e+00, 1.0451548361876000e+03 },
    { 1.9390600000000000e-05, 5.9188341286400004e+00, 6.2567019231239999e+02 },
    { 1.8700970000000001e-05, 5.2019313301000003e-01, 1.1553611574070001e+03 },
    { 1.6633309999999999e-05, 1.9401062919400001e+00, 8.3896928775039999e+02 },
    { 1.6314999999999999e-05, 4.4191038346599996e+00, 9.4206206196899996e+02 },
    { 1.6112290000000001e-05, 5.4932497484500002e+00, 7.4781598567299994e+01 },
    { 1.4516670000000001e-05, 5.7611270603999998e+00, 8.5319638175199998e+02 },
    { 1.3612860000000000e-05, 1.3479274883700001e+00, 1.3686602528450001e+03 },
    { 1.0882840000000000e-05, 1.1340610418999999e+00, 5.2724328453980002e+02 },
    { 1.0806430000000001e-05, 3.2052836257299999e+00, 5.3213864564940002e+02 },
    { 1.0335700000000000e-05, 8.9072087889999998e-02, 1.7952584437210000e+03 },
    { 9.9148099999999994e-06, 3.0860950581400002e+00, 1.2726810256271999e+03 },
    { 9.5918800000000007e-06, 1.3478949420999999e+00, 1.4956319713459999e+02 },
    { 9.4199699999999992e-06, 1.5761290265600001e+00, 4.1237109687440000e+02 },
    { 9.3478900000000000e-06, 3.1115134163299998e+00, 1.9907200143640000e+02 },
    { 9.3450400000000006e-06, 1.9993880133599999e+00, 2.6484548254729998e+03 },
    { 9.3260000000000002e-06, 5.3459678298200002e+00, 3.8012776796000003e+02 },
    { 8.5882899999999997e-06, 3.7131687955700001e+00, 5.2964278098479997e+02 },
    { 8.5873399999999993e-06, 6.2779464689999998e-01, 5.2973914920439995e+02 },
    { 8.4004500000000002e-06, 4.1439092407700002e+00, 3.9321532631000000e+00 },
    { 7.4773499999999997e-06, 4.7095456132500004e+00, 3.3061896365820002e+02 },
    { 6.9331099999999996e-06, 1.3375428932000000e+00, 1.0633140834523001e+03 },
    { 6.8070700000000000e-06, 2.9377240206999999e-01, 5.2872775724810003e+02 },
    { 6.7881899999999998e-06, 4.0466990313100002e+00, 5.3065417294110000e+02 },
    { 6.6571100000000000e-06, 6.0844626248100004e+00, 2.0085575391590000e+03 },
    { 6.4798199999999999e-06, 5.1250809938200002e+00, 9.8460033162189995e+02 },
    { 6.0676100000000000e-06, 1.1410967423000000e-01, 2.1116503133776000e+03 },
    { 5.7294300000000001e-06, 7.4312663769999998e-01, 2.2218566345969998e+03 },
    { 5.6430400000000003e-06, 4.0633134184099999e+00, 1.0554497769261000e+03 },
    { 5.3958000000000001e-06, 5.3145833375500002e+00, 3.8133035637799999e+01 },
    { 5.3762700000000001e-06, 6.7272668190999996e-01, 1.6850521225016000e+03 },
    { 5.0830299999999998e-06, 6.2450579764400000e+00, 7.2876296653099996e+02 },
    { 4.6019099999999998e-06, 3.8264027775499998e+00, 1.0736090241908000e+03 },
    { 4.1331000000000003e-06, 4.5268901273199997e+00, 4.5490936652729999e+02 },
    { 4.0530400000000002e-06, 2.9879735364400002e+00, 9.0981873305459999e+02 },
    { 3.8947000000000002e-06, 6.0599951523100000e+00, 6.0447256366190004e+02 },
    { 3.8550800000000000e-06, 4.9113719148300001e+00, 3.1813937377000001e+00 },
    { 3.3467500000000002e-06, 9.5326265643999997e-01, 7.6266071275599998e+01 },
    { 3.0410499999999999e-06, 1.7537674128200000e+00, 6.2830758499914000e+03 },
    { 3.0321700000000000e-06, 5.0648963707199997e+00, 5.2916970023279998e+02 },
    { 3.0311399999999999e-06, 4.0724974639699996e+00, 1.9054647649404001e+03 },
    { 3.0282600000000000e-06, 5.5601311783099998e+00, 5.3021222995640005e+02 },
    { 2.8728799999999999e-06, 2.9563537006699998e+00, 3.0534616939270001e+02 },
    { 2.7767900000000000e-06, 6.5883071470999999e-01, 1.4859801210651999e+03 },
    { 2.5445800000000000e-06, 3.5845521027400000e+00, 1.0625633239269000e+03 },
    { 2.4484400000000000e-06, 1.5929825386900001e+00, 1.2584539316256000e+03 },
    { 2.3557799999999998e-06, 9.2416895439000002e-01, 1.0562005364515001e+03 },
    { 2.2706400000000002e-06, 4.6576521585600004e+00, 2.1258774073792001e+03 },
    { 2.2485400000000000e-06, 3.5009976525300002e+00, 1.6992792165031999e+03 },
    { 2.1761300000000002e-06, 5.6832681492499999e+00, 4.5342489381899998e+02 },
    { 2.1060900000000002e-06, 4.0213459186199998e+00, 4.9033408917940000e+02 },
    { 2.0831099999999999e-06, 3.3928705911299999e+00, 1.4395096981491999e+03 },
    { 2.0602600000000002e-06, 1.0319238589299999e+00, 1.8983512179396000e+03 },
    { 1.9485600000000000e-06, 5.3045982096199999e+00, 1.0212488945514000e+03 },
    { 1.9201800000000001e-06, 3.1042772808000002e-01, 5.6904784100979998e+02 },
    { 1.8968500000000001e-06, 1.0412469804700000e+00, 5.0835032409220003e+02 },
    { 1.8874500000000000e-06, 5.7146334684399998e+00, 9.9160620955499994e+01 },
    { 1.7908200000000000e-06, 6.1022200295099998e+00, 5.2677020378780003e+02 },
    { 1.7700500000000001e-06, 4.5232305251599998e+00, 5.3261172640140001e+02 },
    { 1.7686100000000000e-06, 3.1752591889300001e+00, 1.0213285546211000e+04 },
    { 1.7484700000000001e-06, 1.2155027909700000e+00, 1.4717530270636000e+03 },
    { 1.6556399999999999e-06, 3.6634445944600000e+00, 2.2434479570190001e+02 },
    { 1.6468900000000000e-06, 3.4771484277400000e+00, 5.2894020556919997e+02 },
    { 1.6458500000000001e-06, 8.6380555242000001e-01, 5.3044172461999995e+02 },
    { 1.6293000000000000e-06, 4.9071349850000001e-01, 2.0014439921582000e+03 },
    { 1.6287299999999999e-06, 5.6256179898000003e-01, 8.3503713448730002e+02 },
    { 1.5804599999999999e-06, 4.3280501438299996e+00, 5.2549817940059995e+02 },
    { 1.5547800000000000e-06, 1.4482436959999999e-02, 5.3388375078859997e+02 },
    { 1.5462600000000001e-06, 1.2624250143000000e+00, 5.1939602435610004e+02 },
    { 1.5359300000000000e-06, 2.2003224008400002e+00, 1.1045700263900001e+01 },
    { 1.5308900000000000e-06, 2.0229760602400000e+00, 1.4844727083000000e+00 },
    { 1.4529699999999999e-06, 4.0262117568400004e+00, 3.0216477565500003e+02 },
    { 1.4417300000000001e-06, 1.3317668677900001e+00, 1.7881448967202000e+03 },
    { 1.4246300000000001e-06, 3.7554524965199998e+00, 2.2752618943959999e+02 },
    { 1.3973500000000001e-06, 3.3551792319199998e+00, 6.4701083331480004e+02 },
    { 1.3567000000000001e-06, 3.8001292347800000e+00, 5.4073666535849998e+02 },
    { 1.3422600000000000e-06, 1.0239699641000000e-01, 1.5748458012822000e+03 },
    { 1.2404900000000000e-06, 3.3883431101600001e+00, 9.8311585891360005e+02 },
    { 1.2379499999999999e-06, 3.3419948289299999e+00, 5.2502509864859996e+02 },
    { 1.2280500000000000e-06, 4.9063034029499999e+00, 5.2427433912319998e+02 },
    { 1.2181499999999999e-06, 3.7887022120500000e+00, 5.2406189080210004e+02 },
    { 1.2089700000000000e-06, 5.7156535912699997e+00, 5.3510759106600005e+02 },
    { 1.2057800000000000e-06, 3.0830014641600001e+00, 5.3998590583309999e+02 },
    { 1.2003200000000001e-06, 5.5319601435999999e-01, 5.3532003938710000e+02 },
    { 1.1615799999999999e-06, 9.9764796582000004e-01, 5.3435683154059996e+02 },
    { 1.1434400000000000e-06, 3.3397861349500002e+00, 7.5010360753340001e+02 },
    { 1.1064200000000000e-06, 3.2527165996000001e-01, 2.3249494088155998e+03 },
    { 1.0465400000000001e-06, 3.3289302430399998e+00, 5.5103160609700001e+02 },
    { 1.0442999999999999e-06, 3.7655169905000001e+00, 5.2953090640020002e+02 },
    { 1.0442999999999999e-06, 5.7516865787000004e-01, 5.2985102378900001e+02 },
    { 1.0306999999999999e-06, 4.4145575303099998e+00, 9.6340270297140000e+02 },
    { 9.6206000000000003e-07, 4.8871670957899997e+00, 2.1340641002400002e+01 },
    { 9.5403999999999993e-07, 5.9381725625400001e+00, 3.5424722652100002e+01 },
    { 9.3824999999999997e-07, 3.2372088223000000e+00, 1.8023719907218001e+03 },
    { 9.3157000000000002e-07, 2.7249765230000000e-02, 1.3828873468465999e+03 },
    { 9.1908000000000000e-07, 3.0337090867400001e+00, 4.4082528487759998e+02 },
    { 8.7496000000000005e-07, 1.6667978361900000e+00, 8.3185574074960005e+02 },
    { 8.2322000000000004e-07, 2.5831669607699999e+00, 5.3117543780289998e+02 },
    { 7.9197999999999998e-07, 1.7307501529600000e+00, 5.2820649238630006e+02 },
    { 7.8833000000000002e-07, 5.7154199861799997e+00, 4.1555249061209997e+02 },
    { 7.4038000000000002e-07, 4.4199400812100000e+00, 1.7394221952280000e+02 },
    { 7.3547999999999997e-07, 7.3533236562000004e-01, 2.2147430875962000e+03 },
    { 7.2877999999999999e-07, 4.4613274764000002e+00, 3.9356875915200000e+01 },
    { 7.2139000000000003e-07, 2.6833059020000000e-01, 5.1864526483069994e+02 },
    { 7.2096000000000002e-07, 4.0226131862500001e+00, 5.2990341341570002e+02 },
    { 7.2096000000000002e-07, 3.1807246212000001e-01, 5.2947851677350002e+02 },
    { 7.1452000000000002e-07, 5.6720235418299998e+00, 5.1153171782990000e+02 },
    { 6.8706999999999997e-07, 4.9855348886000002e+00, 5.4785021235930003e+02 },
    { 6.7372000000000003e-07, 1.0282583203800000e+00, 1.3647280995819001e+03 },
    { 6.6064999999999998e-07, 5.9752988902000004e-01, 2.3056457082540001e+02 },
    { 6.5957999999999998e-07, 2.9867934562999998e-01, 2.3178358618148000e+03 },
    { 6.5451000000000000e-07, 1.0200472970900001e+00, 2.1045367663768002e+03 },
    { 6.0119999999999996e-07, 3.7802249596999999e+00, 1.0618296107440001e+03 },
    { 6.0011000000000004e-07, 1.6427439870800000e+00, 1.3615467058442000e+03 },
    { 5.9167000000000000e-07, 8.9478754329999999e-01, 4.9426624244250002e+02 },
    { 5.8551000000000001e-07, 2.3129899922199999e+00, 1.0578974574808999e+03 },
    { 5.7133000000000002e-07, 7.2237577019999999e-01, 9.2047073954699997e+01 },
    { 5.5535999999999996e-07, 4.7922393626000002e+00, 6.4382943957709995e+02 },
    { 5.5262000000000003e-07, 2.2782584079600001e+00, 1.0702492748170000e+02 },
    { 5.2715000000000001e-07, 1.4837917576499999e+00, 1.0569342496344000e+03 },
    { 5.2147999999999998e-07, 1.9872962672900001e+00, 1.0483362299252999e+03 },
    { 5.2104999999999997e-07, 4.6258894296499999e+00, 9.6022130923370003e+02 },
    { 5.1793000000000003e-07, 3.4665889589900001e+00, 5.6511568774670002e+02 },
    { 5.1559999999999998e-07, 1.6466949905399999e+00, 5.1471311156759998e+02 },
    { 5.0617000000000004e-07, 3.1750972006000000e-01, 2.6413412784722000e+03 },
    { 5.0111000000000002e-07, 3.1352877329700002e+00, 3.8724131496080003e+02 },
    { 4.9747999999999998e-07, 1.8815254261700001e+00, 6.7925416222920001e+02 },
    { 4.9711000000000004e-07, 2.6941066121900001e+00, 5.4466881862160005e+02 },
    { 4.9180999999999995e-07, 6.2053150174400002e+00, 3.3406124266997999e+03 },
    { 4.9105000000000005e-07, 4.1509634561500004e+00, 5.2354062594029995e+02 },
    { 4.8551000000000001e-07, 5.9431633641500001e+00, 1.0380412891868000e+03 },
    { 4.8294000000000001e-07, 5.4709133882299996e+00, 1.0627416795630000e+02 },
    { 4.7067000000000002e-07, 2.3523485756100002e+00, 3.1781457905676002e+03 },
    { 4.6172999999999999e-07, 5.5067659751600004e+00, 2.4280421830342002e+03 },
    { 4.6144999999999999e-07, 3.7458977052900000e+00, 1.5128068240082000e+03 },
    { 4.5698000000000001e-07, 2.3070104258999999e-01, 2.5382485042536000e+03 },
    { 4.5597999999999998e-07, 4.9167989085899997e+00, 3.1245971639350000e+02 },
    { 4.4448999999999998e-07, 1.9000018391000001e-01, 5.3584130424889997e+02 },
    { 4.4411000000000002e-07, 4.4063513865699999e+00, 7.6025553592000006e+02 },
    { 4.3977999999999998e-07, 1.0880077740700000e+00, 4.3053034413910001e+02 },
    { 4.3673999999999998e-07, 3.6671900326000001e-01, 2.4476805547999998e+00 },
    { 4.2076000000000002e-07, 3.4481972117500002e+00, 7.0849445304200003e+01 },
    { 4.1687000000000000e-07, 9.7719980018999997e-01, 1.0594301142990000e+03 },
    { 4.1687000000000000e-07, 4.0625739797299998e+00, 1.0593337460794000e+03 },
    { 4.1540999999999999e-07, 5.4468090082699998e+00, 1.5142912967165000e+03 },
    { 4.1451999999999998e-07, 1.7760607115200000e+00, 1.5930050485469001e+03 },
    { 4.0915999999999999e-07, 4.6246615688499997e+00, 1.0294940738499999e+01 },
    { 4.0751999999999999e-07, 1.8388543996000001e-01, 1.8159247264699999e+01 },
    { 4.0142000000000000e-07, 1.5361300942100000e+00, 6.1855664531160005e+02 },
    { 3.9658000000000002e-07, 2.0024825896900000e+00, 1.1482476104062000e+03 },
    { 3.7509000000000001e-07, 1.1299218312799999e+00, 2.7515475996916002e+03 },
    { 3.6843999999999998e-07, 5.7375508146499996e+00, 5.2943033266370003e+02 },
    { 3.6843999999999998e-07, 4.8863201408899997e+00, 5.2995159752550001e+02 },
    { 3.6613000000000001e-07, 4.1893520899499999e+00, 9.3494851496820002e+02 },
    { 3.6012000000000002e-07, 2.2010977997299999e+00, 1.8912376709388000e+03 },
    { 3.5445000000000000e-07, 7.7567886722000001e-01, 5.2262560220360001e+02 },
    { 3.4849999999999998e-07, 4.4956426353300003e+00, 1.0603451380357001e+03 },
    { 3.4397999999999999e-07, 4.9088065652299999e+00, 1.4244965013379999e+02 },
    { 3.4069000000000002e-07, 1.1640484272700000e+00, 6.7214061522839995e+02 },
    { 3.3966000000000001e-07, 6.2036887673100001e+00, 8.8543971066639995e+02 },
    { 3.3780000000000002e-07, 2.9528775598500001e+00, 6.2885158605009997e+02 },
    { 3.3061000000000002e-07, 6.0846915979000000e+00, 5.6782400073240001e+02 },
    { 3.3000000000000002e-07, 3.8459518675700002e+00, 5.2252923398400003e+02 },
    { 3.2992999999999999e-07, 6.4338422668999995e-01, 1.0584187223427000e+03 },
    { 3.2614999999999999e-07, 5.8371084319400000e+00, 1.2443341522100000e+02 },
    { 3.1356000000000001e-07, 8.3981449550000004e-02, 4.3764389113990001e+02 },
    { 3.0582999999999998e-07, 4.7111818765100004e+00, 1.0239572075371000e+03 },
    { 3.0459000000000000e-07, 4.3856993256300001e+00, 1.5851407420206999e+03 },
    { 3.0433999999999998e-07, 3.2988979626399999e+00, 2.1723124870109999e+02 },
    { 2.9909000000000001e-07, 3.5591889990999999e+00, 5.3675632798560002e+02 },
    { 2.9411999999999999e-07, 4.6822577817199997e+00, 4.2266603761290003e+02 },
    { 2.8933000000000001e-07, 1.1979811005200001e+00, 1.2125202148330000e+02 },
    { 2.8696000000000002e-07, 5.7293204229799999e+00, 7.4692221379570003e+02 },
    { 2.8393999999999999e-07, 9.9780388701000000e-01, 2.4209286360333999e+03 },
    { 2.7888999999999998e-07, 3.8593538990000002e-02, 2.6342277314714001e+03 },
    { 2.7674999999999999e-07, 5.3383199942799999e+00, 5.2841894070740000e+02 },
    { 2.7585999999999998e-07, 5.2855101124899999e+00, 5.3096298948180004e+02 },
    { 2.7496999999999998e-07, 4.9110736349000000e-01, 5.3685269620520000e+02 },
    { 2.7098000000000001e-07, 9.3568939987999999e-01, 4.9744763618019999e+02 },
    { 2.6906000000000000e-07, 6.1243856321800001e+00, 2.9912639426919998e+02 },
    { 2.6726999999999998e-07, 3.2046438007700000e+00, 1.1585425511446999e+03 },
    { 2.5670999999999997e-07, 5.7144448089499997e+00, 6.3671589257630001e+02 },
    { 2.5017999999999998e-07, 5.1390903222200004e+00, 8.6030992875280003e+02 },
    { 2.4825000000000002e-07, 1.5973065560999999e-01, 1.1341635287565000e+03 },
    { 2.4685000000000002e-07, 4.2205604146900004e+00, 6.2173803904930003e+02 },
    { 2.4660000000000000e-07, 7.1366313976999995e-01, 2.5311349572528002e+03 },
    { 2.4445000000000000e-07, 2.5067009049000002e+00, 9.9911380480899993e+01 },
    { 2.4004000000000002e-07, 5.6780671906300002e+00, 5.2309868295560000e+02 },
    { 2.3213000000000000e-07, 3.4319539579099998e+00, 5.6193429400900004e+02 },
    { 2.2303999999999999e-07, 4.1915862680500000e+00, 4.9155792945680003e+02 },
    { 2.2284000000000000e-07, 4.6679621200999999e-01, 5.2161421024729998e+02 },
    { 2.1869999999999999e-07, 4.9449021714999999e+00, 5.3628324723360004e+02 },
    { 2.1220000000000001e-07, 4.1260454699999997e-02, 4.9575071515079998e+02 },
    { 2.0953000000000000e-07, 2.5786998768100000e+00, 4.5884151979040001e+02 },
    { 2.0924000000000001e-07, 3.0357950630000002e-02, 1.1664068576709001e+03 },
    { 2.0802000000000001e-07, 4.9773659312899996e+00, 8.8865680217000005e+01 },
    { 1.9777999999999999e-07, 9.3154236593999995e-01, 5.0531194270639998e+02 },
    { 1.9711000000000000e-07, 2.0281060982299999e+00, 4.2341679713830001e+02 },
    { 1.9591000000000001e-07, 4.1743182349200003e+00, 1.6032999892854000e+03 },
    { 1.9507000000000001e-07, 1.9102901731399999e+00, 1.0987388061044001e+03 },
    { 1.9198000000000000e-07, 4.2979254519600003e+00, 5.6363121503840000e+02 },
    { 1.9149000000000000e-07, 2.1052830889799998e+00, 3.2743756992049998e+02 },
    { 1.8867000000000001e-07, 2.1087517108600000e+00, 5.6290742925000004e+00 },
    { 1.8715000000000000e-07, 7.3543500526000005e-01, 5.2347118997109999e+02 },
    { 1.8655000000000001e-07, 4.9789747062299998e+00, 1.1514290041439001e+03 },
    { 1.8509000000000000e-07, 3.4168229215900001e+00, 5.5406998748280000e+02 },
    { 1.8386999999999999e-07, 3.7528491518500000e+00, 1.9692006632437999e+03 },
    { 1.8237000000000001e-07, 4.5905654908400004e+00, 9.5310776223289997e+02 },
    { 1.8075000000000000e-07, 1.9992886268600001e+00, 6.0054041039879996e+02 },
    { 1.8008999999999999e-07, 3.9663935479700001e+00, 2.4379022388199999e+01 },
    { 1.7922000000000001e-07, 3.6081104203500001e+00, 5.3591074021810005e+02 },
    { 1.7873000000000001e-07, 3.8667847080200000e+00, 5.3776771994190005e+02 },
    { 1.7585999999999999e-07, 2.0261468091600001e+00, 2.2076295405954002e+03 },
    { 1.7310000000000001e-07, 3.2737870127800002e+00, 1.9195845443560000e+02 },
    { 1.7177000000000001e-07, 2.0343780561700000e+00, 1.4646394800628000e+03 },
    { 1.7116999999999999e-07, 4.7977883008000000e-01, 1.1413847448250000e+02 },
    { 1.6936000000000000e-07, 4.2657132903799999e+00, 2.4351557300350000e+03 },
    { 1.6624000000000001e-07, 3.8536110196200002e+00, 2.2289701815978001e+03 },
    { 1.6271999999999999e-07, 3.9078616812600000e+00, 5.2698265210889997e+02 },
    { 1.6175999999999999e-07, 2.5264460385500001e+00, 4.6658664459999999e+00 },
    { 1.6170000000000000e-07, 4.3269852872999998e-01, 5.3239927808029995e+02 },
    { 1.6159000000000001e-07, 2.1846540542000001e-01, 1.0490869894507000e+03 },
    { 1.6117000000000001e-07, 1.1721687534700000e+00, 1.6779385755008000e+03 },
    { 1.6073000000000001e-07, 5.9289977862999999e-01, 3.2243328914400003e+01 },
    { 1.5853000000000000e-07, 3.0258948312600000e+00, 5.0137978944330001e+02 },
    { 1.5548000000000001e-07, 5.7771044039400001e+00, 6.3735898303399999e+01 },
    { 1.5363000000000001e-07, 1.1045612771100000e+00, 3.8161224066829999e+02 },
    { 1.5356000000000001e-07, 5.6968257849499997e+00, 3.3940249943799998e+01 },
    { 1.5272999999999999e-07, 3.5231511102000002e+00, 4.6053844081979997e+02 },
    { 1.5211999999999999e-07, 9.5366780101000004e-01, 9.8853248488500003e+02 },
    { 1.5092000000000000e-07, 1.6220117053600001e+00, 2.0156710861598001e+03 },
    { 1.5036999999999999e-07, 4.5129668855400000e+00, 1.5578297225809999e+02 },
    { 1.4740999999999999e-07, 1.0933580125600000e+00, 9.4524345570670005e+02 },
    { 1.4712000000000001e-07, 5.4138626056500003e+00, 1.0588606653274001e+03 },
    { 1.4712000000000001e-07, 5.9090964814499998e+00, 1.0599031950510000e+03 },
    { 1.4583000000000000e-07, 3.9901021568099999e+00, 2.8311175651300001e+01 },
    { 1.4529000000000000e-07, 4.8595900617799996e+00, 1.0704276304530999e+03 },
    { 1.4496000000000000e-07, 1.3386429607699999e+00, 5.5800214074589996e+02 },
    { 1.4315000000000001e-07, 5.6409749122799999e+00, 1.5509398596460001e+03 },
    { 1.4298000000000000e-07, 4.5856078156700004e+00, 1.1735204046716999e+03 },
    { 1.4250000000000001e-07, 3.2932664828800000e+00, 4.6202291352809999e+02 },
    { 1.4177000000000000e-07, 2.6373282554999999e-01, 1.5104766984290001e+02 },
    { 1.4119000000000001e-07, 1.3240053257000001e-01, 3.7864329525170001e+02 },
    { 1.3957000000000000e-07, 3.9349673076600000e+00, 1.5922542890215000e+03 },
    { 1.3832000000000001e-07, 1.0736124387199999e+00, 1.5858915015461000e+03 },
    { 1.3780999999999999e-07, 4.5512964933499997e+00, 3.3773251065900001e+02 },
    { 1.3768000000000000e-07, 4.4823601547400003e+00, 6.7668051566499997e+01 },
    { 1.3750000000000001e-07, 2.0972225238700002e+00, 4.5097721326419997e+02 },
    { 1.3719000000000000e-07, 5.6389319563700004e+00, 1.1338771495710000e+02 },
    { 1.3707000000000000e-07, 3.6086843507499999e+00, 4.9867147645760002e+02 },
    { 1.3535999999999999e-07, 7.6908575684000002e-01, 2.7373205056900001e+03 },
    { 1.3524000000000001e-07, 5.0195678584500003e+00, 2.6555683724738001e+03 },
    { 1.3428000000000001e-07, 3.1562549215500000e+00, 5.4166259714000002e+00 },
    { 1.3322000000000000e-07, 8.2171739268999999e-01, 9.4498282327580000e+02 },
    { 1.3150999999999999e-07, 4.4361425339100000e+00, 1.0412226829245001e+03 },
    { 1.3012999999999999e-07, 7.8393846490999997e-01, 5.9884348936940000e+02 },
    { 1.3010000000000001e-07, 5.2593558973599999e+00, 2.7444340526907999e+03 },
    { 1.2686000000000001e-07, 1.5257940486999999e+00, 5.3138788612400003e+02 },
    { 1.2686000000000001e-07, 2.8148915996700001e+00, 5.2799404406520000e+02 },
    { 1.2616000000000001e-07, 7.3397485702999998e-01, 5.6071045373159996e+02 },
    { 1.2263000000000001e-07, 1.8512547403199999e+00, 6.3596513305090002e+02 },
    { 1.2230999999999999e-07, 6.1442294321400004e+00, 7.2073285581600004e+01 },
    { 1.2109000000000001e-07, 4.8522395141899999e+00, 1.0200250542740000e+03 },
    { 1.2034000000000000e-07, 3.1296429460200001e+00, 9.3812990870589999e+02 },
    { 1.2032000000000001e-07, 9.9005156583999998e-01, 5.9735901666109999e+02 },
    { 1.1861000000000000e-07, 6.0593778331200001e+00, 7.2164941953020002e+02 },
    { 1.1828000000000000e-07, 2.2334071586499999e+00, 6.0840471692500000e+02 },
    { 1.1798000000000000e-07, 2.4270937103999999e+00, 7.3194436026870005e+02 },
    { 1.1694000000000000e-07, 6.1011076715100003e+00, 2.9506196011279999e+03 },
    { 1.1590000000000000e-07, 3.3970066149300000e+00, 7.3297125859000005e+01 },
    { 1.1471000000000000e-07, 1.1782815922500001e+00, 9.6320784650000002e-01 },
    { 1.1266000000000000e-07, 3.4409958853500000e+00, 2.1011770170029999e+02 },
    { 1.1186000000000000e-07, 1.6225045870799999e+00, 5.2825467649610005e+02 },
    { 1.1186000000000000e-07, 2.7181810612900001e+00, 5.3112725369309999e+02 },
    { 1.0946000000000000e-07, 5.5857502476300001e+00, 4.6470422915999997e+01 },
    { 1.0863000000000000e-07, 4.6411786873400001e+00, 2.4872381809010000e+02 },
    { 1.0789999999999999e-07, 6.3662383032000003e-01, 5.3058473697190004e+02 },
    { 1.0789999999999999e-07, 3.7040618180500000e+00, 5.2879719321729999e+02 },
    { 1.0614999999999999e-07, 6.1351105325299997e+00, 6.2960234557549995e+02 },
    { 1.0592000000000000e-07, 6.3539606038999996e-01, 2.8475268269093999e+03 },
    { 1.0477000000000000e-07, 3.3777470465100001e+00, 4.5639383923560001e+02 },
    { 1.0370000000000000e-07, 4.4259970340799999e+00, 4.2977958461370002e+02 },
    { 1.0339000000000000e-07, 4.5861630350800002e+00, 3.1957326339430000e+02 },
    { 1.0323000000000001e-07, 1.2410474487800001e+00, 6.0595703637019994e+02 },
    { 9.9719999999999996e-08, 2.0436689694500001e+00, 6.8547393735269998e+02 },
    { 9.9699999999999999e-08, 4.5216030483500003e+00, 9.5612275556000004e+00 },
    { 9.8160000000000000e-08, 4.6475178742800001e+00, 6.9152524274800001e+01 },
    { 9.7320000000000001e-08, 2.6231831014600000e+00, 1.2513403846248000e+03 },
    { 9.7279999999999995e-08, 5.8564087027000000e-01, 1.1592933106701000e+03 },
    { 9.5920000000000001e-08, 1.7180651872099999e+00, 1.7810313497193999e+03 },
    { 9.5420000000000003e-08, 1.8449036389400000e+00, 2.5240214102519999e+03 },
    { 9.3190000000000003e-08, 1.2358538926700000e+00, 4.8322054217860000e+02 },
    { 9.2309999999999997e-08, 5.4314882480000000e+00, 8.0767548473000002e+00 },
    { 9.0769999999999998e-08, 3.8534523623900001e+00, 7.5403576079649997e+02 },
    { 9.0050000000000002e-08, 2.6183019015400002e+00, 1.1656560981455000e+03 },
    { 8.9340000000000005e-08, 9.6120225737999998e-01, 6.0298809095360002e+02 },
    { 8.8860000000000003e-08, 1.8901810757499999e+00, 5.3091480537200005e+02 },
    { 8.8860000000000003e-08, 2.4505045726199999e+00, 5.2846712481719999e+02 },
    { 8.8660000000000001e-08, 1.6303508574800001e+00, 1.9513984817330001e+02 },
    { 8.7089999999999994e-08, 3.2644085406799999e+00, 5.7616138801060004e+02 },
    { 8.6350000000000002e-08, 1.6436387126999999e-01, 1.0564611688824000e+03 },
    { 8.6350000000000002e-08, 4.8719774969599996e+00, 1.0623026914960001e+03 },
    { 8.6000000000000002e-08, 6.7499551221999998e-01, 2.8546403739102002e+03 },
    { 8.4400000000000001e-08, 2.5897465579699999e+00, 1.2018315803230000e+03 },
    { 8.3939999999999996e-08, 2.5879209256300002e+00, 5.2554636351040006e+02 },
    { 8.3089999999999998e-08, 1.7533236788100000e+00, 5.3383556667879998e+02 },
    { 8.0260000000000005e-08, 1.9866753112900000e+00, 4.0525754987360000e+02 },
    { 7.9889999999999996e-08, 3.8264174698900000e+00, 1.0586311706638000e+03 },
    { 7.9889999999999996e-08, 1.2133563100200000e+00, 1.0601326897146000e+03 },
    { 7.8590000000000004e-08, 3.8734430101000000e-01, 1.0647985561605999e+03 },
    { 7.7490000000000001e-08, 1.7560035289900000e+00, 5.2182665856840003e+02 },
    { 7.7289999999999999e-08, 2.9729594721599999e+00, 3.2032402291969998e+02 },
    { 7.6850000000000003e-08, 6.1566864952899998e+00, 3.3455111692129998e+02 },
    { 7.6829999999999994e-08, 4.7895356296199996e+00, 2.4723934538180001e+02 },
    { 7.6570000000000003e-08, 1.0315646867000000e-01, 4.0843894361129998e+02 },
    { 7.6090000000000002e-08, 5.6771912888399996e+00, 3.5181659230870002e+02 },
    { 7.6020000000000002e-08, 3.6299136019000000e-01, 1.0635747158832000e+03 },
    { 7.6020000000000002e-08, 4.6723540151099998e+00, 1.0551891444952000e+03 },
    { 7.6010000000000004e-08, 2.5849071749400001e+00, 5.3755527162080000e+02 },
    { 7.5349999999999996e-08, 3.0028931755800001e+00, 1.0608664028974999e+03 },
    { 7.4929999999999997e-08, 3.5196916516700001e+00, 2.3320629558164001e+03 },
    { 7.4840000000000000e-08, 6.4173631183000002e-01, 9.5235700270749999e+02 },
    { 7.3869999999999999e-08, 5.3291759331000002e+00, 1.4807872442630000e+02 },
    { 7.3409999999999994e-08, 6.0435905315299996e+00, 6.2197751234999998e+00 },
    { 7.3210000000000005e-08, 5.9656486855899997e+00, 2.9577331481288002e+03 },
    { 7.2149999999999994e-08, 4.3047597194299998e+00, 3.1019488636999998e+01 },
    { 7.1690000000000002e-08, 1.3492833770000001e-01, 8.4215068148809996e+02 },
    { 7.1359999999999998e-08, 3.4481526679999999e-02, 7.3980866679489998e+02 },
    { 6.9740000000000001e-08, 5.1052139598000004e+00, 2.7703499374140000e+02 },
    { 6.9059999999999998e-08, 3.0791062389300001e+00, 1.0064509366380000e+02 },
    { 6.8589999999999994e-08, 4.9934791684100004e+00, 9.4599421523210003e+02 },
    { 6.8289999999999998e-08, 2.1832288651099998e+00, 2.8414854074220000e+02 },
    { 6.8219999999999998e-08, 2.2986801847699998e+00, 1.6682867252200001e+02 },
    { 6.8149999999999998e-08, 3.1178839097600002e+00, 1.0537528558967001e+03 },
    { 6.7889999999999994e-08, 4.0613230556300000e+00, 3.0386169668439999e+02 },
    { 6.7430000000000002e-08, 2.7868574307000000e+00, 1.0547160637432000e+03 },
    { 6.7300000000000000e-08, 2.1327440050400002e+00, 5.2804643369190001e+02 },
    { 6.7300000000000000e-08, 2.2079416433299999e+00, 5.3133549649730003e+02 },
    { 6.6769999999999995e-08, 4.5441778626000001e-01, 1.9125783119411999e+03 },
    { 6.6289999999999993e-08, 5.3834433593999997e-01, 3.0537123753465999e+03 },
    { 6.3780000000000006e-08, 4.7646995899600002e+00, 1.4930936680660000e+03 },
    { 6.3520000000000002e-08, 4.3837827686899997e+00, 2.6087903141574199e+04 },
    { 6.3220000000000006e-08, 1.0202439734900000e+00, 1.9943304451573999e+03 },
    { 6.1510000000000000e-08, 4.6670502800499998e+00, 1.0916252591036000e+03 },
    { 6.0129999999999996e-08, 3.9505360832000003e-01, 1.0350029078010000e+03 },
    { 5.9709999999999997e-08, 4.4960436707299998e+00, 4.9819839570559998e+02 },
    { 5.9139999999999999e-08, 5.2566317309799997e+00, 1.0539653042177999e+03 },
    { 5.8309999999999997e-08, 9.1029898197000003e-01, 1.2766131788903001e+03 },
    { 5.7359999999999999e-08, 4.9480881504600003e+00, 7.0756533788050001e+02 },
    { 5.7229999999999997e-08, 8.0521226018000003e-01, 1.0650110044817000e+03 },
    { 5.6659999999999999e-08, 1.3458679117600001e+00, 1.0640477966352000e+03 },
    { 5.6659999999999999e-08, 4.1164376193100001e+00, 9.0833426034629997e+02 },
    { 5.6540000000000002e-08, 3.1791288229100001e+00, 2.8096714700450002e+02 },
    { 5.6050000000000002e-08, 2.8344687470199998e+00, 5.1322863885929996e+02 },
    { 5.5070000000000003e-08, 6.1180349336499997e+00, 5.6118353448360006e+02 },
    { 5.4970000000000002e-08, 5.9722071059999999e+00, 8.1065811209909998e+02 },
    { 5.4219999999999998e-08, 3.4231048040699998e+00, 7.5721715453419995e+02 },
    { 5.3820000000000002e-08, 3.6253823686800000e+00, 5.1242548970719997e+02 },
    { 5.3780000000000002e-08, 1.5062170140300000e+00, 5.4615329132989996e+02 },
    { 5.2339999999999998e-08, 3.2393975223700000e+00, 3.5574874557179999e+02 },
    { 5.1749999999999997e-08, 5.8845345810999996e+00, 3.2670114707846001e+03 },
    { 5.1529999999999999e-08, 7.1530340802000003e-01, 5.4695644048199995e+02 },
    { 5.0990000000000002e-08, 4.2292695891200003e+00, 1.1767017984094000e+03 },
    { 5.0860000000000000e-08, 4.8452249535399998e+00, 1.1439910691340000e+02 },
    { 5.0650000000000000e-08, 4.1150610562700001e+00, 1.0592218714948001e+03 },
    { 5.0650000000000000e-08, 9.2471272365000001e-01, 1.0595419888836000e+03 },
    { 5.0069999999999998e-08, 6.0909269079200001e+00, 4.9181856188770001e+02 },
    { 4.9549999999999997e-08, 3.4190630944599998e+00, 1.0696768709277001e+03 },
    { 4.9520000000000003e-08, 1.5800880026599999e+00, 2.8404132799086001e+03 },
    { 4.9110000000000001e-08, 1.5173165291599999e+00, 5.1228247735529999e+02 },
    { 4.8970000000000001e-08, 5.6822636453199999e+00, 2.4554242435239999e+02 },
    { 4.8639999999999997e-08, 5.1744939194999995e-01, 3.1639186965660001e+03 },
    { 4.8060000000000002e-08, 4.8295072320800001e+00, 1.2899465010146000e+03 },
    { 4.7650000000000000e-08, 1.4590260283700001e+00, 2.0974232193759999e+03 },
    { 4.7610000000000001e-08, 3.0826557851600001e+00, 8.5001498801430000e+02 },
    { 4.7600000000000003e-08, 2.4696056536600000e+00, 4.1529185818119998e+02 },
    { 4.7330000000000001e-08, 2.2421653935500001e+00, 1.0876931058405000e+03 },
    { 4.6880000000000000e-08, 8.2987219835000003e-01, 5.0456118318099999e+02 },
    { 4.6719999999999997e-08, 3.6038315922300002e+00, 2.3463973644039999e+02 },
    { 4.6520000000000002e-08, 4.7489663314800001e+00, 1.4899122743283001e+03 },
    { 4.6490000000000001e-08, 4.6475990989499998e+00, 1.0554045477340000e+02 },
    { 4.6280000000000001e-08, 4.5302637808400004e+00, 5.6756336830149996e+02 },
    { 4.6250000000000001e-08, 2.0983870110199998e+00, 8.5712853501510006e+02 },
    { 4.6129999999999997e-08, 4.2396912147799997e+00, 2.9505122865419997e+02 },
    { 4.6049999999999999e-08, 2.0107996909099999e+00, 3.9117346822389999e+02 },
    { 4.5690000000000001e-08, 4.9123703347700003e+00, 7.2483081326790000e+02 },
    { 4.5529999999999998e-08, 3.0507512200300000e+00, 5.0924409596949999e+02 },
    { 4.5499999999999997e-08, 3.9693191688999998e+00, 2.3767811782620001e+02 },
    { 4.4309999999999998e-08, 6.2071235756899998e+00, 5.1965665678699997e+02 },
    { 4.4290000000000002e-08, 4.9056561592700003e+00, 1.0254416802454000e+03 },
    { 4.4139999999999997e-08, 3.5015418001500000e+00, 5.5482074700819999e+02 },
    { 4.4029999999999998e-08, 2.4682685536400002e+00, 1.1270499817556999e+03 },
    { 4.4020000000000000e-08, 4.4786591128800000e+00, 9.8730864460759994e+02 },
    { 4.3240000000000002e-08, 1.2899346123200000e+00, 5.5013783421970004e+02 },
    { 4.3110000000000000e-08, 1.9108736357600000e+00, 4.1630325013750002e+02 },
    { 4.3030000000000002e-08, 2.2304460654499998e+00, 3.7390799283650000e+02 },
    { 4.2559999999999999e-08, 4.5262248976499997e+00, 5.1216485727630004e+02 },
    { 4.2319999999999998e-08, 4.1030525474799999e+00, 2.0424977891028000e+03 },
    { 4.2100000000000000e-08, 3.4256915421100000e+00, 2.0300415469949999e+02 },
    { 4.1659999999999997e-08, 2.6673446793200002e+00, 1.5875884225755001e+03 },
    { 4.1549999999999998e-08, 5.4015184756599997e+00, 7.8234693644779998e+02 },
    { 4.1439999999999999e-08, 3.2854038280100002e+00, 9.7051624997219994e+02 },
    { 4.1269999999999998e-08, 2.4126557912900002e+00, 1.5780271950199001e+03 },
    { 4.1150000000000001e-08, 5.1286524669600002e+00, 1.0129115072732000e+03 },
    { 4.0760000000000002e-08, 4.4309694438000002e-01, 4.6988722005429997e+02 },
    { 4.0719999999999997e-08, 6.0976461955300003e+00, 5.4721707291289999e+02 },
    { 4.0280000000000001e-08, 2.1438637821500000e+00, 8.1001373690799994e+01 },
    { 4.0170000000000002e-08, 7.0412765332000005e-01, 2.0225339517410001e+02 },
    { 4.0119999999999998e-08, 2.1980707907900001e+00, 6.6820846196529999e+02 },
    { 4.0019999999999997e-08, 4.8704674548200000e+00, 9.1693228005540004e+02 },
    { 3.9869999999999999e-08, 2.6344960454800002e+00, 4.4793883187839998e+02 },
    { 3.9820000000000002e-08, 9.4891503004000000e-01, 9.8608480433019997e+02 },
    { 3.9739999999999997e-08, 6.4083518042999998e-01, 3.1710322435667999e+03 },
    { 3.9710000000000003e-08, 5.2104159207900000e+00, 1.8944190646765001e+03 },
    { 3.9669999999999997e-08, 2.9036002612799998e+00, 4.8640193591629998e+02 },
    { 3.9550000000000000e-08, 3.6106300415199999e+00, 1.1231178284926000e+03 },
    { 3.9109999999999997e-08, 1.8127376438500000e+00, 3.0683064210100002e+02 },
    { 3.8999999999999998e-08, 2.3442354888599999e+00, 7.3905790726949999e+02 },
    { 3.8929999999999999e-08, 5.8326063648900002e+00, 8.1214258480740000e+02 },
    { 3.8560000000000003e-08, 4.7974179586899997e+00, 2.2286032299359999e+02 },
    { 3.8400000000000000e-08, 1.9118763371400000e+00, 4.7928838891549998e+02 },
    { 3.8070000000000003e-08, 1.7230476440900000e+00, 1.0212956637210000e+02 },
    { 3.7970000000000002e-08, 5.5298496022699997e+00, 2.0936694217490000e+02 },
    { 3.7289999999999998e-08, 5.2425376514100002e+00, 9.7748678462110001e+02 },
    { 3.6979999999999998e-08, 4.3488132575299998e+00, 3.2668681039510000e+02 },
    { 3.6799999999999999e-08, 2.7835847050500000e+00, 5.4709945283390005e+02 },
    { 3.6739999999999997e-08, 5.1716820230899998e+00, 7.8713751830400000e+01 },
    { 3.6710000000000002e-08, 2.6396694713400000e+00, 2.2582926841020000e+02 },
    { 3.6650000000000001e-08, 3.0027879929600001e+00, 7.7750543983900002e+01 },
    { 3.6589999999999999e-08, 5.4456782091899996e+00, 1.0405598206510000e+02 },
    { 3.6010000000000003e-08, 1.6409390853800001e+00, 8.3355266177900000e+02 },
    { 3.6010000000000003e-08, 2.0422718559800002e+00, 9.9171387862270001e+02 },
    { 3.5710000000000000e-08, 3.6594791282000001e+00, 1.0807225711916001e+03 },
    { 3.5649999999999998e-08, 5.4205942424200000e+00, 8.1752133216199994e+01 },
    { 3.5560000000000002e-08, 1.4372469755299999e+00, 5.7297999427289994e+02 },
    { 3.5439999999999998e-08, 5.9547690821700003e+00, 8.4926422848890002e+02 },
    { 3.5070000000000002e-08, 3.9004489419900001e+00, 5.8949471013489995e+02 },
    { 3.4970000000000001e-08, 4.3721572520300001e+00, 1.0595943785103000e+03 },
    { 3.4970000000000001e-08, 6.6761652789000003e-01, 1.0591694818681001e+03 },
    { 3.4720000000000003e-08, 2.4280763935800000e+00, 5.8009354127370000e+02 },
    { 3.4630000000000000e-08, 4.1728767679300001e+00, 1.5915205758386001e+03 },
    { 3.4550000000000002e-08, 4.9917017171899998e+00, 4.8884961647109998e+02 },
    { 3.4459999999999999e-08, 5.9453800608799998e+00, 1.0498207026335999e+03 },
    { 3.4149999999999998e-08, 9.3397733418999995e-01, 3.0679394693482000e+03 },
    { 3.3500000000000002e-08, 5.3347961943100000e+00, 1.0775411774539000e+03 },
    { 3.3449999999999998e-08, 2.0184388528600001e+00, 1.0314095832840000e+02 },
    { 3.3379999999999998e-08, 5.1164459779599998e+00, 1.0304459010879999e+02 },
    { 3.3370000000000000e-08, 2.5586290849300002e+00, 7.7523338944700004e+02 },
    { 3.3029999999999999e-08, 4.4220506901699999e+00, 5.3972527340219995e+02 },
    { 3.2910000000000002e-08, 5.8946936690999996e-01, 9.7676148247200004e+01 },
    { 3.2829999999999997e-08, 2.9172444270300000e+00, 1.0872184851110001e+02 },
    { 3.2520000000000003e-08, 2.5412920996999999e-01, 3.3701042450032000e+03 },
    { 3.2259999999999999e-08, 6.1739212714400002e+00, 5.2278986641489996e+02 },
    { 3.2259999999999999e-08, 4.4499496840999999e+00, 5.3659206377429996e+02 },
    { 3.1930000000000002e-08, 3.6601295717000002e+00, 1.4563104387150000e+02 },
    { 3.1750000000000003e-08, 1.2555497733900001e+00, 5.2012973753899996e+02 },
    { 3.1590000000000000e-08, 7.6090441014999999e-01, 4.5761767951299998e+02 },
    { 3.1419999999999999e-08, 6.1957018965500001e+00, 5.3925219265019996e+02 },
    { 3.1179999999999999e-08, 5.6321699961500000e+00, 5.7053231371810000e+02 },
    { 3.1000000000000000e-08, 4.4859756781600000e+00, 1.4380252254408999e+03 },
    { 3.0829999999999999e-08, 4.7954990594200000e+00, 3.0608259223474001e+03 },
    { 3.0540000000000001e-08, 2.2160614321900001e+00, 6.5412438031559998e+02 },
    { 3.0309999999999998e-08, 4.9469924936399998e+00, 5.2109294538550000e+02 },
    { 3.0309999999999998e-08, 5.6768784619100003e+00, 5.3828898480370003e+02 },
    { 3.0230000000000000e-08, 2.9412314465699998e+00, 5.2672201967800004e+02 },
    { 3.0230000000000000e-08, 1.3994542018000000e+00, 5.3265991051120000e+02 },
    { 2.9580000000000001e-08, 5.9679143097800003e+00, 5.2768522752449996e+02 },
    { 2.9580000000000001e-08, 4.6559566457699999e+00, 5.3169670266469996e+02 },
    { 2.9469999999999999e-08, 2.5760156205600002e+00, 1.8196374661092000e+03 },
    { 2.9180000000000001e-08, 1.0238961877799999e+00, 9.6733485623449997e+02 },
    { 2.9090000000000001e-08, 4.6460485565400003e+00, 5.2268929267839997e+02 },
    { 2.9090000000000001e-08, 5.9778223990099999e+00, 5.3669263751079995e+02 },
    { 2.9029999999999999e-08, 5.8685837820800000e+00, 5.0441817082910001e+02 },
    { 2.8699999999999999e-08, 1.8338801385800001e+00, 1.5866252147289999e+03 },
    { 2.8489999999999999e-08, 2.5231587953200001e+00, 1.8841241239379999e+03 },
    { 2.8369999999999999e-08, 5.4706467325999997e+00, 1.4409941708575000e+03 },
    { 2.8360000000000001e-08, 1.2073125023700000e+00, 1.2694996318895001e+03 },
    { 2.8209999999999999e-08, 2.6373575003299998e+00, 5.0286426215159997e+02 },
    { 2.7960000000000000e-08, 1.2412856313100000e+00, 3.1568051495651998e+03 },
    { 2.7809999999999999e-08, 3.6298610677499998e+00, 4.5194042111070002e+02 },
    { 2.7789999999999999e-08, 4.0658758728700004e+00, 1.0656017053127000e+03 },
    { 2.7640000000000001e-08, 1.1003155390499999e+00, 3.9617508346100003e+01 },
    { 2.7470000000000000e-08, 3.7465810139100002e+00, 1.2616353253632999e+03 },
    { 2.7310000000000001e-08, 6.2441699716999999e+00, 5.2246554350919996e+02 },
    { 2.7310000000000001e-08, 4.3797009838500003e+00, 5.3691638667999996e+02 },
    { 2.7249999999999999e-08, 1.6269604999600000e+00, 3.2812385647862002e+03 },
    { 2.7150000000000002e-08, 4.7404360122000000e+00, 5.5496375936009997e+02 },
    { 2.6849999999999999e-08, 1.6767701729000000e+00, 5.5651766803759995e+02 },
    { 2.6680000000000001e-08, 5.3758166780999996e+00, 9.1130320576290001e+02 },
    { 2.6610000000000001e-08, 3.4120009151900001e+00, 1.0775864066459999e+02 },
    { 2.6470000000000001e-08, 2.2115097056600002e+00, 6.1144309831080000e+02 },
    { 2.6270000000000000e-08, 3.8221525778699998e+00, 1.4414657116320001e+02 },
    { 2.6270000000000000e-08, 3.8097573988900000e+00, 1.0948066528413001e+03 },
    { 2.6219999999999999e-08, 2.4593606603299998e+00, 1.3458534360760001e+02 },
    { 2.5839999999999998e-08, 2.4103674516600000e+00, 2.1226960136415000e+03 },
    { 2.5659999999999999e-08, 5.1787123376900004e+00, 9.3531546663000000e+01 },
    { 2.5600000000000001e-08, 5.7578998273200002e+00, 2.3204904353370000e+02 },
    { 2.5480000000000001e-08, 2.3002952364599998e+00, 1.1285344544640000e+03 },
    { 2.5370000000000002e-08, 1.6733499153200000e+00, 1.2566151699982800e+04 },
    { 2.5250000000000001e-08, 5.4631298373400003e+00, 3.7301422095919997e+02 },
    { 2.5100000000000000e-08, 3.5654252447800001e+00, 6.0176425067620005e+02 },
    { 2.4800000000000000e-08, 5.9475880025899999e+00, 6.1069233878540001e+02 },
    { 2.4560000000000000e-08, 3.0435890405100001e+00, 1.0743597837161999e+03 },
    { 2.4560000000000000e-08, 1.9961847394100001e+00, 1.0444040766621999e+03 },
    { 2.4520000000000001e-08, 2.1910672433100000e+00, 1.0904014188261999e+03 },
    { 2.4350000000000000e-08, 5.8069034545299996e+00, 3.2741250177853999e+03 },
    { 2.4190000000000000e-08, 1.6559225975800000e+00, 3.1492569389000000e+01 },
    { 2.4070000000000000e-08, 1.6559345342900000e+00, 8.2474219374879999e+02 },
    { 2.3800000000000001e-08, 4.1200137357999997e+00, 4.2904587143079999e+02 },
    { 2.3689999999999999e-08, 8.9961386019000000e-01, 2.3107223148140001e+03 },
    { 2.3660000000000001e-08, 2.8148396465699999e+00, 3.7078367556622002e+03 },
    { 2.3619999999999999e-08, 5.6389570904399999e+00, 3.5834033404411998e+03 },
    { 2.3600000000000000e-08, 2.9631311769900002e+00, 4.8863716814999998e+02 },
    { 2.3580000000000000e-08, 2.8705194880000001e-01, 3.4803105662225998e+03 },
    { 2.3520000000000001e-08, 9.7276625670000005e-02, 3.6648562929500002e+01 },
    { 2.3499999999999999e-08, 4.8502146061899998e+00, 2.8537238101960003e+02 },
    { 2.3499999999999999e-08, 3.1477159402199999e+00, 9.0270518605380005e+02 },
    { 2.3370000000000000e-08, 2.5260050022499998e+00, 1.4749344208012999e+03 },
    { 2.3230000000000000e-08, 4.1790406508100002e+00, 5.2905782564820004e+02 },
    { 2.3230000000000000e-08, 1.6164499756000000e-01, 5.3032410454100000e+02 },
    { 2.2790000000000001e-08, 5.7287453430599999e+00, 1.4827987273275000e+03 },
    { 2.2780000000000000e-08, 2.7874028612000001e-01, 6.6812248533995999e+03 },
    { 2.2729999999999999e-08, 1.3268490866800000e+00, 1.5891210793936000e+03 },
    { 2.2729999999999999e-08, 4.4122232662299998e+00, 1.5890247111740000e+03 },
    { 2.2510000000000001e-08, 3.7328559676299999e+00, 1.0674586850364999e+03 },
    { 2.2230000000000001e-08, 3.5429008297900002e+00, 8.4290144101349995e+02 },
    { 2.2160000000000001e-08, 4.4041362946599998e+00, 1.0532315910349000e+03 },
    { 2.2010000000000000e-08, 5.7255033224699998e+00, 2.0439822618111000e+03 },
    { 2.2010000000000000e-08, 3.1427215056000002e-01, 3.7619561469690001e+02 },
    { 2.1990000000000000e-08, 2.8148830496400001e+00, 2.9823262239190001e+02 },
    { 2.1859999999999999e-08, 2.3539098641299998e+00, 1.6289651925890001e+02 },
    { 2.1859999999999999e-08, 5.0566232385300003e+00, 1.5536481726316999e+03 },
    { 2.1769999999999999e-08, 1.2849127438700001e+00, 2.4138150890326001e+03 },
    { 2.1730000000000000e-08, 5.3973434031000000e-01, 1.0655322693435000e+03 },
    { 2.1460000000000002e-08, 2.1228507761900000e+00, 1.3632436268736001e+03 },
    { 2.1400000000000000e-08, 3.0047344818899999e+00, 2.4161027108930000e+02 },
    { 2.1279999999999999e-08, 3.0012341312000002e-01, 8.2170381236300000e+02 },
    { 2.0960000000000000e-08, 1.3483026158100000e+00, 5.7074476203920005e+02 },
    { 2.0899999999999999e-08, 3.5198792254300000e+00, 4.1703696332039999e+02 },
    { 2.0459999999999999e-08, 3.0198726360200001e+00, 1.5677322542813999e+03 },
    { 2.0420000000000000e-08, 4.5345339204000001e-01, 7.3269511979410004e+02 },
    { 2.0350000000000000e-08, 5.4179357057099997e+00, 5.2384944248099998e+02 },
    { 2.0350000000000000e-08, 5.2059352498399996e+00, 5.3553248770820005e+02 },
    { 2.0269999999999999e-08, 1.0585963779800001e+00, 5.0402576179100002e+01 },
    { 2.0240000000000001e-08, 7.5324302754000005e-01, 5.9803745040300001e+01 },
    { 1.9980000000000001e-08, 3.6527951674299999e+00, 1.4323961511483999e+03 },
    { 1.9700000000000001e-08, 4.2508165894800003e+00, 3.6286229257259998e+02 },
    { 1.9580000000000001e-08, 2.6576034846300001e+00, 1.0924311337290000e+02 },
    { 1.9540000000000002e-08, 7.8523241390999998e-01, 1.4151306757610000e+03 },
    { 1.9499999999999999e-08, 2.5346826401300002e+00, 4.8715269544170002e+02 },
    { 1.9480000000000000e-08, 6.7842016860000007e-02, 1.4466232451500000e+03 },
    { 1.9110000000000000e-08, 1.2784730799799999e+00, 5.0123677709139997e+02 },
    { 1.9029999999999999e-08, 5.9722285218800000e+00, 1.2545217783625001e+03 },
    { 1.8940000000000000e-08, 1.6894141090799999e+00, 2.2908009811709999e+02 },
    { 1.8749999999999999e-08, 4.9640984532400001e+00, 1.5900361031303000e+03 },
    { 1.8730000000000000e-08, 2.1613044842300000e+00, 1.2089451273238001e+03 },
    { 1.8720000000000002e-08, 2.9468034413400002e+00, 1.1411340634054000e+03 },
    { 1.8600000000000001e-08, 5.9121810077000003e-01, 1.1600270238529999e+03 },
    { 1.8510000000000002e-08, 2.9490939073200000e+00, 1.1025450532920000e+02 },
    { 1.8290000000000000e-08, 3.2427522713000001e-01, 1.5787779545453000e+03 },
    { 1.8229999999999998e-08, 1.1309992342000000e-01, 1.1116952906589999e+02 },
    { 1.8119999999999999e-08, 1.1863647245700000e+00, 1.3403490771936999e+03 },
    { 1.8019999999999999e-08, 1.8198105628300001e+00, 5.7222923474749996e+02 },
    { 1.7960000000000000e-08, 2.2774803443299998e+00, 4.2415051032119999e+02 },
    { 1.7910000000000000e-08, 1.0409422402000000e-01, 5.0260362972069998e+02 },
    { 1.7870000000000001e-08, 5.2358642066699996e+00, 1.0596425626201001e+03 },
    { 1.7870000000000001e-08, 6.0870948804299996e+00, 1.0591212977583000e+03 },
    { 1.7750000000000000e-08, 3.6837856318600002e+00, 1.6882335162392999e+03 },
    { 1.7459999999999999e-08, 4.4336246205699998e+00, 1.4678208738005001e+03 },
    { 1.7439999999999999e-08, 5.7594478900899997e+00, 1.0513051753418999e+03 },
    { 1.7159999999999999e-08, 4.2263388036100000e+00, 5.0382746999810001e+02 },
    { 1.7140000000000000e-08, 2.6654981626000001e+00, 1.1649223849626001e+03 },
    { 1.6960000000000001e-08, 4.0489397455300002e+00, 3.1321047591889999e+02 },
    { 1.6910000000000001e-08, 6.9039681436000000e-01, 1.2262106027112000e+03 },
    { 1.6899999999999999e-08, 4.4964896141999999e-01, 1.6960978227655000e+03 },
    { 1.6849999999999999e-08, 4.4636865378300001e+00, 5.6622351302600002e+01 },
    { 1.6810000000000000e-08, 5.0561547043299999e+00, 5.4318434591330004e+02 },
    { 1.6810000000000000e-08, 5.5677162512100002e+00, 5.1619758427590000e+02 },
    { 1.6730000000000002e-08, 4.2365917412499998e+00, 5.5677830046849999e+02 },
    { 1.6569999999999999e-08, 2.9569686220800002e+00, 4.7306861379200001e+02 },
    { 1.6560000000000001e-08, 6.1852542398799999e+00, 3.6864961146597998e+03 },
    { 1.6490000000000001e-08, 2.8351667172399999e+00, 1.3544331588434000e+03 },
    { 1.6449999999999998e-08, 2.5602007166000001e-01, 6.3033605875839999e+02 },
    { 1.6199999999999999e-08, 4.0460444576299999e+00, 6.8318631549229997e+02 },
    { 1.6130000000000000e-08, 3.5249429060000002e+00, 1.2687488723641000e+03 },
    { 1.6079999999999999e-08, 1.1434714708000000e-01, 5.5555446019110002e+02 },
    { 1.6070000000000001e-08, 1.8621770244900000e+00, 4.6595506679120001e+02 },
    { 1.5990000000000000e-08, 9.9304321708999999e-01, 1.5881096874373000e+03 },
    { 1.5970000000000000e-08, 2.2782919945900000e+00, 2.2005159935945999e+03 },
    { 1.5950000000000001e-08, 1.9920339820999999e-01, 1.3851749687070000e+02 },
    { 1.5810000000000001e-08, 6.2656515677900000e+00, 6.9651963761659999e+02 },
    { 1.5810000000000001e-08, 4.2493803143999997e-01, 6.4234496686880004e+02 },
    { 1.5760000000000000e-08, 3.1444350094000001e-01, 4.2240540518199998e+02 },
    { 1.5729999999999999e-08, 9.9918451715000001e-01, 3.4731970192218000e+03 },
    { 1.5729999999999999e-08, 1.5498374850200001e+00, 4.2563498302950001e+02 },
    { 1.5690000000000000e-08, 5.5945561191000004e-01, 8.2792358748649997e+02 },
    { 1.5510000000000001e-08, 2.3427636132299998e+00, 5.9342686339800002e+02 },
    { 1.5469999999999999e-08, 4.3253240963000000e-01, 1.0975149658270000e+03 },
    { 1.5329999999999999e-08, 9.1717120274999997e-01, 1.0523165672982000e+03 },
    { 1.5320000000000001e-08, 6.1130441623999996e+00, 3.5033211960040001e+02 },
    { 1.5250000000000001e-08, 3.5211329965500000e+00, 2.5265597135319999e+02 },
    { 1.4960000000000000e-08, 5.9035914693600002e+00, 3.8846515523820000e+02 },
    { 1.4860000000000000e-08, 1.7496446041000000e+00, 1.5182234499796000e+03 },
    { 1.4839999999999999e-08, 3.9862899661100002e+00, 9.9564603188579997e+02 },
    { 1.4710000000000001e-08, 2.0705683946500000e+00, 6.3523141986799999e+02 },
    { 1.4660000000000000e-08, 4.9686628528099996e+00, 4.2655000676620000e+02 },
    { 1.4640000000000001e-08, 3.9091174894499998e+00, 1.0664472930802001e+03 },
    { 1.4590000000000000e-08, 1.8668081033399999e+00, 4.2664637498579998e+02 },
    { 1.4510000000000001e-08, 2.9931241623400000e+00, 2.4431858407499999e+02 },
    { 1.4230000000000001e-08, 3.9689944451799999e+00, 1.0522201990786000e+03 },
    { 1.4190000000000000e-08, 3.1114952471300001e+00, 1.1380956820196000e+03 },
    { 1.3980000000000000e-08, 1.2322431325200001e+00, 2.6271141844705999e+03 },
    { 1.3949999999999999e-08, 1.4045574764600000e+00, 5.8631331639719997e+02 },
    { 1.3930000000000000e-08, 4.4415833191999998e-01, 9.5873683652540001e+02 },
    { 1.3869999999999999e-08, 2.5367096621000002e-01, 2.1648048917570000e+02 },
    { 1.3799999999999999e-08, 4.1306005059100004e+00, 3.8405992122309999e+02 },
    { 1.3750000000000001e-08, 3.9804443753699998e+00, 1.0933221801330001e+03 },
    { 1.3730000000000000e-08, 1.1386932789899999e+00, 4.5909101804889997e+03 },
    { 1.3580000000000000e-08, 4.6226709240600004e+00, 4.4868959140380002e+02 },
    { 1.3529999999999999e-08, 4.1637312503299997e+00, 2.4988916283384001e+03 },
    { 1.3459999999999999e-08, 8.4106067158999998e-01, 1.0665436612998001e+03 },
    { 1.3399999999999999e-08, 5.6878846818400000e+00, 1.0581099058020000e+03 },
    { 1.3399999999999999e-08, 6.0306810334399996e+00, 1.1615114965613000e+03 },
    { 1.3399999999999999e-08, 5.6350744052600001e+00, 1.0606539545764001e+03 },
    { 1.3370000000000000e-08, 5.1832974855599998e+00, 4.2756139872249997e+02 },
    { 1.3290000000000000e-08, 2.3793044972600002e+00, 1.6284297711990000e+03 },
    { 1.3210000000000000e-08, 5.1804037273899999e+00, 9.9022940591439999e+02 },
    { 1.3090000000000000e-08, 3.4807825338899998e+00, 1.1634379122543000e+03 },
    { 1.3080000000000000e-08, 1.2713677586500001e+00, 9.1375088631769995e+02 },
    { 1.3020000000000000e-08, 1.3024727461900001e+00, 4.9447869076360001e+02 },
    { 1.3020000000000000e-08, 4.0737225713500003e+00, 6.3697652500720005e+02 },
    { 1.3000000000000001e-08, 5.4525683063799999e+00, 7.5877106321170004e+02 },
    { 1.2989999999999999e-08, 4.8747523458999997e-01, 1.3576145525811000e+03 },
    { 1.2930000000000001e-08, 2.9865104085800001e+00, 3.9435486196160002e+02 },
    { 1.2919999999999999e-08, 1.3483229175900000e+00, 9.8163138620530003e+02 },
    { 1.2670000000000000e-08, 5.4805226800000002e-02, 1.1625228885176000e+03 },
    { 1.2600000000000000e-08, 2.7849369694199999e+00, 6.3374694715969997e+02 },
    { 1.2590000000000001e-08, 5.0192709982299997e+00, 1.8873055176757000e+03 },
    { 1.2580000000000001e-08, 3.1549358407699999e+00, 1.1624265202980000e+03 },
    { 1.2429999999999999e-08, 4.7198149752600003e+00, 6.7091677495099998e+02 },
    { 1.2350000000000000e-08, 1.2943483743999999e+00, 1.0271386012748001e+03 },
    { 1.2340000000000000e-08, 1.8969291075500001e+00, 4.3615941843159999e+02 },
    { 1.2320000000000000e-08, 1.1376804760100001e+00, 2.7302069586891998e+03 },
    { 1.2290000000000000e-08, 5.4083503515100002e+00, 2.0043647534649999e+03 },
    { 1.2019999999999999e-08, 3.6193397288900000e+00, 2.6992144674060000e+02 },
    { 1.1980000000000000e-08, 2.5643170066100001e+00, 8.5211198767086007e+03 },
    { 1.1940000000000000e-08, 5.7801306772199998e+00, 7.7400954916959995e+02 },
    { 1.1920000000000000e-08, 3.0382134089499999e+00, 5.6490323942559996e+02 },
    { 1.1900000000000001e-08, 5.9267119341900001e+00, 2.1574677599280000e+02 },
    { 1.1860000000000000e-08, 4.6706446506199999e+00, 2.1148317071153001e+03 },
    { 1.1860000000000000e-08, 3.8760897283300002e+00, 1.2914309737229000e+03 },
    { 1.1760000000000001e-08, 5.6525058500499998e+00, 6.3283192342300003e+02 },
    { 1.1720000000000000e-08, 2.4723860568900000e+00, 6.3273555520340005e+02 },
    { 1.1720000000000000e-08, 1.1078488056999999e-01, 3.7967024358792000e+03 },
    { 1.1679999999999999e-08, 6.6078054683999998e-01, 4.1406801790380001e+02 },
    { 1.1660000000000000e-08, 1.6805325604300001e+00, 1.5113223512999000e+03 },
    { 1.1530000000000000e-08, 3.8930087007699998e+00, 3.9510562148700001e+02 },
    { 1.1469999999999999e-08, 5.6223762587400001e+00, 1.3900008938474000e+03 },
    { 1.1370000000000000e-08, 1.2521787874999999e+00, 9.5066008167810003e+02 },
    { 1.1320000000000000e-08, 5.1083517504999998e-01, 1.6638544938511000e+03 },
    { 1.1290000000000001e-08, 4.8786080710000002e-01, 9.1523535902599997e+02 },
    { 1.1240000000000000e-08, 1.8883399793800000e+00, 9.2479658658159997e+02 },
    { 1.1199999999999999e-08, 4.5970974193399998e+00, 7.0904981058880003e+02 },
    { 1.1199999999999999e-08, 7.0794225878999995e-01, 1.0361403908040000e+02 },
    { 1.1190000000000000e-08, 1.8921146283200001e+00, 1.9015326116773001e+03 },
    { 1.1180000000000000e-08, 3.0247731972100000e+00, 8.9255325766720000e+02 },
    { 1.1170000000000000e-08, 2.1068971066000000e-01, 1.0257150935679999e+02 },
    { 1.1120000000000000e-08, 5.2941632099399998e+00, 1.0659742123282001e+03 },
    { 1.1110000000000000e-08, 3.3643182186799998e+00, 1.7787437278589999e+02 },
    { 1.1090000000000000e-08, 1.1071723091400001e+00, 3.4788443904560000e+02 },
    { 1.1080000000000001e-08, 1.2651220501000000e+00, 1.2797945726280000e+03 },
    { 1.0989999999999999e-08, 2.8085480397499998e+00, 6.1158611066269998e+02 },
    { 1.0940000000000001e-08, 4.4953632469900002e+00, 4.4779581952650000e+02 },
    { 1.0890000000000000e-08, 5.1767855368599998e+00, 9.4672792841499995e+02 },
    { 1.0760000000000000e-08, 5.0811919636700003e+00, 1.6811199692385001e+03 },
    { 1.0760000000000000e-08, 3.7787556032900000e+00, 2.8617539209110000e+03 },
    { 1.0690000000000000e-08, 5.4674886279899999e+00, 3.8997952100978000e+03 },
    { 1.0640000000000000e-08, 3.9107139363900001e+00, 1.3049243545416000e+03 },
    { 1.0610000000000000e-08, 5.4342024043299997e+00, 1.0141353475506000e+03 },
    { 1.0610000000000000e-08, 5.9192476469899997e+00, 1.0527896480502000e+03 },
    { 1.0420000000000000e-08, 5.4804455685700004e+00, 1.7076082578510000e+02 },
    { 1.0420000000000000e-08, 9.3682125769000002e-01, 2.9764192156090002e+02 },
    { 1.0400000000000000e-08, 2.4082230893799998e+00, 3.1394418910180002e+02 },
    { 1.0379999999999999e-08, 2.6080928060100002e+00, 2.5453620512543998e+03 },
    { 1.0370000000000000e-08, 2.1054283278599999e+00, 1.7984398374586999e+03 },
    { 1.0330000000000001e-08, 2.6574293306599999e+00, 1.0283624415521999e+03 },
    { 1.0330000000000001e-08, 5.4405782552600002e+00, 6.3182053146670000e+02 },
    { 1.0320000000000001e-08, 4.7701857598900004e+00, 3.0750530163489998e+03 },
    { 1.0250000000000001e-08, 8.1899907031999997e-01, 9.5162328952459995e+02 },
    { 1.0220000000000000e-08, 1.3592214352000000e+00, 1.2623860848887000e+03 },
    { 1.0179999999999999e-08, 2.4417790068400000e+00, 6.2322251175760005e+02 },
    { 1.0169999999999999e-08, 3.2300956526500002e+00, 1.0197644218431000e+03 },
    { 1.0050000000000001e-08, 1.0222393274399999e+00, 3.3843313390048002e+03 },
    { 1.0029999999999999e-08, 6.0627637166700001e+00, 1.1265400177420000e+02 },
    { 9.9699999999999993e-09, 3.8726751474999999e+00, 1.5056932770074000e+03 },
    { 9.8899999999999996e-09, 3.8499005127400001e+00, 5.6633952802409999e+02 },
    { 9.8899999999999996e-09, 4.9078513562999998e-01, 4.9304240216509999e+02 },
    { 9.8799999999999998e-09, 3.7740212907699999e+00, 7.2109158184941998e+03 },
    { 9.8000000000000001e-09, 3.5599702026300002e+00, 9.6170578194200004e+02 },
    { 9.7599999999999994e-09, 7.9158818735000003e-01, 8.0672595883600002e+02 },
    { 9.7499999999999996e-09, 1.3480188330700000e+00, 6.6502706822760001e+02 },
    { 9.7399999999999999e-09, 3.0728906971600001e+00, 2.5035634758385801e+04 },
    { 9.6699999999999999e-09, 1.9419846630000001e-01, 8.2881735936380005e+02 },
    { 9.6699999999999999e-09, 3.3624470495700001e+00, 1.7448558675418999e+03 },
    { 9.6500000000000004e-09, 4.7893715885999999e-01, 1.5199203710090001e+03 },
    { 9.6099999999999997e-09, 4.3405759627900000e+00, 6.9158595893046004e+03 },
    { 9.5499999999999995e-09, 3.7124457651100000e+00, 6.4531391228539997e+02 },
    { 9.5100000000000005e-09, 1.7980074695000001e-01, 2.5021407664384202e+04 },
    { 9.5000000000000007e-09, 1.3142596894200000e+00, 1.3725924061081000e+03 },
    { 9.3900000000000000e-09, 4.9076799095099997e+00, 1.7032113697663001e+03 },
    { 9.3000000000000006e-09, 2.4954832346000000e+00, 1.4365407527325999e+03 },
    { 9.2900000000000008e-09, 2.3311028880100002e+00, 1.1302313754934000e+03 },
    { 9.2900000000000008e-09, 5.9986476724599997e+00, 1.1015813710960001e+02 },
    { 9.2300000000000006e-09, 4.6388670430000001e-02, 1.7935884549420001e+02 },
    { 9.1800000000000001e-09, 3.8673996933399999e+00, 3.4077089204480001e+02 },
    { 9.1399999999999995e-09, 1.8519016389400000e+00, 1.0310707545379000e+03 },
    { 9.0699999999999995e-09, 5.7288020210599999e+00, 3.5905168874420001e+03 },
    { 9.0200000000000007e-09, 2.8172008795700001e+00, 3.2591471642584002e+03 },
    { 9.0099999999999993e-09, 1.9903167840900000e+00, 1.4820479678020999e+03 },
    { 8.9899999999999998e-09, 1.0852556825299999e+00, 1.0531621550657001e+03 },
    { 8.9899999999999998e-09, 4.9732091792600004e+00, 1.5709136480191000e+03 },
    { 8.9700000000000003e-09, 1.2806230741600000e+00, 5.7462713378959997e+03 },
    { 8.8800000000000008e-09, 3.2570697301000001e-01, 1.4756851803267000e+03 },
    { 8.8300000000000003e-09, 5.6753979482399997e+00, 5.2616784674559995e+02 },
    { 8.8300000000000003e-09, 4.9484730073099996e+00, 5.3321408344359997e+02 },
    { 8.8100000000000008e-09, 6.2411444135199998e+00, 4.0028879843164000e+03 },
    { 8.7999999999999994e-09, 6.1203803765600000e+00, 1.2151649024472999e+03 },
    { 8.7700000000000001e-09, 2.5772086441300002e+00, 3.5744566660120000e+02 },
    { 8.7600000000000004e-09, 2.0444550032800000e+00, 2.5169078632512001e+03 },
    { 8.7500000000000006e-09, 1.3864980078500000e+00, 7.0193626358799997e+02 },
    { 8.7299999999999994e-09, 1.7882393858500001e+00, 3.2202094394910000e+02 },
    { 8.7000000000000001e-09, 2.2911758119100001e+00, 5.3732577695719999e+02 },
    { 8.7000000000000001e-09, 5.1911024900499996e+00, 5.2205615323200004e+02 },
    { 8.6599999999999995e-09, 5.7654022927300002e+00, 1.0846069285524200e+04 },
    { 8.5299999999999993e-09, 4.2881073116200001e+00, 2.9648466951296000e+03 },
    { 8.4800000000000005e-09, 4.7250410727699999e+00, 4.4475743814070000e+02 },
    { 8.4499999999999996e-09, 3.5917127046399999e+00, 8.8114920691600005e+01 },
    { 8.4399999999999998e-09, 4.8095183653199998e+00, 2.3145834270270001e+02 },
    { 8.4200000000000003e-09, 8.7893029284000002e-01, 3.6908206769610001e+02 },
    { 8.4200000000000003e-09, 3.3885688458300001e+00, 1.5726744496640001e+02 },
    { 8.4100000000000005e-09, 3.1489140766600001e+00, 6.4307868005169996e+02 },
    { 8.3899999999999994e-09, 3.4691868728499999e+00, 4.6868894077067998e+03 },
    { 8.3199999999999994e-09, 3.0355557156000001e-01, 3.4863519857099999e+02 },
    { 8.2700000000000006e-09, 4.3396212207099998e+00, 1.3718416465826999e+03 },
    { 8.2599999999999992e-09, 1.2623708655500001e+00, 2.0956262057516600e+04 },
    { 8.2299999999999999e-09, 2.2147951466200002e+00, 7.0363318461740005e+02 },
    { 8.1500000000000002e-09, 4.1855362759899997e+00, 3.3772177920039999e+03 },
    { 8.1500000000000002e-09, 2.4217165225000001e+00, 3.5893013930950002e+02 },
    { 8.0999999999999997e-09, 7.3866187949999995e-01, 3.7895888888784002e+03 },
    { 8.0999999999999997e-09, 3.4831745938899998e+00, 6.7776968952089999e+02 },
    { 8.0900000000000000e-09, 5.1702495324099997e+00, 2.4319743362972999e+03 },
    { 8.0499999999999993e-09, 9.9017863829999997e-02, 1.4746737883704000e+03 },
    { 8.0399999999999995e-09, 2.6384052272299998e+00, 9.6764810341156008e+03 },
    { 7.9900000000000007e-09, 3.7403459461900002e+00, 2.2745468326364999e+03 },
    { 7.9200000000000008e-09, 1.4996596092800001e+00, 3.7715882254339999e+02 },
    { 7.9200000000000008e-09, 2.4370973638000001e-01, 3.4874241132234001e+03 },
    { 7.8999999999999996e-09, 6.2015436625299998e+00, 1.0601353552540000e+02 },
    { 7.8899999999999998e-09, 1.6709237015100000e+00, 9.5013881681630005e+02 },
    { 7.8700000000000003e-09, 4.2574687191500002e+00, 1.0566736172035000e+03 },
    { 7.8700000000000003e-09, 1.2045751127900000e+00, 2.9435060541272001e+03 },
    { 7.8399999999999994e-09, 2.5454185957600002e+00, 4.5172797278960002e+02 },
    { 7.8199999999999999e-09, 2.2383233524600001e+00, 1.7315225454176000e+03 },
    { 7.8000000000000004e-09, 1.3619516795600000e+00, 9.4912742486000002e+02 },
    { 7.7799999999999992e-09, 3.2337232310099999e+00, 1.1529134768521999e+03 },
    { 7.7699999999999994e-09, 4.7641510051999996e+00, 5.0921519581158000e+03 },
    { 7.7400000000000002e-09, 4.8944089812899998e+00, 8.6170991039264009e+03 },
    { 7.7400000000000002e-09, 3.1413062599299999e+00, 7.1149749114359997e+02 },
    { 7.7300000000000004e-09, 1.7606583864300001e+00, 3.3339287628256998e+03 },
    { 7.6700000000000002e-09, 3.6065218409300002e+00, 9.2783496796739996e+02 },
    { 7.6600000000000004e-09, 4.5947989880000000e+00, 4.0221916848780000e+02 },
    { 7.6600000000000004e-09, 2.3920210581500001e+00, 1.8346153196362000e+03 },
    { 7.6500000000000007e-09, 4.5676284976999998e-01, 7.7734018459627987e+04 },
    { 7.6399999999999993e-09, 9.9241506715000005e-01, 1.5426024723677999e+03 },
    { 7.5400000000000000e-09, 4.2336227193199996e+00, 1.0227333672597000e+03 },
    { 7.5100000000000007e-09, 5.8139070371299999e+00, 4.4631134681819998e+02 },
    { 7.4999999999999993e-09, 5.5290509814000000e-01, 1.5349535039770001e+02 },
    { 7.4999999999999993e-09, 3.5232005659499999e+00, 1.8181529934009000e+03 },
    { 7.4899999999999996e-09, 4.3100576344299997e+00, 4.2193232442999999e+02 },
    { 7.4499999999999997e-09, 4.4585454823499999e+00, 3.0609692891809999e+02 },
    { 7.4300000000000002e-09, 6.1313876451800002e+00, 4.3201481684740003e+02 },
    { 7.4300000000000002e-09, 8.9577704031000005e-01, 1.6700742689746000e+03 },
    { 7.4199999999999996e-09, 2.0035417279300001e+00, 8.9648541093029996e+02 },
    { 7.4199999999999996e-09, 4.2170336126199999e+00, 9.4821240112329997e+02 },
    { 7.3600000000000002e-09, 3.2013015489500001e+00, 7.7962992305000000e+01 },
    { 7.3600000000000002e-09, 2.5044227155400001e+00, 2.3969139281195799e+04 },
    { 7.3600000000000002e-09, 2.1677397997200001e+00, 6.0150361824530000e+02 },
    { 7.3600000000000002e-09, 2.1729458486500000e+00, 4.5787831194389997e+02 },
    { 7.3300000000000001e-09, 3.9841498875100001e+00, 3.4470304530790003e+02 },
    { 7.3200000000000004e-09, 5.4294365215199996e+00, 1.6001185955476999e+03 },
    { 7.3099999999999998e-09, 5.8326849222300003e+00, 2.1084689196398999e+03 },
    { 7.2699999999999999e-09, 3.5558842025900002e+00, 6.0114455234753996e+03 },
    { 7.2600000000000002e-09, 2.0943357626800001e+00, 2.1510071892927999e+03 },
    { 7.2500000000000004e-09, 5.3245023398200004e+00, 1.7224529849340001e+02 },
    { 7.2300000000000001e-09, 3.1726564192700000e+00, 3.3334988796990001e+03 },
    { 7.2099999999999997e-09, 4.9346306208600001e+00, 6.4959738580800007e+01 },
    { 7.2000000000000000e-09, 5.0189286563100000e+00, 8.7832616366560001e+02 },
    { 7.1699999999999998e-09, 5.7231204836899998e+00, 9.8899988524600005e+01 },
    { 7.1699999999999998e-09, 1.7823751300799999e+00, 9.0562601246400007e+01 },
    { 7.1500000000000003e-09, 6.0773911138700001e+00, 3.3477259737006002e+03 },
    { 7.0900000000000001e-09, 4.5974093444099999e+00, 3.3914448860056000e+03 },
    { 7.0699999999999998e-09, 3.8479744900000001e-02, 2.2186752408593002e+03 },
    { 7.0500000000000003e-09, 4.2710602504599997e+00, 2.5976951835400001e+02 },
    { 7.0500000000000003e-09, 6.1237443662300004e+00, 6.5716276170139997e+02 },
    { 7.0399999999999997e-09, 4.8325377506000000e-01, 7.1861103814440003e+02 },
    { 7.0299999999999999e-09, 1.3732411730600000e+00, 7.3832419408659996e+02 },
    { 6.9999999999999998e-09, 5.5985249487399997e+00, 4.4386366626339998e+02 },
    { 6.9900000000000001e-09, 3.6117043886400002e+00, 9.9957818514890005e+02 },
    { 6.9800000000000003e-09, 5.0428184833299996e+00, 1.3140394986990000e+02 },
    { 6.9399999999999996e-09, 4.9770561192300002e+00, 1.9677161905354999e+03 },
    { 6.8400000000000004e-09, 2.8121237391000000e+00, 4.2511371816769997e+02 },
    { 6.8200000000000000e-09, 2.6050257479000000e-01, 2.2257887878601000e+03 },
    { 6.6999999999999996e-09, 1.3603870588700000e+00, 1.0017201291180000e+02 },
    { 6.6599999999999997e-09, 3.0481422305899999e+00, 2.4390878832981002e+03 },
    { 6.6199999999999999e-09, 4.8157920190399999e+00, 1.5952926704073000e+03 },
    { 6.6100000000000001e-09, 2.9017865189099998e+00, 7.0045179087969996e+02 },
    { 6.6000000000000004e-09, 3.2349572737700001e+00, 4.3121663069721999e+03 },
    { 6.5899999999999998e-09, 5.7597194808000003e+00, 4.2161870797543997e+03 },
    { 6.5700000000000003e-09, 6.0457512346999998e-01, 1.7244089984167999e+03 },
    { 6.5700000000000003e-09, 2.1978003452700001e+00, 4.6473122651379998e+02 },
    { 6.5700000000000003e-09, 1.9707303185800000e+00, 2.3508129039121000e+03 },
    { 6.5300000000000004e-09, 1.3032017967200000e+00, 1.0110192771992401e+04 },
    { 6.5199999999999998e-09, 4.1686245555000001e+00, 9.0684978763799995e+02 },
    { 6.5000000000000003e-09, 2.7374933437100002e+00, 5.5814515309779995e+02 },
    { 6.4499999999999999e-09, 7.5334618858000002e-01, 1.7515395314160000e+03 },
    { 6.4499999999999999e-09, 1.8277724678100000e+00, 1.2747179660680000e+02 },
    { 6.4300000000000003e-09, 3.3003051250900000e+00, 2.8245161971279998e+02 },
    { 6.3899999999999996e-09, 8.1563103959999994e-02, 6.5094298657790000e+02 },
    { 6.3799999999999999e-09, 1.1361616532500001e+00, 1.6889842757647000e+03 },
    { 6.3600000000000004e-09, 8.0538758265999999e-01, 6.6427630870220003e+02 },
    { 6.2799999999999998e-09, 3.9726893538399999e+00, 1.4080171287602000e+03 },
    { 6.2499999999999997e-09, 2.8922217966699999e+00, 1.0529677415867600e+04 },
    { 6.2099999999999999e-09, 2.8713834447800002e+00, 1.1976754877499999e+02 },
    { 6.2000000000000001e-09, 9.3234202068000005e-01, 3.0465988283458000e+03 },
    { 6.1900000000000003e-09, 6.9084255400000005e-01, 1.1443154571431000e+03 },
    { 6.1900000000000003e-09, 4.3715214211899998e+00, 3.2595309721220002e+02 },
    { 6.1700000000000000e-09, 3.7448512237300000e+00, 9.2086443331850001e+02 },
    { 6.1700000000000000e-09, 3.0504803394300000e+00, 1.5606187072805999e+03 },
    { 6.1399999999999999e-09, 3.6908086144699999e+00, 6.1216518446948003e+03 },
    { 6.1300000000000001e-09, 2.0378897530300000e+00, 7.4543774108740001e+02 },
    { 6.1300000000000001e-09, 2.4884161564600000e+00, 1.4575259330619999e+03 },
    { 6.1300000000000001e-09, 1.7307236499600001e+00, 6.0765395739960002e+02 },
    { 6.1099999999999998e-09, 3.2273834367699998e+00, 4.4437132691939996e+03 },
    { 6.1099999999999998e-09, 2.0045997747600000e+00, 3.9265794093220001e+02 },
    { 6.1099999999999998e-09, 2.3360858736100001e+00, 6.6672398925699997e+02 },
    { 6.1099999999999998e-09, 6.0472786279399999e+00, 1.3662125722901999e+03 },
    { 6.0699999999999999e-09, 3.0432184347500000e+00, 1.6708250284999999e+03 },
    { 6.0699999999999999e-09, 1.4707099790800000e+00, 6.5994677196479997e+03 },
    { 6.0600000000000002e-09, 6.0547852239599997e+00, 1.8094855377225999e+03 },
    { 6.0500000000000004e-09, 5.0401862765600001e+00, 3.0061086697750000e+02 },
    { 6.0399999999999998e-09, 5.4070600495999999e-01, 4.2711945573780002e+02 },
    { 6.0200000000000003e-09, 8.1687029998000005e-01, 6.8073863493750002e+02 },
    { 6.0200000000000003e-09, 1.8909665527299999e+00, 3.5452490529440001e+02 },
    { 6.0200000000000003e-09, 2.4497190956399999e+00, 7.0485702489480002e+02 },
    { 6.0099999999999997e-09, 2.0123476880000001e-02, 4.2607692601420001e+02 },
    { 6.0000000000000000e-09, 7.7520117110999998e-01, 1.9093969182035000e+03 },
    { 6.0000000000000000e-09, 1.5725840238600000e+00, 6.3426821202149995e+02 },
    { 5.9900000000000002e-09, 4.1487847353399996e+00, 1.8498791978669999e+02 },
    { 5.9699999999999999e-09, 3.8553497717000001e-01, 3.1843655656911001e+03 },
    { 5.9600000000000001e-09, 2.5757557889100000e+00, 7.3736098624010003e+02 },
    { 5.9500000000000003e-09, 2.1504681870600000e+00, 1.7463403402501999e+03 },
    { 5.9200000000000002e-09, 1.6792679447200001e+00, 7.9961241183519996e+02 },
    { 5.9099999999999997e-09, 3.4817821113099998e+00, 5.7604984318976003e+03 },
    { 5.8999999999999999e-09, 5.7661516900700001e+00, 3.6346210245184002e+03 },
    { 5.8999999999999999e-09, 4.5577313861600004e+00, 6.2736711334179995e+02 },
    { 5.8900000000000001e-09, 5.5495059581400001e+00, 2.0863322899200000e+02 },
    { 5.8900000000000001e-09, 3.8514838418299999e+00, 2.0053761454213000e+03 },
    { 5.8900000000000001e-09, 1.8391885135899999e+00, 2.7405018994277002e+03 },
    { 5.8800000000000004e-09, 4.8807716857900001e+00, 9.6907081281171995e+03 },
    { 5.8800000000000004e-09, 1.6775709575700000e+00, 4.5809076026499997e+02 },
    { 5.8699999999999998e-09, 4.6205537811699999e+00, 1.7842127434571000e+03 },
    { 5.8500000000000003e-09, 3.3129526251699999e+00, 1.7913262904579001e+03 },
    { 5.8200000000000002e-09, 5.4318198686499999e+00, 2.9538009948657000e+03 },
    { 5.8200000000000002e-09, 4.7843699684900001e+00, 5.4817545583807996e+03 },
    { 5.7999999999999998e-09, 1.2493880340400001e+00, 5.1795897827258004e+03 },
    { 5.7900000000000001e-09, 2.3689221312000000e-01, 1.8180652604900001e+02 },
    { 5.7900000000000001e-09, 5.5731676146800000e+00, 4.7529915918498000e+03 },
    { 5.7900000000000001e-09, 2.2868460241699999e+00, 5.9757146498220004e+02 },
    { 5.7900000000000001e-09, 2.0538396242000001e+00, 4.6181046520699999e+02 },
    { 5.7800000000000003e-09, 6.8382188540000000e-01, 8.8150755740329998e+02 },
    { 5.7600000000000000e-09, 4.3306304577799999e+00, 4.8418572720667999e+03 },
    { 5.7500000000000002e-09, 1.4073851992999999e+00, 1.6567409468502999e+03 },
    { 5.7299999999999999e-09, 1.3792415466100001e+00, 1.1326790560482000e+03 },
    { 5.6900000000000000e-09, 4.9968489806900003e+00, 2.9201284726839998e+02 },
    { 5.6800000000000002e-09, 2.9749349628099999e+00, 4.8042092759269999e+03 },
    { 5.6599999999999999e-09, 2.7263318555599998e+00, 5.3715482371613998e+03 },
    { 5.6299999999999998e-09, 3.9518750685100001e+00, 2.4169964827703002e+03 },
    { 5.6200000000000000e-09, 4.8083792934000003e-01, 1.8589943420244001e+03 },
    { 5.5999999999999997e-09, 4.7104503767399999e+00, 4.2096911658350001e+02 },
    { 5.5899999999999999e-09, 5.2799219942000004e+00, 3.8872578766909999e+02 },
    { 5.5800000000000002e-09, 2.1428872665599998e+00, 5.9465070367540000e+02 },
    { 5.5800000000000002e-09, 1.0298723201600000e+00, 2.3478439584954999e+03 },
    { 5.5700000000000004e-09, 4.0305167397800004e+00, 2.0426571092422000e+04 },
    { 5.5500000000000001e-09, 2.4126425202999999e+00, 1.3797059531089001e+03 },
    { 5.5400000000000003e-09, 4.2404149372199997e+00, 3.8575684225250001e+02 },
    { 5.5299999999999997e-09, 4.4395214498000002e-01, 3.9957744373156002e+03 },
    { 5.5200000000000000e-09, 3.5292885266799998e+00, 2.0373786788240000e+02 },
    { 5.4899999999999999e-09, 2.4677263671500000e+00, 6.7025604938666002e+03 },
    { 5.4899999999999999e-09, 5.4309212070999999e+00, 8.8375117463652005e+03 },
    { 5.4500000000000000e-09, 5.9069468633900000e+00, 2.0767002114549999e+02 },
    { 5.4299999999999997e-09, 5.6381517839799997e+00, 5.0622699145056004e+03 },
    { 5.4299999999999997e-09, 1.5669159539299999e+00, 5.7844900987100004e+02 },
    { 5.4299999999999997e-09, 2.7737696944399999e+00, 4.8093292031819999e+02 },
    { 5.4299999999999997e-09, 5.9746982830500004e+00, 3.8309671337660001e+02 },
    { 5.3800000000000000e-09, 9.2867899703000001e-01, 7.5648307207380003e+03 },
    { 5.3800000000000000e-09, 3.8915574688399999e+00, 1.0632770190086199e+04 },
    { 5.3700000000000003e-09, 2.7020999889500001e+00, 2.0522234059069999e+02 },
    { 5.3700000000000003e-09, 4.3985677795899996e+00, 8.7344189721465991e+03 },
    { 5.3599999999999997e-09, 6.1517068874000003e-01, 4.7011165017084004e+03 },
    { 5.3400000000000002e-09, 4.2804092977500003e+00, 8.4363515419639998e+02 },
    { 5.2700000000000002e-09, 1.5994444842500000e+00, 1.3364169239306000e+03 },
    { 5.2499999999999999e-09, 2.6746062201999998e+00, 7.6674636529438780e+04 },
    { 5.2000000000000002e-09, 6.1291304733000000e+00, 5.8564776591153995e+03 },
    { 5.1899999999999997e-09, 5.1443621129299997e+00, 3.9814400287280000e+02 },
    { 5.1700000000000001e-09, 5.2759365553000004e+00, 6.0185590704761998e+03 },
    { 5.1600000000000004e-09, 2.6350469761000000e-01, 5.5990744266009997e+03 },
    { 5.1600000000000004e-09, 2.2590098116999999e-01, 1.0457724692690000e+02 },
    { 5.1600000000000004e-09, 4.7650921109000004e+00, 4.1943645976540000e+02 },
    { 5.1499999999999998e-09, 6.1366361755899996e+00, 1.0908744995781999e+03 },
    { 5.0899999999999996e-09, 1.4288365887600001e+00, 4.8631979130691998e+03 },
    { 5.0799999999999998e-09, 2.0792996237199999e+00, 7.3683972137830006e+02 },
    { 5.0700000000000001e-09, 5.0734615719700003e+00, 6.1551826392580006e+02 },
    { 5.0700000000000001e-09, 4.9105572748300004e+00, 2.2036973873323000e+03 },
    { 5.0400000000000000e-09, 2.3004563656400001e+00, 1.9903982918943000e+03 },
    { 5.0400000000000000e-09, 5.2935824057499996e+00, 5.6502921106781996e+03 },
    { 5.0000000000000001e-09, 7.7202686942999998e-01, 1.9731328165069001e+03 },
    { 4.9700000000000000e-09, 4.9075913061499996e+00, 1.0234201469320000e+02 },
    { 4.9600000000000002e-09, 5.9535165348900003e+00, 4.3192798539730002e+03 },
    { 4.9600000000000002e-09, 4.4132567050100002e+00, 2.7586611466924001e+03 },
    { 4.9499999999999997e-09, 1.7470093004800000e+00, 4.1953282798499998e+02 },
    { 4.9499999999999997e-09, 3.1515155253299998e+00, 3.9828701522469999e+02 },
    { 4.9399999999999999e-09, 1.7691775323000000e-01, 5.3605375257000003e+02 },
    { 4.9399999999999999e-09, 1.0221752415500001e+00, 5.2332817761920001e+02 },
    { 4.9200000000000004e-09, 1.6485769060400000e+00, 1.3654788591073000e+03 },
    { 4.9099999999999998e-09, 2.6631170131799999e+00, 6.0129116992419995e+02 },
    { 4.8900000000000003e-09, 5.2681941610900003e+00, 2.4241100297711000e+03 },
    { 4.8799999999999997e-09, 5.5360549457100001e+00, 5.7021672008196001e+03 },
    { 4.8699999999999999e-09, 2.2743391542300002e+00, 3.2105773610260002e+02 },
    { 4.8300000000000001e-09, 5.6109229387699999e+00, 1.5429849954980000e+02 },
    { 4.8200000000000003e-09, 2.3608482841499998e+00, 1.0384353374400000e+02 },
    { 4.8200000000000003e-09, 6.1900701165800003e+00, 7.3342883297699996e+02 },
    { 4.8200000000000003e-09, 1.9606116279500001e+00, 8.6313261979279996e+03 },
    { 4.8200000000000003e-09, 4.2323321860399998e+00, 6.1145382976940000e+03 },
    { 4.8200000000000003e-09, 6.2126501685099997e+00, 2.5028521211384999e+04 },
    { 4.8200000000000003e-09, 8.3788877001999995e-01, 5.2175806283148399e+04 },
    { 4.8000000000000000e-09, 4.5242781897700004e+00, 2.1329909543799999e+03 },
    { 4.8000000000000000e-09, 5.1541399676799999e+00, 1.1058523531052001e+03 },
    { 4.7699999999999999e-09, 1.1512270325600000e+00, 3.2598979237837998e+03 },
    { 4.7600000000000001e-09, 2.0616725378999998e+00, 5.5848473325994000e+03 },
    { 4.7399999999999998e-09, 3.5023297708000001e-01, 2.8790193962983999e+03 },
    { 4.7300000000000000e-09, 5.7606261015499998e+00, 1.6740064222377000e+03 },
    { 4.7200000000000002e-09, 5.6454626690399996e+00, 4.1677633088950000e+02 },
    { 4.7099999999999997e-09, 4.4572545738200002e+00, 2.5721887541974002e+03 },
    { 4.6900000000000001e-09, 6.3364469059999995e-01, 1.8951698242018999e+03 },
    { 4.6800000000000004e-09, 3.3922584751199998e+00, 4.7700076705510003e+02 },
    { 4.6800000000000004e-09, 9.4842717325000003e-01, 5.8238116313410001e+02 },
    { 4.6800000000000004e-09, 2.0613896593800001e+00, 2.0479144150742000e+03 },
    { 4.6800000000000004e-09, 5.8527867785899996e+00, 1.9706851359520999e+03 },
    { 4.6699999999999998e-09, 4.4909285037000002e-01, 5.2826825569443999e+03 },
    { 4.6600000000000000e-09, 6.0089115860999998e-01, 2.6452734317353002e+03 },
    { 4.6600000000000000e-09, 3.7999845821200000e+00, 6.3226247445139995e+02 },
    { 4.6600000000000000e-09, 1.7111224133100000e+00, 2.8332997329077998e+03 },
    { 4.6500000000000003e-09, 3.7677429761400001e+00, 1.0837609525774001e+03 },
    { 4.6500000000000003e-09, 7.6802832089999995e-02, 2.0714875628370001e+02 },
    { 4.6399999999999997e-09, 2.0379824912500002e+00, 1.4434418514122999e+03 },
    { 4.6299999999999999e-09, 5.7847641825500000e+00, 4.2951895218279998e+02 },
    { 4.6299999999999999e-09, 4.3205671677000002e+00, 6.3330500417500002e+02 },
    { 4.6099999999999996e-09, 3.9395878821000002e-01, 1.7991905969841000e+03 },
    { 4.6099999999999996e-09, 4.3280745402999998e-01, 9.5805018068978006e+03 },
    { 4.5900000000000001e-09, 6.2259990128800000e+00, 4.1130943055358002e+03 },
    { 4.5800000000000003e-09, 3.5302112602100002e+00, 4.1273213995373999e+03 },
    { 4.5100000000000003e-09, 2.8632608728400002e+00, 2.5272028039897000e+03 },
    { 4.4999999999999998e-09, 3.0282559978300001e+00, 5.4099729778940002e+02 },
    { 4.4999999999999998e-09, 4.4540223041300004e+00, 5.1838463239980001e+02 },
    { 4.4900000000000000e-09, 4.7027124336700004e+00, 1.2218485663214001e+03 },
    { 4.4800000000000002e-09, 5.3354347729999996e-01, 1.0977078804699000e+04 },
    { 4.4699999999999997e-09, 5.4055074835500001e+00, 4.7535623565240002e+02 },
    { 4.4699999999999997e-09, 5.2183634720000001e+00, 5.8402569453679996e+02 },
    { 4.4699999999999997e-09, 2.4245776625200000e+00, 1.8837498197138189e+04 },
    { 4.4500000000000001e-09, 6.0510605612399999e+00, 2.5350671105158999e+03 },
    { 4.4500000000000001e-09, 7.1549866195000000e-01, 3.7007232086613999e+03 },
    { 4.4400000000000004e-09, 4.3147130420300002e+00, 4.2044785172169998e+02 },
    { 4.4200000000000000e-09, 1.0890800646800001e+00, 1.0689431577447999e+03 },
    { 4.3500000000000001e-09, 5.9178540882900004e+00, 4.4294861751924000e+03 },
    { 4.3500000000000001e-09, 2.6253250664699999e+00, 1.2301427559742999e+03 },
    { 4.3200000000000000e-09, 6.0097831366100003e+00, 2.0806308247406000e+03 },
    { 4.3200000000000000e-09, 1.2956776402700001e+00, 1.3009922012785000e+03 },
    { 4.2700000000000004e-09, 5.5245087868100002e+00, 3.9069087570985998e+03 },
    { 4.2100000000000001e-09, 3.1010101757899999e+00, 6.2247446189133998e+03 },
    { 4.2100000000000001e-09, 5.7457509837899998e+00, 6.3841281360569997e+02 },
    { 4.2000000000000004e-09, 5.8940280233099998e+00, 4.7458780448489997e+03 },
    { 4.1800000000000000e-09, 6.0808504223600002e+00, 2.0613736432740001e+02 },
    { 4.1499999999999999e-09, 2.9622893576600000e+00, 4.7601051388506003e+03 },
    { 4.1400000000000002e-09, 3.1623884127399999e+00, 1.2442268376239999e+03 },
    { 4.1299999999999996e-09, 7.3884800861000000e-01, 3.7824753418775999e+03 },
    { 4.1199999999999998e-09, 4.3255499792000002e-01, 4.1059807585349999e+03 },
    { 4.1199999999999998e-09, 2.1415795873499999e+00, 8.3578789401270001e+02 },
    { 4.1199999999999998e-09, 3.9581731479300002e+00, 8.3379618276399995e+01 },
    { 4.0499999999999999e-09, 5.5711445470300003e+00, 2.4248607892965001e+03 },
    { 4.0000000000000002e-09, 2.7746042465900000e+00, 2.7554797529547000e+03 },
    { 3.9899999999999997e-09, 3.1941201281799998e+00, 1.9377080938547999e+03 },
    { 3.9799999999999999e-09, 2.9518944348999998e+00, 9.4921463150048003e+03 },
    { 3.9700000000000001e-09, 1.1410695116000000e-01, 5.3339002410215999e+03 },
    { 3.9700000000000001e-09, 5.9079875700400004e+00, 5.0586733387864588e+04 },
    { 3.9600000000000004e-09, 1.9473798418599999e+00, 2.1871572140940000e+02 },
    { 3.9499999999999998e-09, 3.5541054600600002e+00, 3.6936096616606001e+03 },
    { 3.9099999999999999e-09, 5.3439581540700001e+00, 6.7065614252010005e+02 },
    { 3.8600000000000003e-09, 3.3070019592200000e+00, 6.3278373931320002e+03 },
    { 3.8400000000000000e-09, 1.0028038290000001e-01, 6.7362508793669997e+02 },
    { 3.7600000000000003e-09, 3.1684357109499999e+00, 5.9083527492568001e+03 },
    { 3.7300000000000001e-09, 1.5127345693500001e+00, 9.2641099372412009e+03 },
};

static const VSOP87Term VSOP87A_JUPITER_X1[488] = {
    { 8.8238925100000004e-03, 3.1415926535900001e+00, 0.0000000000000000e+00 },
    { 6.3529717200000004e-03, 1.0662156868000000e-01, 1.0593819301891999e+03 },
    { 5.9972048200000001e-03, 2.4299667827500002e+00, 5.2257741809380002e+02 },
    { 5.8915705999999998e-03, 1.9155631463700000e+00, 5.3680451209540001e+02 },
    { 8.1697203999999996e-04, 3.4666810879700001e+00, 7.1135470007999997e+00 },
    { 4.6201898000000001e-04, 4.5714214032000000e-01, 1.5890728952838001e+03 },
    { 3.3891192999999998e-04, 4.1011348275200001e+00, 5.2969096509459996e+02 },
    { 3.2508590000000000e-04, 1.7464884992800001e+00, 1.0522683831884001e+03 },
    { 3.1234303000000002e-04, 2.3469805150199998e+00, 1.0664954771900000e+03 },
    { 2.1244362999999999e-04, 4.3657617895299996e+00, 1.1020632121940000e+02 },
    { 1.8156701000000001e-04, 4.0057223877899997e+00, 4.2659819087599999e+02 },
    { 1.3577576000000001e-04, 3.0008010246000000e-01, 6.3278373931320004e+02 },
    { 1.2889505000000001e-04, 2.5748929406199998e+00, 5.1546387109299997e+02 },
    { 9.1258749999999998e-05, 1.7808246996199999e+00, 5.4391805909619995e+02 },
    { 8.0859910000000002e-05, 6.1613651890199996e+00, 9.4917560896980001e+02 },
    { 7.1425469999999994e-05, 3.1726780120300000e+00, 3.2350541665740002e+02 },
    { 4.3939770000000000e-05, 1.1477078806300001e+00, 7.3587651353180001e+02 },
    { 4.2922400000000001e-05, 4.7497062665499996e+00, 2.0618554843720000e+02 },
    { 3.8734669999999998e-05, 3.3364887010099999e+00, 1.4227094001599999e+01 },
    { 3.3991639999999998e-05, 2.9009145074700000e+00, 5.2650957135689998e+02 },
    { 3.3333439999999998e-05, 1.4369165296699999e+00, 5.3287235883230005e+02 },
    { 3.0444080000000001e-05, 1.6542804866900001e+00, 5.2575881183150000e+02 },
    { 3.0018740000000001e-05, 2.6837698274599999e+00, 5.3362311835770004e+02 },
    { 2.9902450000000000e-05, 8.0692155638999996e-01, 2.1187638603783998e+03 },
    { 2.9333589999999999e-05, 2.6189985500500002e+00, 4.1948464387519999e+02 },
    { 2.8042179999999999e-05, 4.8974259132000002e+00, 1.0309277421860000e+02 },
    { 2.4381990000000001e-05, 3.6065564453699999e+00, 3.1639186965660002e+02 },
    { 1.9775719999999999e-05, 5.0891548908799997e+00, 9.5628915597059995e+02 },
    { 1.9040670000000000e-05, 1.8593787370299999e+00, 1.5819593482830001e+03 },
    { 1.8536790000000001e-05, 2.7694100174699998e+00, 1.5961864422845999e+03 },
    { 1.8129650000000000e-05, 3.8460214874699998e+00, 9.5979227217800002e+01 },
    { 1.7728000000000000e-05, 7.2631739445999999e-01, 7.4299006053259995e+02 },
    { 1.6323619999999999e-05, 1.4150421240800000e+00, 1.0451548361876000e+03 },
    { 1.5392119999999999e-05, 1.4789917282100000e+00, 6.3989728631399998e+02 },
    { 1.5329449999999999e-05, 4.3155671450100002e+00, 1.1731986822019999e+02 },
    { 1.0554219999999999e-05, 2.5084422297700000e+00, 1.2655674786264001e+03 },
    { 1.0238120000000001e-05, 2.5718269771500002e+00, 4.3371173787679999e+02 },
    { 9.8573300000000000e-06, 1.4274683426500001e+00, 1.1695882514085999e+03 },
    { 9.8177499999999993e-06, 2.1880002261399998e+00, 2.2041264243879999e+02 },
    { 9.4009399999999995e-06, 1.3487301447300000e+00, 6.2567019231239999e+02 },
    { 8.3971199999999993e-06, 6.2053487161199996e+00, 9.4206206196899996e+02 },
    { 7.7893899999999996e-06, 5.4932353368299998e+00, 3.0927832265580003e+02 },
    { 7.6519200000000001e-06, 1.9689206785600000e+00, 1.1553611574070001e+03 },
    { 7.3437800000000002e-06, 1.1449859192000000e-01, 1.1624747044077999e+03 },
    { 6.4397499999999996e-06, 4.2583878498800001e+00, 2.1329909543799999e+02 },
    { 5.3831500000000003e-06, 4.2457528015000001e+00, 8.5319638175199998e+02 },
    { 5.0190300000000001e-06, 4.8138672150800002e+00, 1.9907200143640000e+02 },
    { 4.9987299999999997e-06, 3.0204173565899999e+00, 3.3061896365820002e+02 },
    { 4.7142599999999997e-06, 5.9121318041900004e+00, 1.6921656695024001e+03 },
    { 4.2818100000000001e-06, 3.2006578448999998e+00, 4.1237109687440000e+02 },
    { 3.9231400000000000e-06, 6.2104273422200000e+00, 1.4788665740644001e+03 },
    { 3.8379299999999997e-06, 2.1714385466600001e+00, 1.0736090241908000e+03 },
    { 3.4935099999999999e-06, 1.4989840068000000e+00, 7.2876296653099996e+02 },
    { 3.1282899999999999e-06, 2.9231361324999998e+00, 8.3896928775039999e+02 },
    { 2.4460000000000000e-06, 1.7602488974799999e+00, 1.2726810256271999e+03 },
    { 2.2873199999999998e-06, 2.8490176649699999e+00, 1.3757737998458001e+03 },
    { 2.2720000000000000e-06, 2.7383903950900002e+00, 5.0835032409220003e+02 },
    { 2.2154000000000000e-06, 3.0981926040099999e+00, 3.1813937377000001e+00 },
    { 1.8949200000000000e-06, 1.1574227092000000e+00, 2.6484548254729998e+03 },
    { 1.8077900000000000e-06, 3.6499060164400001e+00, 1.3686602528450001e+03 },
    { 1.7394000000000000e-06, 1.8532639085000000e+00, 1.0625633239269000e+03 },
    { 1.6085600000000001e-06, 3.0274460008999999e+00, 5.1939602435610004e+02 },
    { 1.4670600000000001e-06, 2.7243058052800002e+00, 1.0562005364515001e+03 },
    { 1.4532999999999999e-06, 6.0820873475399999e+00, 1.4859801210651999e+03 },
    { 1.4431000000000000e-06, 2.0846379434300002e+00, 1.0554497769261000e+03 },
    { 1.3131400000000000e-06, 2.9789815738500001e+00, 1.2584539316256000e+03 },
    { 1.2562900000000001e-06, 3.3185085511199999e+00, 1.0633140834523001e+03 },
    { 1.2516199999999999e-06, 1.3155846925300001e+00, 5.3998590583309999e+02 },
    { 1.2485199999999999e-06, 1.6203103651599999e+00, 5.5103160609700001e+02 },
    { 1.1850000000000001e-06, 5.9627059237199997e+00, 3.9321532631000000e+00 },
    { 1.1591099999999999e-06, 3.1814775430500002e+00, 2.1258774073792001e+03 },
    { 1.1422000000000001e-06, 2.0569891417899999e+00, 2.2752618943959999e+02 },
    { 1.0693800000000000e-06, 1.6788575840800000e+00, 1.5748458012822000e+03 },
    { 1.0341200000000001e-06, 2.0292676193400001e+00, 2.1116503133776000e+03 },
    { 1.0254800000000000e-06, 3.1792897330500001e+00, 2.1340641002400002e+01 },
    { 9.7462000000000008e-07, 3.8358128621400001e+00, 8.4608283475120004e+02 },
    { 9.6362000000000010e-07, 5.7096959738799997e+00, 3.0216477565500003e+02 },
    { 9.1984000000000001e-07, 1.8594696442000000e+00, 1.6992792165031999e+03 },
    { 8.2834000000000002e-07, 1.2875470849599999e+00, 4.4082528487759998e+02 },
    { 7.8205999999999995e-07, 5.0927277232400003e+00, 5.2502509864859996e+02 },
    { 7.4807000000000000e-07, 5.5286779102199999e+00, 5.3435683154059996e+02 },
    { 7.4173000000000001e-07, 1.5770391229700000e+00, 6.4701083331480004e+02 },
    { 6.6588999999999999e-07, 2.6339876231199999e+00, 1.7952584437210000e+03 },
    { 6.6496000000000005e-07, 2.2857123373000001e+00, 1.6850521225016000e+03 },
    { 6.0063000000000000e-07, 6.2396518031800001e+00, 2.2218566345969998e+03 },
    { 5.8286000000000002e-07, 3.0625893351200002e+00, 8.3185574074960005e+02 },
    { 5.2157999999999999e-07, 1.5249576333799999e+00, 7.5010360753340001e+02 },
    { 5.1890999999999998e-07, 1.3753818117700001e+00, 1.0380412891868000e+03 },
    { 4.9429999999999997e-07, 2.7565774354700001e+00, 9.6340270297140000e+02 },
    { 4.8729000000000002e-07, 5.8382094896599996e+00, 2.0085575391590000e+03 },
    { 4.7789000000000005e-07, 1.2453011290600000e+00, 1.4717530270636000e+03 },
    { 4.6409000000000002e-07, 2.4983268436500001e+00, 5.2427433912319998e+02 },
    { 4.5782000000000001e-07, 1.8423588188500000e+00, 5.3510759106600005e+02 },
    { 4.3686000000000001e-07, 4.0678646558999998e-01, 1.4956319713459999e+02 },
    { 4.2921999999999997e-07, 3.1324137377099999e+00, 6.1855664531160005e+02 },
    { 4.0582000000000002e-07, 4.8305732261100003e+00, 1.3828873468465999e+03 },
    { 3.7950999999999999e-07, 1.2700277961899999e+00, 5.1153171782990000e+02 },
    { 3.6867999999999999e-07, 1.5399582421900000e+00, 5.2677020378780003e+02 },
    { 3.6601999999999999e-07, 2.8009552425700002e+00, 5.3261172640140001e+02 },
    { 3.6521999999999998e-07, 3.0850035833999998e+00, 5.4785021235930003e+02 },
    { 3.5242999999999999e-07, 2.8011885383199999e+00, 1.0294940738499999e+01 },
    { 3.3062999999999999e-07, 3.4414131281400002e+00, 1.3615467058442000e+03 },
    { 3.0442000000000002e-07, 4.1408637058000002e+00, 1.2443341522100000e+02 },
    { 3.0104000000000000e-07, 6.0047872583800004e+00, 9.3494851496820002e+02 },
    { 2.9742999999999998e-07, 3.0999094485300001e+00, 1.9054647649404001e+03 },
    { 2.8717000000000000e-07, 5.5505619678200002e+00, 1.1045700263900001e+01 },
    { 2.7711000000000002e-07, 2.7327466606400002e+00, 1.7881448967202000e+03 },
    { 2.7352999999999999e-07, 1.6821416584900000e+00, 1.8023719907218001e+03 },
    { 2.5726000000000002e-07, 2.4446208841299999e+00, 1.8983512179396000e+03 },
    { 2.5166999999999998e-07, 1.8044350353900001e+00, 5.4073666535849998e+02 },
    { 2.4774999999999998e-07, 2.9932742360500000e+00, 9.9160620955499994e+01 },
    { 2.4465000000000001e-07, 3.4366776924900000e+00, 1.0627416795630000e+02 },
    { 2.2193000000000000e-07, 3.5919389107700002e+00, 1.1482476104062000e+03 },
    { 1.8419000000000001e-07, 4.5830047829899998e+00, 1.0702492748170000e+02 },
    { 1.8077000000000000e-07, 3.9250457142399999e+00, 9.0981873305459999e+02 },
    { 1.7783000000000000e-07, 3.5333771889299999e+00, 8.6030992875280003e+02 },
    { 1.7546000000000001e-07, 4.4161533817000000e+00, 1.8159247264699999e+01 },
    { 1.7538000000000000e-07, 6.0415215609299997e+00, 5.2549817940059995e+02 },
    { 1.7452999999999999e-07, 1.0706951113000000e-01, 7.6266071275599998e+01 },
    { 1.7368000000000000e-07, 4.5821966817700002e+00, 5.3388375078859997e+02 },
    { 1.7147000000000001e-07, 4.4287741204999997e-01, 8.8865680217000005e+01 },
    { 1.7088999999999999e-07, 2.5612591173200001e+00, 1.6032999892854000e+03 },
    { 1.6173000000000001e-07, 4.3125221099199997e+00, 9.9911380480899993e+01 },
    { 1.6077999999999999e-07, 4.9719733835900000e+00, 1.9195845443560000e+02 },
    { 1.5125999999999999e-07, 1.8083147943200000e+00, 5.1864526483069994e+02 },
    { 1.4850000000000001e-07, 2.0839645199499999e+00, 1.0490869894507000e+03 },
    { 1.4698999999999999e-07, 2.8589676064999998e+00, 3.3773251065900001e+02 },
    { 1.4282999999999999e-07, 2.2612856498500000e+00, 2.1045367663768002e+03 },
    { 1.2875000000000001e-07, 1.4631500224600000e+00, 7.2164941953020002e+02 },
    { 1.1691000000000001e-07, 1.5011018815699999e+00, 3.1781457905676002e+03 },
    { 1.1491000000000000e-07, 3.6720505257900000e+00, 1.8912376709388000e+03 },
    { 1.1346000000000000e-07, 1.8339153343500001e+00, 2.2147430875962000e+03 },
    { 1.1042000000000001e-07, 4.7580028707400004e+00, 6.0447256366190004e+02 },
    { 1.0955000000000000e-07, 5.2903548096999997e-01, 2.0156710861598001e+03 },
    { 1.0898000000000000e-07, 3.8339492640700001e+00, 1.1338771495710000e+02 },
    { 1.0134000000000001e-07, 2.2601342898699999e+00, 1.5922542890215000e+03 },
    { 1.0028000000000000e-07, 5.5184535001199997e+00, 4.9744763618019999e+02 },
    { 1.0023000000000000e-07, 2.7917473176600001e+00, 4.2266603761290003e+02 },
    { 9.2890000000000007e-08, 4.0149102428400001e+00, 4.2341679713830001e+02 },
    { 9.2890000000000007e-08, 5.0959822161700004e+00, 5.6193429400900004e+02 },
    { 8.9550000000000004e-08, 5.9244301069300001e+00, 4.5490936652729999e+02 },
    { 8.8500000000000005e-08, 3.2954129243099999e+00, 9.2047073954699997e+01 },
    { 8.6210000000000002e-08, 3.7560813917200000e+00, 1.4646394800628000e+03 },
    { 8.4450000000000005e-08, 2.6465248059800000e+00, 2.2434479570190001e+02 },
    { 8.3680000000000006e-08, 2.5348440284799998e+00, 1.5851407420206999e+03 },
    { 8.0900000000000003e-08, 2.8758055145000001e+00, 1.5858915015461000e+03 },
    { 7.8699999999999997e-08, 2.5542414583299999e+00, 4.1555249061209997e+02 },
    { 7.8470000000000001e-08, 4.7229152807499997e+00, 1.1413847448250000e+02 },
    { 7.7089999999999997e-08, 3.6575346874800001e+00, 4.0525754987360000e+02 },
    { 7.6280000000000005e-08, 2.3792377412899999e+00, 2.0014439921582000e+03 },
    { 7.5600000000000002e-08, 1.3435919128600000e+00, 6.3671589257630001e+02 },
    { 7.5069999999999997e-08, 5.3848238442299996e+00, 1.2125202148330000e+02 },
    { 7.4579999999999997e-08, 2.9675865852699999e+00, 1.6779385755008000e+03 },
    { 7.3580000000000001e-08, 3.6482248533799999e+00, 2.6555683724738001e+03 },
    { 7.3560000000000005e-08, 1.9054339846399999e+00, 3.9356875915200000e+01 },
    { 7.3339999999999994e-08, 5.4794481470600003e+00, 5.3044172461999995e+02 },
    { 7.3339999999999994e-08, 5.1444228084899999e+00, 5.2894020556919997e+02 },
    { 7.2919999999999994e-08, 2.6978094162400001e+00, 4.2977958461370002e+02 },
    { 7.2100000000000004e-08, 2.2400708338799999e+00, 2.2289701815978001e+03 },
    { 7.1799999999999994e-08, 1.5636830239199999e+00, 6.2960234557549995e+02 },
    { 6.9349999999999996e-08, 1.2704673971200000e+00, 3.8724131496080003e+02 },
    { 6.8509999999999996e-08, 2.2239992232000000e-01, 9.5310776223289997e+02 },
    { 6.7840000000000004e-08, 4.3484843112400000e+00, 3.0534616939270001e+02 },
    { 6.6880000000000001e-08, 4.1755795441699997e+00, 1.2513403846248000e+03 },
    { 6.6549999999999997e-08, 1.1537899239500000e+00, 1.1585425511446999e+03 },
    { 6.6430000000000007e-08, 3.4232458487200002e+00, 4.3053034413910001e+02 },
    { 6.5449999999999994e-08, 1.0032284925000000e-01, 6.3596513305090002e+02 },
    { 6.4679999999999994e-08, 8.3651032578999995e-01, 1.1656560981455000e+03 },
    { 6.4469999999999994e-08, 4.2380199748600003e+00, 9.8311585891360005e+02 },
    { 6.4399999999999994e-08, 5.0632321692500000e+00, 1.0483362299252999e+03 },
    { 6.4130000000000006e-08, 2.3153075552900000e+00, 1.1592933106701000e+03 },
    { 6.1179999999999996e-08, 2.6944534153799999e+00, 3.1245971639350000e+02 },
    { 6.0090000000000004e-08, 1.0875806681000000e-01, 1.0412226829245001e+03 },
    { 6.0069999999999995e-08, 3.0197494591000001e-01, 2.7515475996916002e+03 },
    { 5.9730000000000006e-08, 2.7615263546800000e+00, 3.1957326339430000e+02 },
    { 5.9419999999999999e-08, 3.3258519234900001e+00, 5.1228247735529999e+02 },
    { 5.9170000000000000e-08, 4.1224010594599996e+00, 4.5884151979040001e+02 },
    { 5.8420000000000003e-08, 2.0761475252700001e+00, 1.1664068576709001e+03 },
    { 5.6599999999999997e-08, 6.0613810414799998e+00, 2.5382485042536000e+03 },
    { 5.6120000000000002e-08, 4.4133608597200000e+00, 4.3764389113990001e+02 },
    { 5.5910000000000002e-08, 1.9103376080700001e+00, 2.3463973644039999e+02 },
    { 5.5810000000000001e-08, 1.6658804653499999e+00, 7.0849445304200003e+01 },
    { 5.5190000000000000e-08, 3.6409968504200001e+00, 1.5930050485469001e+03 },
    { 5.3060000000000000e-08, 1.6904937560400000e+00, 1.0696768709277001e+03 },
    { 5.3060000000000000e-08, 5.4548610589199997e+00, 2.1011770170029999e+02 },
    { 5.2859999999999998e-08, 4.4626584379300001e+00, 5.0531194270639998e+02 },
    { 5.2780000000000000e-08, 3.3919752357099999e+00, 1.7810313497193999e+03 },
    { 5.2310000000000003e-08, 3.5504887643399998e+00, 2.2076295405954002e+03 },
    { 5.2029999999999997e-08, 1.8796267814000001e+00, 4.9867147645760002e+02 },
    { 5.1889999999999997e-08, 9.1840951510000002e-01, 4.4793883187839998e+02 },
    { 5.0519999999999998e-08, 5.7298078947999997e-01, 9.4599421523210003e+02 },
    { 4.9719999999999998e-08, 6.1612127884000003e+00, 5.5406998748280000e+02 },
    { 4.8960000000000003e-08, 5.9400405322700003e+00, 2.9505122865419997e+02 },
    { 4.8119999999999997e-08, 2.4610593064600002e+00, 5.6071045373159996e+02 },
    { 4.8070000000000000e-08, 1.8394235829400001e+00, 4.9426624244250002e+02 },
    { 4.7810000000000003e-08, 2.3036668649000000e-01, 6.0054041039879996e+02 },
    { 4.6229999999999998e-08, 6.1864666117400002e+00, 6.2173803904930003e+02 },
    { 4.6049999999999999e-08, 9.4900742335999999e-01, 6.7925416222920001e+02 },
    { 4.5559999999999999e-08, 8.4893193977000003e-01, 6.2885158605009997e+02 },
    { 4.5160000000000002e-08, 1.6747536925099999e+00, 7.5721715453419995e+02 },
    { 4.3970000000000003e-08, 2.5012932603800002e+00, 5.6511568774670002e+02 },
    { 4.3259999999999998e-08, 2.9361711893500000e+00, 6.7214061522839995e+02 },
    { 4.3180000000000000e-08, 8.2050097892999996e-01, 8.1752133216199994e+01 },
    { 4.3049999999999998e-08, 3.1729257482300000e+00, 1.0704276304530999e+03 },
    { 4.2270000000000001e-08, 2.6156262679200002e+00, 2.4209286360333999e+03 },
    { 4.2240000000000000e-08, 9.4453455733000002e-01, 5.4709945283390005e+02 },
    { 4.1479999999999998e-08, 1.9146604536800000e+00, 1.0807225711916001e+03 },
    { 4.1250000000000002e-08, 1.8563531500199999e+00, 6.4382943957709995e+02 },
    { 3.9699999999999998e-08, 2.2142359374499998e+00, 3.2243328914400003e+01 },
    { 3.9650000000000001e-08, 1.3397946256400000e+00, 5.0137978944330001e+02 },
    { 3.9629999999999998e-08, 5.1842722711700002e+00, 9.5235700270749999e+02 },
    { 3.9080000000000003e-08, 1.8394490477600001e+00, 7.4692221379570003e+02 },
    { 3.8980000000000002e-08, 5.9387613991399997e+00, 1.0640477966352000e+03 },
    { 3.8880000000000001e-08, 1.9839549420400000e+00, 2.6413412784722000e+03 },
    { 3.8799999999999997e-08, 5.2954960500199997e+00, 5.0456118318099999e+02 },
    { 3.8530000000000002e-08, 2.5948558243000002e+00, 2.3178358618148000e+03 },
    { 3.8449999999999997e-08, 1.5976712013700001e+00, 9.7051624997219994e+02 },
    { 3.8379999999999997e-08, 3.1919394643699999e+00, 1.4930936680660000e+03 },
    { 3.6930000000000000e-08, 3.0008913838300000e+00, 5.5800214074589996e+02 },
    { 3.6430000000000003e-08, 5.3283751794300001e+00, 5.5482074700819999e+02 },
    { 3.5250000000000001e-08, 3.8120932704600001e+00, 6.1144309831080000e+02 },
    { 3.5000000000000002e-08, 2.3770186352699998e+00, 2.4280421830342002e+03 },
    { 3.4959999999999997e-08, 5.3766264924199998e+00, 1.0547160637432000e+03 },
    { 3.4739999999999999e-08, 1.7608317127599999e+00, 7.3980866679489998e+02 },
    { 3.3600000000000003e-08, 2.9600003604300000e+00, 5.0123677709139997e+02 },
    { 3.3440000000000000e-08, 3.6019761945200002e+00, 6.2759623029905997e+03 },
    { 3.3430000000000002e-08, 1.4899987135199999e+00, 8.3503713448730002e+02 },
    { 3.2520000000000003e-08, 3.0573738848400001e+00, 6.2901893969922003e+03 },
    { 3.2220000000000000e-08, 5.3173534002800000e+00, 1.9125783119411999e+03 },
    { 3.2140000000000002e-08, 6.2107281286499996e+00, 4.9819839570559998e+02 },
    { 3.1940000000000000e-08, 2.8580526063399998e+00, 4.8322054217860000e+02 },
    { 3.1890000000000003e-08, 6.0232354487600004e+00, 2.1723124870109999e+02 },
    { 3.1470000000000003e-08, 4.4530277934000001e-01, 6.5412438031559998e+02 },
    { 3.1280000000000000e-08, 5.0244138565199998e+00, 1.0206171999210201e+04 },
    { 3.0769999999999997e-08, 3.6852471788500001e+00, 4.1630325013750002e+02 },
    { 3.0750000000000001e-08, 1.5626767598500000e+00, 4.6202291352809999e+02 },
    { 3.0419999999999997e-08, 4.4804091647400002e+00, 1.0220399093211799e+04 },
    { 3.0230000000000000e-08, 5.2603776765300001e+00, 9.3812990870589999e+02 },
    { 3.0059999999999999e-08, 2.0866372804400002e+00, 2.3056457082540001e+02 },
    { 2.9700000000000001e-08, 5.2615767861700000e+00, 2.0300415469949999e+02 },
    { 2.9690000000000000e-08, 4.4131432913399999e+00, 5.6118353448360006e+02 },
    { 2.9679999999999998e-08, 2.5281504264899999e+00, 5.9735901666109999e+02 },
    { 2.9309999999999999e-08, 2.7977357610500002e+00, 1.9943304451573999e+03 },
    { 2.9309999999999999e-08, 1.3259267224700000e+00, 2.5311349572528002e+03 },
    { 2.8480000000000001e-08, 2.9221985133100001e+00, 1.1767017984094000e+03 },
    { 2.8290000000000001e-08, 4.3932572215999999e+00, 3.2032402291969998e+02 },
    { 2.8040000000000002e-08, 1.4532926783300000e+00, 5.7616138801060004e+02 },
    { 2.7199999999999999e-08, 2.5801043058699999e+00, 3.2668681039510000e+02 },
    { 2.7050000000000001e-08, 1.5068383710800000e+00, 9.6022130923370003e+02 },
    { 2.7010000000000002e-08, 1.0801622989399999e+00, 1.1514290041439001e+03 },
    { 2.6720000000000000e-08, 1.9632469640000001e+00, 5.2182665856840003e+02 },
    { 2.6720000000000000e-08, 2.3774386843699999e+00, 5.3755527162080000e+02 },
    { 2.6630000000000001e-08, 3.1386339608400000e+00, 8.2474219374879999e+02 },
    { 2.6580000000000000e-08, 2.7153285687599999e+00, 1.0618296107440001e+03 },
    { 2.6580000000000000e-08, 6.4454112605000002e-01, 1.0569342496344000e+03 },
    { 2.6090000000000001e-08, 3.4077081440699999e+00, 2.5240214102519999e+03 },
    { 2.5450000000000000e-08, 6.4046730080000003e-02, 1.0987388061044001e+03 },
    { 2.5060000000000001e-08, 1.9087874937800000e+00, 2.3320629558164001e+03 },
    { 2.3879999999999999e-08, 4.2282571528900004e+00, 3.3455111692129998e+02 },
    { 2.3730000000000001e-08, 5.6485862875899997e+00, 5.2347118997109999e+02 },
    { 2.3730000000000001e-08, 4.9752846679599996e+00, 5.3591074021810005e+02 },
    { 2.3300000000000000e-08, 4.1827537736800000e+00, 8.4926422848890002e+02 },
    { 2.2919999999999999e-08, 5.4224462495200001e+00, 2.3249494088155998e+03 },
    { 2.2840000000000001e-08, 2.6882029173699999e+00, 5.2804643369190001e+02 },
    { 2.2840000000000001e-08, 1.6524827310000001e+00, 5.3133549649730003e+02 },
    { 2.2830000000000000e-08, 8.5697379379000005e-01, 7.4781598567299994e+01 },
    { 2.2560000000000002e-08, 2.9806030563600001e+00, 1.0539653042177999e+03 },
    { 2.2490000000000002e-08, 3.1821504976199999e+00, 2.0974232193759999e+03 },
    { 2.2259999999999999e-08, 2.9761616275299998e+00, 3.8012776796000003e+02 },
    { 2.2119999999999999e-08, 4.9982599462900001e+00, 5.2308074668030004e+03 },
    { 2.1990000000000000e-08, 4.9100670725200004e+00, 9.4524345570670005e+02 },
    { 2.1889999999999999e-08, 4.6850202853400003e+00, 1.5142912967165000e+03 },
    { 2.1830000000000001e-08, 3.6910397926999998e+00, 2.0936694217490000e+02 },
    { 2.1810000000000001e-08, 3.7882992625699998e+00, 2.0225339517410001e+02 },
    { 2.1600000000000002e-08, 2.6227107526700002e+00, 2.7373205056900001e+03 },
    { 2.1120000000000000e-08, 5.5183015378600002e+00, 5.2165803728013998e+03 },
    { 2.1109999999999998e-08, 3.7418808757400002e+00, 1.9513984817330001e+02 },
    { 2.1050000000000000e-08, 3.2226220091800002e+00, 1.0593337460794000e+03 },
    { 2.1050000000000000e-08, 1.3724782964000001e-01, 1.0594301142990000e+03 },
    { 2.0860000000000000e-08, 3.3123818757199999e+00, 1.0623026914960001e+03 },
    { 2.0840000000000000e-08, 5.3657462729600001e+00, 4.9033408917940000e+02 },
    { 2.0570000000000002e-08, 1.3888894147000000e-01, 9.1610171630225996e+03 },
    { 2.0330000000000001e-08, 2.0368982253699999e+00, 1.1735204046716999e+03 },
    { 2.0319999999999999e-08, 2.7405916568199999e+00, 9.8853248488500003e+02 },
    { 2.0249999999999999e-08, 5.2610296211399996e+00, 3.2743756992049998e+02 },
    { 2.0199999999999999e-08, 4.8460719014300002e+00, 5.6782400073240001e+02 },
    { 1.9969999999999999e-08, 1.5865231934199999e+00, 5.0441817082910001e+02 },
    { 1.9810000000000000e-08, 4.6965906360499998e+00, 1.1411340634054000e+03 },
    { 1.9659999999999999e-08, 6.5769985595000002e-01, 9.1467900690210008e+03 },
    { 1.9560000000000001e-08, 3.6137253793199999e+00, 4.6470422915999997e+01 },
    { 1.9390000000000000e-08, 2.2083653849100000e+00, 1.0647985561605999e+03 },
    { 1.9359999999999999e-08, 4.1714069836799998e+00, 1.8841241239379999e+03 },
    { 1.9099999999999999e-08, 5.8071704849500003e+00, 7.6025553592000006e+02 },
    { 1.9049999999999999e-08, 3.5115575155899998e+00, 1.0775411774539000e+03 },
    { 1.8990000000000000e-08, 1.2036803026600000e+00, 1.4244965013379999e+02 },
    { 1.8909999999999999e-08, 5.2600648963800003e+00, 5.6904784100979998e+02 },
    { 1.8790000000000001e-08, 3.2710372236500000e+00, 5.2820649238630006e+02 },
    { 1.8790000000000001e-08, 1.0696484247200000e+00, 5.3117543780289998e+02 },
    { 1.8740000000000001e-08, 2.7541627457800000e+00, 5.5496375936009997e+02 },
    { 1.8550000000000001e-08, 5.8018985628199999e+00, 4.9155792945680003e+02 },
    { 1.8299999999999998e-08, 2.1226266062099999e+00, 2.6342277314714001e+03 },
    { 1.8050000000000000e-08, 2.0925558055400000e+00, 4.6053844081979997e+02 },
    { 1.7940000000000001e-08, 5.5483755832500004e+00, 1.2797945726280000e+03 },
    { 1.7760000000000002e-08, 4.3408698662900003e+00, 8.4215068148809996e+02 },
    { 1.7630000000000000e-08, 6.1937918645599996e+00, 6.7668051566499997e+01 },
    { 1.7059999999999999e-08, 1.3943133227000001e-01, 7.2483081326790000e+02 },
    { 1.6870000000000002e-08, 2.0021269389800000e+00, 2.7444340526907999e+03 },
    { 1.6840000000000001e-08, 2.5397356052700002e+00, 5.9884348936940000e+02 },
    { 1.6660000000000002e-08, 6.0866175200499999e+00, 1.0584187223427000e+03 },
    { 1.6660000000000002e-08, 3.5564375615500001e+00, 1.0603451380357001e+03 },
    { 1.6460000000000000e-08, 4.8904305473000003e+00, 2.9912639426919998e+02 },
    { 1.6390000000000000e-08, 6.2227037776000005e-01, 7.3905790726949999e+02 },
    { 1.6370000000000000e-08, 2.2021610724100000e+00, 7.3269511979410004e+02 },
    { 1.6330000000000001e-08, 3.3368684345599999e+00, 2.4351557300350000e+03 },
    { 1.6269999999999999e-08, 3.0478036961399999e+00, 2.8404132799086001e+03 },
    { 1.6199999999999999e-08, 2.2477721273100002e+00, 1.3647280995819001e+03 },
    { 1.6120000000000001e-08, 4.4622513123700003e+00, 1.3544331588434000e+03 },
    { 1.5480000000000000e-08, 3.0177032371100001e+00, 2.3107223148140001e+03 },
    { 1.5209999999999999e-08, 5.3768080262900000e+00, 8.4290144101349995e+02 },
    { 1.5200000000000000e-08, 6.6505874837000001e-01, 9.7748678462110001e+02 },
    { 1.5170000000000000e-08, 3.2266947096399998e+00, 1.4395096981491999e+03 },
    { 1.5130000000000000e-08, 2.9686119992000002e+00, 1.2694996318895001e+03 },
    { 1.4979999999999999e-08, 5.4029072174100001e+00, 1.1341635287565000e+03 },
    { 1.4370000000000001e-08, 1.7223311857600001e+00, 1.0564611688824000e+03 },
    { 1.4349999999999999e-08, 2.6722898141000001e-01, 9.9171387862270001e+02 },
    { 1.4100000000000001e-08, 2.5813969160000000e-01, 1.0916252591036000e+03 },
    { 1.3770000000000000e-08, 3.5108191634399999e+00, 9.1693228005540004e+02 },
    { 1.3660000000000000e-08, 4.7202636438200001e+00, 1.4466232451500000e+03 },
    { 1.3529999999999999e-08, 3.0451568950399999e+00, 2.4138150890326001e+03 },
    { 1.3230000000000000e-08, 4.5300798060000003e-01, 2.8414854074220000e+02 },
    { 1.3080000000000000e-08, 1.1646702102800000e+00, 5.5814515309779995e+02 },
    { 1.3000000000000001e-08, 7.6466874907000004e-01, 1.4749344208012999e+03 },
    { 1.2520000000000001e-08, 1.0600746021300000e+00, 4.7306861379200001e+02 },
    { 1.2420000000000000e-08, 2.9623498608699999e+00, 5.2012973753899996e+02 },
    { 1.2420000000000000e-08, 4.5199284410900002e+00, 5.3925219265019996e+02 },
    { 1.2310000000000001e-08, 9.1254979842999995e-01, 3.7301422095919997e+02 },
    { 1.1770000000000001e-08, 2.1743986608300001e+00, 3.0537123753465999e+03 },
    { 1.1660000000000000e-08, 3.2340943052900002e+00, 2.7586611466924001e+03 },
    { 1.1469999999999999e-08, 2.7406164689699999e+00, 4.0843894361129998e+02 },
    { 1.1440000000000000e-08, 5.7869563945000002e-01, 1.4820479678020999e+03 },
    { 1.1320000000000000e-08, 5.1950487281399997e+00, 1.0635747158832000e+03 },
    { 1.1260000000000000e-08, 3.2306942268999999e+00, 5.8631331639719997e+02 },
    { 1.1210000000000001e-08, 6.7312913777000005e-01, 4.1643119896130002e+03 },
    { 1.0789999999999999e-08, 1.8315750655000000e+00, 1.4827987273275000e+03 },
    { 1.0589999999999999e-08, 6.0102741021500004e+00, 2.8546403739102002e+03 },
    { 1.0480000000000000e-08, 1.9786068948100000e+00, 2.9506196011279999e+03 },
    { 1.0460000000000001e-08, 3.9416673658899999e+00, 2.2005159935945999e+03 },
    { 1.0400000000000000e-08, 4.0121762821500004e+00, 5.2236939198022001e+03 },
    { 1.0300000000000000e-08, 5.3559386927799997e+00, 8.5001498801430000e+02 },
    { 1.0169999999999999e-08, 2.1033842058399999e+00, 8.0945216858326003e+03 },
    { 1.0150000000000000e-08, 5.1818482402999999e-01, 4.5097721326419997e+02 },
    { 1.0029999999999999e-08, 5.2118465456000003e+00, 1.4323961511483999e+03 },
    { 9.8999999999999993e-09, 2.8805776056400001e+00, 1.9692006632437999e+03 },
    { 9.8400000000000008e-09, 1.9416987169900000e+00, 5.3685269620520000e+02 },
    { 9.8400000000000008e-09, 5.5405795849699997e+00, 5.2252923398400003e+02 },
    { 9.8400000000000008e-09, 2.4552054054200001e+00, 5.2262560220360001e+02 },
    { 9.8400000000000008e-09, 5.0270728965400000e+00, 5.3675632798560002e+02 },
    { 9.8099999999999998e-09, 5.3167463557000003e+00, 9.1539036160218002e+03 },
    { 9.7499999999999996e-09, 5.8311496413599997e+00, 3.1321047591889999e+02 },
    { 9.5599999999999992e-09, 7.3411529838000000e-01, 2.8475268269093999e+03 },
    { 9.2599999999999999e-09, 4.6929397633000001e+00, 2.9823262239190001e+02 },
    { 9.1399999999999995e-09, 5.9947638309300002e+00, 1.5578297225809999e+02 },
    { 9.1000000000000004e-09, 2.3234980520600002e+00, 1.5787779545453000e+03 },
    { 9.0699999999999995e-09, 4.0056377229900004e+00, 4.5342489381899998e+02 },
    { 9.0099999999999993e-09, 2.9540361094400001e+00, 2.4872381809010000e+02 },
    { 8.9100000000000000e-09, 1.6204195658400000e+00, 4.8640193591629998e+02 },
    { 8.7099999999999999e-09, 2.9941734458400000e+00, 2.7302069586891998e+03 },
    { 8.6800000000000006e-09, 2.7871211776500000e+00, 3.1568051495651998e+03 },
    { 8.6800000000000006e-09, 2.8387805532899999e+00, 6.2248879857470001e+02 },
    { 8.5000000000000000e-09, 3.9010977716799999e+00, 7.7523338944700004e+02 },
    { 8.3199999999999994e-09, 3.8225036691000001e+00, 6.0840471692500000e+02 },
    { 8.2499999999999994e-09, 3.5506220653199998e+00, 4.8560843660684004e+03 },
    { 8.2499999999999994e-09, 2.4972568677000001e-01, 8.6170991039264009e+03 },
    { 8.2299999999999999e-09, 3.6427859957900002e+00, 5.7462713378959997e+03 },
    { 8.0999999999999997e-09, 4.5334517048000000e-01, 3.0679394693482000e+03 },
    { 8.0700000000000005e-09, 1.7041038909899999e+00, 1.7920770499833000e+03 },
    { 8.0600000000000007e-09, 4.9831886253000002e-01, 1.7913262904579001e+03 },
    { 8.0200000000000000e-09, 3.9277909667199999e+00, 9.8460033162189995e+02 },
    { 8.0100000000000003e-09, 1.4851661489600001e+00, 7.7204327494533383e+04 },
    { 7.9900000000000007e-09, 2.7202669454200001e+00, 5.7297999427289994e+02 },
    { 7.9699999999999996e-09, 5.5145579197699996e+00, 4.6498988176311996e+03 },
    { 7.9200000000000008e-09, 3.2218110970899998e+00, 1.2623860848887000e+03 },
    { 7.8800000000000001e-09, 4.8813471372799997e+00, 1.6708250284999999e+03 },
    { 7.8800000000000001e-09, 8.2832429041000000e-01, 1.0064509366380000e+02 },
    { 7.8000000000000004e-09, 6.7747505650999995e-01, 6.0595703637019994e+02 },
    { 7.7900000000000006e-09, 2.1213282555599999e+00, 5.2161421024729998e+02 },
    { 7.7900000000000006e-09, 5.3609500464000002e+00, 5.3776771994190005e+02 },
    { 7.7900000000000006e-09, 1.6078215671200000e+00, 5.3584130424889997e+02 },
    { 7.7900000000000006e-09, 5.8744567348399999e+00, 5.2354062594029995e+02 },
    { 7.6700000000000002e-09, 2.2843067250800000e+00, 3.8109295298808001e+03 },
    { 7.6399999999999993e-09, 5.2877329145000003e+00, 9.2783496796739996e+02 },
    { 7.6199999999999997e-09, 6.2572849961800001e+00, 1.0554045477340000e+02 },
    { 7.6100000000000000e-09, 4.7458705484800001e+00, 9.0270518605380005e+02 },
    { 7.5800000000000007e-09, 5.0631621367499999e+00, 9.6764810341156008e+03 },
    { 7.5400000000000000e-09, 5.3379104113999998e-01, 8.6313261979279996e+03 },
    { 7.4400000000000000e-09, 8.7700319351000000e-01, 2.7703499374140000e+02 },
    { 7.4400000000000000e-09, 1.3356021031500001e+00, 6.4307868005169996e+02 },
    { 7.4400000000000000e-09, 4.0345220832499997e+00, 1.8196374661092000e+03 },
    { 7.4300000000000002e-09, 5.0691445109000002e+00, 1.0599031950510000e+03 },
    { 7.3900000000000003e-09, 5.8213157187100002e+00, 9.6733485623449997e+02 },
    { 7.3300000000000001e-09, 3.0269044778300001e+00, 2.6271141844705999e+03 },
    { 7.2900000000000003e-09, 9.4818548170000003e-02, 4.5468060434126000e+03 },
    { 7.2900000000000003e-09, 2.4565368997000001e-01, 1.2899465010146000e+03 },
    { 7.2000000000000000e-09, 3.0601717958600001e+00, 3.0750530163489998e+03 },
    { 7.1600000000000001e-09, 6.3230188605000004e-01, 3.2812385647862002e+03 },
    { 7.0900000000000001e-09, 4.7860351219700004e+00, 5.0693834615063997e+03 },
    { 7.0399999999999997e-09, 4.2034860657000001e-01, 5.1653626887241999e+03 },
    { 7.0299999999999999e-09, 3.6834313941399999e+00, 2.2741169495098002e+03 },
    { 7.0200000000000002e-09, 6.2754823325600002e+00, 5.0622699145056004e+03 },
    { 7.0200000000000002e-09, 3.1812356546200000e+00, 2.2883440435113998e+03 },
    { 6.9999999999999998e-09, 4.5273624919000000e+00, 4.8715269544170002e+02 },
    { 6.9999999999999998e-09, 6.0965084636400002e+00, 5.7222923474749996e+02 },
    { 6.9699999999999997e-09, 1.6921157447999999e+00, 2.5453620512543998e+03 },
    { 6.9600000000000000e-09, 1.8468217345100000e+00, 1.4756851803267000e+03 },
    { 6.8400000000000004e-09, 3.0762745276999998e-01, 1.7984398374586999e+03 },
    { 6.7899999999999999e-09, 3.6793044107699999e+00, 2.4498830246290399e+04 },
    { 6.6700000000000003e-09, 8.7586274198000003e-01, 1.0129115072732000e+03 },
    { 6.5700000000000003e-09, 5.1680753909900004e+00, 3.9957744373156002e+03 },
    { 6.5199999999999998e-09, 6.2512755516300000e+00, 3.0609692891809999e+02 },
    { 6.5000000000000003e-09, 1.5630105328099999e+00, 1.7991905969841000e+03 },
    { 6.4899999999999997e-09, 1.0673286737200001e+00, 2.4554242435239999e+02 },
    { 6.4800000000000000e-09, 1.8218069559300001e+00, 3.0608259223474001e+03 },
    { 6.3400000000000000e-09, 1.3849318234700001e+00, 4.6988722005429997e+02 },
    { 6.2300000000000002e-09, 1.3811735873600000e+00, 2.5035634758385801e+04 },
    { 6.2199999999999996e-09, 9.2043275035000005e-01, 4.5254654024102001e+03 },
    { 6.1600000000000002e-09, 1.8833466160100001e+00, 2.5021407664384202e+04 },
    { 6.0900000000000003e-09, 4.7326013789800001e+00, 1.5606187072805999e+03 },
    { 6.0699999999999999e-09, 2.9231980382499998e+00, 2.1329909543799999e+03 },
    { 6.0600000000000002e-09, 5.4859936898399999e+00, 1.7515395314160000e+03 },
    { 6.0200000000000003e-09, 5.0423273015000003e-01, 3.9814400287280000e+02 },
    { 5.8999999999999999e-09, 1.9298326161299999e+00, 3.3701042450032000e+03 },
    { 5.8200000000000002e-09, 4.2067008179099998e+00, 1.4575259330619999e+03 },
    { 5.8100000000000004e-09, 2.8104024853600000e+00, 8.2792358748649997e+02 },
    { 5.7999999999999998e-09, 5.6915725642200004e+00, 7.3194436026870005e+02 },
    { 5.7600000000000000e-09, 6.0677839867200003e+00, 4.7387644978482003e+03 },
    { 5.7299999999999999e-09, 1.2629475250800000e+00, 3.9435486196160002e+02 },
    { 5.6699999999999997e-09, 6.2029159479999996e-01, 1.8873055176757000e+03 },
    { 5.6400000000000004e-09, 4.0905133120499997e+00, 1.1270499817556999e+03 },
    { 5.6299999999999998e-09, 1.7918130295000001e+00, 1.2687488723641000e+03 },
    { 5.5899999999999999e-09, 2.4982368644599999e+00, 4.4475743814070000e+02 },
    { 5.4700000000000003e-09, 2.0012150023499999e+00, 4.1273213995373999e+03 },
    { 5.4599999999999998e-09, 2.9557555113200000e+00, 5.8949471013489995e+02 },
    { 5.4500000000000000e-09, 1.3845676631199999e+00, 8.1065811209909998e+02 },
    { 5.4299999999999997e-09, 4.6977584554599998e+00, 2.1648048917570000e+02 },
    { 5.4100000000000001e-09, 3.4650782969999998e+00, 2.5169078632512001e+03 },
    { 5.3899999999999998e-09, 1.6974189350300000e+00, 6.1462449204849997e+02 },
    { 5.3700000000000003e-09, 3.4227001862000002e+00, 7.8234693644779998e+02 },
    { 5.3400000000000002e-09, 1.4575561426800001e+00, 4.3689313161450002e+02 },
    { 5.2800000000000000e-09, 1.2088747954200001e+00, 3.6936096616606001e+03 },
    { 5.2599999999999996e-09, 1.2857914027899999e+00, 3.3772177920039999e+03 },
    { 5.2400000000000001e-09, 1.3691906752999999e-01, 1.5780271950199001e+03 },
    { 5.2100000000000000e-09, 8.1875815113999995e-01, 6.9152524274800001e+01 },
    { 5.1400000000000000e-09, 6.5565842899000004e-01, 3.2527843767830000e+03 },
    { 5.1300000000000003e-09, 4.1830859389899997e+00, 1.8094855377225999e+03 },
    { 5.0899999999999996e-09, 5.0709691775300003e+00, 4.6868894077067998e+03 },
    { 5.0700000000000001e-09, 3.4928821557699998e+00, 2.0903096723752001e+03 },
    { 5.0499999999999997e-09, 1.7809391740000000e+00, 7.5403576079649997e+02 },
    { 5.0000000000000001e-09, 3.1101880578099999e+00, 4.3263934009738005e+03 },
    { 4.9900000000000003e-09, 2.9033722949600000e+00, 1.6889842757647000e+03 },
    { 4.9300000000000001e-09, 5.4321564352899996e+00, 1.0021837280099400e+04 },
    { 4.8699999999999999e-09, 2.7721159194000000e-01, 1.9015326116773001e+03 },
    { 4.7699999999999999e-09, 5.3766888476299997e+00, 4.9662906872878002e+03 },
    { 4.7699999999999999e-09, 6.2368980679600003e+00, 3.1710322435667999e+03 },
    { 4.6900000000000001e-09, 1.7924392780400000e+00, 3.2670114707846001e+03 },
    { 4.6699999999999998e-09, 2.4926909585299999e+00, 3.4731970192218000e+03 },
    { 4.6699999999999998e-09, 1.6698229037600001e+00, 1.4899122743283001e+03 },
    { 4.6200000000000002e-09, 2.5774167220700002e+00, 3.4660834722210002e+03 },
    { 4.6200000000000002e-09, 7.6919542271999997e-01, 2.6617594106668799e+04 },
    { 4.5699999999999997e-09, 1.4424613594300000e+00, 2.1084689196398999e+03 },
    { 4.5299999999999999e-09, 4.9622611371699996e+00, 4.3192798539730002e+03 },
    { 4.5200000000000001e-09, 4.8173547703099997e+00, 1.5509398596460001e+03 },
    { 4.5100000000000003e-09, 2.7095202105999999e-01, 2.1077181601144998e+03 },
    { 4.5100000000000003e-09, 3.3156330776999998e+00, 5.1724762357250002e+03 },
    { 4.4999999999999998e-09, 6.6226303752000004e-01, 1.4678208738005001e+03 },
    { 4.4800000000000002e-09, 2.5739342144299999e+00, 1.2036460734888200e+04 },
    { 4.3800000000000002e-09, 1.1100833646699999e+00, 6.1158611066269998e+02 },
    { 4.3200000000000000e-09, 5.4410751026400002e+00, 4.7011165017084004e+03 },
    { 4.2400000000000002e-09, 1.7453405766900001e+00, 1.2545217783625001e+03 },
    { 4.2299999999999997e-09, 5.0278301811899997e+00, 2.9577331481288002e+03 },
    { 4.2299999999999997e-09, 4.5266252256700001e+00, 4.1529185818119998e+02 },
    { 4.1599999999999997e-09, 1.5208165273900001e+00, 1.2616353253632999e+03 },
    { 4.1299999999999996e-09, 2.4778698676399999e+00, 1.9022833712027000e+03 },
    { 4.1199999999999998e-09, 6.1809390952100003e+00, 4.8631979130691998e+03 },
    { 4.0899999999999997e-09, 2.9647092908200001e+00, 1.3576145525811000e+03 },
    { 4.0400000000000001e-09, 3.2470991462400001e+00, 2.0424977891028000e+03 },
    { 4.0199999999999998e-09, 3.2054658720600000e+00, 6.6502706822760001e+02 },
    { 3.9600000000000004e-09, 3.8675281312399998e+00, 1.0200250542740000e+03 },
    { 3.7900000000000004e-09, 1.7279558418300001e+00, 4.4437132691939996e+03 },
};

static const VSOP87Term VSOP87A_JUPITER_X2[255] = {
    { 1.2386464399999999e-03, 4.1356327751300004e+00, 5.2257741809380002e+02 },
    { 1.2152129600000001e-03, 2.1155109275000000e-01, 5.3680451209540001e+02 },
    { 8.5355502999999997e-04, 0.0000000000000000e+00, 0.0000000000000000e+00 },
    { 7.7685546999999998e-04, 5.2977615445800001e+00, 5.2969096509459996e+02 },
    { 4.1410886999999999e-04, 5.1229158993899997e+00, 1.0593819301891999e+03 },
    { 1.1423070000000000e-04, 1.7291787823800000e+00, 7.1135470007999997e+00 },
    { 7.0515870000000005e-05, 7.4163703419000004e-01, 1.0664954771900000e+03 },
    { 5.7110290000000000e-05, 3.6317284649400001e+00, 1.0522683831884001e+03 },
    { 5.2426439999999999e-05, 4.2748237944099996e+00, 5.1546387109299997e+02 },
    { 4.0395400000000003e-05, 5.5841773211700003e+00, 1.5890728952838001e+03 },
    { 3.7064570000000002e-05, 7.7699813490000005e-02, 5.4391805909619995e+02 },
    { 1.6988169999999999e-05, 2.4428441806599999e+00, 1.1020632121940000e+02 },
    { 1.3226730000000000e-05, 1.6314254998000000e+00, 1.4227094001599999e+01 },
    { 1.1345980000000000e-05, 2.3580706180900002e+00, 4.2659819087599999e+02 },
    { 9.9400800000000002e-06, 1.4698552225299999e+00, 3.2350541665740002e+02 },
    { 8.8820299999999994e-06, 4.6662729024400003e+00, 5.2650957135689998e+02 },
    { 8.6554700000000000e-06, 5.9559688853899999e+00, 5.3287235883230005e+02 },
    { 8.2257899999999994e-06, 1.9647399507800001e+00, 6.3278373931320004e+02 },
    { 7.3338599999999999e-06, 3.7132887987000002e-01, 1.0309277421860000e+02 },
    { 5.9592999999999998e-06, 2.8599317150500001e+00, 7.3587651353180001e+02 },
    { 5.7406599999999997e-06, 1.6692658814800001e+00, 9.4917560896980001e+02 },
    { 5.7171099999999997e-06, 3.1691209590899998e+00, 1.0451548361876000e+03 },
    { 5.1425600000000003e-06, 5.9710333068599999e+00, 5.2575881183150000e+02 },
    { 5.1222500000000000e-06, 4.6553500001000003e+00, 5.3362311835770004e+02 },
    { 4.5853300000000004e-06, 1.2445006828600000e+00, 1.5961864422845999e+03 },
    { 4.1912599999999997e-06, 2.6104223842400001e+00, 1.1731986822019999e+02 },
    { 3.7484000000000000e-06, 5.5582152647100003e+00, 9.5979227217800002e+01 },
    { 3.4176499999999998e-06, 3.9491407125000000e-01, 2.0618554843720000e+02 },
    { 3.3292600000000002e-06, 6.0000875215200002e+00, 2.1187638603783998e+03 },
    { 2.9474299999999999e-06, 4.4187127489800000e+00, 4.1948464387519999e+02 },
    { 2.8201800000000000e-06, 3.7109826237000001e+00, 1.5819593482830001e+03 },
    { 2.6446399999999999e-06, 3.6800767374399999e+00, 9.5628915597059995e+02 },
    { 2.2562399999999999e-06, 1.6750167448900000e+00, 9.4206206196899996e+02 },
    { 2.2558699999999999e-06, 3.0353034598000002e+00, 6.2567019231239999e+02 },
    { 2.2180799999999999e-06, 8.8408008288999995e-01, 4.3371173787679999e+02 },
    { 2.0441999999999998e-06, 4.7807959065000000e-01, 2.2041264243879999e+02 },
    { 1.9674999999999998e-06, 6.0038889057500002e+00, 1.1695882514085999e+03 },
    { 1.9604800000000000e-06, 5.9864829574499998e+00, 6.3989728631399998e+02 },
    { 1.7029600000000000e-06, 1.3251441230700001e+00, 3.3061896365820002e+02 },
    { 1.6483299999999999e-06, 5.3542284920099998e+00, 3.1639186965660002e+02 },
    { 1.6362800000000000e-06, 5.2534435976000005e-01, 1.0736090241908000e+03 },
    { 1.6081399999999999e-06, 3.5985635133899998e+00, 1.1553611574070001e+03 },
    { 1.3742900000000001e-06, 2.4727364545000000e-01, 1.9907200143640000e+02 },
    { 1.3715299999999999e-06, 4.4413058415000002e+00, 5.0835032409220003e+02 },
    { 1.2182900000000000e-06, 3.1143765199300000e+00, 7.2876296653099996e+02 },
    { 1.1832599999999999e-06, 2.4856624680100001e+00, 2.1329909543799999e+02 },
    { 1.1644600000000000e-06, 5.4908048502500000e+00, 7.4299006053259995e+02 },
    { 1.0799700000000001e-06, 4.6329868291900000e+00, 1.1624747044077999e+03 },
    { 1.0530100000000000e-06, 2.6878308994400002e+00, 8.5319638175199998e+02 },
    { 1.0411700000000001e-06, 7.9163400572999998e-01, 3.0927832265580003e+02 },
    { 1.0330200000000000e-06, 4.8383307725500000e+00, 4.1237109687440000e+02 },
    { 8.5092000000000000e-07, 4.8305497922900003e+00, 5.1939602435610004e+02 },
    { 7.5825999999999996e-07, 4.2352295497699997e+00, 1.2655674786264001e+03 },
    { 7.4748999999999998e-07, 6.1915406296100004e+00, 5.5103160609700001e+02 },
    { 6.7576000000000001e-07, 5.7669049613499999e+00, 5.3998590583309999e+02 },
    { 5.7861999999999998e-07, 4.1102531931100001e+00, 8.3896928775039999e+02 },
    { 5.5782999999999996e-07, 1.3145207272199999e+00, 3.1813937377000001e+00 },
    { 5.5764000000000001e-07, 1.4604626993200001e+00, 2.1340641002400002e+01 },
    { 4.8874000000000002e-07, 1.7659043470999999e-01, 1.0625633239269000e+03 },
    { 4.6659000000000000e-07, 3.5544852318999998e-01, 2.2752618943959999e+02 },
    { 3.9238999999999998e-07, 1.8292411580000001e+00, 8.4608283475120004e+02 },
    { 3.8445999999999999e-07, 3.3552280724400001e+00, 1.5748458012822000e+03 },
    { 3.8080000000000002e-07, 5.8205282046200004e+00, 4.4082528487759998e+02 },
    { 3.7163000000000000e-07, 4.5211462661899997e+00, 1.2584539316256000e+03 },
    { 3.6511999999999998e-07, 4.4043424106800000e+00, 1.0562005364515001e+03 },
    { 3.6486000000000000e-07, 4.7432805585800004e+00, 1.6921656695024001e+03 },
    { 3.2376999999999999e-07, 1.1198158920400001e+00, 3.0216477565500003e+02 },
    { 3.1862999999999999e-07, 2.8733457947000002e-01, 1.2726810256271999e+03 },
    { 3.1825999999999999e-07, 1.7217629585900001e+00, 2.1258774073792001e+03 },
    { 2.9754000000000000e-07, 4.7849223189999996e+00, 1.4859801210651999e+03 },
    { 2.8626000000000002e-07, 3.1037285350900001e+00, 1.0380412891868000e+03 },
    { 2.8570000000000002e-07, 3.3181498895000000e-01, 1.0554497769261000e+03 },
    { 2.8363000000000001e-07, 5.4036673293700002e+00, 1.3686602528450001e+03 },
    { 2.5133000000000001e-07, 1.1303788218999999e-01, 2.6484548254729998e+03 },
    { 2.2231999999999999e-07, 4.5134447182099997e+00, 8.3185574074960005e+02 },
    { 2.2070000000000001e-07, 4.8122766576300000e+00, 6.1855664531160005e+02 },
    { 2.0494000000000001e-07, 2.5456467022000001e-01, 1.6992792165031999e+03 },
    { 2.0382000000000001e-07, 6.1029885458699997e+00, 6.4701083331480004e+02 },
    { 1.9784999999999999e-07, 4.8868752219899996e+00, 1.0633140834523001e+03 },
    { 1.8022999999999999e-07, 2.0154772619400001e+00, 1.4788665740644001e+03 },
    { 1.7438000000000000e-07, 1.5092462383100000e+00, 1.3757737998458001e+03 },
    { 1.5148000000000000e-07, 2.4918523107300001e+00, 1.4717530270636000e+03 },
    { 1.4147999999999999e-07, 2.4270010636800001e+00, 1.2443341522100000e+02 },
    { 1.3409000000000000e-07, 3.8594762925800001e+00, 2.1116503133776000e+03 },
    { 1.2898000000000001e-07, 1.5212517897800000e+00, 9.3494851496820002e+02 },
    { 1.2697999999999999e-07, 4.0191557716400004e+00, 1.6850521225016000e+03 },
    { 1.2646000000000001e-07, 1.0699186874500000e+00, 9.6340270297140000e+02 },
    { 1.2496999999999999e-07, 1.0641296145300001e+00, 1.0294940738499999e+01 },
    { 1.2491000000000000e-07, 5.9831915049899997e+00, 7.5010360753340001e+02 },
    { 1.1227000000000000e-07, 3.1676233498199999e+00, 5.1153171782990000e+02 },
    { 1.0726000000000000e-07, 1.1730624478300000e+00, 5.4785021235930003e+02 },
    { 1.0421000000000000e-07, 5.2480703075999999e+00, 1.3615467058442000e+03 },
    { 9.1139999999999994e-08, 3.3210491920599998e+00, 1.3828873468465999e+03 },
    { 8.0179999999999994e-08, 1.1641708203100001e+00, 3.3773251065900001e+02 },
    { 7.6689999999999994e-08, 3.8211083277000002e-01, 1.9195845443560000e+02 },
    { 7.5030000000000004e-08, 9.5949306974000004e-01, 1.6032999892854000e+03 },
    { 7.5009999999999995e-08, 3.1194099201499998e+00, 7.2164941953020002e+02 },
    { 7.1799999999999994e-08, 2.2139665100500001e+00, 8.8865680217000005e+01 },
    { 6.8029999999999994e-08, 1.4292002967399999e+00, 1.0627416795630000e+02 },
    { 6.3969999999999996e-08, 1.9263581432700001e+00, 8.6030992875280003e+02 },
    { 6.3510000000000004e-08, 5.0943613317500001e+00, 1.1482476104062000e+03 },
    { 5.5640000000000001e-08, 5.1731873879599997e+00, 2.2218566345969998e+03 },
    { 5.5110000000000002e-08, 3.9694046998900001e+00, 1.0490869894507000e+03 },
    { 5.3890000000000001e-08, 4.4891315437500001e+00, 1.7952584437210000e+03 },
    { 5.0150000000000002e-08, 6.1632620804300000e+00, 9.9911380480899993e+01 },
    { 4.4199999999999999e-08, 4.2933315149800002e+00, 1.7881448967202000e+03 },
    { 3.9260000000000002e-08, 3.7329718523200000e+00, 4.9744763618019999e+02 },
    { 3.7870000000000001e-08, 1.9793794060000001e-02, 1.8023719907218001e+03 },
    { 3.7700000000000000e-08, 2.0052708112900000e+00, 1.1338771495710000e+02 },
    { 3.7289999999999998e-08, 3.5216371388000001e-01, 1.0702492748170000e+02 },
    { 3.6200000000000000e-08, 6.0771427323000005e-01, 5.6193429400900004e+02 },
    { 3.5220000000000000e-08, 4.6666357179000002e+00, 5.0123677709139997e+02 },
    { 3.5010000000000000e-08, 6.1890795899000004e+00, 1.0696768709277001e+03 },
    { 3.4020000000000003e-08, 1.6878191367000001e-01, 2.3463973644039999e+02 },
    { 3.3309999999999999e-08, 5.4023722356899997e+00, 4.0525754987360000e+02 },
    { 3.1739999999999998e-08, 5.4770026592900001e+00, 4.4793883187839998e+02 },
    { 3.1559999999999999e-08, 4.9306039131899997e+00, 1.8983512179396000e+03 },
    { 2.9550000000000000e-08, 6.2556987114999996e-01, 1.5922542890215000e+03 },
    { 2.7540000000000000e-08, 9.8877084311999996e-01, 4.2977958461370002e+02 },
    { 2.7510000000000000e-08, 3.6436999549000002e+00, 2.1045367663768002e+03 },
    { 2.7470000000000000e-08, 2.6101764113399999e+00, 8.1752133216199994e+01 },
    { 2.6689999999999999e-08, 5.5969390905100003e+00, 2.0156710861598001e+03 },
    { 2.6630000000000001e-08, 1.7775893029100001e+00, 1.9054647649404001e+03 },
    { 2.6610000000000001e-08, 1.3351983877400000e+00, 2.9505122865419997e+02 },
    { 2.5390000000000001e-08, 5.6049462435099997e+00, 1.2513403846248000e+03 },
    { 2.4340000000000002e-08, 1.6399596747000000e-01, 1.0807225711916001e+03 },
    { 2.4220000000000001e-08, 1.1729241541500000e+00, 9.9160620955499994e+01 },
    { 2.3520000000000001e-08, 5.4027951338899998e+00, 1.4956319713459999e+02 },
    { 2.2840000000000001e-08, 3.3698896603500001e+00, 6.2960234557549995e+02 },
    { 2.2340000000000000e-08, 5.2879089028899999e+00, 6.2759623029905997e+03 },
    { 2.2320000000000001e-08, 1.3634379238700001e+00, 6.2901893969922003e+03 },
    { 2.2230000000000001e-08, 9.3651582781999998e-01, 4.2266603761290003e+02 },
    { 2.1929999999999999e-08, 5.5158071781200002e+00, 6.1144309831080000e+02 },
    { 2.1640000000000001e-08, 5.0792051820399999e+00, 1.8912376709388000e+03 },
    { 2.1550000000000001e-08, 5.4210078246300002e+00, 1.4646394800628000e+03 },
    { 2.1240000000000000e-08, 5.7702024334899997e+00, 4.2341679713830001e+02 },
    { 2.1039999999999998e-08, 4.5327966023100004e+00, 2.0085575391590000e+03 },
    { 2.0940000000000001e-08, 4.8766728181300003e+00, 1.6779385755008000e+03 },
    { 2.0750000000000001e-08, 4.2762472761999998e-01, 1.0206171999210201e+04 },
    { 2.0730000000000001e-08, 2.7863422607400001e+00, 1.0220399093211799e+04 },
    { 2.0599999999999999e-08, 2.1847664696100000e+00, 2.6555683724738001e+03 },
    { 1.9810000000000000e-08, 4.3298320984899998e+00, 2.0014439921582000e+03 },
    { 1.8189999999999999e-08, 5.3619992496900002e+00, 1.1656560981455000e+03 },
    { 1.8150000000000000e-08, 5.8586504545100002e+00, 5.5814515309779995e+02 },
    { 1.8040000000000001e-08, 4.1400207771000002e+00, 1.1592933106701000e+03 },
    { 1.7970000000000001e-08, 7.7606737877999998e-01, 1.5851407420206999e+03 },
    { 1.7820000000000000e-08, 6.0447629298000005e-01, 3.1781457905676002e+03 },
    { 1.7640000000000001e-08, 6.2623194948799998e+00, 9.7051624997219994e+02 },
    { 1.7529999999999999e-08, 1.3075072507999999e-01, 7.5721715453419995e+02 },
    { 1.7459999999999999e-08, 4.4248969308100001e+00, 1.5858915015461000e+03 },
    { 1.7190000000000000e-08, 4.7520295612599996e+00, 6.5412438031559998e+02 },
    { 1.6169999999999999e-08, 4.6609109055399998e+00, 8.2474219374879999e+02 },
    { 1.5580000000000001e-08, 4.8017795301600001e+00, 6.3596513305090002e+02 },
    { 1.5480000000000000e-08, 2.3534281878700001e+00, 9.4599421523210003e+02 },
    { 1.4660000000000000e-08, 4.7654271859500001e+00, 2.2147430875962000e+03 },
    { 1.4410000000000000e-08, 1.0097087189100000e+00, 2.2289701815978001e+03 },
    { 1.3920000000000000e-08, 4.0452751244599998e+00, 1.2797945726280000e+03 },
    { 1.3830000000000000e-08, 3.3019486270099998e+00, 5.2308074668030004e+03 },
    { 1.3809999999999999e-08, 9.4385809616000005e-01, 5.2165803728013998e+03 },
    { 1.3510000000000000e-08, 2.1589157649500001e+00, 5.2236939198022001e+03 },
    { 1.2700000000000000e-08, 4.7249357744399996e+00, 9.1610171630225996e+03 },
    { 1.2680000000000000e-08, 2.3668585837600000e+00, 9.1467900690210008e+03 },
    { 1.2660000000000001e-08, 5.0849021985500000e+00, 1.7810313497193999e+03 },
    { 1.2550000000000000e-08, 1.2927220992399999e+00, 2.1011770170029999e+02 },
    { 1.2200000000000000e-08, 9.2941284572000005e-01, 3.1957326339430000e+02 },
    { 1.2010000000000000e-08, 5.7322934001899997e+00, 3.2812385647862002e+03 },
    { 1.1980000000000000e-08, 7.9239741189000001e-01, 3.2668681039510000e+02 },
    { 1.1830000000000001e-08, 3.2589170571300001e+00, 6.3671589257630001e+02 },
    { 1.1830000000000001e-08, 5.2768528013599996e+00, 4.1630325013750002e+02 },
    { 1.1830000000000001e-08, 2.1375356228400002e+00, 9.5310776223289997e+02 },
    { 1.1760000000000001e-08, 3.0881191422500001e+00, 7.3269511979410004e+02 },
    { 1.1180000000000000e-08, 3.0440295871699998e+00, 1.9125783119411999e+03 },
    { 1.0690000000000000e-08, 4.1713835660600003e+00, 2.9435060541272001e+03 },
    { 1.0630000000000000e-08, 1.9054310246099999e+00, 5.1864526483069994e+02 },
    { 1.0630000000000000e-08, 5.5768472773499997e+00, 5.4073666535849998e+02 },
    { 1.0519999999999999e-08, 2.4418182736200000e+00, 1.4930936680660000e+03 },
    { 1.0360000000000000e-08, 1.4407854373300000e+00, 5.7533848848968000e+03 },
    { 1.0190000000000001e-08, 2.6639453048000000e-01, 1.1411340634054000e+03 },
    { 1.0169999999999999e-08, 4.8963526138100004e+00, 1.9943304451573999e+03 },
    { 9.8899999999999996e-09, 1.6818876296999999e-01, 6.2830758499914000e+03 },
    { 9.8500000000000005e-09, 4.9207395620299996e+00, 2.2076295405954002e+03 },
    { 9.6600000000000001e-09, 5.0519479268899996e+00, 2.3249494088155998e+03 },
    { 9.5800000000000004e-09, 4.5429856857799997e+00, 3.3843313390048002e+03 },
    { 9.5499999999999995e-09, 2.8580082822600001e+00, 9.6835945811164001e+03 },
    { 9.3499999999999994e-09, 8.4952544347000003e-01, 2.0300415469949999e+02 },
    { 9.3200000000000001e-09, 3.3878380676900002e+00, 9.5235700270749999e+02 },
    { 9.2300000000000006e-09, 5.2069876701800002e+00, 9.0981873305459999e+02 },
    { 9.0500000000000000e-09, 5.4584737572800002e+00, 1.1585425511446999e+03 },
    { 8.8900000000000005e-09, 3.3258852140499999e+00, 7.3980866679489998e+02 },
    { 8.6900000000000004e-09, 3.3528849464500001e+00, 3.5905168874420001e+03 },
    { 8.5899999999999995e-09, 6.0231752959499998e+00, 4.5884151979040001e+02 },
    { 8.4200000000000003e-09, 2.3862159927600000e+00, 3.7895888888784002e+03 },
    { 8.2999999999999999e-09, 3.5369424113300001e+00, 3.5762897934404000e+03 },
    { 8.2200000000000002e-09, 3.2337837102600000e+00, 3.8997952100978000e+03 },
    { 8.1899999999999992e-09, 2.7581986856600000e+00, 3.6936096616606001e+03 },
    { 7.8999999999999996e-09, 4.0300534638299998e+00, 1.1664068576709001e+03 },
    { 7.8500000000000008e-09, 5.6620772306599996e+00, 1.3544331588434000e+03 },
    { 7.8100000000000001e-09, 2.8734181865899999e+00, 3.7967024358792000e+03 },
    { 7.7099999999999992e-09, 7.5516833087000002e-01, 9.2783496796739996e+02 },
    { 7.6100000000000000e-09, 3.6138384202400000e+00, 9.1539036160218002e+03 },
    { 7.5800000000000007e-09, 4.8781738664200001e+00, 2.5311349572528002e+03 },
    { 7.5699999999999993e-09, 4.2069980277600001e+00, 2.3178358618148000e+03 },
    { 7.5400000000000000e-09, 4.2489717535700002e+00, 3.7007232086613999e+03 },
    { 7.5100000000000007e-09, 5.4959780030200003e+00, 3.5976304344427999e+03 },
    { 7.4999999999999993e-09, 1.8243753507800000e+00, 3.1496916025644000e+03 },
    { 7.4600000000000003e-09, 5.7546909890200002e+00, 1.8841241239379999e+03 },
    { 7.4400000000000000e-09, 4.4686175307299996e+00, 2.6271141844705999e+03 },
    { 7.3399999999999999e-09, 5.2444533782100002e+00, 1.6708250284999999e+03 },
    { 7.2500000000000004e-09, 3.3273237223300001e+00, 1.5677322542813999e+03 },
    { 7.2300000000000001e-09, 2.2621771325000002e+00, 1.8094855377225999e+03 },
    { 7.2200000000000003e-09, 2.1354860124799999e+00, 3.3701042450032000e+03 },
    { 7.2099999999999997e-09, 2.3068981877899999e+00, 4.6940029547076001e+03 },
    { 7.1900000000000002e-09, 4.6006979536300001e+00, 6.0054041039879996e+02 },
    { 7.1600000000000001e-09, 4.3569749110800000e+00, 2.6413412784722000e+03 },
    { 7.0999999999999999e-09, 4.4096644632800004e+00, 3.8038159828799999e+03 },
    { 7.0900000000000001e-09, 4.5513773567399998e+00, 2.3107223148140001e+03 },
    { 7.0299999999999999e-09, 1.9520856370899999e+00, 2.5453620512543998e+03 },
    { 6.8900000000000000e-09, 4.6711282766500002e+00, 2.7302069586891998e+03 },
    { 6.6100000000000001e-09, 3.3250727583100002e+00, 3.5834033404411998e+03 },
    { 6.5599999999999997e-09, 3.7287038744699998e+00, 8.6242126509272002e+03 },
    { 6.4899999999999997e-09, 4.0775407325700002e+00, 2.6342277314714001e+03 },
    { 6.3700000000000001e-09, 4.8829544852000000e+00, 1.9872168981566001e+03 },
    { 6.2400000000000000e-09, 4.9100156179800001e+00, 2.0974232193759999e+03 },
    { 6.1300000000000001e-09, 4.1846333675100000e+00, 2.4209286360333999e+03 },
    { 6.1000000000000000e-09, 5.6565017397300004e+00, 1.4575259330619999e+03 },
    { 6.0799999999999997e-09, 4.8879589350300003e+00, 3.4874241132234001e+03 },
    { 6.0399999999999998e-09, 4.9150607013199998e+00, 2.5240214102519999e+03 },
    { 5.9500000000000003e-09, 5.0800407265200000e+00, 7.6266071275599998e+01 },
    { 5.8800000000000004e-09, 5.0228287322499998e+00, 4.3053034413910001e+02 },
    { 5.8800000000000004e-09, 3.7657461666800001e+00, 3.2598979237837998e+03 },
    { 5.7399999999999996e-09, 1.5417054480000000e-01, 2.3320629558164001e+03 },
    { 5.6999999999999998e-09, 5.3731503759399999e+00, 2.5382485042536000e+03 },
    { 5.6599999999999999e-09, 1.7009427937699999e+00, 2.8617539209110000e+03 },
    { 5.4599999999999998e-09, 2.4483770268999998e+00, 4.3764389113990001e+02 },
    { 5.3700000000000003e-09, 3.2637918091799998e+00, 3.8926816630970002e+03 },
    { 5.3000000000000003e-09, 2.5898823907100001e+00, 5.4709945283390005e+02 },
    { 5.3000000000000003e-09, 4.8923959112500004e+00, 5.1228247735529999e+02 },
    { 5.2700000000000002e-09, 6.2114775321500000e+00, 4.6202291352809999e+02 },
    { 5.2700000000000002e-09, 4.4123934234000002e+00, 5.9735901666109999e+02 },
    { 5.2100000000000000e-09, 3.0306676036200000e+00, 4.2161870797543997e+03 },
    { 5.1700000000000001e-09, 4.4426737201300002e+00, 2.3036087678131998e+03 },
    { 5.1099999999999999e-09, 6.0374672243900003e+00, 6.4307868005169996e+02 },
    { 5.1000000000000002e-09, 1.7493208890100000e+00, 2.0936694217490000e+02 },
    { 5.0600000000000003e-09, 5.0101883015000004e-01, 1.2442268376239999e+03 },
    { 4.9499999999999997e-09, 3.1381334509799999e+00, 3.9069087570985998e+03 },
    { 4.8499999999999996e-09, 4.9163605854599997e+00, 2.8475268269093999e+03 },
    { 4.7799999999999996e-09, 5.0713259913100002e+00, 1.5930050485469001e+03 },
    { 4.5800000000000003e-09, 5.6010467342599997e+00, 6.2885158605009997e+02 },
    { 4.5299999999999999e-09, 2.5087193362700000e+00, 4.0100015313171998e+03 },
    { 4.4999999999999998e-09, 5.3069249467699997e+00, 3.9140223040994001e+03 },
    { 4.3800000000000002e-09, 5.4935502015999997e+00, 2.2005159935945999e+03 },
    { 4.3299999999999997e-09, 1.8923124032500001e+00, 6.2173803904930003e+02 },
    { 4.3100000000000002e-09, 4.6907718207600002e+00, 2.8332997329077998e+03 },
    { 4.3100000000000002e-09, 2.4220955824399999e+00, 8.4926422848890002e+02 },
    { 4.1199999999999998e-09, 1.4853392213000001e+00, 2.7586611466924001e+03 },
};

static const VSOP87Term VSOP87A_JUPITER_X3[140] = {
    { 1.7071322999999999e-04, 5.8613302227800004e+00, 5.2257741809380002e+02 },
    { 1.6713548000000001e-04, 4.7745879448500004e+00, 5.3680451209540001e+02 },
    { 3.3486099999999998e-05, 0.0000000000000000e+00, 0.0000000000000000e+00 },
    { 1.7878380000000000e-05, 3.5655029803099998e+00, 1.0593819301891999e+03 },
    { 1.4354490000000001e-05, 5.9850203658699996e+00, 5.1546387109299997e+02 },
    { 1.0801940000000000e-05, 5.4253030591400000e+00, 1.0664954771900000e+03 },
    { 1.0731749999999999e-05, 6.2231446796399998e+00, 7.1135470007999997e+00 },
    { 1.0142060000000000e-05, 4.6477390207699996e+00, 5.4391805909619995e+02 },
    { 7.1106500000000003e-06, 5.5068051520500001e+00, 1.0522683831884001e+03 },
    { 3.0105400000000001e-06, 6.1984132109000001e+00, 1.4227094001599999e+01 },
    { 2.6108900000000000e-06, 4.2826983439399999e+00, 1.5890728952838001e+03 },
    { 1.3473800000000001e-06, 4.9474619792699999e+00, 1.0451548361876000e+03 },
    { 1.2429000000000000e-06, 3.7523072266000002e-01, 1.1020632121940000e+02 },
    { 9.6082999999999996e-07, 1.9117186884499999e+00, 5.2969096509459996e+02 },
    { 9.1757999999999999e-07, 6.0106809945300004e+00, 3.2350541665740002e+02 },
    { 7.8210999999999995e-07, 6.0135706720200002e+00, 1.5961864422845999e+03 },
    { 7.6743000000000004e-07, 8.8793596850000001e-01, 1.1731986822019999e+02 },
    { 6.1058000000000001e-07, 4.9120147122000002e-01, 4.2659819087599999e+02 },
    { 5.8133000000000003e-07, 6.1540580109900000e+00, 5.0835032409220003e+02 },
    { 5.6622000000000000e-07, 4.6237810655900002e+00, 7.3587651353180001e+02 },
    { 5.1910000000000004e-07, 1.0162468958899999e+00, 9.5979227217800002e+01 },
    { 4.7020000000000001e-07, 5.1495136377800002e+00, 1.0736090241908000e+03 },
    { 4.3113999999999998e-07, 3.7595242106300000e+00, 6.3278373931320004e+02 },
    { 4.1242999999999999e-07, 3.4205369133799999e+00, 9.4206206196899996e+02 },
    { 3.8596000000000000e-07, 5.9076398205400000e+00, 3.3061896365820002e+02 },
    { 3.7231999999999999e-07, 2.0860962803200001e+00, 1.0309277421860000e+02 },
    { 3.5616999999999999e-07, 4.6919515436300001e+00, 6.2567019231239999e+02 },
    { 3.2286999999999998e-07, 4.4623163838900002e+00, 5.5103160609700001e+02 },
    { 3.2071999999999998e-07, 5.6033035689800004e+00, 1.5819593482830001e+03 },
    { 3.1385000000000002e-07, 5.4666070118899999e+00, 4.3371173787679999e+02 },
    { 3.0092000000000002e-07, 3.6230629449600000e+00, 9.4917560896980001e+02 },
    { 2.8375999999999999e-07, 4.7667540987399999e+00, 2.1187638603783998e+03 },
    { 2.8027000000000001e-07, 5.0201209900699997e+00, 2.2041264243879999e+02 },
    { 2.7813999999999997e-07, 4.7774647078500001e+00, 7.2876296653099996e+02 },
    { 2.6408000000000002e-07, 6.0228349169600000e+00, 4.1948464387519999e+02 },
    { 2.6398000000000001e-07, 4.2931741689400003e+00, 1.1695882514085999e+03 },
    { 2.4896999999999999e-07, 3.8867758331000002e-01, 5.1939602435610004e+02 },
    { 2.4882999999999999e-07, 1.9797981692100000e+00, 1.9907200143640000e+02 },
    { 2.4427000000000000e-07, 2.4676738439900001e+00, 2.0618554843720000e+02 },
    { 2.2586000000000001e-07, 2.1193339573899999e+00, 9.5628915597059995e+02 },
    { 2.2343000000000001e-07, 5.3076179880499996e+00, 1.1553611574070001e+03 },
    { 2.0475000000000000e-07, 3.9118415331000000e+00, 5.3998590583309999e+02 },
    { 1.9628999999999999e-07, 4.2139701342500002e+00, 6.3989728631399998e+02 },
    { 1.7704000000000000e-07, 6.0246646138099997e+00, 2.1340641002400002e+01 },
    { 1.6866000000000000e-07, 2.2131400816999999e-01, 4.1237109687440000e+02 },
    { 1.3416999999999999e-07, 4.9272991927999996e+00, 2.2752618943959999e+02 },
    { 1.3342000000000001e-07, 1.0925712179300000e+00, 8.5319638175199998e+02 },
    { 1.1494999999999999e-07, 4.0504300532400004e+00, 4.4082528487759998e+02 },
    { 9.7010000000000006e-08, 2.2359744827600001e+00, 3.0927832265580003e+02 },
    { 9.4769999999999994e-08, 4.8435823645599996e+00, 1.0380412891868000e+03 },
    { 8.7690000000000000e-08, 8.9310739987999999e-01, 3.1639186965660002e+02 },
    { 8.6280000000000002e-08, 5.0637137551200002e+00, 1.5748458012822000e+03 },
    { 8.2199999999999995e-08, 5.7098597527499999e+00, 8.3896928775039999e+02 },
    { 8.1219999999999995e-08, 8.0758964342999995e-01, 2.1329909543799999e+02 },
    { 7.4550000000000003e-08, 2.4841580448000000e-01, 6.1855664531160005e+02 },
    { 7.4070000000000001e-08, 2.8291722725800001e+00, 3.0216477565500003e+02 },
    { 6.8420000000000000e-08, 6.2066465244500000e+00, 1.2584539316256000e+03 },
    { 6.0870000000000002e-08, 6.0185303366099996e+00, 8.3185574074960005e+02 },
    { 5.9459999999999998e-08, 2.5737442462999999e-01, 2.1258774073792001e+03 },
    { 4.2050000000000003e-08, 4.0435545134000002e+00, 7.4299006053259995e+02 },
    { 4.1519999999999997e-08, 5.9259378476000002e+00, 1.2655674786264001e+03 },
    { 4.0980000000000000e-08, 7.4383398898999997e-01, 1.2443341522100000e+02 },
    { 3.8010000000000001e-08, 3.4155751339200000e+00, 1.4859801210651999e+03 },
    { 3.7039999999999999e-08, 3.2483123624300001e+00, 9.3494851496820002e+02 },
    { 3.2409999999999997e-08, 4.9481476982799997e+00, 1.6992792165031999e+03 },
    { 2.9880000000000000e-08, 4.1530490542000003e+00, 6.4701083331480004e+02 },
    { 2.8439999999999999e-08, 5.0426220811800002e+00, 1.2726810256271999e+03 },
    { 2.7680000000000000e-08, 3.5719223806399998e+00, 1.1624747044077999e+03 },
    { 2.7470000000000000e-08, 4.0470020549200001e+00, 8.8865680217000005e+01 },
    { 2.6270000000000000e-08, 5.5559683897300003e+00, 1.0625633239269000e+03 },
    { 2.5749999999999999e-08, 4.1246603289500001e+00, 1.4717530270636000e+03 },
    { 2.4719999999999999e-08, 4.8534849674200000e+00, 7.2164941953020002e+02 },
    { 2.4599999999999999e-08, 5.2490110762700004e+00, 2.6484548254729998e+03 },
    { 2.3590000000000001e-08, 5.7442494712799999e+00, 3.3773251065900001e+02 },
    { 2.3230000000000000e-08, 9.0656927683999999e-01, 1.3686602528450001e+03 },
    { 2.3219999999999999e-08, 5.6253033368300001e+00, 1.6032999892854000e+03 },
    { 2.1769999999999999e-08, 5.6485710508300002e+00, 9.6340270297140000e+02 },
    { 2.0680000000000001e-08, 7.4233309260000002e-01, 1.3615467058442000e+03 },
    { 2.0179999999999999e-08, 2.1525766846700001e+00, 1.9195845443560000e+02 },
    { 1.9950000000000000e-08, 4.5830038409500000e+00, 7.5010360753340001e+02 },
    { 1.9670000000000000e-08, 2.5692061323999998e-01, 6.1144309831080000e+02 },
    { 1.8110000000000001e-08, 2.2566389001000001e-01, 8.6030992875280003e+02 },
    { 1.7840000000000000e-08, 1.8871768378000001e-01, 5.2650957135689998e+02 },
    { 1.7840000000000000e-08, 1.0103753110000000e+00, 5.3287235883230005e+02 },
    { 1.7339999999999998e-08, 5.7168150659899997e+00, 1.0554497769261000e+03 },
    { 1.7170000000000001e-08, 3.7611291412300001e+00, 4.4793883187839998e+02 },
    { 1.7000000000000000e-08, 4.1696535430599999e+00, 1.0562005364515001e+03 },
    { 1.6800000000000002e-08, 6.3242761280000004e-02, 1.6850521225016000e+03 },
    { 1.4320000000000000e-08, 4.4330888212600001e+00, 8.1752133216199994e+01 },
    { 1.3930000000000000e-08, 3.5826879373899998e+00, 1.4788665740644001e+03 },
    { 1.3599999999999999e-08, 7.2191961038999997e-01, 1.1482476104062000e+03 },
    { 1.3189999999999999e-08, 6.1971681021000002e-01, 4.0525754987360000e+02 },
    { 1.2110000000000001e-08, 4.4138938645400003e+00, 1.0696768709277001e+03 },
    { 1.1980000000000000e-08, 5.1284411102299998e+00, 1.3757737998458001e+03 },
    { 1.0430000000000000e-08, 4.5535991049200000e+00, 1.0807225711916001e+03 },
    { 1.0130000000000000e-08, 7.3509414390000005e-01, 1.0490869894507000e+03 },
    { 1.0000000000000000e-08, 1.6839140624900000e+00, 1.3828873468465999e+03 },
    { 9.8600000000000003e-09, 5.7481099696999998e+00, 2.1116503133776000e+03 },
    { 9.6199999999999995e-09, 9.3080390060000004e-02, 2.3178358618148000e+03 },
    { 9.6099999999999997e-09, 3.5855176011099998e+00, 2.0974232193759999e+03 },
    { 9.5900000000000002e-09, 1.4875112596100000e+00, 1.6779385755008000e+03 },
    { 9.0500000000000000e-09, 3.6044185388800001e+00, 1.6921656695024001e+03 },
    { 9.0300000000000005e-09, 2.5250598145999997e-01, 8.4608283475120004e+02 },
    { 8.8900000000000005e-09, 3.1042732331799998e+00, 2.9505122865419997e+02 },
    { 8.6800000000000006e-09, 2.6075561298000000e-01, 1.8983512179396000e+03 },
    { 8.6200000000000004e-09, 2.0992928664899999e+00, 2.0085575391590000e+03 },
    { 8.5799999999999997e-09, 4.5156344860899997e+00, 2.4209286360333999e+03 },
    { 8.3199999999999994e-09, 1.4644054564000000e-01, 2.0014439921582000e+03 },
    { 8.2299999999999999e-09, 3.6101276100400002e+00, 6.5412438031559998e+02 },
    { 7.9300000000000005e-09, 4.5370720623400000e+00, 2.3463973644039999e+02 },
    { 7.8899999999999998e-09, 6.1943463299200001e+00, 2.2147430875962000e+03 },
    { 7.6199999999999997e-09, 9.4919223996000002e-01, 1.2513403846248000e+03 },
    { 7.4199999999999996e-09, 6.1562551337000004e-01, 5.3362311835770004e+02 },
    { 7.4199999999999996e-09, 5.8346748141000004e-01, 5.2575881183150000e+02 },
    { 7.3499999999999996e-09, 4.3213614739999997e+00, 1.9054647649404001e+03 },
    { 7.2600000000000002e-09, 4.9542670165700002e+00, 2.8404132799086001e+03 },
    { 7.1399999999999997e-09, 5.0121156633700004e+00, 2.0156710861598001e+03 },
    { 6.9900000000000001e-09, 5.1311753119999999e+00, 2.5240214102519999e+03 },
    { 6.9600000000000000e-09, 3.3722606607999999e+00, 2.4351557300350000e+03 },
    { 6.9200000000000001e-09, 6.4255931050000004e-01, 1.9943304451573999e+03 },
    { 6.8400000000000004e-09, 3.6143853397500001e+00, 1.1411340634054000e+03 },
    { 6.6700000000000003e-09, 1.0114093196000000e-01, 1.7881448967202000e+03 },
    { 6.3600000000000004e-09, 6.1585811042599996e+00, 2.6271141844705999e+03 },
    { 6.1799999999999998e-09, 6.1260765165400004e+00, 2.6342277314714001e+03 },
    { 5.9900000000000002e-09, 3.1561824783599999e+00, 2.7515475996916002e+03 },
    { 5.9399999999999998e-09, 6.6342094558999998e-01, 1.7810313497193999e+03 },
    { 5.6599999999999999e-09, 4.8177920335600000e+00, 2.3320629558164001e+03 },
    { 5.4400000000000002e-09, 3.8548422760499998e+00, 1.8023719907218001e+03 },
    { 5.2499999999999999e-09, 4.0322300188800000e+00, 2.2218566345969998e+03 },
    { 4.8399999999999998e-09, 5.0274363296000004e+00, 2.2005159935945999e+03 },
    { 4.7799999999999996e-09, 3.0022321250000000e-01, 2.3107223148140001e+03 },
    { 4.7699999999999999e-09, 5.9429678895800002e+00, 2.5311349572528002e+03 },
    { 4.6500000000000003e-09, 5.9043113218299998e+00, 2.9435060541272001e+03 },
    { 4.6299999999999999e-09, 2.3881154687000000e-01, 1.4646394800628000e+03 },
    { 4.3599999999999998e-09, 5.8825008003899999e+00, 2.1045367663768002e+03 },
    { 4.2899999999999999e-09, 4.3926928149200002e+00, 2.7373205056900001e+03 },
    { 4.1700000000000003e-09, 8.9795666781000005e-01, 1.8912376709388000e+03 },
    { 4.1499999999999999e-09, 6.2555819142700004e+00, 5.0123677709139997e+02 },
    { 4.1499999999999999e-09, 1.2266963876899999e+00, 5.5814515309779995e+02 },
    { 4.1199999999999998e-09, 9.2939189488999996e-01, 1.8841241239379999e+03 },
};

static const VSOP87Term VSOP87A_JUPITER_X4[58] = {
    { 1.7624020000000001e-05, 1.3286303975700000e+00, 5.2257741809380002e+02 },
    { 1.7178459999999999e-05, 3.0333153184300001e+00, 5.3680451209540001e+02 },
    { 3.0406300000000001e-06, 1.4314409625700000e+00, 5.1546387109299997e+02 },
    { 2.1650800000000001e-06, 2.9120559552600001e+00, 5.4391805909619995e+02 },
    { 1.6057100000000000e-06, 3.1415926535900001e+00, 0.0000000000000000e+00 },
    { 1.2819299999999999e-06, 3.8302226533599999e+00, 1.0664954771900000e+03 },
    { 8.1343000000000000e-07, 4.4795799927399997e+00, 7.1135470007999997e+00 },
    { 6.8446000000000003e-07, 1.0649840482699999e+00, 1.0522683831884001e+03 },
    { 5.6005000000000004e-07, 1.8094471024100001e+00, 1.0593819301891999e+03 },
    { 5.0538000000000004e-07, 4.4715393562700001e+00, 1.4227094001599999e+01 },
    { 2.3705000000000000e-07, 4.6845139714000000e-01, 1.0451548361876000e+03 },
    { 1.5040000000000000e-07, 1.6416455924200000e+00, 5.0835032409220003e+02 },
    { 1.3698000000000000e-07, 2.9081720932600001e+00, 1.5890728952838001e+03 },
    { 1.0460000000000000e-07, 4.4972015720999998e+00, 1.5961864422845999e+03 },
    { 1.0393000000000000e-07, 3.4606676748299998e+00, 1.0736090241908000e+03 },
    { 9.8500000000000002e-08, 5.4923011219799998e+00, 1.1731986822019999e+02 },
    { 9.6050000000000003e-08, 4.6656040146800004e+00, 1.1020632121940000e+02 },
    { 7.2909999999999996e-08, 2.5769513360300000e+00, 5.5103160609700001e+02 },
    { 7.2800000000000003e-08, 3.0643998803399999e+00, 5.2969096509459996e+02 },
    { 5.8080000000000001e-08, 2.7569401838299998e+00, 9.5979227217800002e+01 },
    { 5.2819999999999999e-08, 4.1915492903900002e+00, 3.3061896365820002e+02 },
    { 5.2269999999999998e-08, 4.2016633903100002e+00, 3.2350541665740002e+02 },
    { 5.1230000000000003e-08, 5.1904451600500003e+00, 9.4206206196899996e+02 },
    { 4.5529999999999998e-08, 8.9617728420000001e-02, 7.2876296653099996e+02 },
    { 3.5959999999999999e-08, 1.4061655375000001e-01, 7.3587651353180001e+02 },
    { 3.5110000000000001e-08, 3.6129476313999997e-01, 6.2567019231239999e+02 },
    { 3.2859999999999998e-08, 3.6240163270600001e+00, 1.9907200143640000e+02 },
    { 3.1809999999999998e-08, 4.6865320798000001e+00, 4.2659819087599999e+02 },
    { 3.1769999999999999e-08, 3.3790788707099999e+00, 4.3371173787679999e+02 },
    { 3.1200000000000001e-08, 3.8198500772699999e+00, 1.0309277421860000e+02 },
    { 2.9530000000000000e-08, 1.7452880500900001e+00, 6.1855664531160005e+02 },
    { 2.8600000000000001e-08, 1.1344596654500001e+00, 1.5819593482830001e+03 },
    { 2.7039999999999999e-08, 2.3972410035900000e+00, 4.4082528487759998e+02 },
    { 2.5889999999999999e-08, 2.5739446200999998e+00, 1.1695882514085999e+03 },
    { 2.3269999999999999e-08, 8.0202919113000004e-01, 1.1553611574070001e+03 },
    { 2.1249999999999998e-08, 1.5645702145700000e+00, 4.1948464387519999e+02 },
    { 1.9160000000000001e-08, 1.9146051817800001e+00, 4.1237109687440000e+02 },
    { 1.8040000000000001e-08, 5.6681404650499996e+00, 6.3278373931320004e+02 },
    { 1.6000000000000001e-08, 3.3393105508700001e+00, 2.1187638603783998e+03 },
    { 1.5309999999999999e-08, 2.6036141338199998e+00, 6.3989728631399998e+02 },
    { 1.4960000000000000e-08, 6.3527761947000005e-01, 1.5748458012822000e+03 },
    { 1.4769999999999999e-08, 5.2953212716999998e-01, 8.3185574074960005e+02 },
    { 1.4680000000000000e-08, 3.2440793698700001e+00, 2.2041264243879999e+02 },
    { 1.4160000000000001e-08, 3.2312030250200001e+00, 2.2752618943959999e+02 },
    { 1.3240000000000000e-08, 4.5738333619800002e+00, 3.0216477565500003e+02 },
    { 1.3160000000000000e-08, 5.2819139641000001e-01, 9.5628915597059995e+02 },
    { 1.1980000000000000e-08, 1.6079524525700000e+00, 1.2584539316256000e+03 },
    { 1.1870000000000000e-08, 5.7386355269999996e+00, 8.5319638175199998e+02 },
    { 1.1420000000000001e-08, 5.5655494981800002e+00, 9.4917560896980001e+02 },
    { 9.5300000000000000e-09, 1.7683702654300000e+00, 3.0927832265580003e+02 },
    { 8.9000000000000003e-09, 2.7018398572500000e+00, 6.4701083331480004e+02 },
    { 8.6800000000000006e-09, 6.7942468080999996e-01, 1.0380412891868000e+03 },
    { 8.1699999999999997e-09, 8.7495679358000000e-01, 7.4299006053259995e+02 },
    { 8.0399999999999995e-09, 1.1265603438300000e+00, 8.3896928775039999e+02 },
    { 5.9799999999999996e-09, 4.0322652264899999e+00, 3.3773251065900001e+02 },
    { 5.5700000000000004e-09, 2.2950871058800000e+00, 1.4859801210651999e+03 },
    { 4.5100000000000003e-09, 6.1566349461000003e-01, 1.4717530270636000e+03 },
    { 3.9600000000000004e-09, 3.9038414304300000e+00, 1.6032999892854000e+03 },
};

static const VSOP87Term VSOP87A_JUPITER_X5[11] = {
    { 1.3147099999999999e-06, 3.2128492886700002e+00, 5.2257741809380002e+02 },
    { 1.2674800000000000e-06, 1.1630700213400000e+00, 5.3680451209540001e+02 },
    { 4.8324000000000003e-07, 3.1965772312800000e+00, 5.1546387109299997e+02 },
    { 3.4033999999999999e-07, 1.1280191325800000e+00, 5.4391805909619995e+02 },
    { 1.1212000000000000e-07, 2.1934061860099998e+00, 1.0664954771900000e+03 },
    { 4.0600000000000000e-08, 3.3424754556299998e+00, 1.0522683831884001e+03 },
    { 1.9429999999999999e-08, 1.7523788575499999e+00, 1.0736090241908000e+03 },
    { 1.7859999999999999e-08, 3.0710374065999999e+00, 1.0451548361876000e+03 },
    { 8.8599999999999996e-09, 3.2549926571399999e+00, 5.0835032409220003e+02 },
    { 8.8599999999999996e-09, 4.2272856448200002e+00, 5.5103160609700001e+02 },
    { 4.6699999999999998e-09, 2.9492134784899999e+00, 1.5961864422845999e+03 },
};

static const VSOP87Term VSOP87A_JUPITER_Y0[1037] = {
    { 5.1952004658900002e+00, 5.3120316273099997e+00, 5.2969096509459996e+02 },
    { 1.2592862601999999e-01, 5.6616022772800001e+00, 1.0593819301891999e+03 },
    { 9.3636706159999999e-02, 3.1415926535900001e+00, 0.0000000000000000e+00 },
    { 1.5082752990000000e-02, 5.4393496810200004e+00, 5.2257741809380002e+02 },
    { 1.4758093700000001e-02, 2.0467956649499999e+00, 5.3680451209540001e+02 },
    { 4.5775080599999999e-03, 6.0112909350099999e+00, 1.5890728952838001e+03 },
    { 3.7828557799999998e-03, 3.5300678238300001e+00, 1.0309277421860000e+02 },
    { 3.0068667899999998e-03, 3.6094805073999998e+00, 7.1135470007999997e+00 },
    { 1.9233312800000001e-03, 3.4569056477100002e+00, 4.2659819087599999e+02 },
    { 1.4610465600000001e-03, 4.6226722443100003e+00, 1.1020632121940000e+02 },
    { 1.3948005799999999e-03, 4.0007530770599997e+00, 6.3278373931320004e+02 },
    { 1.3269676399999999e-03, 5.6218458185899998e+00, 2.1329909543799999e+02 },
    { 1.1404310999999999e-03, 4.7298226296900001e+00, 1.1624747044077999e+03 },
    { 1.0199980700000000e-03, 4.5759459888400000e+00, 1.0522683831884001e+03 },
    { 7.2091178000000000e-04, 2.3904865914800002e+00, 1.0664954771900000e+03 },
    { 6.8374489000000004e-04, 1.8653707437400000e+00, 8.4608283475120004e+02 },
    { 5.9051768999999999e-04, 2.8952907096799998e+00, 9.4917560896980001e+02 },
    { 2.9807369000000000e-04, 4.5210577273999997e+00, 2.0618554843720000e+02 },
    { 2.6933578999999998e-04, 3.8623395682699999e+00, 4.1948464387519999e+02 },
    { 2.6619714000000001e-04, 3.2820317495100002e+00, 3.2350541665740002e+02 },
    { 2.0873779999999999e-04, 3.7936988175700002e+00, 7.3587651353180001e+02 },
    { 1.9727397000000001e-04, 7.8185345320000002e-02, 2.1187638603783998e+03 },
    { 1.8639846000000000e-04, 3.8751972138000002e-01, 3.1639186965660002e+02 },
    { 1.6355726000000001e-04, 5.5699788160399999e+00, 5.1546387109299997e+02 },
    { 1.4606858000000000e-04, 4.7759399144999998e-01, 7.4299006053259995e+02 },
    { 1.2153427000000000e-04, 2.1815197249899998e+00, 5.2575881183150000e+02 },
    { 1.1988875000000000e-04, 5.2968760208900001e+00, 5.3362311835770004e+02 },
    { 1.1419853000000000e-04, 1.9108934146800001e+00, 5.4391805909619995e+02 },
    { 8.7327889999999994e-05, 5.0960706609699997e+00, 1.6921656695024001e+03 },
    { 8.4431069999999997e-05, 1.9143580169700001e+00, 6.3989728631399998e+02 },
    { 8.1631629999999995e-05, 4.0030374237500004e+00, 1.4788665740644001e+03 },
    { 7.7791840000000006e-05, 4.8334630066199997e+00, 1.5819593482830001e+03 },
    { 7.6194860000000002e-05, 5.5955415199700003e+00, 1.2655674786264001e+03 },
    { 7.4141150000000004e-05, 4.4114199046099998e+00, 9.5628915597059995e+02 },
    { 5.7721320000000000e-05, 3.4691571692699998e+00, 1.4227094001599999e+01 },
    { 5.3228820000000002e-05, 5.8174047264500004e+00, 5.2650957135689998e+02 },
    { 5.2170250000000001e-05, 1.6617864354200000e+00, 5.3287235883230005e+02 },
    { 4.5671809999999999e-05, 3.7130077693499999e+00, 9.5979227217800002e+01 },
    { 4.5283549999999998e-05, 2.1837755803799999e+00, 1.3757737998458001e+03 },
    { 3.9398750000000003e-05, 2.7383053105399999e+00, 1.5961864422845999e+03 },
    { 3.2354190000000002e-05, 4.7660034706200003e+00, 6.2567019231239999e+02 },
    { 3.1407400000000000e-05, 5.5956679692200000e+00, 3.0927832265580003e+02 },
    { 2.8554230000000001e-05, 4.4447828600600001e+00, 1.1731986822019999e+02 },
    { 2.6722619999999999e-05, 1.6185789706900000e+00, 1.1695882514085999e+03 },
    { 2.4456250000000002e-05, 2.6703695223000001e+00, 4.3371173787679999e+02 },
    { 2.4236390000000000e-05, 2.3294233983899999e+00, 2.2041264243879999e+02 },
    { 2.4095810000000001e-05, 4.3319630160899996e+00, 1.0451548361876000e+03 },
    { 2.3626620000000000e-05, 4.6041758020700003e+00, 1.1553611574070001e+03 },
    { 2.2535450000000001e-05, 4.2846282572199996e+00, 8.3896928775039999e+02 },
    { 1.6465679999999999e-05, 3.9196587656199999e+00, 7.4781598567299994e+01 },
    { 1.4581690000000001e-05, 4.1876188127700003e+00, 8.5319638175199998e+02 },
    { 1.4321949999999999e-05, 3.2482455450000001e+00, 9.4206206196899996e+02 },
    { 1.0877270000000000e-05, 5.8467308693900000e+00, 5.2724328453980002e+02 },
    { 1.0795120000000001e-05, 1.6344850734600000e+00, 5.3213864564940002e+02 },
    { 1.0502700000000000e-05, 4.8370601432699996e+00, 1.7952584437210000e+03 },
    { 1.0023550000000000e-05, 1.5093193987000000e+00, 1.2726810256271999e+03 },
    { 9.8075099999999995e-06, 6.2999941323999997e-01, 3.8012776796000003e+02 },
    { 9.5727000000000007e-06, 6.0600222916300002e+00, 1.4956319713459999e+02 },
    { 9.3447599999999996e-06, 4.2886055429999997e-01, 2.6484548254729998e+03 },
    { 9.2297200000000005e-06, 4.6872779257500001e+00, 1.9907200143640000e+02 },
    { 8.5832200000000008e-06, 2.1423748981699999e+00, 5.2964278098479997e+02 },
    { 8.5822700000000004e-06, 5.3401860256399996e+00, 5.2973914920439995e+02 },
    { 8.4712699999999993e-06, 5.9304314008199999e+00, 3.9321532631000000e+00 },
    { 8.0600599999999999e-06, 1.6826763933400000e+00, 1.3686602528450001e+03 },
    { 7.4775900000000003e-06, 3.1398049203300000e+00, 3.3061896365820002e+02 },
    { 6.9393099999999996e-06, 6.0521392726299998e+00, 1.0633140834523001e+03 },
    { 6.8208000000000000e-06, 4.5294252832400002e+00, 2.0085575391590000e+03 },
    { 6.7999699999999997e-06, 5.0063220430199999e+00, 5.2872775724810003e+02 },
    { 6.7836499999999998e-06, 2.4763088177500001e+00, 5.3065417294110000e+02 },
    { 6.6975699999999996e-06, 4.0301640619999999e+00, 7.2876296653099996e+02 },
    { 6.4709700000000000e-06, 3.5549639159000002e+00, 9.8460033162189995e+02 },
    { 6.2327200000000002e-06, 4.8489747837400001e+00, 2.1116503133776000e+03 },
    { 5.7531900000000004e-06, 5.4604967436500003e+00, 2.2218566345969998e+03 },
    { 5.6290099999999997e-06, 2.4930554766999999e+00, 1.0554497769261000e+03 },
    { 5.4007599999999998e-06, 3.7472311552200002e+00, 3.8133035637799999e+01 },
    { 5.1893600000000004e-06, 4.1735549310300000e+00, 4.1237109687440000e+02 },
    { 4.5995200000000003e-06, 2.2558951289900002e+00, 1.0736090241908000e+03 },
    { 4.5607799999999999e-06, 3.6546666540100001e+00, 1.4717530270636000e+03 },
    { 4.1034399999999998e-06, 1.4177989953200001e+00, 9.0981873305459999e+02 },
    { 3.8844999999999998e-06, 4.4902976198799998e+00, 6.0447256366190004e+02 },
    { 3.8103700000000000e-06, 3.3079928754300001e+00, 3.1813937377000001e+00 },
    { 3.6073600000000001e-06, 2.9568602950199998e+00, 4.5490936652729999e+02 },
    { 3.5702300000000001e-06, 5.3858030203400000e+00, 1.2584539316256000e+03 },
    { 3.3364700000000002e-06, 5.6659303844000002e+00, 7.6266071275599998e+01 },
    { 3.0414199999999999e-06, 1.8267926862000000e-01, 6.2830758499914000e+03 },
    { 3.0384300000000000e-06, 3.4929749240899999e+00, 5.2916970023279998e+02 },
    { 3.0345100000000002e-06, 3.9882079095499998e+00, 5.3021222995640005e+02 },
    { 2.9928099999999998e-06, 2.4854639870000002e+00, 1.9054647649404001e+03 },
    { 2.9194599999999999e-06, 4.5543666451900000e+00, 3.0534616939270001e+02 },
    { 2.7573299999999998e-06, 5.3630208899799996e+00, 1.4859801210651999e+03 },
    { 2.6379100000000000e-06, 4.5770854550399998e+00, 1.5748458012822000e+03 },
    { 2.5432699999999999e-06, 2.0140414705800000e+00, 1.0625633239269000e+03 },
    { 2.3328500000000000e-06, 5.6354831992400003e+00, 1.0562005364515001e+03 },
    { 2.2696299999999998e-06, 3.0874906240000000e+00, 2.1258774073792001e+03 },
    { 2.2558000000000001e-06, 1.9311446596199999e+00, 1.6992792165031999e+03 },
    { 2.1523300000000001e-06, 9.7247023786999998e-01, 4.5342489381899998e+02 },
    { 2.1021400000000000e-06, 2.4486338863100001e+00, 4.9033408917940000e+02 },
    { 2.0822900000000001e-06, 1.8228161402200000e+00, 1.4395096981491999e+03 },
    { 1.9586799999999999e-06, 5.7118687557400003e+00, 5.0835032409220003e+02 },
    { 1.9458000000000001e-06, 3.7346118314400001e+00, 1.0212488945514000e+03 },
    { 1.9149699999999999e-06, 5.0169305618199997e+00, 5.6904784100979998e+02 },
    { 1.7883900000000000e-06, 4.5300699290199997e+00, 5.2677020378780003e+02 },
    { 1.7700500000000001e-06, 2.9524332165399998e+00, 5.3261172640140001e+02 },
    { 1.7690699999999999e-06, 1.6058917643799999e+00, 1.0213285546211000e+04 },
    { 1.7481700000000001e-06, 4.3249591547999999e-01, 1.4844727083000000e+00 },
    { 1.7235100000000000e-06, 5.1872271224900004e+00, 1.8983512179396000e+03 },
    { 1.6671200000000000e-06, 2.0863116984899999e+00, 2.2434479570190001e+02 },
    { 1.6505300000000001e-06, 1.9057890708500000e+00, 5.2894020556919997e+02 },
    { 1.6458500000000001e-06, 5.5761942887600000e+00, 5.3044172461999995e+02 },
    { 1.5799600000000001e-06, 2.7576658024100000e+00, 5.2549817940059995e+02 },
    { 1.5547900000000001e-06, 4.7268699763499997e+00, 5.3388375078859997e+02 },
    { 1.5540099999999999e-06, 5.9657999238499997e+00, 5.1939602435610004e+02 },
    { 1.5263300000000001e-06, 5.7047031015699998e+00, 3.0216477565500003e+02 },
    { 1.4715400000000001e-06, 2.1768711464299999e+00, 1.6850521225016000e+03 },
    { 1.4250200000000000e-06, 2.1874605326699998e+00, 2.2752618943959999e+02 },
    { 1.3940200000000000e-06, 1.7855393434100000e+00, 6.4701083331480004e+02 },
    { 1.3638599999999999e-06, 9.6879673889999995e-01, 1.1045700263900001e+01 },
    { 1.3433600000000001e-06, 2.2267752385600001e+00, 5.4073666535849998e+02 },
    { 1.3366499999999999e-06, 4.6349161209199998e+00, 8.3185574074960005e+02 },
    { 1.2553600000000000e-06, 5.6086665684400003e+00, 6.1855664531160005e+02 },
    { 1.2450700000000000e-06, 3.4593336374699999e+00, 1.7881448967202000e+03 },
    { 1.2442699999999999e-06, 1.8177937520200000e+00, 9.8311585891360005e+02 },
    { 1.2356900000000000e-06, 1.7757870520000001e+00, 5.2502509864859996e+02 },
    { 1.2280500000000000e-06, 3.3355092720399999e+00, 5.2427433912319998e+02 },
    { 1.2187699999999999e-06, 2.2185973899699998e+00, 5.2406189080210004e+02 },
    { 1.2095799999999999e-06, 4.1452485604599998e+00, 5.3510759106600005e+02 },
    { 1.2057900000000001e-06, 1.5122017752500001e+00, 5.3998590583309999e+02 },
    { 1.2003200000000001e-06, 5.2655829150000004e+00, 5.3532003938710000e+02 },
    { 1.1964600000000001e-06, 5.0239412570899997e+00, 2.3249494088155998e+03 },
    { 1.1766000000000001e-06, 1.7999231581899999e+00, 7.5010360753340001e+02 },
    { 1.1615900000000000e-06, 5.7100355222300001e+00, 5.3435683154059996e+02 },
    { 1.1120500000000000e-06, 1.8692834498599999e+00, 2.0014439921582000e+03 },
    { 1.0664500000000000e-06, 4.9615394878599997e+00, 9.9160620955499994e+01 },
    { 1.0480700000000000e-06, 1.7547305191400000e+00, 5.5103160609700001e+02 },
    { 1.0442999999999999e-06, 2.1947206637000001e+00, 5.2953090640020002e+02 },
    { 1.0442999999999999e-06, 5.2875576382600000e+00, 5.2985102378900001e+02 },
    { 1.0313800000000001e-06, 2.8430408573300001e+00, 9.6340270297140000e+02 },
    { 9.6101000000000005e-07, 3.3155974187399999e+00, 2.1340641002400002e+01 },
    { 9.5413000000000009e-07, 4.3660356245400003e+00, 3.5424722652100002e+01 },
    { 9.4763999999999998e-07, 1.6797996059200000e+00, 1.8023719907218001e+03 },
    { 9.1885999999999996e-07, 4.7307218872899996e+00, 1.3828873468465999e+03 },
    { 9.1856000000000004e-07, 1.4648991893800001e+00, 4.4082528487759998e+02 },
    { 8.2393000000000005e-07, 1.0110507294500000e+00, 5.3117543780289998e+02 },
    { 7.6558999999999996e-07, 1.6914007578000001e-01, 5.2820649238630006e+02 },
    { 7.3989000000000004e-07, 2.8474864280099998e+00, 1.7394221952280000e+02 },
    { 7.3325000000000003e-07, 4.0575021511899996e+00, 5.1153171782990000e+02 },
    { 7.2096000000000002e-07, 2.4518168594600001e+00, 5.2990341341570002e+02 },
    { 7.2096000000000002e-07, 5.0304614425000000e+00, 5.2947851677350002e+02 },
    { 6.9890999999999995e-07, 1.5960118806300001e+00, 1.1413847448250000e+02 },
    { 6.8945000000000002e-07, 3.4097602853700000e+00, 5.4785021235930003e+02 },
    { 6.8200999999999995e-07, 4.3553025246099999e+00, 4.1555249061209997e+02 },
    { 6.7561000000000000e-07, 4.9949065446800001e+00, 5.1864526483069994e+02 },
    { 6.6311000000000001e-07, 2.1687315155100002e+00, 2.3056457082540001e+02 },
    { 6.4547999999999998e-07, 5.7442560409699999e+00, 1.3647280995819001e+03 },
    { 6.3600000000000003e-07, 4.6069096452400000e+00, 2.2147430875962000e+03 },
    { 6.2656999999999998e-07, 2.1856643583799999e+00, 9.2047073954699997e+01 },
    { 6.0722999999999997e-07, 5.6040129448000000e+00, 4.9426624244250002e+02 },
    { 6.0019000000000003e-07, 2.2088526168699998e+00, 1.0618296107440001e+03 },
    { 5.9078000000000000e-07, 3.2428411283199998e+00, 2.1045367663768002e+03 },
    { 5.8551000000000001e-07, 7.4219324450000002e-01, 1.0578974574808999e+03 },
    { 5.6418999999999997e-07, 3.1967228365899998e+00, 1.0702492748170000e+02 },
    { 5.5825999999999997e-07, 3.2335628560499998e+00, 6.4382943957709995e+02 },
    { 5.4799999999999998e-07, 5.0017759723399999e+00, 2.6413412784722000e+03 },
    { 5.4010000000000003e-07, 3.8559528823000000e-01, 1.0483362299252999e+03 },
    { 5.2715000000000001e-07, 6.1961810297000000e+00, 1.0569342496344000e+03 },
    { 5.2462000000000005e-07, 3.0435172527900001e+00, 9.6022130923370003e+02 },
    { 5.2219999999999995e-07, 1.8973088302100001e+00, 5.6511568774670002e+02 },
    { 5.1559000000000003e-07, 7.5903748579999999e-02, 5.1471311156759998e+02 },
    { 5.1167999999999999e-07, 4.1541716916200002e+00, 2.4280421830342002e+03 },
    { 4.9754000000000005e-07, 3.1051950136000001e-01, 6.7925416222920001e+02 },
    { 4.9711000000000004e-07, 1.1233049503999999e+00, 5.4466881862160005e+02 },
    { 4.9388999999999998e-07, 4.3836964711700004e+00, 1.0380412891868000e+03 },
    { 4.9040999999999996e-07, 2.5833012718199999e+00, 5.2354062594029995e+02 },
    { 4.8788999999999996e-07, 4.6285231497000003e+00, 3.3406124266997999e+03 },
    { 4.8106999999999995e-07, 1.5744380599400001e+00, 2.3178358618148000e+03 },
    { 4.8032999999999996e-07, 1.6476968228400000e+00, 3.8724131496080003e+02 },
    { 4.7081999999999998e-07, 2.1647901351000001e+00, 1.5128068240082000e+03 },
    { 4.7066000000000001e-07, 7.8305409046999996e-01, 3.1781457905676002e+03 },
    { 4.6791000000000001e-07, 4.9903227816799998e+00, 2.5382485042536000e+03 },
    { 4.5707000000000000e-07, 5.7660154169700002e+00, 3.9356875915200000e+01 },
    { 4.4448999999999998e-07, 4.9023868398700001e+00, 5.3584130424889997e+02 },
    { 4.4121999999999998e-07, 6.0619696373199998e+00, 4.3053034413910001e+02 },
    { 4.3994000000000001e-07, 3.2860634130899999e+00, 3.1245971639350000e+02 },
    { 4.3765000000000000e-07, 1.9354116885699999e+00, 2.4476805547999998e+00 },
    { 4.1708000000000003e-07, 3.0813190147500000e+00, 1.0294940738499999e+01 },
    { 4.1687000000000000e-07, 5.6895887805700003e+00, 1.0594301142990000e+03 },
    { 4.1687000000000000e-07, 2.4917776529400002e+00, 1.0593337460794000e+03 },
    { 4.1522999999999999e-07, 3.8760247796599998e+00, 1.5142912967165000e+03 },
    { 4.1507000000000002e-07, 2.1319785414000000e-01, 1.5930050485469001e+03 },
    { 4.1488999999999998e-07, 1.8747114759800001e+00, 7.0849445304200003e+01 },
    { 4.0515000000000000e-07, 4.9077189630499998e+00, 1.8159247264699999e+01 },
    { 4.0368000000000003e-07, 4.1195413216299999e+00, 1.1482476104062000e+03 },
    { 4.0007000000000001e-07, 4.9467677631500004e+00, 1.0627416795630000e+02 },
    { 3.7225000000000001e-07, 5.8361578984599998e+00, 2.7515475996916002e+03 },
    { 3.6843999999999998e-07, 4.1667544878599996e+00, 5.2943033266370003e+02 },
    { 3.6843999999999998e-07, 3.3155238141000001e+00, 5.2995159752550001e+02 },
    { 3.5862999999999998e-07, 4.3504645507599999e+00, 1.8912376709388000e+03 },
    { 3.5790000000000000e-07, 5.8558588869000001e+00, 6.7214061522839995e+02 },
    { 3.5390000000000001e-07, 5.4921093465100004e+00, 5.2262560220360001e+02 },
    { 3.4849999999999998e-07, 2.9248463085299998e+00, 1.0603451380357001e+03 },
    { 3.3893999999999999e-07, 4.6346433378400000e+00, 8.8543971066639995e+02 },
    { 3.3186000000000001e-07, 1.4204474192500001e+00, 6.2885158605009997e+02 },
    { 3.3053999999999999e-07, 4.5135901487299996e+00, 5.6782400073240001e+02 },
    { 3.2992999999999999e-07, 5.3557732070700004e+00, 1.0584187223427000e+03 },
    { 3.2943000000000001e-07, 2.2794672603500001e+00, 5.2252923398400003e+02 },
    { 3.2870000000000003e-07, 4.2535285687700002e+00, 1.2443341522100000e+02 },
    { 3.1537000000000000e-07, 4.7908939999599998e+00, 4.3764389113990001e+02 },
    { 3.0585000000000000e-07, 3.1403602715000001e+00, 1.0239572075371000e+03 },
    { 3.0492000000000001e-07, 2.8157623096500002e+00, 1.5851407420206999e+03 },
    { 2.9909000000000001e-07, 1.9883899155400000e+00, 5.3675632798560002e+02 },
    { 2.9097999999999998e-07, 5.8734860001099998e+00, 1.2125202148330000e+02 },
    { 2.8101000000000000e-07, 1.3711179625200001e+00, 1.3615467058442000e+03 },
    { 2.7935999999999998e-07, 3.4015551841699998e+00, 1.2899465010146000e+03 },
    { 2.7674999999999999e-07, 3.7675241384399998e+00, 5.2841894070740000e+02 },
    { 2.7669000000000002e-07, 6.1357138585200000e+00, 1.4244965013379999e+02 },
    { 2.7585999999999998e-07, 3.7147133343999998e+00, 5.3096298948180004e+02 },
    { 2.7496999999999998e-07, 5.2034935239999998e+00, 5.3685269620520000e+02 },
    { 2.7261000000000000e-07, 2.7203134832200000e+00, 2.4209286360333999e+03 },
    { 2.7158000000000000e-07, 4.0417246261399997e+00, 9.3494851496820002e+02 },
    { 2.7098000000000001e-07, 5.6480883195200002e+00, 4.9744763618019999e+02 },
    { 2.6903000000000002e-07, 4.5533846653000003e+00, 2.9912639426919998e+02 },
    { 2.6669000000000001e-07, 1.6120174354200001e+00, 1.1585425511446999e+03 },
    { 2.5974999999999998e-07, 4.2506468896299996e+00, 6.3671589257630001e+02 },
    { 2.5302000000000002e-07, 6.9085103669000003e-01, 4.5884151979040001e+02 },
    { 2.4955000000000001e-07, 3.5666867160300000e+00, 8.6030992875280003e+02 },
    { 2.4824000000000000e-07, 4.8721306796799997e+00, 1.1341635287565000e+03 },
    { 2.4690000000000002e-07, 4.1516544706099996e+00, 7.4692221379570003e+02 },
    { 2.4545000000000002e-07, 4.0715302969199998e+00, 9.9911380480899993e+01 },
    { 2.4392999999999999e-07, 1.7692251436299999e+00, 5.6193429400900004e+02 },
    { 2.3976000000000002e-07, 4.1101550185800004e+00, 5.2309868295560000e+02 },
    { 2.3620000000000001e-07, 3.1978935801900001e+00, 4.2266603761290003e+02 },
    { 2.3519999999999999e-07, 2.6826380286300000e+00, 6.2173803904930003e+02 },
    { 2.2230000000000000e-07, 5.1859676015399998e+00, 5.2161421024729998e+02 },
    { 2.1869999999999999e-07, 3.3741033647199998e+00, 5.3628324723360004e+02 },
    { 2.1575000000000001e-07, 4.6963152598500004e+00, 1.1664068576709001e+03 },
    { 2.1553000000000000e-07, 2.9372828887000002e-01, 8.8865680217000005e+01 },
    { 2.1220000000000001e-07, 4.7536633740900003e+00, 4.9575071515079998e+02 },
    { 2.0372000000000000e-07, 1.3853811649000001e+00, 2.6342277314714001e+03 },
    { 1.9791999999999999e-07, 5.6421039364899999e+00, 5.0531194270639998e+02 },
    { 1.9758000000000000e-07, 3.5738461976200000e+00, 1.4646394800628000e+03 },
    { 1.9640000000000001e-07, 8.1920620311000003e-01, 8.3503713448730002e+02 },
    { 1.9591000000000001e-07, 2.6035210873799999e+00, 1.6032999892854000e+03 },
    { 1.9558999999999999e-07, 3.3422760418000003e-01, 1.0987388061044001e+03 },
    { 1.9490000000000000e-07, 3.9944970553000001e+00, 2.5311349572528002e+03 },
    { 1.9198999999999999e-07, 2.7271145364699998e+00, 5.6363121503840000e+02 },
    { 1.8999000000000000e-07, 5.4653451643999995e-01, 3.2743756992049998e+02 },
    { 1.8929000000000000e-07, 2.0317596842499999e+00, 5.3591074021810005e+02 },
    { 1.8715000000000000e-07, 5.4478261897699998e+00, 5.2347118997109999e+02 },
    { 1.8510000000000001e-07, 1.8460169271000000e+00, 5.5406998748280000e+02 },
    { 1.8456999999999999e-07, 2.1828984144299999e+00, 1.9692006632437999e+03 },
    { 1.8395000000000000e-07, 3.0943223660800001e+00, 9.5310776223289997e+02 },
    { 1.8357000000000000e-07, 6.2106608417999998e-01, 5.6290742925000004e+00 },
    { 1.8236000000000000e-07, 3.9561351770800002e+00, 2.2076295405954002e+03 },
    { 1.8076000000000001e-07, 4.2845386572999999e-01, 6.0054041039879996e+02 },
    { 1.7929000000000001e-07, 3.4031183014800002e+00, 1.1514290041439001e+03 },
    { 1.7873000000000001e-07, 2.2959851412300001e+00, 5.3776771994190005e+02 },
    { 1.7574000000000001e-07, 4.3364629239700001e+00, 2.1723124870109999e+02 },
    { 1.7517000000000000e-07, 2.6819507153100002e+00, 2.4351557300350000e+03 },
    { 1.7253000000000000e-07, 4.8647579993700001e+00, 1.9195845443560000e+02 },
    { 1.7065999999999999e-07, 2.4735554414399998e+00, 4.9155792945680003e+02 },
    { 1.6950999999999999e-07, 4.0305049254899998e+00, 2.7444340526907999e+03 },
    { 1.6488000000000000e-07, 2.2776357829600000e+00, 2.2289701815978001e+03 },
    { 1.6271999999999999e-07, 2.3370662879799999e+00, 5.2698265210889997e+02 },
    { 1.6238999999999999e-07, 2.2009026979400002e+00, 3.2243328914400003e+01 },
    { 1.6170000000000000e-07, 5.1450866348000002e+00, 5.3239927808029995e+02 },
    { 1.6163000000000000e-07, 4.8695115244200000e+00, 1.0490869894507000e+03 },
    { 1.5853000000000000e-07, 1.4551082771699999e+00, 5.0137978944330001e+02 },
    { 1.5841000000000000e-07, 5.6125268750000001e-01, 7.3194436026870005e+02 },
    { 1.5808000000000000e-07, 5.6723334446499996e+00, 9.8853248488500003e+02 },
    { 1.5690999999999999e-07, 2.4851358272500002e+00, 2.4379022388199999e+01 },
    { 1.5643000000000001e-07, 9.7730443591000005e-01, 4.6658664459999999e+00 },
    { 1.5484000000000000e-07, 1.7216750045300000e+00, 3.7864329525170001e+02 },
    { 1.5402000000000000e-07, 9.9057013095000002e-01, 2.0225339517410001e+02 },
    { 1.5118999999999999e-07, 5.8115644317399999e+00, 7.6025553592000006e+02 },
    { 1.5020000000000001e-07, 6.0820860347399996e+00, 1.5578297225809999e+02 },
    { 1.4994000000000000e-07, 2.0119821039999999e+00, 4.6053844081979997e+02 },
    { 1.4894999999999999e-07, 1.1330960665000001e-01, 2.0156710861598001e+03 },
    { 1.4868999999999999e-07, 5.9836824535700002e+00, 5.5800214074589996e+02 },
    { 1.4774000000000000e-07, 5.8062516161500000e+00, 9.4524345570670005e+02 },
    { 1.4712000000000001e-07, 3.8430662788599999e+00, 1.0588606653274001e+03 },
    { 1.4712000000000001e-07, 4.3383001546499997e+00, 1.0599031950510000e+03 },
    { 1.4439000000000001e-07, 3.2863715827800002e+00, 1.0704276304530999e+03 },
    { 1.4314000000000000e-07, 4.0701973320500002e+00, 1.5509398596460001e+03 },
    { 1.4249000000000000e-07, 1.7224877968400001e+00, 4.6202291352809999e+02 },
    { 1.4238999999999999e-07, 4.9774070278300000e+00, 1.5104766984290001e+02 },
    { 1.4182000000000001e-07, 3.0120284606099998e+00, 1.1735204046716999e+03 },
    { 1.4077000000000001e-07, 5.1698231922600000e+00, 1.2513403846248000e+03 },
    { 1.3957000000000000e-07, 2.3641709808700000e+00, 1.5922542890215000e+03 },
    { 1.3791000000000000e-07, 2.9797134784499999e+00, 3.3773251065900001e+02 },
    { 1.3761000000000000e-07, 6.0454338852799996e+00, 6.7668051566499997e+01 },
    { 1.3749000000000000e-07, 5.2644877497999998e-01, 4.5097721326419997e+02 },
    { 1.3736000000000001e-07, 4.0712967994799998e+00, 1.1338771495710000e+02 },
    { 1.3707000000000000e-07, 2.0378988137300000e+00, 4.9867147645760002e+02 },
    { 1.3568999999999999e-07, 5.7790664206000004e+00, 1.5858915015461000e+03 },
    { 1.3524000000000001e-07, 3.4487632073699999e+00, 2.6555683724738001e+03 },
    { 1.3484999999999999e-07, 2.7071179860900001e+00, 3.8161224066829999e+02 },
    { 1.3472000000000000e-07, 4.5030581084400003e+00, 7.2164941953020002e+02 },
    { 1.3435000000000001e-07, 2.7486916653700000e+00, 1.0412226829245001e+03 },
    { 1.3428000000000001e-07, 5.5062654649400002e+00, 5.9735901666109999e+02 },
    { 1.3318999999999999e-07, 2.4253895363800000e+00, 2.7373205056900001e+03 },
    { 1.3201000000000000e-07, 5.9663713572299999e+00, 2.8311175651300001e+01 },
    { 1.3072000000000000e-07, 4.7691168530199999e+00, 5.4166259714000002e+00 },
    { 1.2895000000000000e-07, 2.7168506029100001e+00, 9.4498282327580000e+02 },
    { 1.2856999999999999e-07, 5.5073927711900001e+00, 5.9884348936940000e+02 },
    { 1.2753000000000000e-07, 5.4440391575899998e+00, 5.6071045373159996e+02 },
    { 1.2686000000000001e-07, 6.2381830290800000e+00, 5.3138788612400003e+02 },
    { 1.2686000000000001e-07, 1.2440952728800001e+00, 5.2799404406520000e+02 },
    { 1.2149000000000001e-07, 4.5784442250500001e+00, 7.2073285581600004e+01 },
    { 1.2120000000000000e-07, 3.2813039495399998e+00, 1.0200250542740000e+03 },
    { 1.2011000000000001e-07, 1.6352132577600000e+00, 9.3812990870589999e+02 },
    { 1.1885000000000000e-07, 5.6823593188999999e-01, 6.3735898303399999e+01 },
    { 1.1721000000000000e-07, 3.8196247217799999e+00, 4.6470422915999997e+01 },
    { 1.1629000000000000e-07, 6.7354350592000001e-01, 6.0840471692500000e+02 },
    { 1.1546000000000000e-07, 3.1915225651000001e-01, 6.3596513305090002e+02 },
    { 1.1513000000000000e-07, 2.7439356908999999e+00, 9.6320784650000002e-01 },
    { 1.1192000000000000e-07, 4.3998037984999998e+00, 3.3940249943799998e+01 },
    { 1.1189000000000000e-07, 4.6216791566099999e+00, 6.2960234557549995e+02 },
    { 1.1186000000000000e-07, 5.1708260290000002e-02, 5.2825467649610005e+02 },
    { 1.1186000000000000e-07, 1.1473847344899999e+00, 5.3112725369309999e+02 },
    { 1.0861000000000000e-07, 3.0677449749900001e+00, 2.4872381809010000e+02 },
    { 1.0799000000000000e-07, 2.8527878322900002e+00, 4.2977958461370002e+02 },
    { 1.0789999999999999e-07, 5.3490128106999997e+00, 5.3058473697190004e+02 },
    { 1.0789999999999999e-07, 2.1332654912600000e+00, 5.2879719321729999e+02 },
    { 1.0492000000000000e-07, 3.0206226665800000e+00, 3.1957326339430000e+02 },
    { 1.0405999999999999e-07, 4.3664554442499997e+00, 2.1011770170029999e+02 },
    { 1.0332000000000000e-07, 5.9536130869199999e+00, 6.0595703637019994e+02 },
    { 1.0153999999999999e-07, 1.8158358375899999e+00, 4.5639383923560001e+02 },
    { 1.0132000000000000e-07, 3.1992319435500001e+00, 9.5612275556000004e+00 },
    { 9.8679999999999994e-08, 5.4409153682199998e+00, 2.8546403739102002e+03 },
    { 9.7409999999999997e-08, 5.1933513232399999e+00, 1.1592933106701000e+03 },
    { 9.4549999999999996e-08, 4.4286083318699996e+00, 2.9577331481288002e+03 },
    { 9.3789999999999995e-08, 1.0962671346000000e-01, 3.2032402291969998e+02 },
    { 9.3229999999999995e-08, 3.6328090493500000e+00, 2.5240214102519999e+03 },
    { 9.3019999999999995e-08, 4.5731289331099996e+00, 1.5677322542813999e+03 },
    { 9.0830000000000000e-08, 1.8396014261400000e+00, 7.3297125859000005e+01 },
    { 9.0709999999999996e-08, 2.2704057451100002e+00, 7.5403576079649997e+02 },
    { 9.0349999999999998e-08, 1.1776786038400000e+00, 2.9506196011279999e+03 },
    { 9.0139999999999999e-08, 1.0250660331300001e+00, 1.2018315803230000e+03 },
    { 8.9799999999999997e-08, 1.0539613959100000e+00, 1.1656560981455000e+03 },
    { 8.9239999999999997e-08, 5.6729799946600004e+00, 6.0298809095360002e+02 },
    { 8.8860000000000003e-08, 3.1938474896000002e-01, 5.3091480537200005e+02 },
    { 8.8860000000000003e-08, 8.7970824581999996e-01, 5.2846712481719999e+02 },
    { 8.8780000000000005e-08, 3.1975671425300001e+00, 1.9513984817330001e+02 },
    { 8.7100000000000006e-08, 1.6935949458499999e+00, 5.7616138801060004e+02 },
    { 8.7009999999999996e-08, 3.3540791435199999e+00, 1.7810313497193999e+03 },
    { 8.6579999999999998e-08, 1.0367708883000000e+00, 1.2151649024472999e+03 },
    { 8.6350000000000002e-08, 4.8767528516500001e+00, 1.0564611688824000e+03 },
    { 8.6350000000000002e-08, 3.3011811701700000e+00, 1.0623026914960001e+03 },
    { 8.5590000000000001e-08, 2.1635923262199999e+00, 1.6779385755008000e+03 },
    { 8.4400000000000001e-08, 5.6646024962599997e+00, 4.8322054217860000e+02 },
    { 8.3939999999999996e-08, 1.0171259036800000e+00, 5.2554636351040006e+02 },
    { 8.3900000000000004e-08, 1.9211871299600001e+00, 2.3320629558164001e+03 },
    { 8.3089999999999998e-08, 1.8252574300999999e-01, 5.3383556667879998e+02 },
    { 8.1250000000000003e-08, 6.0298154969400004e+00, 4.0525754987360000e+02 },
    { 7.9889999999999996e-08, 2.2556211430999999e+00, 1.0586311706638000e+03 },
    { 7.9889999999999996e-08, 5.9257452904100001e+00, 1.0601326897146000e+03 },
    { 7.9660000000000000e-08, 4.9752295162099998e+00, 8.4215068148809996e+02 },
    { 7.8870000000000004e-08, 3.4583929126699999e+00, 2.8475268269093999e+03 },
    { 7.8590000000000004e-08, 5.0997318237800000e+00, 1.0647985561605999e+03 },
    { 7.8549999999999999e-08, 4.0873859147399996e+00, 7.3980866679489998e+02 },
    { 7.7490000000000001e-08, 1.8520987807000000e-01, 5.2182665856840003e+02 },
    { 7.6739999999999998e-08, 4.5897467529000000e+00, 3.3455111692129998e+02 },
    { 7.6479999999999994e-08, 3.2213226514600000e+00, 2.4723934538180001e+02 },
    { 7.6150000000000004e-08, 4.1046352039500000e+00, 3.5181659230870002e+02 },
    { 7.6020000000000002e-08, 5.0753803405699998e+00, 1.0635747158832000e+03 },
    { 7.6020000000000002e-08, 3.1015576883099998e+00, 1.0551891444952000e+03 },
    { 7.6010000000000004e-08, 1.0141080023900000e+00, 5.3755527162080000e+02 },
    { 7.5390000000000002e-08, 5.3582724978899998e+00, 9.5235700270749999e+02 },
    { 7.5349999999999996e-08, 1.4320968487900001e+00, 1.0608664028974999e+03 },
    { 7.3659999999999999e-08, 1.1890893012499999e+00, 4.2341679713830001e+02 },
    { 7.3189999999999996e-08, 1.3613108257800000e+00, 6.2197751234999998e+00 },
    { 7.2720000000000005e-08, 3.7497788602100002e+00, 1.4807872442630000e+02 },
    { 7.1719999999999996e-08, 5.7887289509500004e+00, 3.1019488636999998e+01 },
    { 6.9450000000000003e-08, 3.8292995118900000e+00, 2.7703499374140000e+02 },
    { 6.9120000000000000e-08, 3.4082066717699999e+00, 9.4599421523210003e+02 },
    { 6.8610000000000004e-08, 1.2445516461299999e+00, 1.0547160637432000e+03 },
    { 6.8229999999999996e-08, 6.0912657610999998e-01, 2.8414854074220000e+02 },
    { 6.8149999999999998e-08, 1.5470817809799999e+00, 1.0537528558967001e+03 },
    { 6.7930000000000000e-08, 5.6275398025500003e+00, 3.0386169668439999e+02 },
    { 6.7700000000000004e-08, 5.5365078455300001e+00, 6.9152524274800001e+01 },
    { 6.7659999999999998e-08, 5.1819142944100003e+00, 1.9125783119411999e+03 },
    { 6.7300000000000000e-08, 5.6194767825000003e-01, 5.2804643369190001e+02 },
    { 6.7300000000000000e-08, 6.3714531652999995e-01, 5.3133549649730003e+02 },
    { 6.7150000000000002e-08, 2.2648558995000001e+00, 4.0843894361129998e+02 },
    { 6.5659999999999994e-08, 4.0728171279999997e-01, 3.5574874557179999e+02 },
    { 6.4119999999999994e-08, 2.1318355685600001e+00, 3.0537123753465999e+03 },
    { 6.3360000000000006e-08, 2.5007751409700001e+00, 9.0833426034629997e+02 },
    { 6.2600000000000005e-08, 3.1972782549700001e+00, 1.4930936680660000e+03 },
    { 6.0520000000000002e-08, 2.8188555951700001e+00, 2.6087903141574199e+04 },
    { 6.0129999999999996e-08, 5.1074267290600002e+00, 1.0350029078010000e+03 },
    { 5.9489999999999999e-08, 2.5711257553000002e+00, 7.2483081326790000e+02 },
    { 5.9349999999999999e-08, 2.9445331567799999e+00, 4.9819839570559998e+02 },
    { 5.9139999999999999e-08, 3.6858354041800001e+00, 1.0539653042177999e+03 },
    { 5.8560000000000003e-08, 5.6108562405600004e+00, 1.2766131788903001e+03 },
    { 5.7390000000000000e-08, 3.3772044436500002e+00, 7.0756533788050001e+02 },
    { 5.7229999999999997e-08, 5.5176012405600003e+00, 1.0650110044817000e+03 },
    { 5.6930000000000001e-08, 4.6764322025300000e+00, 1.6682867252200001e+02 },
    { 5.6659999999999999e-08, 6.0582568921500002e+00, 1.0640477966352000e+03 },
    { 5.6050000000000002e-08, 1.2636782042900001e+00, 5.1322863885929996e+02 },
    { 5.5890000000000000e-08, 4.7744367343400000e+00, 2.8096714700450002e+02 },
    { 5.5840000000000002e-08, 3.3998842557700000e+00, 1.0064509366380000e+02 },
    { 5.5070000000000003e-08, 4.5472271059700002e+00, 5.6118353448360006e+02 },
    { 5.5019999999999999e-08, 3.8969589173900001e+00, 3.0608259223474001e+03 },
    { 5.4970000000000002e-08, 4.4114396331899997e+00, 8.1065811209909998e+02 },
    { 5.4700000000000000e-08, 1.8571116019100000e+00, 7.5721715453419995e+02 },
    { 5.3839999999999998e-08, 8.1789121947999999e-01, 2.0936694217490000e+02 },
    { 5.3820000000000002e-08, 2.0545921021300000e+00, 5.1242548970719997e+02 },
    { 5.3780000000000002e-08, 6.2185999659000002e+00, 5.4615329132989996e+02 },
    { 5.2040000000000001e-08, 3.2389456550700002e+00, 1.0916252591036000e+03 },
    { 5.1529999999999999e-08, 5.4276860592600000e+00, 5.4695644048199995e+02 },
    { 5.1310000000000001e-08, 2.5819247138699999e+00, 1.1767017984094000e+03 },
    { 5.0869999999999998e-08, 3.2744041237900001e+00, 1.1439910691340000e+02 },
    { 5.0650000000000000e-08, 2.5442647294800000e+00, 1.0592218714948001e+03 },
    { 5.0650000000000000e-08, 5.6371017040300000e+00, 1.0595419888836000e+03 },
    { 5.0069999999999998e-08, 4.5201420014600000e+00, 4.9181856188770001e+02 },
    { 4.9850000000000000e-08, 3.3522174748600002e+00, 2.8404132799086001e+03 },
    { 4.9730000000000003e-08, 1.0342589447699999e+00, 2.4554242435239999e+02 },
    { 4.9549999999999997e-08, 1.8482653155800000e+00, 1.0696768709277001e+03 },
    { 4.9110000000000001e-08, 6.2297106190299996e+00, 5.1228247735529999e+02 },
    { 4.8940000000000000e-08, 3.3758490678000003e-01, 7.7523338944700004e+02 },
    { 4.8940000000000000e-08, 3.0289339862999998e-01, 7.3905790726949999e+02 },
    { 4.8039999999999999e-08, 6.8976934372999998e-01, 3.9117346822389999e+02 },
    { 4.7489999999999997e-08, 4.0404342088499998e+00, 4.1529185818119998e+02 },
    { 4.7330000000000001e-08, 6.7139053203999999e-01, 1.0876931058405000e+03 },
    { 4.6880000000000000e-08, 5.5422703090300001e+00, 5.0456118318099999e+02 },
    { 4.6810000000000000e-08, 5.8366269028100000e+00, 2.9505122865419997e+02 },
    { 4.6719999999999997e-08, 2.0411607139000001e+00, 2.3463973644039999e+02 },
    { 4.6280000000000001e-08, 2.9594565456100002e+00, 5.6756336830149996e+02 },
    { 4.5529999999999998e-08, 1.4799620421599999e+00, 5.0924409596949999e+02 },
    { 4.5419999999999999e-08, 6.1469038410600003e+00, 1.0554045477340000e+02 },
    { 4.5230000000000002e-08, 6.1542163797700002e+00, 6.1144309831080000e+02 },
    { 4.4689999999999998e-08, 5.2405952072999995e-01, 8.5712853501510006e+02 },
    { 4.4309999999999998e-08, 4.6363309303499998e+00, 5.1965665678699997e+02 },
    { 4.4290000000000002e-08, 3.3348330321000001e+00, 1.0254416802454000e+03 },
    { 4.4180000000000003e-08, 3.0868155513300000e+00, 2.0974232193759999e+03 },
    { 4.4169999999999998e-08, 8.9935900799000001e-01, 1.1270499817556999e+03 },
    { 4.4029999999999998e-08, 2.9078214662300002e+00, 9.8730864460759994e+02 },
    { 4.3929999999999997e-08, 5.3819857584899999e+00, 3.1710322435667999e+03 },
    { 4.3690000000000003e-08, 1.9483467103000001e+00, 5.5482074700819999e+02 },
    { 4.3240000000000002e-08, 6.0023160665699997e+00, 5.5013783421970004e+02 },
    { 4.3030000000000002e-08, 6.5968200894999995e-01, 3.7390799283650000e+02 },
    { 4.2559999999999999e-08, 2.9554347206000000e+00, 5.1216485727630004e+02 },
    { 4.2489999999999999e-08, 5.0438949178500003e+00, 2.0300415469949999e+02 },
    { 4.2330000000000003e-08, 2.5320117533800000e+00, 2.0424977891028000e+03 },
    { 4.2170000000000000e-08, 3.2482702193200002e+00, 1.4899122743283001e+03 },
    { 4.2020000000000002e-08, 5.1522983830999998e+00, 4.6988722005429997e+02 },
    { 4.1659999999999997e-08, 1.0965483525199999e+00, 1.5875884225755001e+03 },
    { 4.1439999999999999e-08, 1.7146116342600000e+00, 9.7051624997219994e+02 },
    { 4.0860000000000003e-08, 2.5542715486200001e+00, 2.3767811782620001e+02 },
    { 4.0849999999999999e-08, 3.3214408017600001e+00, 9.1693228005540004e+02 },
    { 4.0830000000000002e-08, 3.8694852484000002e+00, 7.8234693644779998e+02 },
    { 4.0719999999999997e-08, 4.5268434418999997e+00, 5.4721707291289999e+02 },
    { 4.0539999999999998e-08, 3.7141907101100000e+00, 8.1001373690799994e+01 },
    { 4.0119999999999998e-08, 6.2727446398999998e-01, 6.6820846196529999e+02 },
    { 3.9949999999999997e-08, 8.5610382029999998e-01, 1.5780271950199001e+03 },
    { 3.9869999999999999e-08, 9.7522695807000004e-01, 3.2670114707846001e+03 },
    { 3.9860000000000001e-08, 1.0637222250400000e+00, 4.4793883187839998e+02 },
    { 3.9820000000000002e-08, 5.6612790233699997e+00, 9.8608480433019997e+02 },
    { 3.9669999999999997e-08, 1.3328186898400001e+00, 4.8640193591629998e+02 },
    { 3.9589999999999999e-08, 3.3776487321099999e+00, 3.0683064210100002e+02 },
    { 3.9529999999999997e-08, 2.0408796076500000e+00, 1.1231178284926000e+03 },
    { 3.9470000000000002e-08, 3.8024209618000002e+00, 1.1265400177420000e+02 },
    { 3.9239999999999999e-08, 4.1831400634999998e+00, 8.3652160719560004e+02 },
    { 3.9010000000000003e-08, 3.5813805151600002e+00, 1.8944190646765001e+03 },
    { 3.8929999999999999e-08, 4.2617375098599997e+00, 8.1214258480740000e+02 },
    { 3.8659999999999997e-08, 4.9582173879100004e+00, 8.2474219374879999e+02 },
    { 3.8519999999999997e-08, 2.5243860775300000e+00, 1.0656017053127000e+03 },
    { 3.8449999999999997e-08, 3.2254377856500001e+00, 2.2286032299359999e+02 },
    { 3.8400000000000000e-08, 3.4109385144999999e-01, 4.7928838891549998e+02 },
    { 3.7510000000000003e-08, 3.6062836867799999e+00, 9.7748678462110001e+02 },
    { 3.7109999999999999e-08, 2.9877504902299998e+00, 3.1639186965660001e+03 },
    { 3.6970000000000000e-08, 2.7902934668000001e+00, 3.2668681039510000e+02 },
    { 3.6909999999999998e-08, 3.2335124709300000e+00, 1.0212956637210000e+02 },
    { 3.6820000000000001e-08, 1.0694454263399999e+00, 2.2582926841020000e+02 },
    { 3.6799999999999999e-08, 1.2127820793799999e+00, 5.4709945283390005e+02 },
    { 3.6659999999999999e-08, 1.4319839795700000e+00, 7.7750543983900002e+01 },
    { 3.6300000000000001e-08, 6.9909450749000002e-01, 8.1752133216199994e+01 },
    { 3.6190000000000002e-08, 6.9464229609999995e-01, 1.0405598206510000e+02 },
    { 3.6010000000000003e-08, 4.7144134781000002e-01, 9.9171387862270001e+02 },
    { 3.5999999999999998e-08, 4.2955883949000002e+00, 3.2741250177853999e+03 },
    { 3.5710000000000000e-08, 2.0886796681400002e+00, 1.0807225711916001e+03 },
    { 3.5630000000000002e-08, 2.0468748794699998e+00, 1.9943304451573999e+03 },
    { 3.5560000000000002e-08, 6.1496194052400002e+00, 5.7297999427289994e+02 },
    { 3.5439999999999998e-08, 4.3839480866600002e+00, 8.4926422848890002e+02 },
    { 3.5390000000000001e-08, 3.6198321346800002e+00, 1.0129115072732000e+03 },
    { 3.5339999999999997e-08, 4.7144281608999999e-01, 7.8713751830400000e+01 },
    { 3.5070000000000002e-08, 2.3296265308900002e+00, 5.8949471013489995e+02 },
    { 3.4970000000000001e-08, 2.8013609252300000e+00, 1.0595943785103000e+03 },
    { 3.4970000000000001e-08, 5.3800055082800000e+00, 1.0591694818681001e+03 },
    { 3.4720000000000003e-08, 8.5726515863999997e-01, 5.8009354127370000e+02 },
    { 3.4630000000000000e-08, 2.6020904766799999e+00, 1.5915205758386001e+03 },
    { 3.4550000000000002e-08, 3.4209193217500000e+00, 4.8884961647109998e+02 },
    { 3.4480000000000002e-08, 3.4721109438100002e+00, 6.8547393735269998e+02 },
    { 3.3689999999999999e-08, 3.5533045792100002e+00, 1.0314095832840000e+02 },
    { 3.3500000000000002e-08, 3.7639975575700002e+00, 1.0775411774539000e+03 },
    { 3.3349999999999998e-08, 4.4423574572500000e+00, 1.0498207026335999e+03 },
    { 3.3250000000000003e-08, 3.7561309842000001e-01, 1.0304459010879999e+02 },
    { 3.3029999999999999e-08, 2.8512500106499998e+00, 5.3972527340219995e+02 },
    { 3.2259999999999999e-08, 4.6031249446500002e+00, 5.2278986641489996e+02 },
    { 3.2259999999999999e-08, 2.8791533573099999e+00, 5.3659206377429996e+02 },
    { 3.2089999999999998e-08, 2.1020907707599998e+00, 1.4563104387150000e+02 },
    { 3.1960000000000003e-08, 5.6045795089599997e+00, 3.0679394693482000e+03 },
    { 3.1590000000000000e-08, 5.4733256561000001e+00, 4.5761767951299998e+02 },
    { 3.1419999999999999e-08, 4.6249055697600001e+00, 5.3925219265019996e+02 },
    { 3.1179999999999999e-08, 4.0613582300599997e+00, 5.7053231371810000e+02 },
    { 3.1119999999999997e-08, 6.0306044408000004e+00, 5.2012973753899996e+02 },
    { 3.0640000000000002e-08, 4.4417032517199999e+00, 1.8841241239379999e+03 },
    { 3.0549999999999999e-08, 6.4523665181000001e-01, 6.5412438031559998e+02 },
    { 3.0309999999999998e-08, 3.3761961668399998e+00, 5.2109294538550000e+02 },
    { 3.0309999999999998e-08, 4.1060821351200003e+00, 5.3828898480370003e+02 },
    { 3.0230000000000000e-08, 1.3704351197800000e+00, 5.2672201967800004e+02 },
    { 3.0230000000000000e-08, 6.1118431821800003e+00, 5.3265991051120000e+02 },
    { 3.0120000000000001e-08, 1.9725994307100001e+00, 3.3701042450032000e+03 },
    { 2.9999999999999997e-08, 3.7497190027700000e+00, 9.1130320576290001e+02 },
    { 2.9970000000000003e-08, 2.9134672153499999e+00, 1.4380252254408999e+03 },
    { 2.9580000000000001e-08, 4.3971179829900002e+00, 5.2768522752449996e+02 },
    { 2.9580000000000001e-08, 3.0851603189699999e+00, 5.3169670266469996e+02 },
    { 2.9429999999999999e-08, 5.6664452761200002e+00, 1.2694996318895001e+03 },
    { 2.9259999999999999e-08, 4.3952215749699999e+00, 7.3269511979410004e+02 },
    { 2.9180000000000001e-08, 5.7362816671600001e+00, 9.6733485623449997e+02 },
    { 2.9090000000000001e-08, 3.0752522297499998e+00, 5.2268929267839997e+02 },
    { 2.9090000000000001e-08, 4.4070260722099999e+00, 5.3669263751079995e+02 },
    { 2.9090000000000001e-08, 3.9566052300100001e+00, 1.4827987273275000e+03 },
    { 2.9060000000000000e-08, 4.2922754632300002e+00, 4.1630325013750002e+02 },
    { 2.8749999999999999e-08, 4.3262243446399999e+00, 5.0441817082910001e+02 },
    { 2.8699999999999999e-08, 2.6308381178000001e-01, 1.5866252147289999e+03 },
    { 2.8579999999999999e-08, 3.8974517438400000e+00, 1.4409941708575000e+03 },
    { 2.8550000000000001e-08, 2.0304405046700000e+00, 8.5001498801430000e+02 },
    { 2.8390000000000001e-08, 4.9768757351300001e+00, 1.1403833038800001e+03 },
    { 2.8240000000000000e-08, 1.7122798180000001e+00, 1.0872184851110001e+02 },
    { 2.8209999999999999e-08, 1.0665704803200000e+00, 5.0286426215159997e+02 },
    { 2.7820000000000000e-08, 5.2007354460000004e+00, 4.5194042111070002e+02 },
    { 2.7700000000000000e-08, 5.8370434492400003e+00, 3.9617508346100003e+01 },
    { 2.7689999999999998e-08, 4.8561505308399999e+00, 1.7448558675418999e+03 },
    { 2.7310000000000001e-08, 4.6733736448999998e+00, 5.2246554350919996e+02 },
    { 2.7310000000000001e-08, 2.8089046570599998e+00, 5.3691638667999996e+02 },
    { 2.7150000000000002e-08, 3.1696307596600000e+00, 5.5496375936009997e+02 },
    { 2.6720000000000000e-08, 8.7511357041000004e-01, 1.3458534360760001e+02 },
    { 2.6510000000000000e-08, 2.2706379399499999e+00, 1.4414657116320001e+02 },
    { 2.6280000000000001e-08, 8.4720826300000007e-03, 3.2812385647862002e+03 },
    { 2.6270000000000000e-08, 2.2389563142200002e+00, 1.0948066528413001e+03 },
    { 2.6140000000000001e-08, 4.1964493730000002e-01, 9.3531546663000000e+01 },
    { 2.5900000000000000e-08, 1.6004731508000000e-01, 5.5651766803759995e+02 },
    { 2.5510000000000001e-08, 1.0397161149800001e+00, 2.3204904353370000e+02 },
    { 2.5480000000000001e-08, 7.2953715709000000e-01, 1.1285344544640000e+03 },
    { 2.5300000000000002e-08, 1.0552341266000000e-01, 1.2566151699982800e+04 },
    { 2.5220000000000000e-08, 7.4750896561000002e-01, 3.7301422095919997e+02 },
    { 2.5119999999999999e-08, 2.0406756830399999e+00, 1.2616353253632999e+03 },
    { 2.5100000000000000e-08, 1.9945967377400000e+00, 6.0176425067620005e+02 },
    { 2.4769999999999999e-08, 3.2108484238699999e+00, 3.1568051495651998e+03 },
    { 2.4560000000000000e-08, 1.4727927137100001e+00, 1.0743597837161999e+03 },
    { 2.4560000000000000e-08, 4.2538841261999999e-01, 1.0444040766621999e+03 },
    { 2.4520000000000001e-08, 6.2028796575000000e-01, 1.0904014188261999e+03 },
    { 2.4410000000000002e-08, 8.3069302598000005e-01, 1.4749344208012999e+03 },
    { 2.4389999999999999e-08, 2.0855892300500001e+00, 1.0775864066459999e+02 },
    { 2.3930000000000000e-08, 2.8513543955100000e+00, 1.0532315910349000e+03 },
    { 2.3750000000000001e-08, 7.0058711647000005e-01, 2.1226960136415000e+03 },
    { 2.3660000000000001e-08, 1.2439448324500000e+00, 3.7078367556622002e+03 },
    { 2.3630000000000000e-08, 3.2896254938100000e+00, 2.8537238101960003e+02 },
    { 2.3230000000000000e-08, 2.6082443240200002e+00, 5.2905782564820004e+02 },
    { 2.3230000000000000e-08, 4.8740339779399999e+00, 5.3032410454100000e+02 },
    { 2.2910000000000001e-08, 1.4517294108800001e+00, 4.8863716814999998e+02 },
    { 2.2729999999999999e-08, 6.0392380670700003e+00, 1.5891210793936000e+03 },
    { 2.2729999999999999e-08, 2.8414269394399998e+00, 1.5890247111740000e+03 },
    { 2.2700000000000001e-08, 4.8401280036900003e+00, 3.6648562929500002e+01 },
    { 2.2510000000000001e-08, 2.1620596408399999e+00, 1.0674586850364999e+03 },
    { 2.2340000000000000e-08, 3.7459338503400001e+00, 3.3772177920039999e+03 },
    { 2.2220000000000000e-08, 4.9872537441000002e+00, 3.7619561469690001e+02 },
    { 2.2200000000000000e-08, 2.8087415519899999e+00, 2.4138150890326001e+03 },
    { 2.2169999999999999e-08, 4.3337324784900000e+00, 1.2545217783625001e+03 },
    { 2.2169999999999999e-08, 1.5649560605799999e+00, 9.0270518605380005e+02 },
    { 2.2060000000000000e-08, 4.9756221723599996e+00, 6.6812248533995999e+03 },
    { 2.2010000000000000e-08, 4.1547109845400003e+00, 2.0439822618111000e+03 },
    { 2.1979999999999999e-08, 2.5104936500299999e+00, 4.2904587143079999e+02 },
    { 2.1959999999999999e-08, 4.3894698507300003e+00, 2.9823262239190001e+02 },
    { 2.1909999999999999e-08, 2.8358866932200000e+00, 8.4290144101349995e+02 },
    { 2.1900000000000001e-08, 5.7761849351899999e+00, 9.7676148247200004e+01 },
    { 2.1900000000000001e-08, 2.6535679103200001e+00, 4.1703696332039999e+02 },
    { 2.1859999999999999e-08, 3.4858055645000001e+00, 1.5536481726316999e+03 },
    { 2.1730000000000000e-08, 5.2521233206900000e+00, 1.0655322693435000e+03 },
    { 2.1570000000000001e-08, 5.3757004791099998e+00, 5.9803745040300001e+01 },
    { 2.1290000000000001e-08, 5.0124377344899997e+00, 8.2170381236300000e+02 },
    { 2.1010000000000001e-08, 5.1136191966500002e+00, 2.4161027108930000e+02 },
    { 2.0999999999999999e-08, 1.9756381757799999e+00, 1.6882335162392999e+03 },
    { 2.0969999999999999e-08, 6.0606760722399997e+00, 5.7074476203920005e+02 },
    { 2.0660000000000001e-08, 9.4018645457000005e-01, 4.8715269544170002e+02 },
    { 2.0350000000000000e-08, 3.8471393789200001e+00, 5.2384944248099998e+02 },
    { 2.0350000000000000e-08, 3.6351389230400000e+00, 5.3553248770820005e+02 },
    { 1.9830000000000000e-08, 1.6301472924100000e+00, 1.4323961511483999e+03 },
    { 1.9700000000000001e-08, 2.6800555163699999e+00, 3.6286229257259998e+02 },
    { 1.9620000000000000e-08, 1.1267167509700000e+00, 1.0924311337290000e+02 },
    { 1.9610000000000002e-08, 4.7783238509899997e+00, 1.4466232451500000e+03 },
    { 1.9530000000000000e-08, 5.4972828101499998e+00, 1.4151306757610000e+03 },
    { 1.9259999999999998e-08, 3.2593368421300002e+00, 2.2908009811709999e+02 },
    { 1.8940000000000000e-08, 2.6611314897399998e+00, 3.4803105662225998e+03 },
    { 1.8749999999999999e-08, 3.3933021264400001e+00, 1.5900361031303000e+03 },
    { 1.8730000000000000e-08, 5.9052322459999995e-01, 1.2089451273238001e+03 },
    { 1.8699999999999999e-08, 6.0419684213600000e+00, 5.0123677709139997e+02 },
    { 1.8600000000000001e-08, 5.3036982669099997e+00, 1.1600270238529999e+03 },
    { 1.8590000000000000e-08, 5.8710083030300000e+00, 3.1321047591889999e+02 },
    { 1.8510000000000002e-08, 4.8601839956299999e+00, 1.1116952906589999e+02 },
    { 1.8110000000000001e-08, 5.8989386684999996e+00, 1.3403490771936999e+03 },
    { 1.8060000000000001e-08, 5.0436149351899999e+00, 9.1375088631769995e+02 },
    { 1.8050000000000000e-08, 1.4430343497200000e+00, 1.1025450532920000e+02 },
    { 1.8019999999999999e-08, 2.4899836512000001e-01, 5.7222923474749996e+02 },
    { 1.7960000000000000e-08, 7.0671519409000005e-01, 4.2415051032119999e+02 },
    { 1.7910000000000000e-08, 4.8164925986400000e+00, 5.0260362972069998e+02 },
    { 1.7870000000000001e-08, 3.6650678798700000e+00, 1.0596425626201001e+03 },
    { 1.7870000000000001e-08, 4.5162985536300004e+00, 1.0591212977583000e+03 },
    { 1.7859999999999999e-08, 5.4400194864999998e+00, 1.2262106027112000e+03 },
    { 1.7800000000000001e-08, 4.2915870368400002e+00, 1.0513051753418999e+03 },
    { 1.7780000000000001e-08, 1.0821742122400000e+00, 1.8196374661092000e+03 },
    { 1.7330000000000000e-08, 7.3534814864999998e-01, 1.6289651925890001e+02 },
    { 1.7249999999999999e-08, 5.0809840197799998e+00, 1.6960978227655000e+03 },
    { 1.7159999999999999e-08, 2.6555514599599999e+00, 5.0382746999810001e+02 },
    { 1.7140000000000000e-08, 1.0947745451199999e+00, 1.1649223849626001e+03 },
    { 1.7109999999999999e-08, 4.1479901456999997e+00, 2.2005159935945999e+03 },
    { 1.7010000000000001e-08, 4.9140454581800004e+00, 4.2240540518199998e+02 },
    { 1.6969999999999999e-08, 5.6609896740999996e+00, 1.0523165672982000e+03 },
    { 1.6810000000000000e-08, 3.4853583775399999e+00, 5.4318434591330004e+02 },
    { 1.6810000000000000e-08, 3.9969199244200002e+00, 5.1619758427590000e+02 },
    { 1.6740000000000000e-08, 7.6441019022000001e-01, 3.5834033404411998e+03 },
    { 1.6730000000000002e-08, 2.6657853578999999e+00, 5.5677830046849999e+02 },
    { 1.6569999999999999e-08, 1.3861916129100000e+00, 4.7306861379200001e+02 },
    { 1.6449999999999998e-08, 4.9683906802799997e+00, 6.3033605875839999e+02 },
    { 1.6199999999999999e-08, 2.4751961620100000e+00, 6.8318631549229997e+02 },
    { 1.6079999999999999e-08, 4.8267265405600002e+00, 5.5555446019110002e+02 },
    { 1.5990000000000000e-08, 5.7054321974800004e+00, 1.5881096874373000e+03 },
    { 1.5869999999999999e-08, 1.9703530840900001e+00, 1.2687488723641000e+03 },
    { 1.5860000000000001e-08, 2.4352009649699999e+00, 1.0522201990786000e+03 },
    { 1.5830000000000000e-08, 6.0474006269600000e+00, 5.6622351302600002e+01 },
    { 1.5810000000000001e-08, 5.1373057985599999e+00, 6.4234496686880004e+02 },
    { 1.5729999999999999e-08, 6.2622560654399999e+00, 4.2563498302950001e+02 },
    { 1.5510000000000001e-08, 7.7205866411000001e-01, 5.9342686339800002e+02 },
    { 1.5469999999999999e-08, 5.1449213900100004e+00, 1.0975149658270000e+03 },
    { 1.5370000000000001e-08, 1.9658665242000001e-01, 1.5182234499796000e+03 },
    { 1.4970000000000001e-08, 4.6005454389000000e-01, 1.3797059531089001e+03 },
    { 1.4960000000000000e-08, 4.3328193636899996e+00, 3.8846515523820000e+02 },
    { 1.4839999999999999e-08, 2.4154576152899998e+00, 9.9564603188579997e+02 },
    { 1.4790000000000000e-08, 5.1207833786099997e+00, 1.3851749687070000e+02 },
    { 1.4730000000000000e-08, 4.1213984285700000e+00, 1.4365407527325999e+03 },
    { 1.4660000000000000e-08, 3.3978952844900001e+00, 4.2655000676620000e+02 },
    { 1.4640000000000001e-08, 2.3383211626499998e+00, 1.0664472930802001e+03 },
    { 1.4590000000000000e-08, 2.9604013368999998e-01, 4.2664637498579998e+02 },
    { 1.4530000000000000e-08, 4.9087312816399997e+00, 2.1648048917570000e+02 },
    { 1.4510000000000001e-08, 6.0645762141099997e+00, 5.8631331639719997e+02 },
    { 1.4489999999999999e-08, 1.5342460562700000e+00, 3.6864961146597998e+03 },
    { 1.4460000000000000e-08, 5.8600455995900003e+00, 3.8405992122309999e+02 },
    { 1.4279999999999999e-08, 4.5843130490200004e+00, 2.4431858407499999e+02 },
    { 1.4190000000000000e-08, 1.5407359052800000e+00, 1.1380956820196000e+03 },
    { 1.4140000000000000e-08, 4.4308473265400004e+00, 3.5033211960040001e+02 },
    { 1.4090000000000001e-08, 5.6082836686400004e+00, 1.4746737883704000e+03 },
    { 1.3949999999999999e-08, 4.6166683815999998e-01, 6.3523141986799999e+02 },
    { 1.3939999999999999e-08, 5.1564979756999998e+00, 9.5873683652540001e+02 },
    { 1.3750000000000001e-08, 2.4096828932099998e+00, 1.0933221801330001e+03 },
    { 1.3690000000000000e-08, 2.7060615389599998e+00, 4.5909101804889997e+03 },
    { 1.3580000000000000e-08, 3.0518980297499998e+00, 4.4868959140380002e+02 },
    { 1.3529999999999999e-08, 2.5931839976800002e+00, 2.4988916283384001e+03 },
    { 1.3480000000000001e-08, 3.8118402157500002e+00, 2.0043647534649999e+03 },
    { 1.3459999999999999e-08, 5.5534496519800003e+00, 1.0665436612998001e+03 },
    { 1.3440000000000000e-08, 2.5794016040200001e+00, 2.5265597135319999e+02 },
    { 1.3420000000000000e-08, 6.2786640357500003e+00, 1.3576145525811000e+03 },
    { 1.3399999999999999e-08, 4.1170883550399999e+00, 1.0581099058020000e+03 },
    { 1.3399999999999999e-08, 4.0642780784600001e+00, 1.0606539545764001e+03 },
    { 1.3389999999999999e-08, 4.4599685352699998e+00, 1.1615114965613000e+03 },
    { 1.3370000000000000e-08, 3.6125336174000000e+00, 4.2756139872249997e+02 },
    { 1.3319999999999999e-08, 6.0706051289799996e+00, 9.8163138620530003e+02 },
    { 1.3290000000000000e-08, 8.0851044181999998e-01, 1.6284297711990000e+03 },
    { 1.3130000000000001e-08, 2.8770046151400002e+00, 8.2792358748649997e+02 },
    { 1.3090000000000000e-08, 1.9100777294600000e+00, 1.1634379122543000e+03 },
    { 1.3090000000000000e-08, 2.9912235984300000e+00, 3.4731970192218000e+03 },
    { 1.3020000000000000e-08, 6.0148738192000000e+00, 4.9447869076360001e+02 },
    { 1.3020000000000000e-08, 2.5028933155100002e+00, 6.3697652500720005e+02 },
    { 1.2930000000000001e-08, 1.4157490157299999e+00, 3.9435486196160002e+02 },
    { 1.2870000000000000e-08, 9.8828251480000004e-02, 1.3544331588434000e+03 },
    { 1.2750000000000000e-08, 5.2108766366000003e-01, 1.3632436268736001e+03 },
    { 1.2660000000000001e-08, 4.7672680491500001e+00, 1.1625228885176000e+03 },
    { 1.2600000000000000e-08, 1.2141071404399999e+00, 6.3374694715969997e+02 },
    { 1.2590000000000001e-08, 3.5620885188300000e+00, 9.9022940591439999e+02 },
    { 1.2580000000000001e-08, 1.5842162602900001e+00, 1.1624265202980000e+03 },
    { 1.2429999999999999e-08, 3.1489897642200000e+00, 6.7091677495099998e+02 },
    { 1.2410000000000000e-08, 5.2304409177500002e+00, 2.6992144674060000e+02 },
    { 1.2350000000000000e-08, 6.0067373547900003e+00, 1.0271386012748001e+03 },
    { 1.2340000000000000e-08, 3.2615815170000001e-01, 4.3615941843159999e+02 },
    { 1.2340000000000000e-08, 2.8869806918799998e+00, 1.4678208738005001e+03 },
    { 1.2310000000000001e-08, 5.3066439539900001e+00, 4.1406801790380001e+02 },
    { 1.2190000000000000e-08, 4.1443284709399997e+00, 6.1462449204849997e+02 },
    { 1.2120000000000000e-08, 4.1034188928799997e+00, 3.5905168874420001e+03 },
    { 1.1940000000000000e-08, 4.2092451847400003e+00, 7.7400954916959995e+02 },
    { 1.1940000000000000e-08, 4.1314318147400000e+00, 8.5211198767086007e+03 },
    { 1.1920000000000000e-08, 1.4674038694000000e+00, 5.6490323942559996e+02 },
    { 1.1860000000000000e-08, 3.0997935232499998e+00, 2.1148317071153001e+03 },
    { 1.1819999999999999e-08, 5.2974408869000000e-01, 4.6595506679120001e+02 },
    { 1.1760000000000001e-08, 4.0816752535700003e+00, 6.3283192342300003e+02 },
    { 1.1730000000000000e-08, 2.0435115315300001e+00, 2.9648466951296000e+03 },
    { 1.1720000000000000e-08, 9.0155530148999996e-01, 6.3273555520340005e+02 },
    { 1.1700000000000001e-08, 3.4951136248100001e+00, 3.6936096616606001e+03 },
    { 1.1679999999999999e-08, 2.2435855171100001e+00, 2.8617539209110000e+03 },
    { 1.1650000000000000e-08, 1.1002151063000000e-01, 1.5113223512999000e+03 },
    { 1.1560000000000001e-08, 1.6537839673500001e+00, 1.0257150935679999e+02 },
    { 1.1560000000000001e-08, 5.0160886481500002e+00, 6.9651963761659999e+02 },
    { 1.1469999999999999e-08, 4.0516035875599998e+00, 1.3900008938474000e+03 },
    { 1.1390000000000000e-08, 4.3654003687699996e+00, 1.0527896480502000e+03 },
    { 1.1370000000000000e-08, 3.2743387191200002e+00, 4.4779581952650000e+02 },
    { 1.1370000000000000e-08, 5.9645082537300000e+00, 9.5066008167810003e+02 },
    { 1.1320000000000000e-08, 5.2232241554299996e+00, 1.6638544938511000e+03 },
    { 1.1310000000000000e-08, 3.6119144594499999e+00, 1.8873055176757000e+03 },
    { 1.1199999999999999e-08, 3.0262453633200002e+00, 7.0904981058880003e+02 },
    { 1.1180000000000000e-08, 1.4538990730600001e+00, 8.9255325766720000e+02 },
    { 1.1129999999999999e-08, 6.0810759885400003e+00, 3.3843313390048002e+03 },
    { 1.1120000000000000e-08, 3.7233668831500002e+00, 1.0659742123282001e+03 },
    { 1.1110000000000000e-08, 2.1546212200500001e+00, 1.0361403908040000e+02 },
    { 1.1090000000000000e-08, 5.8196053725399999e+00, 3.4788443904560000e+02 },
    { 1.1050000000000000e-08, 4.6869566994999996e+00, 1.3662125722901999e+03 },
    { 1.0989999999999999e-08, 1.2377517129600000e+00, 6.1158611066269998e+02 },
    { 1.0929999999999999e-08, 1.7405245920300001e+00, 7.6174000862829996e+02 },
    { 1.0890000000000000e-08, 3.6059262876200000e+00, 9.4672792841499995e+02 },
    { 1.0880000000000000e-08, 1.9529284489500001e+00, 2.3107223148140001e+03 },
    { 1.0680000000000000e-08, 6.0418518586800003e+00, 1.2797945726280000e+03 },
    { 1.0670000000000001e-08, 4.1622320101700003e+00, 7.9961241183519996e+02 },
    { 1.0610000000000000e-08, 3.8634060775300001e+00, 1.0141353475506000e+03 },
    { 1.0480000000000000e-08, 3.8010986538000002e-01, 1.9015326116773001e+03 },
    { 1.0449999999999999e-08, 4.2211696761699997e+00, 1.5709136480191000e+03 },
    { 1.0420000000000000e-08, 5.6492552571000001e+00, 2.9764192156090002e+02 },
    { 1.0400000000000000e-08, 8.3747359096999996e-01, 3.1394418910180002e+02 },
    { 1.0379999999999999e-08, 7.6497233984000002e-01, 1.7076082578510000e+02 },
    { 1.0360000000000000e-08, 5.3491669676999998e-01, 1.7984398374586999e+03 },
    { 1.0360000000000000e-08, 5.2689999476700002e+00, 9.1523535902599997e+02 },
    { 1.0350000000000000e-08, 1.2867193623600000e+00, 2.5453620512543998e+03 },
    { 1.0330000000000001e-08, 1.0866330038700001e+00, 1.0283624415521999e+03 },
    { 1.0330000000000001e-08, 3.8697399076200001e+00, 6.3182053146670000e+02 },
    { 1.0320000000000001e-08, 3.1996856590100000e+00, 3.0750530163489998e+03 },
    { 1.0300000000000000e-08, 2.1031170772299999e+00, 3.9510562148700001e+02 },
    { 1.0250000000000001e-08, 5.5313245801799997e+00, 9.5162328952459995e+02 },
    { 1.0210000000000000e-08, 2.6795260478200000e+00, 2.7302069586891998e+03 },
    { 1.0179999999999999e-08, 8.7095306528000005e-01, 6.2322251175760005e+02 },
    { 1.0169999999999999e-08, 1.6592846226000000e+00, 1.0197644218431000e+03 },
    { 1.0150000000000000e-08, 5.0774958849500003e+00, 3.4874241132234001e+03 },
    { 1.0120000000000001e-08, 4.0691222962499998e+00, 1.6811199692385001e+03 },
    { 1.0000000000000000e-08, 3.5908379113000000e-01, 9.2479658658159997e+02 },
    { 9.9900000000000005e-09, 2.2991490581099998e+00, 1.5056932770074000e+03 },
    { 9.9200000000000005e-09, 4.4896489708800003e+00, 4.1202078525365996e+03 },
    { 9.8899999999999996e-09, 2.2791041859400001e+00, 5.6633952802409999e+02 },
    { 9.8899999999999996e-09, 5.2031741160199996e+00, 4.9304240216509999e+02 },
    { 9.8700000000000000e-09, 2.2025444078000000e+00, 7.2109158184941998e+03 },
    { 9.8500000000000005e-09, 2.4088992371000001e-01, 1.2623860848887000e+03 },
    { 9.8299999999999993e-09, 4.5083501355399997e+00, 1.4756851803267000e+03 },
    { 9.8299999999999993e-09, 2.3878910771299999e+00, 1.6700742689746000e+03 },
    { 9.8099999999999998e-09, 4.4894644170299998e+00, 1.2884620283063000e+03 },
    { 9.8000000000000001e-09, 1.9891188548900001e+00, 9.6170578194200004e+02 },
    { 9.8000000000000001e-09, 4.6521362365699996e+00, 2.5035634758385801e+04 },
    { 9.7599999999999994e-09, 6.0865622396100001e+00, 7.0045179087969996e+02 },
    { 9.6799999999999997e-09, 5.5196442053999997e+00, 8.0672595883600002e+02 },
    { 9.6699999999999999e-09, 6.0633179249299998e+00, 6.6502706822760001e+02 },
    { 9.6699999999999999e-09, 4.9066022548400001e+00, 8.2881735936380005e+02 },
    { 9.6500000000000004e-09, 5.1916211232100000e+00, 1.5199203710090001e+03 },
    { 9.6400000000000006e-09, 2.7660415622499999e+00, 6.9158595893046004e+03 },
    { 9.5999999999999999e-09, 6.1739002880299996e+00, 4.8631979130691998e+03 },
    { 9.5499999999999995e-09, 2.1416135397299998e+00, 6.4531391228539997e+02 },
    { 9.5000000000000007e-09, 1.7505366931399999e+00, 2.5021407664384202e+04 },
    { 9.3900000000000000e-09, 3.3369630466700002e+00, 1.7032113697663001e+03 },
    { 9.3299999999999998e-09, 5.3959087191800004e+00, 1.5787779545453000e+03 },
    { 9.3299999999999998e-09, 3.8685464525300000e+00, 2.5169078632512001e+03 },
    { 9.2900000000000008e-09, 7.6029521873000006e-01, 1.1302313754934000e+03 },
    { 9.2699999999999996e-09, 2.3403733817700001e+00, 3.7967024358792000e+03 },
    { 9.2400000000000004e-09, 1.9365306269600000e+00, 1.8181529934009000e+03 },
    { 9.2300000000000006e-09, 4.7588156343300003e+00, 1.7935884549420001e+02 },
    { 9.1800000000000001e-09, 2.2966346656800001e+00, 3.4077089204480001e+02 },
    { 9.1399999999999995e-09, 2.8107178678000000e-01, 1.0310707545379000e+03 },
    { 9.1199999999999999e-09, 1.1926223674900001e+00, 1.7913262904579001e+03 },
    { 9.0099999999999993e-09, 1.2467002021100000e+00, 3.2591471642584002e+03 },
    { 8.9899999999999998e-09, 5.7976446629199998e+00, 1.0531621550657001e+03 },
    { 8.8900000000000005e-09, 4.9571931331099996e+00, 1.5726744496640001e+02 },
    { 8.8599999999999996e-09, 4.5458100887499997e+00, 1.1015813710960001e+02 },
    { 8.8300000000000003e-09, 4.1046016214499996e+00, 5.2616784674559995e+02 },
    { 8.8300000000000003e-09, 3.3776766805100000e+00, 5.3321408344359997e+02 },
    { 8.7700000000000001e-09, 1.0064513680600000e+00, 3.5744566660120000e+02 },
    { 8.7500000000000006e-09, 1.9299640269400000e+00, 4.1273213995373999e+03 },
    { 8.7299999999999994e-09, 2.1748412885000001e-01, 3.2202094394910000e+02 },
    { 8.7199999999999997e-09, 4.6189921103999998e-01, 1.4820479678020999e+03 },
    { 8.7000000000000001e-09, 7.2037948511000005e-01, 5.3732577695719999e+02 },
    { 8.7000000000000001e-09, 3.6203061632600000e+00, 5.2205615323200004e+02 },
    { 8.6900000000000004e-09, 5.5334929762199998e+00, 2.1574677599280000e+02 },
    { 8.6900000000000004e-09, 4.1912969024400004e+00, 1.0846069285524200e+04 },
    { 8.6499999999999997e-09, 1.0126946187999999e-01, 3.8997952100978000e+03 },
    { 8.5999999999999993e-09, 1.5952443299100001e+00, 1.1529134768521999e+03 },
    { 8.5600000000000002e-09, 2.4439741654799998e+00, 3.6908206769610001e+02 },
    { 8.5099999999999998e-09, 3.9678869089800002e+00, 5.7021672008196001e+03 },
    { 8.5099999999999998e-09, 3.8406910946999999e+00, 2.1249836355019002e+03 },
    { 8.4800000000000005e-09, 3.1542726327400001e+00, 4.4475743814070000e+02 },
    { 8.4399999999999998e-09, 3.2387243038200002e+00, 2.3145834270270001e+02 },
    { 8.4100000000000005e-09, 1.5780533994399999e+00, 6.4307868005169996e+02 },
    { 8.3899999999999994e-09, 5.0396929703400000e+00, 4.6868894077067998e+03 },
    { 8.3899999999999994e-09, 6.0332714152199998e+00, 3.7007232086613999e+03 },
    { 8.3199999999999994e-09, 5.9119778563500001e+00, 1.6889842757647000e+03 },
    { 8.2700000000000006e-09, 5.9749431009600000e+00, 2.0956262057516600e+04 },
    { 8.2000000000000006e-09, 2.5829995248399999e+00, 1.3718416465826999e+03 },
    { 8.1699999999999997e-09, 1.9269031254300000e+00, 1.3725924061081000e+03 },
    { 8.1500000000000002e-09, 8.5095606738999996e-01, 3.5893013930950002e+02 },
    { 8.1099999999999995e-09, 5.1591358002999996e+00, 5.2826825569443999e+03 },
    { 8.0999999999999997e-09, 1.9123782670900000e+00, 6.7776968952089999e+02 },
    { 8.0900000000000000e-09, 1.9319570119200000e+00, 3.4863519857099999e+02 },
    { 8.0399999999999995e-09, 2.9174027099700002e+00, 1.1411340634054000e+03 },
    { 7.9799999999999993e-09, 3.0678650296100001e+00, 3.7715882254339999e+02 },
    { 7.9300000000000005e-09, 3.8978711440000001e-02, 5.0921519581158000e+03 },
    { 7.8899999999999998e-09, 1.0006236927999999e-01, 9.5013881681630005e+02 },
    { 7.8899999999999998e-09, 6.2852846983999999e-01, 7.0363318461740005e+02 },
    { 7.8700000000000003e-09, 2.6866723923500002e+00, 1.0566736172035000e+03 },
    { 7.8700000000000003e-09, 4.4395954035100003e+00, 4.3201481684740003e+02 },
    { 7.8199999999999999e-09, 6.6782646307000004e-01, 1.7315225454176000e+03 },
    { 7.8100000000000001e-09, 4.8241202246499997e+00, 2.2257887878601000e+03 },
    { 7.8100000000000001e-09, 2.3793119181700000e+00, 2.4169964827703002e+03 },
    { 7.8000000000000004e-09, 6.0742808742400003e+00, 9.4912742486000002e+02 },
    { 7.7799999999999992e-09, 3.2549986319499999e+00, 9.0562601246400007e+01 },
    { 7.7599999999999997e-09, 1.7466059191500001e+00, 7.1149749114359997e+02 },
    { 7.7400000000000002e-09, 3.7023231607799998e+00, 2.4319743362972999e+03 },
    { 7.7300000000000004e-09, 1.8165556769999999e-01, 8.6170991039264009e+03 },
    { 7.7200000000000006e-09, 1.9013329643999999e-01, 3.3339287628256998e+03 },
    { 7.6899999999999997e-09, 2.2022368211300001e+00, 6.0114455234753996e+03 },
    { 7.6600000000000004e-09, 3.0239921176800002e+00, 4.0221916848780000e+02 },
    { 7.6500000000000007e-09, 8.2175084218000005e-01, 1.8346153196362000e+03 },
    { 7.6500000000000007e-09, 5.2174383314500004e+00, 9.2783496796739996e+02 },
    { 7.5900000000000005e-09, 5.6988329181199999e+00, 1.5426024723677999e+03 },
    { 7.5599999999999995e-09, 5.7260333982000000e-01, 1.7224529849340001e+02 },
    { 7.5400000000000000e-09, 2.6628263925200000e+00, 1.0227333672597000e+03 },
    { 7.5300000000000003e-09, 2.4539018099400001e+00, 8.8114920691600005e+01 },
    { 7.5100000000000007e-09, 1.1021104404299999e+00, 4.4631134681819998e+02 },
    { 7.4799999999999998e-09, 2.3998257174300002e+00, 1.9975118388951000e+03 },
    { 7.4600000000000003e-09, 3.0715456859000001e-01, 7.4543774108740001e+02 },
    { 7.4400000000000000e-09, 9.4568343594000004e-01, 9.8899988524600005e+01 },
    { 7.4300000000000002e-09, 6.0572785812200003e+00, 3.0609692891809999e+02 },
    { 7.4199999999999996e-09, 4.3265335833000002e-01, 8.9648541093029996e+02 },
    { 7.4199999999999996e-09, 2.6461735315400001e+00, 9.4821240112329997e+02 },
    { 7.3600000000000002e-09, 4.0749188522399997e+00, 2.3969139281195799e+04 },
    { 7.3600000000000002e-09, 5.9694347292000005e-01, 6.0150361824530000e+02 },
    { 7.3600000000000002e-09, 6.0214952186000004e-01, 4.5787831194389997e+02 },
    { 7.3499999999999996e-09, 3.3474292280000001e-01, 6.4959738580800007e+01 },
    { 7.3200000000000004e-09, 3.8587250033000000e+00, 1.6001185955476999e+03 },
    { 7.2699999999999999e-09, 1.1263281516999999e+00, 4.2511371816769997e+02 },
    { 7.2699999999999999e-09, 5.7073797151000001e+00, 3.4470304530790003e+02 },
    { 7.2600000000000002e-09, 5.2382471287999999e-01, 2.1510071892927999e+03 },
    { 7.1600000000000001e-09, 1.6148352386200000e+00, 3.3334988796990001e+03 },
    { 7.1500000000000003e-09, 5.8013394012299999e+00, 3.8554439393140001e+02 },
    { 7.1500000000000003e-09, 4.5065450261700004e+00, 3.3477259737006002e+03 },
    { 7.1200000000000002e-09, 4.7095937826899998e+00, 2.1084689196398999e+03 },
    { 7.1200000000000002e-09, 1.2323026209300001e+00, 4.0028879843164000e+03 },
    { 7.1099999999999996e-09, 1.1096937803400000e+00, 4.5172797278960002e+02 },
    { 7.0800000000000004e-09, 3.0269061413300000e+00, 3.3914448860056000e+03 },
    { 7.0699999999999998e-09, 6.0198922161000001e-01, 2.9201284726839998e+02 },
    { 7.0500000000000003e-09, 2.7003271079500002e+00, 2.5976951835400001e+02 },
    { 7.0500000000000003e-09, 4.5529097746999998e+00, 6.5716276170139997e+02 },
    { 7.0399999999999997e-09, 5.1955960097200000e+00, 7.1861103814440003e+02 },
    { 6.9999999999999998e-09, 5.0794526300699996e+00, 3.1843655656911001e+03 },
    { 6.9900000000000001e-09, 2.0408690026400000e+00, 9.9957818514890005e+02 },
    { 6.9299999999999999e-09, 3.4065188751100002e+00, 1.9677161905354999e+03 },
    { 6.9200000000000001e-09, 2.7951080581999999e+00, 3.7895888888784002e+03 },
    { 6.8500000000000001e-09, 5.2093871192799996e+00, 2.2186752408593002e+03 },
    { 6.7400000000000003e-09, 5.1473052625999998e+00, 2.8245161971279998e+02 },
    { 6.7299999999999997e-09, 1.8572920762300000e+00, 7.7962992305000000e+01 },
    { 6.6599999999999997e-09, 1.4776251948000001e+00, 2.4390878832981002e+03 },
    { 6.6400000000000002e-09, 3.3511557626899999e+00, 1.2747179660680000e+02 },
    { 6.6199999999999999e-09, 3.0651417095300002e+00, 4.9591771402869999e+03 },
    { 6.5700000000000003e-09, 6.2702555569999996e-01, 4.6473122651379998e+02 },
    { 6.5700000000000003e-09, 4.0021926878000003e-01, 2.3508129039121000e+03 },
    { 6.5499999999999999e-09, 4.6579209449999999e+00, 1.8094855377225999e+03 },
    { 6.5000000000000003e-09, 1.1666866225500001e+00, 5.5814515309779995e+02 },
    { 6.4899999999999997e-09, 5.1179756920299999e+00, 2.0373786788240000e+02 },
    { 6.4400000000000001e-09, 2.2005912430899999e+00, 1.3049243545416000e+03 },
    { 6.4300000000000003e-09, 9.1695869023999999e-01, 2.0863322899200000e+02 },
    { 6.4199999999999998e-09, 1.7746626723300001e+00, 4.1130943055358002e+03 },
    { 6.4100000000000000e-09, 6.0907626740499996e+00, 5.2175806283148399e+04 },
    { 6.4100000000000000e-09, 5.9312731089900002e+00, 4.2193232442999999e+02 },
    { 6.3899999999999996e-09, 4.7935213234000003e+00, 6.5094298657790000e+02 },
    { 6.3899999999999996e-09, 2.7595787856400000e+00, 1.0017201291180000e+02 },
    { 6.3600000000000004e-09, 6.0224353362200000e+00, 1.0110192771992401e+04 },
    { 6.3600000000000004e-09, 5.5176856895900004e+00, 6.6427630870220003e+02 },
    { 6.3400000000000000e-09, 1.6241551680699999e+00, 5.9083527492568001e+03 },
    { 6.3000000000000002e-09, 2.4524162535700000e+00, 1.7515395314160000e+03 },
    { 6.2900000000000004e-09, 1.9322913986800001e+00, 1.8180652604900001e+02 },
    { 6.2799999999999998e-09, 4.7512742571500004e+00, 1.5606187072805999e+03 },
    { 6.2499999999999997e-09, 1.3452491388100001e+00, 1.1976754877499999e+02 },
    { 6.2400000000000000e-09, 1.3280340950800000e+00, 1.0529677415867600e+04 },
    { 6.2099999999999999e-09, 3.4899181637200001e+00, 4.5396924964117998e+03 },
    { 6.1900000000000003e-09, 2.8007476305300001e+00, 3.2595309721220002e+02 },
    { 6.1700000000000000e-09, 3.1666852307000002e+00, 4.8560843660684004e+03 },
    { 6.1700000000000000e-09, 2.1739979711299999e+00, 9.2086443331850001e+02 },
    { 6.1499999999999996e-09, 2.0004847357500002e+00, 3.4945376602242000e+03 },
    { 6.1399999999999999e-09, 2.1202915210800000e+00, 6.1216518446948003e+03 },
    { 6.1399999999999999e-09, 2.3160730782100001e+00, 1.1443154571431000e+03 },
    { 6.1300000000000001e-09, 1.5989670052999999e-01, 6.0765395739960002e+02 },
    { 6.1099999999999998e-09, 1.6568413271400000e+00, 4.4437132691939996e+03 },
    { 6.1099999999999998e-09, 4.3380344796999998e-01, 3.9265794093220001e+02 },
    { 6.1099999999999998e-09, 7.6528954681000005e-01, 6.6672398925699997e+02 },
    { 6.0600000000000002e-09, 4.7408063704499996e+00, 3.8038159828799999e+03 },
    { 6.0500000000000004e-09, 4.4611289389900000e+00, 8.3578789401270001e+02 },
    { 6.0500000000000004e-09, 1.4990158284099999e+00, 2.0767002114549999e+02 },
    { 6.0500000000000004e-09, 3.4694333334200000e+00, 3.0061086697750000e+02 },
    { 6.0500000000000004e-09, 6.1904279666599997e+00, 6.5994677196479997e+03 },
    { 6.0399999999999998e-09, 5.2531275683900001e+00, 4.2711945573780002e+02 },
    { 6.0200000000000003e-09, 5.7392813892500003e+00, 1.6708250284999999e+03 },
    { 6.0200000000000003e-09, 5.5292592803599998e+00, 6.8073863493750002e+02 },
    { 6.0200000000000003e-09, 3.2017022594000000e-01, 3.5452490529440001e+02 },
    { 6.0200000000000003e-09, 8.7892276883999998e-01, 7.0485702489480002e+02 },
    { 6.0099999999999997e-09, 4.7325453333000000e+00, 4.2607692601420001e+02 },
    { 6.0000000000000000e-09, 1.7676799599999999e-03, 6.3426821202149995e+02 },
    { 5.9900000000000002e-09, 2.5780457458799999e+00, 1.8498791978669999e+02 },
    { 5.9799999999999996e-09, 5.1769443783000002e-01, 1.1562294719080001e+02 },
    { 5.9600000000000001e-09, 1.0048846054700000e+00, 7.3736098624010003e+02 },
    { 5.9300000000000000e-09, 6.2671244929000003e-01, 7.5877106321170004e+02 },
    { 5.9200000000000002e-09, 4.8058901083000000e-01, 1.0601353552540000e+02 },
    { 5.8999999999999999e-09, 1.0534586882500001e+00, 3.6346210245184002e+03 },
    { 5.8999999999999999e-09, 2.9868922657899999e+00, 6.2736711334179995e+02 },
    { 5.8900000000000001e-09, 2.6867746379000002e-01, 2.7405018994277002e+03 },
    { 5.8800000000000004e-09, 1.0679818786000000e-01, 4.5809076026499997e+02 },
    { 5.8299999999999999e-09, 3.3500074589000000e+00, 4.3263934009738005e+03 },
    { 5.8100000000000004e-09, 3.8612826737499999e+00, 2.9538009948657000e+03 },
    { 5.7999999999999998e-09, 2.4710019206300000e+00, 2.1006046131137000e+03 },
    { 5.7900000000000001e-09, 5.9620518273899998e+00, 5.1795897827258004e+03 },
    { 5.7900000000000001e-09, 7.1604969737000002e-01, 5.9757146498220004e+02 },
    { 5.7900000000000001e-09, 4.8304329741000002e-01, 4.6181046520699999e+02 },
    { 5.7800000000000003e-09, 5.3962537610599997e+00, 8.8150755740329998e+02 },
    { 5.7500000000000002e-09, 6.1204635415600004e+00, 1.6567409468502999e+03 },
    { 5.7299999999999999e-09, 6.0916305269900004e+00, 1.1326790560482000e+03 },
    { 5.7100000000000003e-09, 1.6927475924199999e+00, 1.9377080938547999e+03 },
    { 5.6800000000000002e-09, 3.6654156763000001e-01, 1.3140394986990000e+02 },
    { 5.6100000000000003e-09, 3.9196306698800001e+00, 4.4386366626339998e+02 },
    { 5.6100000000000003e-09, 2.5102067285000000e+00, 2.0426571092422000e+04 },
    { 5.6100000000000003e-09, 8.9037751064000004e-01, 6.7025604938666002e+03 },
    { 5.6100000000000003e-09, 5.1930699885999996e+00, 1.8589943420244001e+03 },
    { 5.5999999999999997e-09, 3.1397233355499998e+00, 4.2096911658350001e+02 },
    { 5.5999999999999997e-09, 2.1016083929399998e+00, 4.7011165017084004e+03 },
    { 5.5899999999999999e-09, 3.7091686107499999e+00, 3.8872578766909999e+02 },
    { 5.5800000000000002e-09, 5.7206557221999998e-01, 5.9465070367540000e+02 },
    { 5.5800000000000002e-09, 6.1165798313800002e+00, 2.1997652340691998e+03 },
    { 5.5800000000000002e-09, 5.7425465775399998e+00, 2.3478439584954999e+03 },
    { 5.5599999999999998e-09, 4.5395348878100004e+00, 4.8042092759269999e+03 },
    { 5.5599999999999998e-09, 3.2526362077200002e+00, 1.5952926704073000e+03 },
    { 5.5400000000000003e-09, 2.6696623196400000e+00, 3.8575684225250001e+02 },
    { 5.5299999999999997e-09, 2.1390673461500000e+00, 1.5349535039770001e+02 },
    { 5.5100000000000002e-09, 5.9454158243700004e+00, 4.3335069479745998e+03 },
    { 5.5100000000000002e-09, 7.2617989359000001e-01, 8.8375117463652005e+03 },
    { 5.4899999999999999e-09, 2.3146992551399999e+00, 1.0632770190086199e+04 },
    { 5.4899999999999999e-09, 8.5225402276999995e-01, 7.0193626358799997e+02 },
    { 5.4500000000000000e-09, 2.0690045337499998e+00, 2.0046253858959001e+03 },
    { 5.4299999999999997e-09, 6.2793049343199998e+00, 5.7844900987100004e+02 },
    { 5.4299999999999997e-09, 1.2029733676400001e+00, 4.8093292031819999e+02 },
    { 5.4299999999999997e-09, 4.8848038573000002e-01, 3.9140223040994001e+03 },
    { 5.4199999999999999e-09, 2.1174468158500002e+00, 8.5827298831199997e+01 },
    { 5.3800000000000000e-09, 2.4991010871600001e+00, 7.5648307207380003e+03 },
    { 5.3400000000000002e-09, 2.7095651824800000e+00, 8.4363515419639998e+02 },
    { 5.2700000000000002e-09, 5.9632891137500001e+00, 8.7344189721465991e+03 },
    { 5.2700000000000002e-09, 4.4265241395099997e+00, 4.4365997221932003e+03 },
    { 5.2499999999999999e-09, 4.2450715507299996e+00, 7.6674636529438780e+04 },
    { 5.2400000000000001e-09, 3.0093052759999999e-02, 1.3364169239306000e+03 },
    { 5.2300000000000003e-09, 3.9258199992199998e+00, 3.9069087570985998e+03 },
    { 5.2300000000000003e-09, 3.6690378703500000e+00, 2.4241100297711000e+03 },
    { 5.2199999999999998e-09, 1.2810810370100001e+00, 3.8309671337660001e+02 },
    { 5.1600000000000004e-09, 4.9761671666899998e+00, 5.5990744266009997e+03 },
    { 5.1600000000000004e-09, 3.7054046346799998e+00, 6.0185590704761998e+03 },
    { 5.1499999999999998e-09, 4.5658398487999996e+00, 1.0908744995781999e+03 },
    { 5.1099999999999999e-09, 1.7821225544999999e+00, 2.0714875628370001e+02 },
    { 5.0700000000000001e-09, 3.5026307430300001e+00, 6.1551826392580006e+02 },
    { 5.0499999999999997e-09, 4.4577011956000001e-01, 3.9814400287280000e+02 },
    { 5.0300000000000002e-09, 6.6237185259999998e-01, 1.4434418514122999e+03 },
    { 5.0300000000000002e-09, 1.6279616215099999e+00, 5.8564776591153995e+03 },
    { 5.0000000000000001e-09, 3.4400083580200000e+00, 8.6313261979279996e+03 },
    { 4.9900000000000003e-09, 5.4847011268100001e+00, 1.9731328165069001e+03 },
    { 4.9600000000000002e-09, 5.5069944979000001e+00, 2.3288815620787000e+03 },
    { 4.9600000000000002e-09, 3.0151496850299999e+00, 2.7586611466924001e+03 },
    { 4.9499999999999997e-09, 1.5807596420900001e+00, 3.9828701522469999e+02 },
    { 4.9399999999999999e-09, 4.8893067336199998e+00, 5.3605375257000003e+02 },
    { 4.9399999999999999e-09, 5.7345642219300004e+00, 5.2332817761920001e+02 },
    { 4.9300000000000001e-09, 5.0478237827200001e+00, 2.8790193962983999e+03 },
    { 4.9099999999999998e-09, 1.0922924691600000e+00, 6.0129116992419995e+02 },
    { 4.8799999999999997e-09, 3.3053643836800002e+00, 4.1953282798499998e+02 },
    { 4.8699999999999999e-09, 7.0358405203999996e-01, 3.2105773610260002e+02 },
    { 4.8300000000000001e-09, 8.9846608938000005e-01, 1.5429849954980000e+02 },
    { 4.8200000000000003e-09, 2.6618211362399999e+00, 6.1145382976940000e+03 },
    { 4.8000000000000000e-09, 2.5311079559500000e+00, 3.0465988283458000e+03 },
    { 4.8000000000000000e-09, 2.9534818629699999e+00, 2.1329909543799999e+03 },
    { 4.8000000000000000e-09, 3.5833589088400002e+00, 1.1058523531052001e+03 },
    { 4.8000000000000000e-09, 4.8322803531599998e+00, 9.0508108418032007e+03 },
    { 4.7900000000000002e-09, 3.6956547475799999e+00, 2.1871572140940000e+02 },
    { 4.7600000000000001e-09, 3.5321509896999999e+00, 2.8332997329077998e+03 },
    { 4.7399999999999998e-09, 4.7700766509999999e-02, 1.0234201469320000e+02 },
    { 4.7399999999999998e-09, 3.6171857882900000e+00, 2.2036973873323000e+03 },
    { 4.7200000000000002e-09, 3.7637982934099998e+00, 1.0384353374400000e+02 },
    { 4.7200000000000002e-09, 1.6256866991500001e+00, 2.5272028039897000e+03 },
    { 4.7200000000000002e-09, 9.3318411224999998e-01, 4.1677633088950000e+02 },
    { 4.7099999999999997e-09, 2.8866973284799999e+00, 2.5721887541974002e+03 },
    { 4.6999999999999999e-09, 6.2342702847300000e+00, 8.7832616366560001e+02 },
    { 4.6900000000000001e-09, 5.2063569260299998e+00, 2.2745468326364999e+03 },
    { 4.6800000000000004e-09, 1.8214621483300000e+00, 4.7700076705510003e+02 },
    { 4.6800000000000004e-09, 5.6608161536299999e+00, 5.8238116313410001e+02 },
    { 4.6800000000000004e-09, 1.8128030053999999e-01, 4.1943645976540000e+02 },
    { 4.6800000000000004e-09, 4.2822433179599999e+00, 1.9706851359520999e+03 },
    { 4.6699999999999998e-09, 4.9089030603000000e-01, 2.0479144150742000e+03 },
    { 4.6699999999999998e-09, 4.6270052262799997e+00, 2.6546746005965001e+03 },
    { 4.6600000000000000e-09, 2.2291460256600000e+00, 6.3226247445139995e+02 },
    { 4.6600000000000000e-09, 4.7754019162199999e+00, 7.3342883297699996e+02 },
    { 4.6500000000000003e-09, 2.1969466493500001e+00, 1.0837609525774001e+03 },
    { 4.6500000000000003e-09, 3.6230678879000000e+00, 4.0100015313171998e+03 },
    { 4.6399999999999997e-09, 5.9799699932400001e+00, 4.0171150783180001e+03 },
    { 4.6299999999999999e-09, 4.2139997618500002e+00, 4.2951895218279998e+02 },
    { 4.6299999999999999e-09, 2.7497281177800001e+00, 6.3330500417500002e+02 },
    { 4.6099999999999996e-09, 1.8002508797700001e+00, 1.7787437278589999e+02 },
    { 4.5100000000000003e-09, 3.6225262861799998e+00, 1.4575259330619999e+03 },
    { 4.5100000000000003e-09, 2.4606269661900000e+00, 1.0977078804699000e+04 },
    { 4.5100000000000003e-09, 1.4376044617799999e+00, 2.0613736432740001e+02 },
    { 4.4999999999999998e-09, 1.4574596710400001e+00, 5.4099729778940002e+02 },
    { 4.4999999999999998e-09, 2.8832259773300000e+00, 5.1838463239980001e+02 },
    { 4.4900000000000000e-09, 6.2750628283500003e+00, 1.2218485663214001e+03 },
    { 4.4699999999999997e-09, 3.8347111567600001e+00, 4.7535623565240002e+02 },
    { 4.4699999999999997e-09, 3.6475671452000000e+00, 5.8402569453679996e+02 },
    { 4.4299999999999998e-09, 5.2206847460299999e+00, 1.9872168981566001e+03 },
    { 4.4200000000000000e-09, 6.0013415202299996e+00, 1.0968505635760000e+02 },
    { 4.4200000000000000e-09, 4.8428257619600004e+00, 2.9363925071263998e+03 },
    { 4.4200000000000000e-09, 5.8014690450700002e+00, 1.0689431577447999e+03 },
    { 4.3699999999999996e-09, 4.9255953205500003e+00, 2.5350671105158999e+03 },
    { 4.3200000000000000e-09, 4.4389868098100003e+00, 2.0806308247406000e+03 },
    { 4.3200000000000000e-09, 6.0083518976499999e+00, 1.3009922012785000e+03 },
    { 4.2899999999999999e-09, 1.2676163965600000e+00, 4.6356717236295999e+03 },
    { 4.2100000000000001e-09, 4.1749678443900002e+00, 6.3841281360569997e+02 },
    { 4.1899999999999998e-09, 3.1891541591500001e+00, 2.0903096723752001e+03 },
    { 4.1800000000000000e-09, 4.2402293294300000e+00, 2.0522234059069999e+02 },
    { 4.1499999999999999e-09, 1.3917499662799999e+00, 4.7601051388506003e+03 },
    { 4.1199999999999998e-09, 4.4921747983399998e+00, 2.5421806575167002e+03 },
    { 4.1199999999999998e-09, 2.3873833522900001e+00, 8.3379618276399995e+01 },
    { 4.1000000000000003e-09, 5.6301185651999996e-01, 7.8643065262000000e+00 },
    { 4.0400000000000001e-09, 1.9514763657200001e+00, 6.3278373931320002e+03 },
    { 4.0000000000000002e-09, 1.2040931967899999e+00, 2.7554797529547000e+03 },
    { 3.9899999999999997e-09, 1.8912086074200001e+00, 1.6213162241982000e+03 },
    { 3.9700000000000001e-09, 3.5627080301100000e+00, 5.3715482371613998e+03 },
    { 3.9600000000000004e-09, 1.1942568250800001e+00, 5.0586733387864588e+04 },
    { 3.9400000000000000e-09, 4.4794053402099996e+00, 2.4248607892965001e+03 },
    { 3.9099999999999999e-09, 3.7731003862699999e+00, 6.7065614252010005e+02 },
    { 3.8400000000000000e-09, 4.8126063423199996e+00, 6.7362508793669997e+02 },
};

static const VSOP87Term VSOP87A_JUPITER_Y1[499] = {
    { 1.6947982529999999e-02, 3.1415926535900001e+00, 0.0000000000000000e+00 },
    { 6.3485979799999996e-03, 4.8190319964999997e+00, 1.0593819301891999e+03 },
    { 6.0116043099999999e-03, 8.5811249940000001e-01, 5.2257741809380002e+02 },
    { 5.8892850400000000e-03, 3.4491576889999997e-01, 5.3680451209540001e+02 },
    { 8.1187144999999997e-04, 1.9091431653199999e+00, 7.1135470007999997e+00 },
    { 4.6888090000000001e-04, 1.9129453561800001e+00, 5.2969096509459996e+02 },
    { 4.6194128999999998e-04, 5.1695599456099997e+00, 1.5890728952838001e+03 },
    { 3.2503452999999997e-04, 1.7640743622999999e-01, 1.0522683831884001e+03 },
    { 3.1231694000000002e-04, 7.7623645597000002e-01, 1.0664954771900000e+03 },
    { 1.9462095999999999e-04, 3.0095711946999999e+00, 1.1020632121940000e+02 },
    { 1.7738615000000000e-04, 2.4653178710099999e+00, 4.2659819087599999e+02 },
    { 1.3701691999999999e-04, 5.0207019780400000e+00, 6.3278373931320004e+02 },
    { 1.3034616000000000e-04, 9.8979834442000003e-01, 5.1546387109299997e+02 },
    { 9.1226599999999998e-05, 2.1022587969000001e-01, 5.4391805909619995e+02 },
    { 8.1090500000000006e-05, 4.5812381160099998e+00, 9.4917560896980001e+02 },
    { 7.1452290000000000e-05, 1.6038123609399999e+00, 3.2350541665740002e+02 },
    { 4.3473460000000000e-05, 5.8552283548800004e+00, 7.3587651353180001e+02 },
    { 3.9575920000000003e-05, 6.1855069781700003e+00, 2.0618554843720000e+02 },
    { 3.8661470000000000e-05, 1.7687758203799999e+00, 1.4227094001599999e+01 },
    { 3.4017350000000001e-05, 1.3303322525200001e+00, 5.2650957135689998e+02 },
    { 3.3318870000000002e-05, 6.1495183571199998e+00, 5.3287235883230005e+02 },
    { 3.0942570000000002e-05, 1.0067045470100000e+00, 4.1948464387519999e+02 },
    { 3.0442050000000001e-05, 8.3297798270000001e-02, 5.2575881183150000e+02 },
    { 3.0014839999999999e-05, 1.1128060628300001e+00, 5.3362311835770004e+02 },
    { 2.9901920000000001e-05, 5.5194483050600001e+00, 2.1187638603783998e+03 },
    { 2.9772840000000000e-05, 3.3550702850700000e+00, 1.0309277421860000e+02 },
    { 2.3470999999999999e-05, 2.0678177539000000e+00, 3.1639186965660002e+02 },
    { 1.9684010000000001e-05, 3.5189673984400001e+00, 9.5628915597059995e+02 },
    { 1.8853930000000001e-05, 2.9905973710000000e-01, 1.5819593482830001e+03 },
    { 1.8754640000000001e-05, 5.3265735648900003e+00, 7.4299006053259995e+02 },
    { 1.8540670000000001e-05, 1.1990873419700001e+00, 1.5961864422845999e+03 },
    { 1.8086270000000001e-05, 5.4028754302599999e+00, 9.5979227217800002e+01 },
    { 1.6369130000000001e-05, 6.0927075644700004e+00, 1.0451548361876000e+03 },
    { 1.5304720000000001e-05, 2.7509472223699998e+00, 1.1731986822019999e+02 },
    { 1.5165410000000000e-05, 6.2168420357100000e+00, 6.3989728631399998e+02 },
    { 1.2601229999999999e-05, 7.1431739539999997e-02, 6.2567019231239999e+02 },
    { 1.0359330000000000e-05, 9.8273794152000005e-01, 1.2655674786264001e+03 },
    { 1.0281650000000000e-05, 1.0030148582400000e+00, 4.3371173787679999e+02 },
    { 9.8354200000000004e-06, 6.1429420808900002e+00, 1.1695882514085999e+03 },
    { 9.7250700000000007e-06, 6.3832646360000000e-01, 2.2041264243879999e+02 },
    { 8.8614300000000000e-06, 1.1026926442600000e+00, 3.0927832265580003e+02 },
    { 8.4177599999999999e-06, 1.8391927727999999e-01, 1.1553611574070001e+03 },
    { 7.7870500000000003e-06, 4.8355854363099997e+00, 9.4206206196899996e+02 },
    { 7.6799299999999995e-06, 4.8477876953300001e+00, 1.1624747044077999e+03 },
    { 5.5195200000000000e-06, 5.7275517677299996e+00, 2.1329909543799999e+02 },
    { 5.4153599999999996e-06, 2.6691411863800001e+00, 8.5319638175199998e+02 },
    { 4.9953299999999998e-06, 1.4505742736500000e+00, 3.3061896365820002e+02 },
    { 4.9213400000000000e-06, 1.0409782456000000e-01, 1.9907200143640000e+02 },
    { 4.8271699999999997e-06, 4.3177642139800003e+00, 1.6921656695024001e+03 },
    { 4.7441099999999999e-06, 5.0539326439999997e-02, 8.3896928775039999e+02 },
    { 4.1190099999999997e-06, 4.6399969054600003e+00, 1.4788665740644001e+03 },
    { 4.1180399999999996e-06, 5.8247806596500000e+00, 7.2876296653099996e+02 },
    { 3.8379899999999997e-06, 6.0154705725000002e-01, 1.0736090241908000e+03 },
    { 2.4912700000000002e-06, 1.6936457581000000e-01, 1.2726810256271999e+03 },
    { 2.3309899999999999e-06, 1.1363065775200001e+00, 5.0835032409220003e+02 },
    { 2.1380900000000001e-06, 6.1696975748899998e+00, 4.1237109687440000e+02 },
    { 2.0332700000000001e-06, 1.4070048119500000e+00, 3.1813937377000001e+00 },
    { 1.9881200000000002e-06, 1.3893148634600001e+00, 1.3757737998458001e+03 },
    { 1.8928900000000001e-06, 5.8699486935399996e+00, 2.6484548254729998e+03 },
    { 1.8168900000000001e-06, 2.0791923825700001e+00, 1.3686602528450001e+03 },
    { 1.7442200000000001e-06, 2.8157513495999997e-01, 1.0625633239269000e+03 },
    { 1.6741000000000000e-06, 9.4422476456000004e-01, 1.2584539316256000e+03 },
    { 1.6138500000000000e-06, 1.4497628299200001e+00, 5.1939602435610004e+02 },
    { 1.5527599999999999e-06, 2.1166487022800000e+00, 3.9321532631000000e+00 },
    { 1.4663200000000000e-06, 1.1516085797200000e+00, 1.0562005364515001e+03 },
    { 1.4410099999999999e-06, 5.1310291970999999e-01, 1.0554497769261000e+03 },
    { 1.4356399999999999e-06, 4.5134707721799998e+00, 1.4859801210651999e+03 },
    { 1.4296300000000000e-06, 2.6612591139999999e-02, 1.5748458012822000e+03 },
    { 1.2674700000000000e-06, 1.7445990587200000e+00, 1.0633140834523001e+03 },
    { 1.2549600000000000e-06, 6.0256074080199999e+00, 5.3998590583309999e+02 },
    { 1.2472399999999999e-06, 4.9946331250000003e-02, 5.5103160609700001e+02 },
    { 1.1709600000000001e-06, 5.4559422833699998e+00, 1.4717530270636000e+03 },
    { 1.1587499999999999e-06, 1.6109797281900000e+00, 2.1258774073792001e+03 },
    { 1.1419499999999999e-06, 4.8862834602999999e-01, 2.2752618943959999e+02 },
    { 1.0316499999999999e-06, 1.1484139980100001e+00, 3.0216477565500003e+02 },
    { 1.0212700000000001e-06, 1.6271055210100001e+00, 2.1340641002400002e+01 },
    { 9.9933999999999995e-07, 1.0942732336300001e+00, 6.1855664531160005e+02 },
    { 9.7794000000000003e-07, 4.4122492625999998e-01, 2.1116503133776000e+03 },
    { 9.3129000000000002e-07, 2.8789327737999998e-01, 1.6992792165031999e+03 },
    { 8.7843999999999997e-07, 1.8833403824000000e-01, 8.3185574074960005e+02 },
    { 8.2829999999999997e-07, 6.0025674840800001e+00, 4.4082528487759998e+02 },
    { 7.8184999999999998e-07, 3.5229164386499998e+00, 5.2502509864859996e+02 },
    { 7.4807000000000000e-07, 3.9578798754000002e+00, 5.3435683154059996e+02 },
    { 7.4073000000000004e-07, 5.9620237000000001e-03, 6.4701083331480004e+02 },
    { 6.6889999999999996e-07, 7.0927232633000004e-01, 1.6850521225016000e+03 },
    { 6.1943000000000003e-07, 4.6730593790799997e+00, 2.2218566345969998e+03 },
    { 5.6181000000000003e-07, 1.0595830904300001e+00, 1.7952584437210000e+03 },
    { 5.4236000000000005e-07, 1.8182312070000001e-02, 7.5010360753340001e+02 },
    { 5.3422000000000003e-07, 4.1577386187199998e+00, 2.0085575391590000e+03 },
    { 5.2789000000000000e-07, 6.1037369228899996e+00, 1.0380412891868000e+03 },
    { 4.9462999999999998e-07, 1.1826438748899999e+00, 9.6340270297140000e+02 },
    { 4.6409000000000002e-07, 9.2753244624999998e-01, 5.2427433912319998e+02 },
    { 4.5782000000000001e-07, 2.7156053620999998e-01, 5.3510759106600005e+02 },
    { 4.3364000000000001e-07, 5.1207023581700000e+00, 1.4956319713459999e+02 },
    { 3.9364999999999998e-07, 3.2487411388499998e+00, 1.3828873468465999e+03 },
    { 3.8663000000000002e-07, 5.9548677833100001e+00, 5.1153171782990000e+02 },
    { 3.6867999999999999e-07, 6.2523481903400002e+00, 5.2677020378780003e+02 },
    { 3.6603000000000000e-07, 1.2301578226500001e+00, 5.3261172640140001e+02 },
    { 3.6523000000000000e-07, 1.5142003389500001e+00, 5.4785021235930003e+02 },
    { 3.5968000000000000e-07, 1.2625262390600001e+00, 1.0294940738499999e+01 },
    { 3.0708999999999998e-07, 2.5594821991000001e+00, 1.2443341522100000e+02 },
    { 3.0391000000000003e-07, 3.2247886035500000e+00, 8.4608283475120004e+02 },
    { 3.0162999999999998e-07, 1.3432469332700001e+00, 1.8983512179396000e+03 },
    { 2.7889999999999999e-07, 1.3975598354999999e-01, 1.8023719907218001e+03 },
    { 2.6651999999999998e-07, 1.4819386106500001e+00, 1.9054647649404001e+03 },
    { 2.5353000000000002e-07, 2.4938499571000000e-01, 5.4073666535849998e+02 },
    { 2.2688000000000000e-07, 5.5287857253699997e+00, 9.3494851496820002e+02 },
    { 2.2441999999999999e-07, 5.9343836361799998e+00, 1.1482476104062000e+03 },
    { 1.9602999999999999e-07, 5.8112628603800003e+00, 1.7881448967202000e+03 },
    { 1.9261000000000000e-07, 2.6131152008499998e+00, 1.3615467058442000e+03 },
    { 1.8196000000000000e-07, 2.3727165178399998e+00, 9.0981873305459999e+02 },
    { 1.8148000000000001e-07, 3.4122076047399998e+00, 1.0627416795630000e+02 },
    { 1.7683000000000000e-07, 1.9627299138100001e+00, 8.6030992875280003e+02 },
    { 1.7622000000000000e-07, 2.0702130031200001e+00, 8.8865680217000005e+01 },
    { 1.7538000000000000e-07, 4.4707266541299999e+00, 5.2549817940059995e+02 },
    { 1.7457000000000001e-07, 4.8193844022399999e+00, 7.6266071275599998e+01 },
    { 1.7368000000000000e-07, 3.0113990005500000e+00, 5.3388375078859997e+02 },
    { 1.7237000000000000e-07, 2.8889393944299999e+00, 1.8159247264699999e+01 },
    { 1.7088999999999999e-07, 9.9046298510999997e-01, 1.6032999892854000e+03 },
    { 1.6135000000000000e-07, 5.8598946098100004e+00, 9.9911380480899993e+01 },
    { 1.5991999999999999e-07, 2.8205243772999999e-01, 1.9195845443560000e+02 },
    { 1.4886000000000000e-07, 4.9099752798800003e+00, 1.0702492748170000e+02 },
    { 1.4747000000000001e-07, 4.7450508591000001e-01, 1.0490869894507000e+03 },
    { 1.4681000000000000e-07, 1.2917919066400001e+00, 3.3773251065900001e+02 },
    { 1.4672999999999999e-07, 1.7408284113000000e-01, 5.1864526483069994e+02 },
    { 1.3929000000000000e-07, 6.1703609215100004e+00, 7.2164941953020002e+02 },
    { 1.2277999999999999e-07, 5.6526469098799996e+00, 2.1045367663768002e+03 },
    { 1.1742000000000000e-07, 6.2174754540999997e+00, 3.1781457905676002e+03 },
    { 1.1668999999999999e-07, 8.6641948320999995e-01, 2.2147430875962000e+03 },
    { 1.1395000000000000e-07, 5.2626138478400000e+00, 1.4646394800628000e+03 },
    { 1.1204000000000000e-07, 4.7415156528800004e+00, 9.9160620955499994e+01 },
    { 1.1026000000000000e-07, 5.2838853529199996e+00, 2.0156710861598001e+03 },
    { 1.1017000000000000e-07, 3.1864679840500001e+00, 6.0447256366190004e+02 },
    { 1.0939000000000000e-07, 7.1222731143999995e-01, 1.2513403846248000e+03 },
    { 1.0858000000000000e-07, 2.2787781902200002e+00, 1.1338771495710000e+02 },
    { 1.0833000000000000e-07, 3.9263581065800000e+00, 1.1045700263900001e+01 },
    { 1.0812000000000000e-07, 5.7458804719999999e-02, 1.8912376709388000e+03 },
    { 1.0215000000000000e-07, 4.4018434838199996e+00, 4.5490936652729999e+02 },
    { 1.0134000000000001e-07, 6.8933796307999995e-01, 1.5922542890215000e+03 },
    { 1.0028000000000000e-07, 3.9476685717000000e+00, 4.9744763618019999e+02 },
    { 9.1720000000000003e-08, 3.5438010714299999e+00, 5.6193429400900004e+02 },
    { 8.9890000000000006e-08, 4.8993956558000002e+00, 1.1413847448250000e+02 },
    { 8.9570000000000001e-08, 4.4806186924800002e+00, 9.2047073954699997e+01 },
    { 8.5570000000000005e-08, 1.0060759547700000e+00, 2.2434479570190001e+02 },
    { 8.4470000000000001e-08, 9.6776660731999997e-01, 1.5851407420206999e+03 },
    { 8.0719999999999998e-08, 1.3194018173099999e+00, 1.5858915015461000e+03 },
    { 8.0179999999999994e-08, 5.6240753159999997e-02, 6.2960234557549995e+02 },
    { 7.7089999999999997e-08, 6.0145919438500002e+00, 6.3671589257630001e+02 },
    { 7.6879999999999998e-08, 7.0634789910999995e-01, 2.0014439921582000e+03 },
    { 7.6440000000000002e-08, 3.7862213056199998e+00, 1.2125202148330000e+02 },
    { 7.5899999999999998e-08, 3.7265999100100000e+00, 3.9356875915200000e+01 },
    { 7.5450000000000004e-08, 1.6023930732500000e+00, 4.0525754987360000e+02 },
    { 7.5440000000000006e-08, 7.6057765354999995e-01, 2.2289701815978001e+03 },
    { 7.5310000000000004e-08, 6.2654138017900003e+00, 1.5677322542813999e+03 },
    { 7.3570000000000003e-08, 2.0774222596000000e+00, 2.6555683724738001e+03 },
    { 7.3339999999999994e-08, 3.9086518202599998e+00, 5.3044172461999995e+02 },
    { 7.3339999999999994e-08, 3.5736264816999999e+00, 5.2894020556919997e+02 },
    { 7.2919999999999994e-08, 1.1270393894099999e+00, 4.2977958461370002e+02 },
    { 7.1050000000000004e-08, 3.5223550759300002e+00, 1.0483362299252999e+03 },
    { 7.0850000000000002e-08, 5.8590990131299998e+00, 1.1585425511446999e+03 },
    { 7.0220000000000003e-08, 4.9175054367700000e+00, 9.5310776223289997e+02 },
    { 6.9069999999999996e-08, 6.1059143646800003e+00, 3.0534616939270001e+02 },
    { 6.8219999999999998e-08, 8.8572352734000004e-01, 3.1245971639350000e+02 },
    { 6.5699999999999999e-08, 4.8075228346200003e+00, 6.3596513305090002e+02 },
    { 6.5099999999999994e-08, 4.5282796464499997e+00, 2.5382485042536000e+03 },
    { 6.4669999999999996e-08, 5.5489762704999999e+00, 1.1656560981455000e+03 },
    { 6.3800000000000002e-08, 5.0746579609199998e+00, 2.7515475996916002e+03 },
    { 6.3530000000000000e-08, 7.7690107418999999e-01, 1.1592933106701000e+03 },
    { 6.1749999999999994e-08, 2.6491276756000000e+00, 9.8311585891360005e+02 },
    { 5.9960000000000002e-08, 1.1777886772899999e+00, 3.1957326339430000e+02 },
    { 5.9419999999999999e-08, 1.7550606490500000e+00, 5.1228247735529999e+02 },
    { 5.8409999999999998e-08, 5.0541468251999999e-01, 1.1664068576709001e+03 },
    { 5.8360000000000001e-08, 2.5421278620800001e+00, 4.5884151979040001e+02 },
    { 5.7670000000000000e-08, 2.1303252250000002e-02, 3.8724131496080003e+02 },
    { 5.6710000000000003e-08, 4.9459469382799996e+00, 1.0412226829245001e+03 },
    { 5.6559999999999998e-08, 7.1402216830000004e-02, 7.0849445304200003e+01 },
    { 5.5899999999999998e-08, 3.4952070552999998e-01, 2.3463973644039999e+02 },
    { 5.5470000000000000e-08, 1.6262002386300001e+00, 6.1144309831080000e+02 },
    { 5.5439999999999999e-08, 2.8478012048800001e+00, 4.3764389113990001e+02 },
    { 5.5190000000000000e-08, 2.0702050652700001e+00, 1.5930050485469001e+03 },
    { 5.4980000000000000e-08, 1.4561682800000000e+00, 4.2266603761290003e+02 },
    { 5.4770000000000000e-08, 3.4433658690600000e+00, 2.3249494088155998e+03 },
    { 5.3720000000000001e-08, 5.8659254551700002e+00, 2.2076295405954002e+03 },
    { 5.3060000000000000e-08, 1.1969594807000000e-01, 1.0696768709277001e+03 },
    { 5.2980000000000002e-08, 2.6482086427599998e+00, 4.2341679713830001e+02 },
    { 5.2859999999999998e-08, 2.8918705940099998e+00, 5.0531194270639998e+02 },
    { 5.2479999999999997e-08, 5.1144304259900002e+00, 9.4599421523210003e+02 },
    { 5.2029999999999997e-08, 3.0884116137000001e-01, 4.9867147645760002e+02 },
    { 5.1889999999999997e-08, 5.6308216498799997e+00, 4.4793883187839998e+02 },
    { 4.9810000000000001e-08, 1.2543803239500000e+00, 2.9505122865419997e+02 },
    { 4.9719999999999998e-08, 4.5904074423500001e+00, 5.5406998748280000e+02 },
    { 4.8119999999999997e-08, 8.9025140400000002e-01, 5.6071045373159996e+02 },
    { 4.8070000000000000e-08, 2.6863943288999997e-01, 4.9426624244250002e+02 },
    { 4.7810000000000003e-08, 4.9427206562899997e+00, 6.0054041039879996e+02 },
    { 4.7020000000000000e-08, 4.6282004305399997e+00, 6.2173803904930003e+02 },
    { 4.6450000000000002e-08, 5.0792534539699998e+00, 1.7810313497193999e+03 },
    { 4.6119999999999999e-08, 3.9653929055099999e+00, 3.2243328914400003e+01 },
    { 4.6110000000000001e-08, 5.6608157523599996e+00, 6.7925416222920001e+02 },
    { 4.5669999999999998e-08, 1.1329133595000000e-01, 7.5721715453419995e+02 },
    { 4.4619999999999999e-08, 2.5128459873999998e-01, 8.3503713448730002e+02 },
    { 4.4309999999999998e-08, 1.7368957690700000e+00, 4.3053034413910001e+02 },
    { 4.4239999999999998e-08, 6.0135150764500001e+00, 2.1011770170029999e+02 },
    { 4.4059999999999999e-08, 2.3874666117499999e+00, 8.1752133216199994e+01 },
    { 4.3999999999999997e-08, 5.5377402091099999e+00, 6.2885158605009997e+02 },
    { 4.3970000000000003e-08, 9.3048360297999999e-01, 5.6511568774670002e+02 },
    { 4.3590000000000002e-08, 1.3318129421200000e+00, 6.7214061522839995e+02 },
    { 4.3049999999999998e-08, 1.6021316511200001e+00, 1.0704276304530999e+03 },
    { 4.2240000000000000e-08, 5.6569165961400003e+00, 5.4709945283390005e+02 },
    { 4.1600000000000002e-08, 4.5778233334000001e-01, 8.2474219374879999e+02 },
    { 4.1479999999999998e-08, 3.4385943692999998e-01, 1.0807225711916001e+03 },
    { 4.0919999999999998e-08, 2.7540506225000000e-01, 6.4382943957709995e+02 },
    { 4.0730000000000001e-08, 6.1546471489899996e+00, 7.3980866679489998e+02 },
    { 4.0089999999999997e-08, 3.6140691964900000e+00, 9.5235700270749999e+02 },
    { 3.9650000000000001e-08, 6.0521934101000001e+00, 5.0137978944330001e+02 },
    { 3.8980000000000002e-08, 4.3679650723399996e+00, 1.0640477966352000e+03 },
    { 3.8799999999999997e-08, 3.7247084593499999e+00, 5.0456118318099999e+02 },
    { 3.8610000000000000e-08, 1.6362815949000001e-01, 7.4692221379570003e+02 },
    { 3.8449999999999997e-08, 2.6878376200000002e-02, 9.7051624997219994e+02 },
    { 3.7030000000000001e-08, 1.6337335146400001e+00, 1.4930936680660000e+03 },
    { 3.6930000000000000e-08, 1.4300845293500000e+00, 5.5800214074589996e+02 },
    { 3.6500000000000003e-08, 4.6368574191500000e+00, 2.4209286360333999e+03 },
    { 3.6430000000000003e-08, 3.7575695465400001e+00, 5.5482074700819999e+02 },
    { 3.5010000000000000e-08, 2.3661266374199998e+00, 3.2032402291969998e+02 },
    { 3.4959999999999997e-08, 3.8058301656300002e+00, 1.0547160637432000e+03 },
    { 3.4860000000000002e-08, 9.0717666811999997e-01, 2.3178358618148000e+03 },
    { 3.3570000000000002e-08, 2.0275312571700002e+00, 6.2759623029905997e+03 },
    { 3.3400000000000001e-08, 8.9694401439000004e-01, 2.5311349572528002e+03 },
    { 3.3029999999999999e-08, 1.4272031353900001e+00, 5.0123677709139997e+02 },
    { 3.2509999999999998e-08, 1.4866676753200001e+00, 6.2901893969922003e+03 },
    { 3.2140000000000002e-08, 4.6399426656699996e+00, 4.9819839570559998e+02 },
    { 3.1480000000000001e-08, 5.1576619463100002e+00, 6.5412438031559998e+02 },
    { 3.1400000000000003e-08, 3.4500380476900001e+00, 1.0206171999210201e+04 },
    { 3.1160000000000002e-08, 1.3477072361200000e+00, 4.8322054217860000e+02 },
    { 3.0909999999999997e-08, 2.8283825059000001e-01, 2.3320629558164001e+03 },
    { 3.0759999999999999e-08, 3.3458021944400000e+00, 1.6779385755008000e+03 },
    { 3.0750000000000001e-08, 6.2750857088999998e+00, 4.6202291352809999e+02 },
    { 3.0400000000000001e-08, 2.9097012360200001e+00, 1.0220399093211799e+04 },
    { 3.0300000000000000e-08, 3.6477723482200002e+00, 1.9125783119411999e+03 },
    { 3.0230000000000000e-08, 3.6895180738400000e+00, 9.3812990870589999e+02 },
    { 2.9940000000000002e-08, 3.6547257410000000e+00, 2.3056457082540001e+02 },
    { 2.9919999999999999e-08, 5.9496009117000004e-01, 2.0300415469949999e+02 },
    { 2.9690000000000000e-08, 2.8423352050699999e+00, 5.6118353448360006e+02 },
    { 2.9679999999999998e-08, 9.5733368724000001e-01, 5.9735901666109999e+02 },
    { 2.9239999999999999e-08, 6.9505079748999998e-01, 4.1555249061209997e+02 },
    { 2.8470000000000000e-08, 1.3511028728100001e+00, 1.1767017984094000e+03 },
    { 2.8299999999999999e-08, 7.7469965362000004e-01, 2.6413412784722000e+03 },
    { 2.8040000000000002e-08, 6.1656637160100001e+00, 5.7616138801060004e+02 },
    { 2.7829999999999998e-08, 5.2606272443800002e+00, 2.0936694217490000e+02 },
    { 2.7479999999999999e-08, 5.4472310530100003e+00, 2.5240214102519999e+03 },
    { 2.7409999999999999e-08, 5.8837930668800000e+00, 1.1514290041439001e+03 },
    { 2.7199999999999999e-08, 1.0093491189799999e+00, 3.2668681039510000e+02 },
    { 2.6720000000000000e-08, 3.9245063721000001e-01, 5.2182665856840003e+02 },
    { 2.6720000000000000e-08, 8.0664235757000002e-01, 5.3755527162080000e+02 },
    { 2.6580000000000000e-08, 1.1445322419699999e+00, 1.0618296107440001e+03 },
    { 2.6580000000000000e-08, 5.3569301064300001e+00, 1.0569342496344000e+03 },
    { 2.5990000000000000e-08, 6.2136685550999999e+00, 9.6022130923370003e+02 },
    { 2.5450000000000000e-08, 4.7764604459899997e+00, 1.0987388061044001e+03 },
    { 2.3879999999999999e-08, 2.6575005637900002e+00, 3.3455111692129998e+02 },
    { 2.3730000000000001e-08, 4.0777899607899997e+00, 5.2347118997109999e+02 },
    { 2.3730000000000001e-08, 3.4044883411700000e+00, 5.3591074021810005e+02 },
    { 2.3689999999999999e-08, 6.1895642844200003e+00, 7.3269511979410004e+02 },
    { 2.3460000000000000e-08, 3.3194656310299999e+00, 9.4524345570670005e+02 },
    { 2.3460000000000000e-08, 4.6187942661900001e+00, 7.3905790726949999e+02 },
    { 2.3390000000000000e-08, 6.6339232699999997e-03, 4.6053844081979997e+02 },
    { 2.3300000000000000e-08, 2.6119316812700002e+00, 8.4926422848890002e+02 },
    { 2.2869999999999999e-08, 4.8007529410599998e+00, 2.0974232193759999e+03 },
    { 2.2840000000000001e-08, 1.1174065905799999e+00, 5.2804643369190001e+02 },
    { 2.2840000000000001e-08, 8.1686404200000007e-02, 5.3133549649730003e+02 },
    { 2.2560000000000002e-08, 1.4098067295600001e+00, 1.0539653042177999e+03 },
    { 2.2259999999999999e-08, 2.9140866605999999e-01, 5.2308074668030004e+03 },
    { 2.1889999999999999e-08, 3.1141819966500002e+00, 1.5142912967165000e+03 },
    { 2.1109999999999998e-08, 8.0586078543999995e-01, 5.2165803728013998e+03 },
    { 2.1060000000000001e-08, 6.1878364717899998e+00, 1.8841241239379999e+03 },
    { 2.1050000000000000e-08, 1.6518256823899999e+00, 1.0593337460794000e+03 },
    { 2.1050000000000000e-08, 4.8496368100199998e+00, 1.0594301142990000e+03 },
    { 2.0860000000000000e-08, 1.7415855489300001e+00, 1.0623026914960001e+03 },
    { 2.0840000000000000e-08, 3.7949626849700002e+00, 4.9033408917940000e+02 },
    { 2.0689999999999999e-08, 1.7150939644100001e+00, 9.1610171630225996e+03 },
    { 2.0500000000000002e-08, 5.3123226160600003e+00, 1.9513984817330001e+02 },
    { 2.0470000000000001e-08, 1.1314097714700000e+00, 9.8853248488500003e+02 },
    { 2.0330000000000001e-08, 4.6619876707000002e-01, 1.1735204046716999e+03 },
    { 2.0199999999999999e-08, 3.2752755746400002e+00, 5.6782400073240001e+02 },
    { 2.0149999999999998e-08, 1.8449749611199999e+00, 4.6470422915999997e+01 },
    { 2.0050000000000001e-08, 3.9118341008300002e+00, 7.3194436026870005e+02 },
    { 2.0020000000000000e-08, 5.9119404080600004e+00, 1.4244965013379999e+02 },
    { 1.9969999999999999e-08, 1.5735650920000000e-02, 5.0441817082910001e+02 },
    { 1.9659999999999999e-08, 6.0671867794400001e+00, 4.1630325013750002e+02 },
    { 1.9650000000000001e-08, 2.2283853948200001e+00, 9.1467900690210008e+03 },
    { 1.9390000000000000e-08, 6.3756905810999998e-01, 1.0647985561605999e+03 },
    { 1.9049999999999999e-08, 1.9407611888000000e+00, 1.0775411774539000e+03 },
    { 1.8920000000000000e-08, 3.6892534777399999e+00, 5.6904784100979998e+02 },
    { 1.8790000000000001e-08, 1.7002408968500000e+00, 5.2820649238630006e+02 },
    { 1.8790000000000001e-08, 5.7820374051099996e+00, 5.3117543780289998e+02 },
    { 1.8740000000000001e-08, 1.1833570582099999e+00, 5.5496375936009997e+02 },
    { 1.8270000000000001e-08, 1.1095545376400000e+00, 1.3647280995819001e+03 },
    { 1.8089999999999999e-08, 3.9974441027899998e+00, 4.9155792945680003e+02 },
    { 1.7780000000000001e-08, 2.9545761198400000e+00, 8.4215068148809996e+02 },
    { 1.7730000000000001e-08, 1.3035166989700000e+00, 6.7668051566499997e+01 },
    { 1.7059999999999999e-08, 3.7712973272700001e+00, 1.2797945726280000e+03 },
    { 1.6930000000000000e-08, 4.8745121400700002e+00, 7.2483081326790000e+02 },
    { 1.6840000000000001e-08, 9.6891211771999997e-01, 5.9884348936940000e+02 },
    { 1.6840000000000001e-08, 4.4225272018000004e+00, 2.7373205056900001e+03 },
    { 1.6660000000000002e-08, 4.5158211932499999e+00, 1.0584187223427000e+03 },
    { 1.6660000000000002e-08, 1.9856412347500001e+00, 1.0603451380357001e+03 },
    { 1.6460000000000000e-08, 3.3196775857800001e+00, 2.9912639426919998e+02 },
    { 1.5600000000000001e-08, 2.5982989008600001e+00, 1.0064509366380000e+02 },
    { 1.5550000000000000e-08, 4.5953215872800000e+00, 2.0225339517410001e+02 },
    { 1.5539999999999999e-08, 1.6563833065300000e+00, 2.4351557300350000e+03 },
    { 1.5460000000000001e-08, 3.6858197862600002e+00, 3.2743756992049998e+02 },
    { 1.5200000000000000e-08, 5.3775431717500002e+00, 9.7748678462110001e+02 },
    { 1.5180000000000001e-08, 1.6123388830700001e+00, 1.4395096981491999e+03 },
    { 1.5080000000000000e-08, 5.7548049058999995e-01, 2.6342277314714001e+03 },
    { 1.4979999999999999e-08, 3.8321285827199998e+00, 1.1341635287565000e+03 },
    { 1.4929999999999999e-08, 5.3565233021400003e+00, 3.0679394693482000e+03 },
    { 1.4540000000000000e-08, 4.4016049925000003e+00, 2.8546403739102002e+03 },
    { 1.4370000000000001e-08, 1.5153485896999999e-01, 1.0564611688824000e+03 },
    { 1.4349999999999999e-08, 4.9795875561300003e+00, 9.9171387862270001e+02 },
    { 1.4210000000000000e-08, 5.3391622125099998e+00, 2.8404132799086001e+03 },
    { 1.4100000000000001e-08, 4.9705492644199998e+00, 1.0916252591036000e+03 },
    { 1.4079999999999999e-08, 1.4575421147600001e+00, 1.2694996318895001e+03 },
    { 1.3650000000000000e-08, 3.1497490336600000e+00, 1.4466232451500000e+03 },
    { 1.3430000000000000e-08, 1.9386951997099999e+00, 9.1693228005540004e+02 },
    { 1.3430000000000000e-08, 1.9246731697799999e+00, 1.3544331588434000e+03 },
    { 1.3389999999999999e-08, 9.2704363552000002e-01, 2.8475268269093999e+03 },
    { 1.3230000000000000e-08, 5.1654457310200002e+00, 2.8414854074220000e+02 },
    { 1.3080000000000000e-08, 5.8770473050499996e+00, 5.5814515309779995e+02 },
    { 1.2860000000000001e-08, 1.8445004737999999e+00, 3.8012776796000003e+02 },
    { 1.2730000000000001e-08, 4.6652914010800002e+00, 8.4290144101349995e+02 },
    { 1.2680000000000000e-08, 5.5967666796000000e+00, 1.4749344208012999e+03 },
    { 1.2520000000000001e-08, 5.7724825392100003e+00, 4.7306861379200001e+02 },
    { 1.2429999999999999e-08, 4.2079805121699998e+00, 2.1723124870109999e+02 },
    { 1.2420000000000000e-08, 1.3915535340800000e+00, 5.2012973753899996e+02 },
    { 1.2420000000000000e-08, 2.9491321142900002e+00, 5.3925219265019996e+02 },
    { 1.2350000000000000e-08, 6.6052761200999999e-01, 1.4756851803267000e+03 },
    { 1.2220000000000000e-08, 3.2927194769799999e+00, 1.9943304451573999e+03 },
    { 1.2200000000000000e-08, 2.4748198939699999e+00, 3.7301422095919997e+02 },
    { 1.1780000000000000e-08, 2.1437206256699999e+00, 2.4280421830342002e+03 },
    { 1.1650000000000000e-08, 1.6635938529500001e+00, 2.7586611466924001e+03 },
    { 1.1450000000000000e-08, 4.4780744060900002e+00, 4.0843894361129998e+02 },
    { 1.1440000000000000e-08, 5.2913674630000003e+00, 1.4820479678020999e+03 },
    { 1.1320000000000000e-08, 3.6242524013400002e+00, 1.0635747158832000e+03 },
    { 1.1290000000000001e-08, 5.8707975117400002e+00, 2.2005159935945999e+03 },
    { 1.1210000000000001e-08, 2.2440240967600000e+00, 4.1643119896130002e+03 },
    { 1.0789999999999999e-08, 2.6102705757000000e-01, 1.4827987273275000e+03 },
    { 1.0560000000000000e-08, 8.1526253414000005e-01, 2.5453620512543998e+03 },
    { 1.0470000000000000e-08, 1.7634559312100000e+00, 5.8631331639719997e+02 },
    { 1.0300000000000000e-08, 3.7851593535100001e+00, 8.5001498801430000e+02 },
    { 1.0290000000000000e-08, 1.5676320857299999e+00, 3.1321047591889999e+02 },
    { 1.0169999999999999e-08, 3.6739379100699998e+00, 8.0945216858326003e+03 },
    { 1.0150000000000000e-08, 5.2305994911599996e+00, 4.5097721326419997e+02 },
    { 1.0070000000000000e-08, 3.6358645295700001e+00, 1.4323961511483999e+03 },
    { 9.9400000000000000e-09, 1.3090104046400000e+00, 1.9692006632437999e+03 },
    { 9.8400000000000008e-09, 3.7090239019999999e-01, 5.3685269620520000e+02 },
    { 9.8400000000000008e-09, 3.9697832581700001e+00, 5.2252923398400003e+02 },
    { 9.8400000000000008e-09, 8.8440907862999996e-01, 5.2262560220360001e+02 },
    { 9.8400000000000008e-09, 3.4562765697400000e+00, 5.3675632798560002e+02 },
    { 9.5800000000000004e-09, 1.2797055525800001e+00, 1.5578297225809999e+02 },
    { 9.4899999999999993e-09, 6.2691210219200002e+00, 2.9823262239190001e+02 },
    { 9.4400000000000005e-09, 9.5774392272999997e-01, 3.1639186965660001e+03 },
    { 9.0099999999999993e-09, 1.3832963815999999e+00, 2.4872381809010000e+02 },
    { 8.9999999999999995e-09, 8.3061588530999997e-01, 3.4803105662225998e+03 },
    { 8.9100000000000000e-09, 4.9637962250000001e-02, 4.8640193591629998e+02 },
    { 8.8100000000000008e-09, 5.3111321052799996e+00, 3.3843313390048002e+03 },
    { 8.7299999999999994e-09, 4.7476817399200000e+00, 2.4138150890326001e+03 },
    { 8.6599999999999995e-09, 4.2961185695899999e+00, 1.1411340634054000e+03 },
    { 8.6499999999999997e-09, 5.0216013525000003e-01, 2.9506196011279999e+03 },
    { 8.5600000000000002e-09, 5.4906217540200002e+00, 3.2812385647862002e+03 },
    { 8.5000000000000000e-09, 2.3302167033800001e+00, 7.7523338944700004e+02 },
    { 8.4800000000000005e-09, 4.8195442107200002e+00, 1.2899465010146000e+03 },
    { 8.3199999999999994e-09, 2.2516760138200000e+00, 6.0840471692500000e+02 },
    { 8.2900000000000001e-09, 3.0413655476199999e+00, 2.9577331481288002e+03 },
    { 8.2499999999999994e-09, 1.8202759258300001e+00, 8.6170991039264009e+03 },
    { 8.2200000000000002e-09, 2.2136434329600001e+00, 4.5396924964117998e+03 },
    { 8.2100000000000004e-09, 3.2138854283699998e+00, 9.0270518605380005e+02 },
    { 8.0200000000000000e-09, 2.3569569286299998e+00, 9.8460033162189995e+02 },
    { 8.0100000000000003e-09, 3.0556771987600002e+00, 7.7204327494533383e+04 },
    { 7.9900000000000007e-09, 1.1494542007599999e+00, 5.7297999427289994e+02 },
    { 7.8500000000000008e-09, 3.7019465587800000e+00, 3.8038159828799999e+03 },
    { 7.8399999999999994e-09, 5.3211334612099996e+00, 1.7913262904579001e+03 },
    { 7.8100000000000001e-09, 9.7332239037000001e-01, 1.5787779545453000e+03 },
    { 7.8000000000000004e-09, 5.8285673127199997e+00, 9.1539036160218002e+03 },
    { 7.8000000000000004e-09, 5.3898640368899997e+00, 6.0595703637019994e+02 },
    { 7.7900000000000006e-09, 5.5053192876000001e-01, 5.2161421024729998e+02 },
    { 7.7900000000000006e-09, 3.7901537196100001e+00, 5.3776771994190005e+02 },
    { 7.7900000000000006e-09, 3.7025240330000000e-02, 5.3584130424889997e+02 },
    { 7.7900000000000006e-09, 4.3036604080399998e+00, 5.2354062594029995e+02 },
    { 7.6700000000000002e-09, 6.5207644700000000e-01, 9.2783496796739996e+02 },
    { 7.6600000000000004e-09, 7.1376571141000000e-01, 3.8109295298808001e+03 },
    { 7.6399999999999993e-09, 3.9766128691000002e-01, 1.7920770499833000e+03 },
    { 7.5699999999999993e-09, 1.3180760870499999e+00, 1.0554045477340000e+02 },
    { 7.5499999999999998e-09, 2.1039312597900000e+00, 8.6313261979279996e+03 },
    { 7.5100000000000007e-09, 1.0217320323300001e+00, 1.6708250284999999e+03 },
    { 7.4799999999999998e-09, 9.3805184360000005e-01, 2.8617539209110000e+03 },
    { 7.4799999999999998e-09, 1.9453078630300000e+00, 1.2623860848887000e+03 },
    { 7.4400000000000000e-09, 6.0479277393500004e+00, 6.4307868005169996e+02 },
    { 7.4300000000000002e-09, 3.4983481841000001e+00, 1.0599031950510000e+03 },
    { 7.4300000000000002e-09, 3.0797130906099999e+00, 3.2741250177853999e+03 },
    { 7.3900000000000003e-09, 4.2504738519199998e+00, 9.6733485623449997e+02 },
    { 7.3700000000000000e-09, 5.6985099408700002e+00, 2.7703499374140000e+02 },
    { 7.2900000000000003e-09, 4.8074816178199997e+00, 4.5468060434126000e+03 },
    { 7.2600000000000002e-09, 4.1307730199800003e+00, 3.0537123753465999e+03 },
    { 7.2500000000000004e-09, 5.4306335773800001e+00, 7.4781598567299994e+01 },
    { 7.2000000000000000e-09, 1.4896702124700001e+00, 3.0750530163489998e+03 },
    { 7.1500000000000003e-09, 1.2876533624700000e+00, 1.6889842757647000e+03 },
    { 7.0399999999999997e-09, 5.1330228639500000e+00, 5.1653626887241999e+03 },
    { 7.0299999999999999e-09, 4.7533375237900000e+00, 2.2883440435113998e+03 },
    { 7.0299999999999999e-09, 5.2539483702699998e+00, 2.2741169495098002e+03 },
    { 7.0200000000000002e-09, 4.4785179665200001e+00, 5.2236939198022001e+03 },
    { 7.0100000000000004e-09, 2.8827738392100000e+00, 2.4554242435239999e+02 },
    { 7.0100000000000004e-09, 2.3569220397100001e+00, 1.8196374661092000e+03 },
    { 6.9999999999999998e-09, 2.9565661651099999e+00, 4.8715269544170002e+02 },
    { 6.9999999999999998e-09, 4.5257121368500002e+00, 5.7222923474749996e+02 },
    { 6.9999999999999998e-09, 9.0825418370000000e-01, 1.6682867252200001e+02 },
    { 6.8800000000000002e-09, 4.5344368566100002e+00, 5.0622699145056004e+03 },
    { 6.8400000000000004e-09, 5.0202996900699999e+00, 1.7984398374586999e+03 },
    { 6.7899999999999999e-09, 5.2502221423500002e+00, 2.4498830246290399e+04 },
    { 6.7599999999999998e-09, 2.4079371684300002e+00, 2.3107223148140001e+03 },
    { 6.7400000000000003e-09, 3.8815316459300000e+00, 4.6498988176311996e+03 },
    { 6.6999999999999996e-09, 2.2082995423499998e+00, 3.9814400287280000e+02 },
    { 6.6700000000000003e-09, 5.5882060353299998e+00, 1.0129115072732000e+03 },
    { 6.6599999999999997e-09, 6.0239443025900004e+00, 4.5254654024102001e+03 },
    { 6.6500000000000000e-09, 1.6210395035000000e+00, 3.0609692891809999e+02 },
    { 6.6000000000000004e-09, 5.1150128198999996e+00, 3.1568051495651998e+03 },
    { 6.5100000000000001e-09, 1.9384035554300001e+00, 4.8560843660684004e+03 },
    { 6.5000000000000003e-09, 4.9782549143399999e+00, 4.2304141737560003e+03 },
    { 6.4899999999999997e-09, 6.2756560335499998e+00, 1.7991905969841000e+03 },
    { 6.3400000000000000e-09, 6.0973407511400000e+00, 4.6988722005429997e+02 },
    { 6.3000000000000002e-09, 4.0379593006599999e+00, 4.3335069479745998e+03 },
    { 6.3000000000000002e-09, 3.3071808815499999e+00, 5.0693834615063997e+03 },
    { 6.2900000000000004e-09, 3.9520094572600000e+00, 3.4874241132234001e+03 },
    { 6.2799999999999998e-09, 1.9896395150999999e-01, 1.5606187072805999e+03 },
    { 6.2600000000000003e-09, 6.3607861788999998e-01, 3.7967024358792000e+03 },
    { 6.2400000000000000e-09, 2.9532648234900001e+00, 2.5035634758385801e+04 },
    { 6.1600000000000002e-09, 3.4538636343400002e+00, 2.5021407664384202e+04 },
    { 6.1099999999999998e-09, 5.2298500889500001e+00, 5.7462713378959997e+03 },
    { 6.0699999999999999e-09, 1.3524017114500000e+00, 2.1329909543799999e+03 },
    { 6.0600000000000002e-09, 7.7317400847999995e-01, 1.7515395314160000e+03 },
    { 6.0000000000000000e-09, 5.5697930505700004e+00, 2.5169078632512001e+03 },
    { 5.9600000000000001e-09, 2.6791042005299999e+00, 3.5905168874420001e+03 },
    { 5.9300000000000000e-09, 5.0633822149699999e+00, 4.6427852706304002e+03 },
    { 5.8600000000000000e-09, 2.0386209855600002e+00, 4.5342489381899998e+02 },
    { 5.7800000000000003e-09, 9.0023029993000003e-01, 3.4945376602242000e+03 },
    { 5.7299999999999999e-09, 5.9753779832899996e+00, 3.9435486196160002e+02 },
    { 5.7299999999999999e-09, 1.0515167133400001e+00, 1.3576145525811000e+03 },
    { 5.6900000000000000e-09, 4.6448557101199999e+00, 4.0171150783180001e+03 },
    { 5.6699999999999997e-09, 5.3323491206900000e+00, 1.8873055176757000e+03 },
    { 5.6400000000000004e-09, 2.5197509588000000e+00, 1.1270499817556999e+03 },
    { 5.5899999999999999e-09, 9.2746805833000001e-01, 4.4475743814070000e+02 },
    { 5.5700000000000004e-09, 5.9741893993000001e+00, 2.9648466951296000e+03 },
    { 5.4700000000000003e-09, 4.3067657913000001e-01, 4.1273213995373999e+03 },
    { 5.4599999999999998e-09, 1.3849359941499999e+00, 5.8949471013489995e+02 },
    { 5.4500000000000000e-09, 6.0968646122000001e+00, 8.1065811209909998e+02 },
    { 5.4299999999999997e-09, 2.1383484979999999e-02, 4.7387644978482003e+03 },
    { 5.4299999999999997e-09, 3.1270066010700002e+00, 2.1648048917570000e+02 },
    { 5.4100000000000001e-09, 5.1323434650099999e+00, 3.7007232086613999e+03 },
    { 5.4100000000000001e-09, 2.8060434581999999e-01, 1.2687488723641000e+03 },
    { 5.3700000000000003e-09, 1.8518452832800001e+00, 7.8234693644779998e+02 },
    { 5.3400000000000002e-09, 6.1699749257700001e+00, 4.3689313161450002e+02 },
    { 5.2800000000000000e-09, 2.6904364137500001e+00, 3.2527843767830000e+03 },
    { 5.2599999999999996e-09, 4.3709948682400004e+00, 3.1710322435667999e+03 },
    { 5.2400000000000001e-09, 2.4745179743399999e+00, 2.7444340526907999e+03 },
    { 5.2300000000000003e-09, 2.1398375110400001e+00, 3.0608259223474001e+03 },
    { 5.1499999999999998e-09, 5.0329382671599996e+00, 3.4660834722210002e+03 },
    { 5.1400000000000000e-09, 3.7816890042600000e+00, 4.7529915918498000e+03 },
    { 5.0899999999999996e-09, 3.5840961831000001e-01, 4.6868894077067998e+03 },
    { 5.0499999999999997e-09, 2.1000748847000000e-01, 7.5403576079649997e+02 },
    { 4.9700000000000000e-09, 5.5859982447599998e+00, 6.9152524274800001e+01 },
    { 4.9300000000000001e-09, 3.8616453854900001e+00, 1.0021837280099400e+04 },
    { 4.8699999999999999e-09, 4.9898966622799996e+00, 1.9015326116773001e+03 },
    { 4.8300000000000001e-09, 4.9771677857799999e+00, 8.2792358748649997e+02 },
    { 4.7600000000000001e-09, 4.8373339289499997e+00, 4.4779581952650000e+02 },
    { 4.7500000000000003e-09, 5.7291175802299996e+00, 1.4678208738005001e+03 },
    { 4.6600000000000000e-09, 9.9311853960000002e-02, 1.4899122743283001e+03 },
    { 4.6600000000000000e-09, 5.8993418350600004e+00, 3.2670114707846001e+03 },
    { 4.6200000000000002e-09, 5.4813153517500002e+00, 2.6617594106668799e+04 },
    { 4.5200000000000001e-09, 3.2465584435200001e+00, 1.5509398596460001e+03 },
    { 4.5100000000000003e-09, 4.9836262784400001e+00, 2.1077181601144998e+03 },
    { 4.4200000000000000e-09, 3.4262620875400001e+00, 4.4365997221932003e+03 },
    { 4.3800000000000002e-09, 5.8224723450600004e+00, 6.1158611066269998e+02 },
    { 4.3299999999999997e-09, 7.2835384240000001e-01, 4.7011165017084004e+03 },
    { 4.3100000000000002e-09, 1.5413742649000001e-01, 2.1084689196398999e+03 },
    { 4.2899999999999999e-09, 4.7060667092799999e+00, 4.0028879843164000e+03 },
    { 4.2400000000000002e-09, 1.7482952689000000e-01, 1.2545217783625001e+03 },
    { 4.2299999999999997e-09, 6.0977083444100000e+00, 4.1529185818119998e+02 },
    { 4.1700000000000003e-09, 1.3837774274100001e+00, 6.6502706822760001e+02 },
    { 4.1700000000000003e-09, 7.2571440164000001e-01, 3.7824753418775999e+03 },
    { 4.1199999999999998e-09, 2.5902654057199999e+00, 3.9069087570985998e+03 },
    { 4.1199999999999998e-09, 9.0735881784000005e-01, 1.9022833712027000e+03 },
    { 4.1199999999999998e-09, 4.6104280454100000e+00, 4.8631979130691998e+03 },
    { 4.0899999999999997e-09, 4.0377028387899996e+00, 1.2036460734888200e+04 },
    { 4.0700000000000002e-09, 4.9985205890000001e-01, 4.1130943055358002e+03 },
    { 4.0400000000000001e-09, 1.6766011907300000e+00, 2.0424977891028000e+03 },
    { 3.9899999999999997e-09, 5.2107805261699998e+00, 1.4575259330619999e+03 },
    { 3.9799999999999999e-09, 2.9897350766300002e+00, 3.5574874557179999e+02 },
    { 3.9600000000000004e-09, 3.7618205744700002e+00, 4.9662906872878002e+03 },
    { 3.9600000000000004e-09, 2.2967191672500000e+00, 1.0200250542740000e+03 },
    { 3.7900000000000004e-09, 1.5741978894000000e-01, 4.4437132691939996e+03 },
    { 3.7600000000000003e-09, 4.8003200118000000e-01, 6.1462449204849997e+02 },
    { 3.6899999999999999e-09, 3.5288875951500001e+00, 4.1202078525365996e+03 },
};

static const VSOP87Term VSOP87A_JUPITER_Y2[259] = {
    { 1.2403250900000001e-03, 2.5649557683299999e+00, 5.2257741809380002e+02 },
    { 1.2145599100000001e-03, 4.9239876638000002e+00, 5.3680451209540001e+02 },
    { 7.6943042000000004e-04, 3.1415926535900001e+00, 0.0000000000000000e+00 },
    { 7.6523263000000000e-04, 3.7591337179300002e+00, 5.2969096509459996e+02 },
    { 4.1357600000000003e-04, 3.5522844045699999e+00, 1.0593819301891999e+03 },
    { 1.1277667000000000e-04, 1.8559902388999999e-01, 7.1135470007999997e+00 },
    { 7.0511030000000005e-05, 5.4540436857000003e+00, 1.0664954771900000e+03 },
    { 5.7194400000000002e-05, 2.0597000023000001e+00, 1.0522683831884001e+03 },
    { 5.2861570000000000e-05, 2.6949046506399998e+00, 5.1546387109299997e+02 },
    { 4.0390379999999997e-05, 4.0134103463699997e+00, 1.5890728952838001e+03 },
    { 3.7045280000000001e-05, 4.7902929227100000e+00, 5.4391805909619995e+02 },
    { 1.3206270000000001e-05, 5.7860484169999997e-02, 1.4227094001599999e+01 },
    { 1.2802829999999999e-05, 1.4757400686100000e+00, 1.1020632121940000e+02 },
    { 1.0597830000000001e-05, 8.9610748175999999e-01, 4.2659819087599999e+02 },
    { 9.9372799999999992e-06, 6.1861398022599996e+00, 3.2350541665740002e+02 },
    { 8.8814400000000005e-06, 3.0967519562099999e+00, 5.2650957135689998e+02 },
    { 8.6454399999999997e-06, 4.3853758879500004e+00, 5.3287235883230005e+02 },
    { 8.2022299999999996e-06, 3.7911850134000002e-01, 6.3278373931320004e+02 },
    { 6.2411500000000004e-06, 1.2941427265500001e+00, 7.3587651353180001e+02 },
    { 5.7300100000000000e-06, 1.0744491970000000e-01, 9.4917560896980001e+02 },
    { 5.7147999999999999e-06, 1.5785512686400001e+00, 1.0451548361876000e+03 },
    { 5.1386299999999998e-06, 4.4000069822499999e+00, 5.2575881183150000e+02 },
    { 5.1192699999999999e-06, 3.0849493596199999e+00, 5.3362311835770004e+02 },
    { 4.5831399999999996e-06, 5.9571267160600003e+00, 1.5961864422845999e+03 },
    { 4.1765099999999996e-06, 1.0490992255500000e+00, 1.1731986822019999e+02 },
    { 3.7278900000000001e-06, 8.2429067684000001e-01, 9.5979227217800002e+01 },
    { 3.5761200000000001e-06, 2.5781767919799998e+00, 4.1948464387519999e+02 },
    { 3.3259900000000002e-06, 4.4306468687500002e+00, 2.1187638603783998e+03 },
    { 2.8327999999999998e-06, 2.1360707084800001e+00, 1.5819593482830001e+03 },
    { 2.6341100000000002e-06, 1.6757790507899999e+00, 6.2567019231239999e+02 },
    { 2.6188600000000002e-06, 2.1138456131700001e+00, 9.5628915597059995e+02 },
    { 2.6183800000000001e-06, 1.5765892549900000e+00, 2.0618554843720000e+02 },
    { 2.2403800000000001e-06, 5.6015621894400001e+00, 4.3371173787679999e+02 },
    { 2.1511800000000001e-06, 2.0709824197000001e-01, 9.4206206196899996e+02 },
    { 2.0075499999999999e-06, 5.2222083153599996e+00, 2.2041264243879999e+02 },
    { 1.9580900000000001e-06, 4.4384807338699996e+00, 1.1695882514085999e+03 },
    { 1.8982699999999999e-06, 4.4561578173600003e+00, 6.3989728631399998e+02 },
    { 1.7027199999999999e-06, 6.0373120924299997e+00, 3.3061896365820002e+02 },
    { 1.6972100000000001e-06, 1.9595618686100000e+00, 1.1553611574070001e+03 },
    { 1.6941000000000000e-06, 3.7577394475500001e+00, 3.1639186965660002e+02 },
    { 1.6411799999999999e-06, 5.2363771648800004e+00, 1.0736090241908000e+03 },
    { 1.3991500000000000e-06, 2.8475747247599998e+00, 5.0835032409220003e+02 },
    { 1.3650200000000000e-06, 2.9667732288100002e+00, 3.0927832265580003e+02 },
    { 1.3590700000000001e-06, 3.5756235910199998e+00, 7.4299006053259995e+02 },
    { 1.3490699999999999e-06, 1.3100267739000000e+00, 7.2876296653099996e+02 },
    { 1.3357700000000000e-06, 1.8143294876000000e+00, 1.9907200143640000e+02 },
    { 1.1255900000000000e-06, 3.0781003131400002e+00, 1.1624747044077999e+03 },
    { 1.0663700000000001e-06, 1.1026920561700000e+00, 8.5319638175199998e+02 },
    { 9.7173999999999999e-07, 4.8354067006900001e+00, 1.0309277421860000e+02 },
    { 8.4709000000000005e-07, 3.2695475550699999e+00, 5.1939602435610004e+02 },
    { 7.7754000000000001e-07, 2.0702414304599999e+00, 8.3896928775039999e+02 },
    { 7.7441999999999997e-07, 2.6232167480699999e+00, 1.2655674786264001e+03 },
    { 7.4682999999999997e-07, 4.6212840503699999e+00, 5.5103160609700001e+02 },
    { 6.7576000000000001e-07, 4.1961051517800003e+00, 5.3998590583309999e+02 },
    { 5.5428000000000001e-07, 6.2119212119900000e+00, 2.1340641002400002e+01 },
    { 5.3410999999999996e-07, 4.0890947985199997e+00, 2.1329909543799999e+02 },
    { 5.3076000000000002e-07, 1.9151759286300001e+00, 4.1237109687440000e+02 },
    { 5.1093000000000004e-07, 5.9142603455699998e+00, 3.1813937377000001e+00 },
    { 4.8874000000000002e-07, 4.8889790068899996e+00, 1.0625633239269000e+03 },
    { 4.6634999999999999e-07, 5.0706621496700004e+00, 2.2752618943959999e+02 },
    { 4.3276999999999998e-07, 1.7570265201699999e+00, 1.5748458012822000e+03 },
    { 4.3067999999999999e-07, 2.7348565748200002e+00, 1.2584539316256000e+03 },
    { 4.1421000000000000e-07, 2.8726944611300000e+00, 6.1855664531160005e+02 },
    { 3.8050000000000000e-07, 4.2532545945700004e+00, 4.4082528487759998e+02 },
    { 3.7468999999999998e-07, 3.0852205191199999e+00, 1.6921656695024001e+03 },
    { 3.6591000000000003e-07, 2.8359296660700002e+00, 1.0562005364515001e+03 },
    { 3.6213999999999999e-07, 2.8887865993699999e+00, 3.0216477565500003e+02 },
    { 3.2931999999999999e-07, 4.9553426283200004e+00, 1.2726810256271999e+03 },
    { 3.1824999999999998e-07, 1.5097091165000001e-01, 2.1258774073792001e+03 },
    { 3.1557000000000001e-07, 2.0214566789899999e+00, 8.3185574074960005e+02 },
    { 2.9358000000000001e-07, 3.2309576002600000e+00, 1.4859801210651999e+03 },
    { 2.8785999999999998e-07, 5.0432132243899996e+00, 1.0554497769261000e+03 },
    { 2.8755000000000000e-07, 1.5456777556000001e+00, 1.0380412891868000e+03 },
    { 2.6179000000000002e-07, 2.4148768969000001e-01, 8.4608283475120004e+02 },
    { 2.5839000000000003e-07, 4.8155551374899996e+00, 2.6484548254729998e+03 },
    { 2.2677000000000001e-07, 3.7754194762200002e+00, 1.3686602528450001e+03 },
    { 2.1631000000000001e-07, 8.9648612124000004e-01, 1.4717530270636000e+03 },
    { 2.1128000000000000e-07, 4.9677525725600002e+00, 1.6992792165031999e+03 },
    { 2.0494000000000001e-07, 4.5306369101900001e+00, 6.4701083331480004e+02 },
    { 1.9784999999999999e-07, 3.3160784154599998e+00, 1.0633140834523001e+03 },
    { 1.6107000000000000e-07, 5.4459154986000002e-01, 1.4788665740644001e+03 },
    { 1.4560999999999999e-07, 2.2775960157299999e+00, 2.1116503133776000e+03 },
    { 1.4289999999999999e-07, 8.5041670689000004e-01, 1.2443341522100000e+02 },
    { 1.3890000000000001e-07, 3.8659798494999997e-01, 1.3757737998458001e+03 },
    { 1.3225999999999999e-07, 4.5331734490100004e+00, 7.5010360753340001e+02 },
    { 1.2664000000000000e-07, 5.7800463950500003e+00, 9.6340270297140000e+02 },
    { 1.2226000000000001e-07, 2.2524385363500001e+00, 1.6850521225016000e+03 },
    { 1.2066000000000000e-07, 5.7347299274500001e+00, 1.0294940738499999e+01 },
    { 1.1227000000000000e-07, 1.5968333899900000e+00, 5.1153171782990000e+02 },
    { 1.0726000000000000e-07, 5.8854447637699998e+00, 5.4785021235930003e+02 },
    { 9.9050000000000004e-08, 7.7307644712000001e-01, 9.3494851496820002e+02 },
    { 8.8920000000000005e-08, 1.7600365210700000e+00, 1.3828873468465999e+03 },
    { 8.0039999999999994e-08, 5.8858253768199997e+00, 3.3773251065900001e+02 },
    { 7.7480000000000003e-08, 1.9561484754800000e+00, 1.9195845443560000e+02 },
    { 7.5030000000000004e-08, 5.6718799261199999e+00, 1.6032999892854000e+03 },
    { 7.4670000000000006e-08, 1.6428900137900000e+00, 7.2164941953020002e+02 },
    { 7.2930000000000005e-08, 3.9064097622900000e+00, 8.8865680217000005e+01 },
    { 7.0449999999999999e-08, 4.1061318758900001e+00, 1.3615467058442000e+03 },
    { 6.7990000000000002e-08, 2.8020962465500001e+00, 1.7952584437210000e+03 },
    { 6.3720000000000004e-08, 3.5226310150000001e-01, 8.6030992875280003e+02 },
    { 6.3609999999999999e-08, 3.4852081094199998e+00, 2.2218566345969998e+03 },
    { 6.3150000000000006e-08, 1.5329158098400000e+00, 1.1482476104062000e+03 },
    { 5.5740000000000001e-08, 2.4450979938500002e+00, 1.0490869894507000e+03 },
    { 4.9670000000000001e-08, 1.3758483156700001e+00, 9.9911380480899993e+01 },
    { 4.8119999999999997e-08, 1.9845191422499999e+00, 1.0627416795630000e+02 },
    { 4.1890000000000000e-08, 4.7788592207900003e+00, 1.8023719907218001e+03 },
    { 4.0250000000000000e-08, 2.5334782096200001e+00, 1.2513403846248000e+03 },
    { 3.9360000000000003e-08, 3.5268664186100001e+00, 4.0525754987360000e+02 },
    { 3.9260000000000002e-08, 2.1621866382700001e+00, 4.9744763618019999e+02 },
    { 3.8360000000000001e-08, 1.9408814138499999e+00, 1.7881448967202000e+03 },
    { 3.7480000000000002e-08, 4.7584176486000002e-01, 1.1338771495710000e+02 },
    { 3.6200000000000000e-08, 5.3200912019400004e+00, 5.6193429400900004e+02 },
    { 3.5220000000000000e-08, 3.0958466405700000e+00, 5.0123677709139997e+02 },
    { 3.5010000000000000e-08, 4.6182832631000004e+00, 1.0696768709277001e+03 },
    { 3.3619999999999999e-08, 4.9001480406700004e+00, 2.3463973644039999e+02 },
    { 3.2530000000000001e-08, 3.3410607607000000e+00, 6.1144309831080000e+02 },
    { 3.1739999999999998e-08, 3.9062332538100000e+00, 4.4793883187839998e+02 },
    { 3.1260000000000003e-08, 1.6854160346000000e+00, 1.5677322542813999e+03 },
    { 2.9910000000000001e-08, 1.5867851680000000e+00, 2.1045367663768002e+03 },
    { 2.9690000000000000e-08, 4.1234518004399998e+00, 2.0156710861598001e+03 },
    { 2.9550000000000000e-08, 5.3379588515299998e+00, 1.5922542890215000e+03 },
    { 2.7990000000000001e-08, 2.6581064646999999e+00, 9.9160620955499994e+01 },
    { 2.7570000000000001e-08, 2.4113040882300001e+00, 2.0085575391590000e+03 },
    { 2.7540000000000000e-08, 5.7011878114299996e+00, 4.2977958461370002e+02 },
    { 2.7500000000000001e-08, 8.8132502338999996e-01, 1.4646394800628000e+03 },
    { 2.7440000000000000e-08, 4.1785980027000003e+00, 8.1752133216199994e+01 },
    { 2.7319999999999999e-08, 3.1254090254300002e+00, 1.8983512179396000e+03 },
    { 2.7000000000000000e-08, 2.4738899315999999e-01, 1.0702492748170000e+02 },
    { 2.6930000000000000e-08, 2.9641044251999999e+00, 2.9505122865419997e+02 },
    { 2.4340000000000002e-08, 4.8763805219099998e+00, 1.0807225711916001e+03 },
    { 2.4080000000000001e-08, 3.8256118483699999e+00, 1.4956319713459999e+02 },
    { 2.3800000000000001e-08, 4.2083325194099999e+00, 4.2341679713830001e+02 },
    { 2.3549999999999999e-08, 2.2860431863300001e+00, 8.2474219374879999e+02 },
    { 2.2350000000000002e-08, 3.7169635834500001e+00, 6.2759623029905997e+03 },
    { 2.2309999999999999e-08, 6.0759091198700004e+00, 6.2901893969922003e+03 },
    { 2.1970000000000001e-08, 1.7491188801299999e+00, 6.2960234557549995e+02 },
    { 2.0759999999999999e-08, 5.1398664470900002e+00, 1.0206171999210201e+04 },
    { 2.0720000000000000e-08, 1.2156271925800000e+00, 1.0220399093211799e+04 },
    { 2.0599999999999999e-08, 6.1396408295000005e-01, 2.6555683724738001e+03 },
    { 1.9740000000000000e-08, 2.0657103813700002e+00, 1.8912376709388000e+03 },
    { 1.9580000000000001e-08, 2.4200063355700001e+00, 2.4280421830342002e+03 },
    { 1.8189999999999999e-08, 3.7912838910900000e+00, 1.1656560981455000e+03 },
    { 1.8150000000000000e-08, 4.2878409435300000e+00, 5.5814515309779995e+02 },
    { 1.7970000000000001e-08, 5.4884799990699999e+00, 1.5851407420206999e+03 },
    { 1.7809999999999999e-08, 5.3168230834500001e+00, 3.1781457905676002e+03 },
    { 1.7680000000000000e-08, 2.6322249462200000e+00, 1.1592933106701000e+03 },
    { 1.7640000000000001e-08, 4.6915288789499998e+00, 9.7051624997219994e+02 },
    { 1.7529999999999999e-08, 4.8428655350899996e+00, 7.5721715453419995e+02 },
    { 1.7459999999999999e-08, 2.8541122921600000e+00, 1.5858915015461000e+03 },
    { 1.7350000000000000e-08, 2.2737646203000000e-01, 1.9054647649404001e+03 },
    { 1.7190000000000000e-08, 3.1812565283800001e+00, 6.5412438031559998e+02 },
    { 1.5589999999999999e-08, 3.2309637060299998e+00, 6.3596513305090002e+02 },
    { 1.5510000000000001e-08, 7.1019873146000001e-01, 9.4599421523210003e+02 },
    { 1.3949999999999999e-08, 2.5336628799600001e+00, 1.2797945726280000e+03 },
    { 1.3840000000000000e-08, 4.8729866574100003e+00, 5.2308074668030004e+03 },
    { 1.3799999999999999e-08, 2.5145212224200000e+00, 5.2165803728013998e+03 },
    { 1.3340000000000001e-08, 2.4430215946599998e+00, 2.3249494088155998e+03 },
    { 1.3100000000000000e-08, 2.1006557272099999e+00, 2.0014439921582000e+03 },
    { 1.2730000000000001e-08, 5.6951792911099997e+00, 2.2289701815978001e+03 },
    { 1.2709999999999999e-08, 1.2788403810000000e-02, 9.1610171630225996e+03 },
    { 1.2680000000000000e-08, 3.9375217705000001e+00, 9.1467900690210008e+03 },
    { 1.2620000000000000e-08, 3.6354464687600001e+00, 5.2236939198022001e+03 },
    { 1.2470000000000000e-08, 2.2157825542700000e+00, 3.2741250177853999e+03 },
    { 1.2340000000000000e-08, 2.0897784103600001e+00, 2.7444340526907999e+03 },
    { 1.2200000000000000e-08, 5.6418441883400003e+00, 3.1957326339430000e+02 },
    { 1.1980000000000000e-08, 5.5048302050100002e+00, 3.2668681039510000e+02 },
    { 1.1830000000000001e-08, 1.6880987820000000e+00, 6.3671589257630001e+02 },
    { 1.1830000000000001e-08, 5.6667627747000004e-01, 9.5310776223289997e+02 },
    { 1.1620000000000001e-08, 2.3950552519900001e+00, 2.2147430875962000e+03 },
    { 1.1370000000000000e-08, 7.5125250542999999e-01, 1.7810313497193999e+03 },
    { 1.1020000000000000e-08, 4.1504696489699997e+00, 3.2812385647862002e+03 },
    { 1.0989999999999999e-08, 1.5875543140499999e+00, 1.9125783119411999e+03 },
    { 1.0630000000000000e-08, 3.3463469781999999e-01, 5.1864526483069994e+02 },
    { 1.0630000000000000e-08, 4.0060509505499997e+00, 5.4073666535849998e+02 },
    { 1.0570000000000000e-08, 4.2578629309000000e+00, 1.6779385755008000e+03 },
    { 1.0509999999999999e-08, 8.7130311109000003e-01, 1.4930936680660000e+03 },
    { 1.0340000000000000e-08, 3.3491294610900000e+00, 3.4874241132234001e+03 },
    { 9.8500000000000005e-09, 2.2243263427600000e+00, 2.6413412784722000e+03 },
    { 9.7800000000000006e-09, 4.3316234918200003e+00, 2.9648466951296000e+03 },
    { 9.7800000000000006e-09, 4.8965912913600000e+00, 6.2830758499914000e+03 },
    { 9.6299999999999992e-09, 3.2792790241300001e+00, 2.5382485042536000e+03 },
    { 9.3900000000000000e-09, 3.6945112786799998e+00, 9.0981873305459999e+02 },
    { 9.3200000000000001e-09, 1.8169862730099999e+00, 9.5235700270749999e+02 },
    { 9.1999999999999997e-09, 1.7420496411899999e+00, 3.8997952100978000e+03 },
    { 9.1899999999999999e-09, 2.3903874000499998e+00, 2.0300415469949999e+02 },
    { 9.1499999999999992e-09, 2.1394968262799998e+00, 3.5762897934404000e+03 },
    { 9.1399999999999995e-09, 1.0639601544799999e+00, 2.1011770170029999e+02 },
    { 9.0900000000000007e-09, 3.9738654523400001e+00, 5.7533848848968000e+03 },
    { 9.0500000000000000e-09, 1.7503596876600001e+00, 2.2076295405954002e+03 },
    { 9.0400000000000002e-09, 3.8877615136000001e+00, 1.1585425511446999e+03 },
    { 8.7299999999999994e-09, 4.0807065002299998e+00, 1.6708250284999999e+03 },
    { 8.5899999999999995e-09, 4.4524022994100001e+00, 4.5884151979040001e+02 },
    { 8.3699999999999998e-09, 5.4042694823300002e+00, 9.6835945811164001e+03 },
    { 8.2599999999999992e-09, 3.5861513226900001e+00, 3.1496916025644000e+03 },
    { 8.1799999999999995e-09, 2.7621986839899999e+00, 3.3843313390048002e+03 },
    { 8.1199999999999993e-09, 6.0595897512299999e+00, 4.2266603761290003e+02 },
    { 8.0000000000000005e-09, 4.1279269899399997e+00, 2.7515475996916002e+03 },
    { 7.9599999999999998e-09, 1.9632643352200001e+00, 7.3980866679489998e+02 },
    { 7.9400000000000003e-09, 4.7202160302299996e+00, 2.3320629558164001e+03 },
    { 7.9099999999999994e-09, 1.5257349101200000e+00, 3.7967024358792000e+03 },
    { 7.8999999999999996e-09, 2.4593375882299999e+00, 1.1664068576709001e+03 },
    { 7.8500000000000008e-09, 2.0259712450100000e+00, 4.1630325013750002e+02 },
    { 7.8500000000000008e-09, 2.7422016512499998e+00, 7.3269511979410004e+02 },
    { 7.8100000000000001e-09, 2.4085074945999998e+00, 9.2783496796739996e+02 },
    { 7.7799999999999992e-09, 3.6950784901199998e+00, 1.3544331588434000e+03 },
    { 7.6100000000000000e-09, 1.0862070976700000e+00, 2.4209286360333999e+03 },
    { 7.5100000000000007e-09, 3.9254512342700001e+00, 3.5976304344427999e+03 },
    { 7.3600000000000002e-09, 1.8663797649800000e+00, 3.4803105662225998e+03 },
    { 7.2500000000000004e-09, 1.2601666692300000e+00, 3.7895888888784002e+03 },
    { 7.2399999999999998e-09, 1.5508294867200001e+00, 1.8841241239379999e+03 },
    { 7.2200000000000003e-09, 6.9187773299999999e-01, 1.8094855377225999e+03 },
    { 7.2099999999999997e-09, 3.8776851867000000e+00, 4.6940029547076001e+03 },
    { 7.1900000000000002e-09, 3.0298737344300002e+00, 6.0054041039879996e+02 },
    { 7.1200000000000002e-09, 1.9948478877400000e+00, 3.5905168874420001e+03 },
    { 6.9600000000000000e-09, 3.8788355141399999e+00, 1.9872168981566001e+03 },
    { 6.8100000000000003e-09, 5.0339137475199998e+00, 9.1539036160218002e+03 },
    { 6.6800000000000001e-09, 2.8662087199199999e+00, 2.8546403739102002e+03 },
    { 6.6400000000000002e-09, 2.8374192229899999e+00, 3.0465988283458000e+03 },
    { 6.6199999999999999e-09, 2.4999535401199999e+00, 3.7007232086613999e+03 },
    { 6.5499999999999999e-09, 5.2994931811099999e+00, 8.6242126509272002e+03 },
    { 6.4800000000000000e-09, 1.9318840551700001e+00, 3.0608259223474001e+03 },
    { 6.4400000000000001e-09, 1.2935528434400001e+00, 3.3701042450032000e+03 },
    { 6.3799999999999999e-09, 5.5679578953999997e-01, 2.5453620512543998e+03 },
    { 6.1900000000000003e-09, 3.1170782229600000e+00, 3.8038159828799999e+03 },
    { 5.9500000000000003e-09, 3.5090752596499999e+00, 7.6266071275599998e+01 },
    { 5.9500000000000003e-09, 4.1967523461300003e+00, 2.4351557300350000e+03 },
    { 5.8800000000000004e-09, 3.4520640250799999e+00, 4.3053034413910001e+02 },
    { 5.7800000000000003e-09, 1.2147691683399999e+00, 2.5240214102519999e+03 },
    { 5.7800000000000003e-09, 1.8989875660500000e+00, 3.8926816630970002e+03 },
    { 5.7600000000000000e-09, 1.7397773156800000e+00, 2.3178358618148000e+03 },
    { 5.7100000000000003e-09, 3.4587222439800001e+00, 2.0936694217490000e+02 },
    { 5.6599999999999999e-09, 6.1640670232000003e+00, 2.6271141844705999e+03 },
    { 5.6500000000000001e-09, 1.3040432946000000e-01, 2.8617539209110000e+03 },
    { 5.5800000000000002e-09, 5.7801145762599999e+00, 1.1411340634054000e+03 },
    { 5.4899999999999999e-09, 7.3935228740000003e-01, 2.6342277314714001e+03 },
    { 5.4599999999999998e-09, 8.7761028667999996e-01, 4.3764389113990001e+02 },
    { 5.4199999999999999e-09, 1.0742839124900001e+00, 2.8332997329077998e+03 },
    { 5.3499999999999999e-09, 3.6595053090100000e+00, 2.3036087678131998e+03 },
    { 5.3000000000000003e-09, 1.0190860639199999e+00, 5.4709945283390005e+02 },
    { 5.3000000000000003e-09, 3.3215995844499999e+00, 5.1228247735529999e+02 },
    { 5.2700000000000002e-09, 4.6406812053499999e+00, 4.6202291352809999e+02 },
    { 5.2700000000000002e-09, 2.8415970966100002e+00, 5.9735901666109999e+02 },
    { 5.2100000000000000e-09, 1.4601565538200001e+00, 4.2161870797543997e+03 },
    { 5.1400000000000000e-09, 1.2966701324600001e+00, 3.6936096616606001e+03 },
    { 5.1099999999999999e-09, 4.4666162952699997e+00, 6.4307868005169996e+02 },
    { 5.0799999999999998e-09, 2.1285482061600001e+00, 1.2442268376239999e+03 },
    { 4.8399999999999998e-09, 5.2920634379999998e-01, 2.0974232193759999e+03 },
    { 4.7900000000000002e-09, 8.0014976332999999e-01, 1.4575259330619999e+03 },
    { 4.7799999999999996e-09, 3.5005296645200001e+00, 1.5930050485469001e+03 },
    { 4.7500000000000003e-09, 5.8112280183299996e+00, 3.2670114707846001e+03 },
    { 4.5800000000000003e-09, 4.0302097821500000e+00, 6.2885158605009997e+02 },
    { 4.4999999999999998e-09, 3.7364138969699998e+00, 3.9140223040994001e+03 },
    { 4.3299999999999997e-09, 3.2147874689000000e-01, 6.2173803904930003e+02 },
    { 4.3100000000000002e-09, 8.5126632751999998e-01, 8.4926422848890002e+02 },
    { 4.2899999999999999e-09, 1.7358095728800000e+00, 2.9577331481288002e+03 },
    { 4.2400000000000002e-09, 2.3429452582099999e+00, 4.1555249061209997e+02 },
    { 4.1199999999999998e-09, 6.1980134786800001e+00, 2.7586611466924001e+03 },
    { 4.0199999999999998e-09, 5.9620745717399997e+00, 2.7302069586891998e+03 },
    { 4.0199999999999998e-09, 2.0124383032100002e+00, 2.5311349572528002e+03 },
};

static const VSOP87Term VSOP87A_JUPITER_Y3[136] = {
    { 1.7085515999999999e-04, 4.2909690406300003e+00, 5.2257741809380002e+02 },
    { 1.6701353000000000e-04, 3.2036573710899998e+00, 5.3680451209540001e+02 },
    { 4.0060379999999999e-05, 0.0000000000000000e+00, 0.0000000000000000e+00 },
    { 1.7824509999999999e-05, 1.9928307115299999e+00, 1.0593819301891999e+03 },
    { 1.4438160000000001e-05, 4.4086655526899996e+00, 5.1546387109299997e+02 },
    { 1.0794050000000000e-05, 3.8545079925199999e+00, 1.0664954771900000e+03 },
    { 1.0555649999999999e-05, 4.7018477378899997e+00, 7.1135470007999997e+00 },
    { 1.0131569999999999e-05, 3.0772962127899999e+00, 5.4391805909619995e+02 },
    { 7.1038499999999998e-06, 3.9373406269700002e+00, 1.0522683831884001e+03 },
    { 3.0059899999999998e-06, 4.6215611766100002e+00, 1.4227094001599999e+01 },
    { 2.5960100000000001e-06, 2.7156647838999999e+00, 1.5890728952838001e+03 },
    { 1.4283700000000000e-06, 5.2881430732999997e+00, 5.2969096509459996e+02 },
    { 1.3482600000000000e-06, 3.3627725389799998e+00, 1.0451548361876000e+03 },
    { 9.1645999999999999e-07, 4.4462334804100001e+00, 3.2350541665740002e+02 },
    { 7.8210999999999995e-07, 4.4427715196999999e+00, 1.5961864422845999e+03 },
    { 7.6456999999999997e-07, 5.6158739170600001e+00, 1.1731986822019999e+02 },
    { 6.1114999999999996e-07, 2.6556050313000001e-01, 1.1020632121940000e+02 },
    { 5.8233000000000000e-07, 4.5658481708699998e+00, 5.0835032409220003e+02 },
    { 5.6830999999999998e-07, 3.0484002269700001e+00, 7.3587651353180001e+02 },
    { 5.0933000000000002e-07, 2.5562704456000001e+00, 9.5979227217800002e+01 },
    { 4.9264999999999995e-07, 5.4827548088100002e+00, 4.2659819087599999e+02 },
    { 4.7020000000000001e-07, 3.5787156801600002e+00, 1.0736090241908000e+03 },
    { 4.3313000000000001e-07, 2.2418415874100002e+00, 6.3278373931320004e+02 },
    { 4.0222000000000001e-07, 1.9137500948999999e+00, 9.4206206196899996e+02 },
    { 3.8564000000000000e-07, 4.3379352603900001e+00, 3.3061896365820002e+02 },
    { 3.8462000000000001e-07, 3.2925858182400001e+00, 6.2567019231239999e+02 },
    { 3.8001000000000003e-07, 5.2874503771000003e-01, 1.0309277421860000e+02 },
    { 3.2971000000000001e-07, 3.9095448537599999e+00, 4.3371173787679999e+02 },
    { 3.2318000000000001e-07, 4.0251551225900002e+00, 1.5819593482830001e+03 },
    { 3.2287999999999999e-07, 2.8915119778100000e+00, 5.5103160609700001e+02 },
    { 3.0311000000000002e-07, 3.0712394066200002e+00, 7.2876296653099996e+02 },
    { 2.9979000000000001e-07, 2.0200064995900000e+00, 9.4917560896980001e+02 },
    { 2.8185000000000000e-07, 3.1946448421000002e+00, 2.1187638603783998e+03 },
    { 2.7410000000000000e-07, 3.5005320322000002e+00, 2.2041264243879999e+02 },
    { 2.6996000000000002e-07, 4.4447204171400001e+00, 4.1948464387519999e+02 },
    { 2.6376999999999999e-07, 2.7507241559100000e+00, 1.1695882514085999e+03 },
    { 2.4896999999999999e-07, 5.1010698159599999e+00, 5.1939602435610004e+02 },
    { 2.4556999999999999e-07, 3.5397368744900000e+00, 1.9907200143640000e+02 },
    { 2.3237000000000001e-07, 3.7258964304300002e+00, 1.1553611574070001e+03 },
    { 2.2620000000000000e-07, 5.1667171294000003e-01, 9.5628915597059995e+02 },
    { 2.0475000000000000e-07, 2.3410412810799999e+00, 5.3998590583309999e+02 },
    { 1.8360000000000000e-07, 2.7061088367899999e+00, 6.3989728631399998e+02 },
    { 1.7529000000000000e-07, 4.5924527347600002e+00, 2.1340641002400002e+01 },
    { 1.5785000000000000e-07, 4.8681216974700003e+00, 3.0927832265580003e+02 },
    { 1.4338000000000000e-07, 5.7703666446700002e+00, 8.5319638175199998e+02 },
    { 1.3297999999999999e-07, 3.3660646072099998e+00, 2.2752618943959999e+02 },
    { 1.1612999999999999e-07, 4.6952697255000002e+00, 6.1855664531160005e+02 },
    { 1.1494000000000000e-07, 2.4796547984699999e+00, 4.4082528487759998e+02 },
    { 1.1127000000000000e-07, 3.1032765725000000e+00, 2.0618554843720000e+02 },
    { 1.0710000000000000e-07, 3.9802663534399998e+00, 8.3896928775039999e+02 },
    { 1.0142000000000000e-07, 3.8871306497900000e+00, 4.1237109687440000e+02 },
    { 9.4520000000000002e-08, 3.4815818510600001e+00, 1.5748458012822000e+03 },
    { 9.3359999999999997e-08, 3.2762840852899999e+00, 1.0380412891868000e+03 },
    { 8.1800000000000005e-08, 4.5769932381500000e+00, 3.0216477565500003e+02 },
    { 8.0290000000000000e-08, 2.3440357988599998e+00, 2.1329909543799999e+02 },
    { 7.8860000000000006e-08, 3.9059373275699998e+00, 8.3185574074960005e+02 },
    { 7.7289999999999999e-08, 5.6811078929200001e+00, 3.1639186965660002e+02 },
    { 7.6959999999999996e-08, 1.5966029844700000e+00, 7.4299006053259995e+02 },
    { 7.6829999999999994e-08, 4.4775289842500001e+00, 1.2584539316256000e+03 },
    { 5.9459999999999998e-08, 4.9697738630500004e+00, 2.1258774073792001e+03 },
    { 4.1390000000000002e-08, 5.4787693614300004e+00, 1.2443341522100000e+02 },
    { 3.9349999999999998e-08, 4.4677050776599998e+00, 1.2655674786264001e+03 },
    { 3.6090000000000001e-08, 1.8814680035500000e+00, 1.4859801210651999e+03 },
    { 3.0199999999999999e-08, 3.3830596428900002e+00, 1.6992792165031999e+03 },
    { 2.9900000000000003e-08, 2.5733822109500002e+00, 6.4701083331480004e+02 },
    { 2.9309999999999999e-08, 2.8977091204800001e+00, 1.4717530270636000e+03 },
    { 2.8839999999999999e-08, 2.4095697192799999e+00, 9.3494851496820002e+02 },
    { 2.8590000000000000e-08, 3.6615353506399999e+00, 1.2726810256271999e+03 },
    { 2.7030000000000001e-08, 5.9586572445000003e+00, 8.8865680217000005e+01 },
    { 2.6270000000000000e-08, 3.9851720629399998e+00, 1.0625633239269000e+03 },
    { 2.4599999999999999e-08, 3.6782977388500000e+00, 2.6484548254729998e+03 },
    { 2.3590000000000001e-08, 4.1735026805200004e+00, 3.3773251065900001e+02 },
    { 2.3400000000000001e-08, 5.5444890088100003e+00, 1.3686602528450001e+03 },
    { 2.3219999999999999e-08, 4.0545070100400000e+00, 1.6032999892854000e+03 },
    { 2.2679999999999999e-08, 3.3321965241300000e+00, 7.2164941953020002e+02 },
    { 2.1769999999999999e-08, 4.0777545996300004e+00, 9.6340270297140000e+02 },
    { 2.0140000000000000e-08, 3.6493182721699999e+00, 1.9195845443560000e+02 },
    { 1.8110000000000001e-08, 4.9380883526900003e+00, 8.6030992875280003e+02 },
    { 1.7840000000000000e-08, 4.9011066641700003e+00, 5.2650957135689998e+02 },
    { 1.7840000000000000e-08, 5.7227642913799999e+00, 5.3287235883230005e+02 },
    { 1.7470000000000000e-08, 4.7268311417900000e+00, 1.6850521225016000e+03 },
    { 1.7339999999999998e-08, 4.1460187391899996e+00, 1.0554497769261000e+03 },
    { 1.7170000000000001e-08, 2.1903594003200002e+00, 4.4793883187839998e+02 },
    { 1.7000000000000000e-08, 2.5988572162599999e+00, 1.0562005364515001e+03 },
    { 1.6400000000000001e-08, 2.3528705067800000e+00, 7.5010360753340001e+02 },
    { 1.6110000000000000e-08, 3.7116944339000000e+00, 1.7881448967202000e+03 },
    { 1.4310000000000000e-08, 6.0019175876900004e+00, 8.1752133216199994e+01 },
    { 1.4250000000000000e-08, 5.7692139002599996e+00, 1.3615467058442000e+03 },
    { 1.3540000000000001e-08, 2.5698768282800000e+00, 1.1624747044077999e+03 },
    { 1.3410000000000001e-08, 4.0334983039900001e+00, 1.2513403846248000e+03 },
    { 1.3060000000000001e-08, 1.7505316696000001e-01, 1.3828873468465999e+03 },
    { 1.2610000000000000e-08, 3.2405032481200000e+00, 2.1045367663768002e+03 },
    { 1.2520000000000001e-08, 4.1204948765999996e+00, 2.1116503133776000e+03 },
    { 1.2110000000000001e-08, 2.8430975377399998e+00, 1.0696768709277001e+03 },
    { 1.1180000000000000e-08, 5.8836585005700002e+00, 6.1144309831080000e+02 },
    { 1.1170000000000000e-08, 3.5929602528300002e+00, 1.1482476104062000e+03 },
    { 1.1160000000000000e-08, 6.6928326000000003e-03, 1.8841241239379999e+03 },
    { 1.0929999999999999e-08, 2.2239904128999999e-01, 1.1411340634054000e+03 },
    { 1.0900000000000000e-08, 2.6922904019399998e+00, 1.4646394800628000e+03 },
    { 1.0610000000000000e-08, 2.4211318050299999e+00, 1.4788665740644001e+03 },
    { 1.0430000000000000e-08, 2.9828027781199999e+00, 1.0807225711916001e+03 },
    { 1.0130000000000000e-08, 5.4474831242899997e+00, 1.0490869894507000e+03 },
    { 1.0099999999999999e-08, 5.1307793377799999e+00, 2.0974232193759999e+03 },
    { 9.8700000000000000e-09, 5.5645824435100000e+00, 4.0525754987360000e+02 },
    { 9.6099999999999997e-09, 4.8057409127000001e+00, 2.3178358618148000e+03 },
    { 9.5700000000000007e-09, 4.9672158310999999e+00, 1.9943304451573999e+03 },
    { 9.3600000000000008e-09, 4.2345908753500003e+00, 2.2076295405954002e+03 },
    { 8.9199999999999998e-09, 4.6408040121300003e+00, 2.9505122865419997e+02 },
    { 8.7500000000000006e-09, 1.7255525975599999e+00, 8.4608283475120004e+02 },
    { 8.6399999999999999e-09, 3.8182481441700000e+00, 1.5677322542813999e+03 },
    { 8.3500000000000003e-09, 2.7417521094500001e+00, 2.0085575391590000e+03 },
    { 8.3099999999999996e-09, 4.8590849693899996e+00, 2.0014439921582000e+03 },
    { 8.2499999999999994e-09, 4.7076103195099996e+00, 1.8983512179396000e+03 },
    { 8.2299999999999999e-09, 2.0392794573300002e+00, 6.5412438031559998e+02 },
    { 7.9300000000000005e-09, 2.9663049539899999e+00, 2.3463973644039999e+02 },
    { 7.7900000000000006e-09, 4.4348480095299996e+00, 2.2147430875962000e+03 },
    { 7.4199999999999996e-09, 5.3280144937599996e+00, 5.3362311835770004e+02 },
    { 7.4199999999999996e-09, 5.2958564617899997e+00, 5.2575881183150000e+02 },
    { 7.3499999999999996e-09, 4.6069567813600001e+00, 2.6271141844705999e+03 },
    { 6.9299999999999999e-09, 8.2599527101000003e-01, 1.6921656695024001e+03 },
    { 6.7800000000000002e-09, 4.7793735968400002e+00, 8.2474219374879999e+02 },
    { 6.7100000000000002e-09, 3.0910299294999999e+00, 2.4209286360333999e+03 },
    { 6.6999999999999996e-09, 1.5897671835700000e+00, 2.4351557300350000e+03 },
    { 6.3499999999999998e-09, 3.6190558492200000e+00, 2.8404132799086001e+03 },
    { 6.2300000000000002e-09, 4.7721120853099999e+00, 2.3107223148140001e+03 },
    { 6.1700000000000000e-09, 4.5555533458399999e+00, 2.6342277314714001e+03 },
    { 5.7699999999999997e-09, 3.5247211876200000e+00, 2.0156710861598001e+03 },
    { 5.7200000000000001e-09, 2.3911444924300000e+00, 1.7810313497193999e+03 },
    { 5.7100000000000003e-09, 1.4093267044100000e+00, 2.7515475996916002e+03 },
    { 5.5999999999999997e-09, 3.3647636229900000e+00, 1.9054647649404001e+03 },
    { 5.5999999999999997e-09, 3.9859819096100000e+00, 2.5240214102519999e+03 },
    { 5.2300000000000003e-09, 2.5525048107599999e+00, 1.8023719907218001e+03 },
    { 5.1300000000000003e-09, 3.6901870899000002e+00, 1.8912376709388000e+03 },
    { 4.6600000000000000e-09, 6.1281435508800000e+00, 1.3757737998458001e+03 },
    { 4.1499999999999999e-09, 4.6847855874700004e+00, 5.0123677709139997e+02 },
    { 4.1499999999999999e-09, 5.9390853680799998e+00, 5.5814515309779995e+02 },
};

static const VSOP87Term VSOP87A_JUPITER_Y4[60] = {
    { 1.7626450000000001e-05, 6.0415938655400003e+00, 5.2257741809380002e+02 },
    { 1.7160449999999999e-05, 1.4620628571000001e+00, 5.3680451209540001e+02 },
    { 3.0503599999999998e-06, 6.1405278681900004e+00, 5.1546387109299997e+02 },
    { 2.1620300000000002e-06, 1.3430185666600001e+00, 5.4391805909619995e+02 },
    { 1.2789500000000001e-06, 2.2594166479600002e+00, 1.0664954771900000e+03 },
    { 8.1740000000000000e-07, 2.8476641587899998e+00, 7.1135470007999997e+00 },
    { 7.0306000000000005e-07, 0.0000000000000000e+00, 0.0000000000000000e+00 },
    { 6.8446000000000003e-07, 5.7773691357299999e+00, 1.0522683831884001e+03 },
    { 5.5008000000000001e-07, 2.3109269173999999e-01, 1.0593819301891999e+03 },
    { 5.0861000000000005e-07, 2.9053880962499998e+00, 1.4227094001599999e+01 },
    { 2.3664999999999999e-07, 5.1926186105300003e+00, 1.0451548361876000e+03 },
    { 1.4962000000000001e-07, 8.5261561149999998e-02, 5.0835032409220003e+02 },
    { 1.3643000000000001e-07, 1.3433185190500001e+00, 1.5890728952838001e+03 },
    { 1.0849000000000000e-07, 2.6049372229699999e+00, 5.2969096509459996e+02 },
    { 1.0460000000000000e-07, 2.9264028966100000e+00, 1.5961864422845999e+03 },
    { 1.0393000000000000e-07, 1.8898688489600000e+00, 1.0736090241908000e+03 },
    { 1.0367000000000000e-07, 3.8451384000200002e+00, 1.1731986822019999e+02 },
    { 7.2909999999999996e-08, 1.0061449999100001e+00, 5.5103160609700001e+02 },
    { 5.7940000000000001e-08, 4.2752885666899996e+00, 9.5979227217800002e+01 },
    { 5.2819999999999999e-08, 2.6208008350399998e+00, 3.3061896365820002e+02 },
    { 5.2579999999999998e-08, 3.5859132157300002e+00, 9.4206206196899996e+02 },
    { 5.2259999999999999e-08, 2.6309112834300001e+00, 3.2350541665740002e+02 },
    { 4.9749999999999999e-08, 5.7394310078100004e+00, 1.1020632121940000e+02 },
    { 4.3049999999999998e-08, 4.9094684828900004e+00, 7.2876296653099996e+02 },
    { 3.5959999999999999e-08, 4.8529402563700001e+00, 7.3587651353180001e+02 },
    { 3.4359999999999998e-08, 5.2704726818400003e+00, 1.9907200143640000e+02 },
    { 3.4289999999999998e-08, 4.9883195181600000e+00, 6.2567019231239999e+02 },
    { 3.1179999999999999e-08, 1.7897289503100000e+00, 4.3371173787679999e+02 },
    { 2.8600000000000001e-08, 5.8468374729599999e+00, 1.5819593482830001e+03 },
    { 2.7459999999999999e-08, 2.3235371651699999e+00, 1.0309277421860000e+02 },
    { 2.7039999999999999e-08, 8.2647334354000002e-01, 4.4082528487759998e+02 },
    { 2.5880000000000001e-08, 1.0032314581899999e+00, 1.1695882514085999e+03 },
    { 2.2399999999999999e-08, 5.5187183582900001e+00, 1.1553611574070001e+03 },
    { 2.1249999999999998e-08, 6.2770458070500004e+00, 4.1948464387519999e+02 },
    { 1.8040000000000001e-08, 4.0973236992000004e+00, 6.3278373931320004e+02 },
    { 1.6000000000000001e-08, 1.7685762004000001e+00, 2.1187638603783998e+03 },
    { 1.5399999999999999e-08, 7.5983380154000002e-01, 6.1855664531160005e+02 },
    { 1.5390000000000001e-08, 3.8824694365500001e+00, 4.2659819087599999e+02 },
    { 1.5069999999999999e-08, 9.8837376638999996e-01, 6.3989728631399998e+02 },
    { 1.4820000000000000e-08, 2.8875553209999998e-02, 3.0216477565500003e+02 },
    { 1.4680000000000000e-08, 1.6733469108400001e+00, 2.2041264243879999e+02 },
    { 1.4149999999999999e-08, 1.6604741511500001e+00, 2.2752618943959999e+02 },
    { 1.4000000000000000e-08, 5.3480983215600002e+00, 1.5748458012822000e+03 },
    { 1.3620000000000001e-08, 5.1459413182800002e+00, 9.5628915597059995e+02 },
    { 1.2560000000000000e-08, 3.3648936932000001e-01, 3.0927832265580003e+02 },
    { 1.1870000000000000e-08, 4.1678392002000004e+00, 8.5319638175199998e+02 },
    { 1.1420000000000001e-08, 3.9946972704400001e+00, 9.4917560896980001e+02 },
    { 1.1030000000000000e-08, 5.6014604159500001e+00, 4.1237109687440000e+02 },
    { 1.0940000000000001e-08, 1.2734228732000000e-01, 1.2584539316256000e+03 },
    { 8.9000000000000003e-09, 1.1310092765199999e+00, 6.4701083331480004e+02 },
    { 8.6800000000000006e-09, 5.3917524112399997e+00, 1.0380412891868000e+03 },
    { 8.5199999999999995e-09, 8.1910749925000004e-01, 8.3185574074960005e+02 },
    { 8.1699999999999997e-09, 2.4456437306300001e+00, 7.4299006053259995e+02 },
    { 6.7700000000000004e-09, 5.8894543966599997e+00, 8.3896928775039999e+02 },
    { 5.9799999999999996e-09, 2.4615243576000001e+00, 3.3773251065900001e+02 },
    { 5.5599999999999998e-09, 7.2453025284000006e-01, 1.4859801210651999e+03 },
    { 4.9799999999999998e-09, 1.0660824407200000e+00, 1.3615467058442000e+03 },
    { 4.6800000000000004e-09, 5.0720965065000003e+00, 2.1340641002400002e+01 },
    { 4.5100000000000003e-09, 5.3283579664199996e+00, 1.4717530270636000e+03 },
    { 3.9600000000000004e-09, 2.3330451036300000e+00, 1.6032999892854000e+03 },
};

static const VSOP87Term VSOP87A_JUPITER_Y5[11] = {
    { 1.3147099999999999e-06, 1.6420555406599999e+00, 5.2257741809380002e+02 },
    { 1.2663400000000001e-06, 5.8737267358400000e+00, 5.3680451209540001e+02 },
    { 4.8268999999999999e-07, 1.6278893672300001e+00, 5.1546387109299997e+02 },
    { 3.4033999999999999e-07, 5.8404020700699997e+00, 5.4391805909619995e+02 },
    { 1.1212000000000000e-07, 6.2260985921000001e-01, 1.0664954771900000e+03 },
    { 4.0600000000000000e-08, 1.7716791288300000e+00, 1.0522683831884001e+03 },
    { 1.9429999999999999e-08, 1.8158253075000000e-01, 1.0736090241908000e+03 },
    { 1.7859999999999999e-08, 1.5002152999000000e+00, 1.0451548361876000e+03 },
    { 8.8599999999999996e-09, 1.6841963303400000e+00, 5.0835032409220003e+02 },
    { 8.8599999999999996e-09, 2.6564893180300002e+00, 5.5103160609700001e+02 },
    { 4.6699999999999998e-09, 1.3784171516899999e+00, 1.5961864422845999e+03 },
};

static const VSOP87Term VSOP87A_JUPITER_Z0[216] = {
    { 1.1823100489000000e-01, 3.5584464634300002e+00, 5.2969096509459996e+02 },
    { 8.5903195200000004e-03, 0.0000000000000000e+00, 0.0000000000000000e+00 },
    { 2.8656209399999999e-03, 3.9081223833799998e+00, 1.0593819301891999e+03 },
    { 4.2388591999999999e-04, 3.6014419103200002e+00, 5.2257741809380002e+02 },
    { 3.3295490999999999e-04, 3.0297050585000002e-01, 5.3680451209540001e+02 },
    { 1.0416160000000000e-04, 4.2576459306099999e+00, 1.5890728952838001e+03 },
    { 7.4492940000000002e-05, 5.2421310415000004e+00, 1.0309277421860000e+02 },
    { 6.9101020000000003e-05, 1.7503294575199999e+00, 7.1135470007999997e+00 },
    { 5.2920120000000003e-05, 1.6823144719200001e+00, 4.2659819087599999e+02 },
    { 4.3135980000000000e-05, 3.7067368984100000e+00, 2.1329909543799999e+02 },
    { 3.7980160000000003e-05, 2.1671574317500002e+00, 6.3278373931320004e+02 },
    { 3.7842649999999999e-05, 2.7152254449100002e+00, 1.1020632121940000e+02 },
    { 2.4615470000000000e-05, 2.9988946041100002e+00, 1.1624747044077999e+03 },
    { 2.4553849999999999e-05, 2.9690413565900000e+00, 1.0522683831884001e+03 },
    { 2.1634709999999999e-05, 6.2671825985399998e+00, 8.4608283475120004e+02 },
    { 2.0014510000000000e-05, 2.6853583830900001e+00, 4.1948464387519999e+02 },
    { 1.6336530000000001e-05, 6.4194743492999995e-01, 1.0664954771900000e+03 },
    { 1.4506719999999999e-05, 1.1710841619300001e+00, 9.4917560896980001e+02 },
    { 7.1504199999999999e-06, 1.4121119782000000e+00, 3.2350541665740002e+02 },
    { 6.9309500000000001e-06, 5.1427804116100004e+00, 3.1639186965660002e+02 },
    { 5.4971999999999996e-06, 4.8416427437799996e+00, 7.4299006053259995e+02 },
    { 5.4361899999999999e-06, 3.6963656182200002e+00, 5.1546387109299997e+02 },
    { 4.8385200000000001e-06, 4.7179611016000003e+00, 7.3587651353180001e+02 },
    { 4.4783099999999998e-06, 4.6074658862100000e+00, 2.1187638603783998e+03 },
    { 3.6509999999999999e-06, 6.0606592543700000e+00, 2.0618554843720000e+02 },
    { 2.8897999999999998e-06, 4.0266293089999999e-01, 5.2575881183150000e+02 },
    { 2.5493800000000000e-06, 1.7978560943999999e-01, 5.4391805909619995e+02 },
    { 2.5325400000000002e-06, 3.5748531548700000e+00, 5.3362311835770004e+02 },
    { 1.9669100000000000e-06, 2.6552692356900001e+00, 9.5628915597059995e+02 },
    { 1.9513700000000001e-06, 3.3537480223700000e+00, 1.6921656695024001e+03 },
    { 1.6237100000000001e-06, 3.1216059680699999e+00, 1.5819593482830001e+03 },
    { 1.5437900000000001e-06, 1.4705066947000001e+00, 6.2567019231239999e+02 },
    { 1.4597200000000001e-06, 5.9611290339599998e+00, 8.3896928775039999e+02 },
    { 1.3595399999999999e-06, 2.2372160061000002e+00, 1.4788665740644001e+03 },
    { 1.3537600000000000e-06, 4.0318007527799997e+00, 1.2655674786264001e+03 },
    { 1.3309499999999999e-06, 1.6128364808000000e+00, 1.4227094001599999e+01 },
    { 1.2114700000000000e-06, 4.0625246682700000e+00, 5.2650957135689998e+02 },
    { 1.1873400000000001e-06, 6.1913255429399996e+00, 5.3287235883230005e+02 },
    { 1.1396300000000000e-06, 3.4443034869000000e-01, 1.3757737998458001e+03 },
    { 9.1948000000000003e-07, 5.2952156972099997e+00, 9.5979227217800002e+01 },
    { 8.9416000000000001e-07, 9.8707919437000002e-01, 1.5961864422845999e+03 },
    { 7.1808000000000003e-07, 2.6037025722100000e+00, 1.1731986822019999e+02 },
    { 6.4120999999999996e-07, 4.6352439167000004e+00, 1.3686602528450001e+03 },
    { 6.3527000000000000e-07, 9.7603653342999996e-01, 4.3371173787679999e+02 },
    { 6.0053999999999995e-07, 1.0587522621100001e+00, 9.4206206196899996e+02 },
    { 5.9116000000000001e-07, 4.2038566170000002e-01, 2.2041264243879999e+02 },
    { 5.6616999999999999e-07, 5.7864478693999999e+00, 6.3989728631399998e+02 },
    { 5.4980000000000001e-07, 2.7241404185700002e+00, 1.0451548361876000e+03 },
    { 5.4868000000000001e-07, 6.1486834470499998e+00, 1.1695882514085999e+03 },
    { 4.7058000000000002e-07, 2.3906904286900001e+00, 8.5319638175199998e+02 },
    { 4.1027999999999999e-07, 5.0764981580099997e+00, 3.0927832265580003e+02 },
    { 3.9937000000000001e-07, 2.6301496291999999e+00, 3.8012776796000003e+02 },
    { 3.4887000000000003e-07, 3.0664485870700000e+00, 4.1237109687440000e+02 },
    { 3.3112000000000002e-07, 2.2108401152399999e+00, 7.4781598567299994e+01 },
    { 2.4752999999999999e-07, 4.0931153536700000e+00, 5.2724328453980002e+02 },
    { 2.4578000000000002e-07, 6.1644241496500003e+00, 5.3213864564940002e+02 },
    { 2.1472999999999999e-07, 5.6832484841099999e+00, 1.2726810256271999e+03 },
    { 2.1278999999999999e-07, 3.9810521771300000e+00, 1.7952584437210000e+03 },
    { 2.1259000000000000e-07, 4.1564841862300002e+00, 1.6850521225016000e+03 },
    { 2.1217000000000000e-07, 4.4607122791500000e+00, 1.1553611574070001e+03 },
    { 2.1213999999999999e-07, 4.9574971425200003e+00, 2.6484548254729998e+03 },
    { 1.9570000000000001e-07, 4.3717159862399999e+00, 1.4956319713459999e+02 },
    { 1.9532999999999999e-07, 3.8904271301999999e-01, 5.2964278098479997e+02 },
    { 1.9530999999999999e-07, 3.5868538719299998e+00, 5.2973914920439995e+02 },
    { 1.9446999999999999e-07, 1.2766359846599999e+00, 3.3061896365820002e+02 },
    { 1.7624000000000000e-07, 1.3166782787100000e+00, 3.9321532631000000e+00 },
    { 1.5482000000000001e-07, 3.2529178243799999e+00, 5.2872775724810003e+02 },
    { 1.5440000000000001e-07, 7.2260027140000005e-01, 5.3065417294110000e+02 },
    { 1.5412999999999999e-07, 4.3195805939599996e+00, 1.0633140834523001e+03 },
    { 1.4562000000000000e-07, 5.5013599882199999e+00, 1.4717530270636000e+03 },
    { 1.4193000000000000e-07, 3.6738794399099999e+00, 2.1116503133776000e+03 },
    { 1.3309000000000001e-07, 2.7789753385400000e+00, 2.0085575391590000e+03 },
    { 1.3206000000000001e-07, 3.6497650201699998e+00, 2.2218566345969998e+03 },
    { 1.3141999999999999e-07, 7.2687086888999997e-01, 1.0554497769261000e+03 },
    { 1.2996000000000001e-07, 1.3100092786899999e+00, 4.5342489381899998e+02 },
    { 1.2989999999999999e-07, 1.8723425561500000e+00, 9.8460033162189995e+02 },
    { 1.2279999999999999e-07, 1.9903023672500000e+00, 3.8133035637799999e+01 },
    { 1.1738000000000000e-07, 6.1433108236500003e+00, 1.9907200143640000e+02 },
    { 1.0473000000000000e-07, 2.6689008607999998e-01, 1.0213285546211000e+04 },
    { 1.0389000000000000e-07, 5.1378228979999996e-01, 1.0736090241908000e+03 },
    { 8.8749999999999997e-08, 1.3793367408000001e-01, 3.0534616939270001e+02 },
    { 8.7139999999999998e-08, 1.5882502921799999e+00, 3.1813937377000001e+00 },
    { 8.3470000000000006e-08, 5.9653185382500000e+00, 9.0981873305459999e+02 },
    { 7.8610000000000000e-08, 3.7983224575399999e+00, 7.6266071275599998e+01 },
    { 7.7280000000000001e-08, 4.6160478197500003e+00, 7.2876296653099996e+02 },
    { 7.6920000000000003e-08, 5.0840796537299999e+00, 1.7881448967202000e+03 },
    { 7.4410000000000003e-08, 1.3043935274300000e+00, 6.1855664531160005e+02 },
    { 7.3650000000000001e-08, 3.2634783022099998e+00, 1.4859801210651999e+03 },
    { 7.1429999999999998e-08, 4.0191243877299998e+00, 2.0014439921582000e+03 },
    { 6.8960000000000003e-08, 1.7402591986300000e+00, 5.2916970023279998e+02 },
    { 6.8869999999999994e-08, 2.2354933498600000e+00, 5.3021222995640005e+02 },
    { 6.7519999999999998e-08, 5.5096089298599997e+00, 1.8983512179396000e+03 },
    { 6.5180000000000005e-08, 9.0320354569000005e-01, 1.9054647649404001e+03 },
    { 5.7879999999999999e-08, 2.6042181508000001e-01, 1.0625633239269000e+03 },
    { 5.7830000000000002e-08, 6.0233891563300004e+00, 8.3185574074960005e+02 },
    { 5.5810000000000001e-08, 6.1748922866599996e+00, 1.2584539316256000e+03 },
    { 5.4800000000000001e-08, 3.1281876895199998e+00, 6.0447256366190004e+02 },
    { 5.2689999999999997e-08, 4.0657616079799999e+00, 1.0562005364515001e+03 },
    { 5.1539999999999997e-08, 1.3359225276400000e+00, 2.1258774073792001e+03 },
    { 4.9940000000000002e-08, 3.9029214340000000e-02, 1.4395096981491999e+03 },
    { 4.9670000000000001e-08, 3.9288221024199999e+00, 5.0835032409220003e+02 },
    { 4.9159999999999998e-08, 1.8728472832000001e-01, 1.6992792165031999e+03 },
    { 4.8440000000000002e-08, 1.4781856972200000e+00, 4.5490936652729999e+02 },
    { 4.7110000000000003e-08, 8.7975326502999995e-01, 4.9033408917940000e+02 },
    { 4.3579999999999997e-08, 2.1630055560299999e+00, 1.0212488945514000e+03 },
    { 4.2949999999999997e-08, 3.4520203816700001e+00, 5.6904784100979998e+02 },
    { 4.2209999999999999e-08, 2.5730408148500001e+00, 9.9160620955499994e+01 },
    { 4.0049999999999998e-08, 2.9606273759200001e+00, 5.2677020378780003e+02 },
    { 3.9589999999999999e-08, 1.3816378715700000e+00, 5.3261172640140001e+02 },
    { 3.6830000000000000e-08, 3.3555577414999999e-01, 5.2894020556919997e+02 },
    { 3.6809999999999997e-08, 4.0053982060099997e+00, 5.3044172461999995e+02 },
    { 3.6230000000000001e-08, 2.8674357330000000e-01, 2.2752618943959999e+02 },
    { 3.5350000000000002e-08, 1.1864574902400000e+00, 5.2549817940059995e+02 },
    { 3.4770000000000000e-08, 3.1560750905499999e+00, 5.3388375078859997e+02 },
    { 3.4580000000000003e-08, 4.4040176678899998e+00, 5.1939602435610004e+02 },
    { 3.4350000000000000e-08, 5.3419150544300003e+00, 1.1045700263900001e+01 },
    { 3.4240000000000001e-08, 5.1645687138299996e+00, 1.4844727083000000e+00 },
    { 3.3549999999999999e-08, 4.8006324886599998e+00, 2.1045367663768002e+03 },
    { 3.3339999999999999e-08, 3.8640772449999999e-01, 2.2434479570190001e+02 },
    { 3.2189999999999999e-08, 5.3713079328499997e+00, 2.2147430875962000e+03 },
    { 3.1429999999999998e-08, 1.5836271478000000e-01, 7.5010360753340001e+02 },
    { 2.9390000000000000e-08, 6.4629595692999997e-01, 5.4073666535849998e+02 },
    { 2.8559999999999999e-08, 2.9274516197000000e-01, 9.8311585891360005e+02 },
    { 2.7689999999999998e-08, 2.0040217533999999e-01, 5.2502509864859996e+02 },
    { 2.7470000000000000e-08, 1.7647107493600001e+00, 5.2427433912319998e+02 },
    { 2.7249999999999999e-08, 6.4710955845999996e-01, 5.2406189080210004e+02 },
    { 2.7039999999999999e-08, 2.5740609376800001e+00, 5.3510759106600005e+02 },
    { 2.6969999999999999e-08, 6.2245941177499997e+00, 5.3998590583309999e+02 },
    { 2.6849999999999999e-08, 3.6947886679500002e+00, 5.3532003938710000e+02 },
    { 2.6490000000000001e-08, 3.8043446944300001e+00, 2.3178358618148000e+03 },
    { 2.6230000000000001e-08, 2.5280896703000000e-01, 5.5103160609700001e+02 },
    { 2.5979999999999998e-08, 4.1392406194099998e+00, 5.3435683154059996e+02 },
    { 2.5970000000000000e-08, 1.1582486976199999e+00, 9.6340270297140000e+02 },
    { 2.4909999999999999e-08, 2.8429728794700000e+00, 1.3828873468465999e+03 },
    { 2.4750000000000000e-08, 9.9313692672999998e-01, 9.3494851496820002e+02 },
    { 2.3980000000000000e-08, 4.3661585492599997e+00, 2.4280421830342002e+03 },
    { 2.3359999999999999e-08, 3.7167613114600000e+00, 5.2985102378900001e+02 },
    { 2.3359999999999999e-08, 6.2392433691000004e-01, 5.2953090640020002e+02 },
    { 2.3020000000000000e-08, 5.8429514492900001e+00, 1.1482476104062000e+03 },
    { 2.1520000000000000e-08, 1.7455744421999999e+00, 2.1340641002400002e+01 },
    { 2.1470000000000000e-08, 5.9672406041099997e+00, 1.8912376709388000e+03 },
    { 2.1340000000000001e-08, 2.7965799089500001e+00, 3.5424722652100002e+01 },
    { 2.0680000000000001e-08, 5.9619662379899996e+00, 4.4082528487759998e+02 },
    { 2.0120000000000001e-08, 4.9594668026499997e+00, 1.3615467058442000e+03 },
    { 1.9869999999999999e-08, 1.8375525793000000e-01, 1.8023719907218001e+03 },
    { 1.9399999999999998e-08, 3.6368734381199999e+00, 2.3249494088155998e+03 },
    { 1.9340000000000000e-08, 2.8206615603500000e+00, 8.3503713448730002e+02 },
    { 1.8600000000000001e-08, 5.2423040555900000e+00, 2.5311349572528002e+03 },
    { 1.8410000000000001e-08, 5.7247596143599999e+00, 5.3117543780289998e+02 },
    { 1.7710000000000001e-08, 4.8723428065499998e+00, 5.2820649238630006e+02 },
    { 1.7630000000000000e-08, 2.5738273325900001e+00, 4.1555249061209997e+02 },
    { 1.7070000000000000e-08, 5.3888305856900001e+00, 3.0216477565500003e+02 },
    { 1.6560000000000001e-08, 1.2783474276200000e+00, 1.7394221952280000e+02 },
    { 1.6300000000000000e-08, 1.3197348228100001e+00, 3.9356875915200000e+01 },
    { 1.6130000000000000e-08, 3.4596651157100000e+00, 5.2947851677350002e+02 },
    { 1.6130000000000000e-08, 8.8102053266000002e-01, 5.2990341341570002e+02 },
    { 1.5980000000000001e-08, 2.5304308882400002e+00, 5.1153171782990000e+02 },
    { 1.5930000000000001e-08, 1.3731237044900000e+00, 1.5748458012822000e+03 },
    { 1.5790000000000001e-08, 3.9683684837900000e+00, 2.4209286360333999e+03 },
    { 1.5370000000000001e-08, 1.8439422350100001e+00, 5.4785021235930003e+02 },
    { 1.5100000000000000e-08, 3.4161173645399998e+00, 5.1864526483069994e+02 },
    { 1.5069999999999999e-08, 4.1698509739700000e+00, 1.3647280995819001e+03 },
    { 1.4780000000000001e-08, 3.7391225426100001e+00, 2.3056457082540001e+02 },
    { 1.3450000000000000e-08, 6.3863230610999999e-01, 1.0618296107440001e+03 },
    { 1.3230000000000000e-08, 4.0363801968899997e+00, 4.9426624244250002e+02 },
    { 1.3100000000000000e-08, 5.4545826458000004e+00, 1.0578974574808999e+03 },
    { 1.3020000000000000e-08, 3.5405027146300001e+00, 3.3406124266997999e+03 },
    { 1.2779999999999999e-08, 3.8639684237899998e+00, 9.2047073954699997e+01 },
    { 1.2420000000000000e-08, 1.6506467090100001e+00, 6.4382943957709995e+02 },
    { 1.2360000000000000e-08, 5.4198510615500002e+00, 1.0702492748170000e+02 },
    { 1.2320000000000000e-08, 2.8308841294399998e+00, 2.5382485042536000e+03 },
    { 1.1790000000000000e-08, 4.6253844112399998e+00, 1.0569342496344000e+03 },
    { 1.1700000000000001e-08, 1.0550891131500000e+00, 1.5128068240082000e+03 },
    { 1.1660000000000000e-08, 5.1288889208799997e+00, 1.0483362299252999e+03 },
    { 1.1650000000000000e-08, 1.4842967760600001e+00, 9.6022130923370003e+02 },
    { 1.1580000000000000e-08, 3.2499630540000002e-01, 5.6511568774670002e+02 },
    { 1.1530000000000000e-08, 4.7882876441300004e+00, 5.1471311156759998e+02 },
    { 1.1530000000000000e-08, 4.2274339078900001e+00, 2.7444340526907999e+03 },
    { 1.1210000000000001e-08, 6.2768803865600002e+00, 3.8724131496080003e+02 },
    { 1.1199999999999999e-08, 5.0223827141199999e+00, 1.4646394800628000e+03 },
    { 1.1129999999999999e-08, 5.7421783150900003e+00, 3.4803105662225998e+03 },
    { 1.1120000000000000e-08, 5.8356992657799998e+00, 5.4466881862160005e+02 },
    { 1.0980000000000000e-08, 1.0093708025599999e+00, 5.2354062594029995e+02 },
    { 1.0940000000000001e-08, 2.7337841749799998e+00, 2.9912639426919998e+02 },
    { 1.0859999999999999e-08, 2.8015707105600001e+00, 1.0380412891868000e+03 },
    { 1.0800000000000001e-08, 2.3293207346400000e+00, 1.0627416795630000e+02 },
    { 1.0530000000000001e-08, 5.4939412291999998e+00, 3.1781457905676002e+03 },
    { 1.0509999999999999e-08, 5.6277747303799996e+00, 2.2076295405954002e+03 },
    { 1.0200000000000000e-08, 1.7752062550000001e+00, 3.1245971639350000e+02 },
    { 9.9400000000000000e-09, 3.3315928375000001e+00, 5.3584130424889997e+02 },
    { 9.8400000000000008e-09, 4.2296004276600003e+00, 4.3053034413910001e+02 },
    { 9.7700000000000008e-09, 3.5083116568500001e+00, 2.4476805547999998e+00 },
    { 9.6600000000000001e-09, 3.5136334177199999e+00, 2.6342277314714001e+03 },
    { 9.5399999999999997e-09, 1.3401891197999999e-01, 3.9069087570985998e+03 },
    { 9.5000000000000007e-09, 3.8856762013599999e+00, 3.3772177920039999e+03 },
    { 9.4600000000000000e-09, 4.1371811694199998e+00, 1.6779385755008000e+03 },
    { 9.4099999999999996e-09, 3.0660455815999998e-01, 7.0849445304200003e+01 },
    { 9.3700000000000005e-09, 3.7759097456399999e+00, 2.6413412784722000e+03 },
    { 9.3200000000000001e-09, 4.1187924537700003e+00, 1.0594301142990000e+03 },
    { 9.3200000000000001e-09, 9.2098132613999995e-01, 1.0593337460794000e+03 },
    { 9.2900000000000008e-09, 2.3052163546800002e+00, 1.5142912967165000e+03 },
    { 9.2699999999999996e-09, 4.9176533651099996e+00, 1.5930050485469001e+03 },
    { 9.2500000000000001e-09, 1.2909178547000000e+00, 2.4351557300350000e+03 },
    { 9.1499999999999992e-09, 1.4830689152600001e+00, 1.0294940738499999e+01 },
    { 9.1100000000000002e-09, 3.3254780935500001e+00, 1.8159247264699999e+01 },
    { 8.9000000000000003e-09, 3.5828955799999999e-02, 6.4701083331480004e+02 },
    { 8.7199999999999997e-09, 5.0349895944399998e+00, 6.7925416222920001e+02 },
    { 8.3899999999999994e-09, 3.3453606523600001e+00, 1.2443341522100000e+02 },
    { 8.2599999999999992e-09, 5.0573834333500001e+00, 2.8475268269093999e+03 },
    { 8.2399999999999997e-09, 1.7447274873000000e+00, 5.2995159752550001e+02 },
    { 8.2399999999999997e-09, 2.5959581610700000e+00, 5.2943033266370003e+02 },
    { 8.2100000000000004e-09, 3.4147158932899999e+00, 2.9506196011279999e+03 },
    { 7.9200000000000008e-09, 5.1020310994100004e+00, 2.5240214102519999e+03 },
    { 7.8600000000000006e-09, 2.8228652680200002e+00, 1.1341635287565000e+03 },
    { 7.6199999999999997e-09, 4.3056410808600001e+00, 6.7214061522839995e+02 },
    { 7.3799999999999997e-09, 7.0435921398000001e-01, 5.2252923398400003e+02 },
};

static const VSOP87Term VSOP87A_JUPITER_Z1[104] = {
    { 4.0707217499999998e-03, 1.5269935348200001e+00, 5.2969096509459996e+02 },
    { 2.0307341000000000e-04, 2.5987826924799999e+00, 1.0593819301891999e+03 },
    { 1.5474611000000001e-04, 0.0000000000000000e+00, 0.0000000000000000e+00 },
    { 1.4424952999999999e-04, 4.8540015502499996e+00, 5.3680451209540001e+02 },
    { 1.2730363999999999e-04, 5.4553671573200004e+00, 5.2257741809380002e+02 },
    { 2.1008819999999999e-05, 9.5388642869999998e-02, 7.1135470007999997e+00 },
    { 1.2304250000000000e-05, 3.1422250024400000e+00, 1.5890728952838001e+03 },
    { 7.6063299999999999e-06, 5.2786734816200003e+00, 1.0664954771900000e+03 },
    { 6.7883199999999996e-06, 4.7489542278299997e+00, 1.0522683831884001e+03 },
    { 5.9701800000000003e-06, 1.0474805078200000e+00, 1.1020632121940000e+02 },
    { 5.7002400000000003e-06, 1.0941861936099999e+00, 1.0309277421860000e+02 },
    { 4.7303500000000004e-06, 5.0552897171000000e-01, 4.2659819087599999e+02 },
    { 4.3522800000000001e-06, 6.5531261911000005e-01, 4.1948464387519999e+02 },
    { 3.5161600000000001e-06, 5.4769028804700000e+00, 5.1546387109299997e+02 },
    { 3.4587000000000000e-06, 2.8117200214200002e+00, 6.3278373931320004e+02 },
    { 2.1135000000000002e-06, 4.7464612397200003e+00, 5.4391805909619995e+02 },
    { 1.9636499999999998e-06, 6.0528964281400004e+00, 3.2350541665740002e+02 },
    { 1.8027800000000000e-06, 3.9104110906099998e+00, 7.3587651353180001e+02 },
    { 1.6759900000000001e-06, 5.9704404870600003e+00, 3.1639186965660002e+02 },
    { 1.6476999999999999e-06, 2.8348702993099999e+00, 9.4917560896980001e+02 },
    { 1.2977500000000000e-06, 1.7344823576599999e+00, 2.0618554843720000e+02 },
    { 1.0435099999999999e-06, 1.1301642712100000e+00, 1.1624747044077999e+03 },
    { 9.2829000000000001e-07, 6.2216940823800000e+00, 1.4227094001599999e+01 },
    { 8.2564999999999999e-07, 4.8397147185599998e+00, 2.1329909543799999e+02 },
    { 7.9813000000000001e-07, 4.3840490910399996e+00, 5.3287235883230005e+02 },
    { 7.8752999999999996e-07, 4.6214207297099996e+00, 5.2575881183150000e+02 },
    { 7.4957000000000001e-07, 3.5759645860200000e+00, 2.1187638603783998e+03 },
    { 7.4155000000000002e-07, 5.8952436792500000e+00, 5.2650957135689998e+02 },
    { 6.3605000000000003e-07, 5.7663231125400003e+00, 5.3362311835770004e+02 },
    { 5.1134000000000002e-07, 1.6857109494500000e+00, 9.5628915597059995e+02 },
    { 4.4815000000000000e-07, 5.6977526687499997e+00, 1.5961864422845999e+03 },
    { 4.1188000000000001e-07, 8.2223688308999998e-01, 9.5979227217800002e+01 },
    { 4.0568000000000002e-07, 3.7196145156300000e+00, 7.4299006053259995e+02 },
    { 3.8907999999999999e-07, 9.3087494530000003e-01, 1.1731986822019999e+02 },
    { 3.7170999999999998e-07, 1.1094255018900001e+00, 8.4608283475120004e+02 },
    { 3.6944000000000000e-07, 3.5838560422399999e+00, 6.2567019231239999e+02 },
    { 3.5880000000000001e-07, 4.4407564507300004e+00, 1.0451548361876000e+03 },
    { 3.5615999999999998e-07, 4.8771863994500002e+00, 1.5819593482830001e+03 },
    { 3.4507000000000001e-07, 4.0774425050799996e+00, 6.3989728631399998e+02 },
    { 2.7819999999999999e-07, 5.5687361872399999e+00, 4.3371173787679999e+02 },
    { 2.4522999999999998e-07, 5.0521754898399998e+00, 2.2041264243879999e+02 },
    { 2.2917000000000001e-07, 4.3923247082700003e+00, 1.1695882514085999e+03 },
    { 2.2870000000000000e-07, 2.8959340910500000e+00, 9.4206206196899996e+02 },
    { 1.6843000000000001e-07, 8.3125469323000001e-01, 8.5319638175199998e+02 },
    { 1.4203000000000001e-07, 2.0204280765900000e+00, 1.6921656695024001e+03 },
    { 1.3841999999999999e-07, 4.9684731629399996e+00, 1.1553611574070001e+03 },
    { 1.3119000000000000e-07, 5.8960562282000000e+00, 3.3061896365820002e+02 },
    { 1.2828000000000001e-07, 5.6599976098500004e+00, 1.2655674786264001e+03 },
    { 1.1455000000000000e-07, 1.7147876785100000e+00, 3.0927832265580003e+02 },
    { 9.6950000000000005e-08, 1.2820164301999999e+00, 1.4788665740644001e+03 },
    { 9.0330000000000002e-08, 5.2919483934500002e+00, 4.1237109687440000e+02 },
    { 8.9500000000000001e-08, 5.1273942579399998e+00, 1.0736090241908000e+03 },
    { 8.4330000000000001e-08, 1.6692992177399999e+00, 1.9907200143640000e+02 },
    { 7.5069999999999997e-08, 5.6331462652899997e+00, 1.3757737998458001e+03 },
    { 5.6830000000000000e-08, 5.6484271645100002e+00, 5.0835032409220003e+02 },
    { 5.1219999999999998e-08, 7.8055029480000004e-01, 8.3896928775039999e+02 },
    { 5.0650000000000000e-08, 4.6187283652100000e+00, 1.2726810256271999e+03 },
    { 4.9549999999999997e-08, 6.2397852576000004e+00, 3.1813937377000001e+00 },
    { 4.6329999999999999e-08, 3.9696328608100000e+00, 2.6484548254729998e+03 },
    { 4.1610000000000000e-08, 4.7424179971199996e+00, 7.2876296653099996e+02 },
    { 4.1439999999999999e-08, 2.7205253609000002e-01, 1.7952584437210000e+03 },
    { 3.8899999999999998e-08, 4.9948565620899998e+00, 1.0625633239269000e+03 },
    { 3.7770000000000000e-08, 3.0507316031100000e+00, 6.1855664531160005e+02 },
    { 3.5980000000000002e-08, 6.1690386544900004e+00, 5.1939602435610004e+02 },
    { 3.3409999999999999e-08, 5.1535314086500001e+00, 1.0554497769261000e+03 },
    { 3.2810000000000001e-08, 5.8658984588600003e+00, 1.0562005364515001e+03 },
    { 3.1200000000000001e-08, 2.6141036129000002e+00, 1.3686602528450001e+03 },
    { 3.0190000000000001e-08, 4.8053153481899997e+00, 5.5103160609700001e+02 },
    { 2.9539999999999998e-08, 2.7791550688000000e-01, 1.0633140834523001e+03 },
    { 2.8730000000000000e-08, 2.5004532116900000e+00, 1.4859801210651999e+03 },
    { 2.8590000000000000e-08, 4.9415082586499999e+00, 2.2752618943959999e+02 },
    { 2.7990000000000001e-08, 4.4571773461199999e+00, 5.3998590583309999e+02 },
    { 2.7730000000000001e-08, 2.9591233229600000e+00, 3.9321532631000000e+00 },
    { 2.5920000000000000e-08, 3.9884889460000003e-02, 2.1258774073792001e+03 },
    { 2.5500000000000000e-08, 5.5986879288400004e+00, 2.1116503133776000e+03 },
    { 2.2939999999999999e-08, 3.7697079459999998e-02, 2.1340641002400002e+01 },
    { 2.1179999999999998e-08, 4.8155220182000003e+00, 1.6992792165031999e+03 },
    { 2.0879999999999999e-08, 3.3718500881800000e+00, 1.6850521225016000e+03 },
    { 2.0470000000000001e-08, 4.1412642327900002e+00, 1.5748458012822000e+03 },
    { 2.0210000000000000e-08, 1.4767758567800000e+00, 8.3185574074960005e+02 },
    { 1.9309999999999999e-08, 4.1734824999700004e+00, 4.4082528487759998e+02 },
    { 1.7490000000000000e-08, 1.9511350696500001e+00, 5.2502509864859996e+02 },
    { 1.6730000000000002e-08, 2.3870852566299998e+00, 5.3435683154059996e+02 },
    { 1.6260000000000001e-08, 2.2195319723400000e+00, 3.0216477565500003e+02 },
    { 1.5180000000000001e-08, 2.5438440841499999e+00, 2.2218566345969998e+03 },
    { 1.4620000000000000e-08, 6.1753218446700000e+00, 2.4280421830342002e+03 },
    { 1.3550000000000001e-08, 4.5614681773300001e+00, 7.5010360753340001e+02 },
    { 1.2490000000000000e-08, 1.0131731166999999e-01, 1.2584539316256000e+03 },
    { 1.1620000000000001e-08, 1.5141573892500000e+00, 2.0085575391590000e+03 },
    { 1.1609999999999999e-08, 4.5169744653599997e+00, 1.0380412891868000e+03 },
    { 1.0449999999999999e-08, 3.2458796255700002e+00, 2.0014439921582000e+03 },
    { 1.0379999999999999e-08, 5.6399194972400002e+00, 5.2427433912319998e+02 },
    { 1.0320000000000001e-08, 7.7806611202999998e-01, 7.4781598567299994e+01 },
    { 1.0250000000000001e-08, 2.8774845088499998e+00, 2.3178358618148000e+03 },
    { 1.0239999999999999e-08, 4.9839514724400003e+00, 5.3510759106600005e+02 },
    { 9.6099999999999997e-09, 2.0007035996300000e+00, 1.8983512179396000e+03 },
    { 9.4699999999999998e-09, 1.7438430458200000e+00, 2.2147430875962000e+03 },
    { 9.4600000000000000e-09, 1.4233259725600000e+00, 1.3828873468465999e+03 },
    { 9.0400000000000002e-09, 2.0024344531999999e-01, 1.9054647649404001e+03 },
    { 8.4900000000000003e-09, 4.4116204497800000e+00, 5.1153171782990000e+02 },
    { 8.2499999999999994e-09, 4.6815508957800001e+00, 5.2677020378780003e+02 },
    { 8.1899999999999992e-09, 5.9425478961599998e+00, 5.3261172640140001e+02 },
    { 8.1699999999999997e-09, 6.2265962369899999e+00, 5.4785021235930003e+02 },
    { 7.8800000000000001e-09, 5.9427811919100000e+00, 1.0294940738499999e+01 },
};

static const VSOP87Term VSOP87A_JUPITER_Z2[65] = {
    { 2.8635325999999998e-04, 3.0137416697299999e+00, 5.2969096509459996e+02 },
    { 3.1147520000000001e-05, 3.1322864617600001e+00, 5.3680451209540001e+02 },
    { 2.3797650000000001e-05, 9.5574345340000000e-01, 5.2257741809380002e+02 },
    { 1.3101110000000000e-05, 2.0526370491299999e+00, 1.0593819301891999e+03 },
    { 8.9875699999999997e-06, 0.0000000000000000e+00, 0.0000000000000000e+00 },
    { 3.0563500000000000e-06, 4.6421331843899996e+00, 7.1135470007999997e+00 },
    { 1.7890500000000000e-06, 3.6548266610300000e+00, 1.0664954771900000e+03 },
    { 1.2328300000000001e-06, 9.6219015811999997e-01, 5.1546387109299997e+02 },
    { 1.1667000000000000e-06, 2.2309739758400000e+00, 1.5890728952838001e+03 },
    { 1.1136300000000000e-06, 5.1687136283000001e-01, 1.0522683831884001e+03 },
    { 8.9255000000000003e-07, 3.0265352802500001e+00, 5.4391805909619995e+02 },
    { 4.2084000000000000e-07, 5.4294134590200001e+00, 1.1020632121940000e+02 },
    { 3.2518000000000000e-07, 4.5285029923200000e+00, 1.4227094001599999e+01 },
    { 2.7447000000000000e-07, 4.3564774868000002e+00, 3.2350541665740002e+02 },
    { 2.4417000000000000e-07, 5.4092547554600001e+00, 1.0309277421860000e+02 },
    { 2.2380999999999999e-07, 5.4665892224199997e+00, 4.2659819087599999e+02 },
    { 2.1934000000000000e-07, 2.6015847492000002e+00, 5.3287235883230005e+02 },
    { 1.8014000000000000e-07, 1.4445647182100001e+00, 5.2650957135689998e+02 },
    { 1.6331000000000000e-07, 4.9201419901900003e+00, 6.3278373931320004e+02 },
    { 1.3490000000000000e-07, 2.6154857753499998e+00, 5.2575881183150000e+02 },
    { 1.2741000000000000e-07, 6.0399114103200002e+00, 4.1948464387519999e+02 },
    { 1.2104999999999999e-07, 6.2153597072200002e+00, 1.0451548361876000e+03 },
    { 1.1558000000000001e-07, 4.1412921347699996e+00, 1.5961864422845999e+03 },
    { 1.0935000000000000e-07, 3.6712140123900001e+00, 2.0618554843720000e+02 },
    { 1.0688000000000000e-07, 5.5348454021500002e+00, 1.1731986822019999e+02 },
    { 1.0309000000000001e-07, 1.5676028837700000e+00, 5.3362311835770004e+02 },
    { 1.0158000000000000e-07, 5.2613509598999997e+00, 9.4917560896980001e+02 },
    { 9.7640000000000006e-08, 2.6282895168099998e+00, 9.5979227217800002e+01 },
    { 9.6849999999999997e-08, 2.1142805495400001e+00, 3.1639186965660002e+02 },
    { 9.1290000000000005e-08, 2.5833950527799998e+00, 2.1187638603783998e+03 },
    { 9.1129999999999996e-08, 2.2173856330000001e+00, 1.1624747044077999e+03 },
    { 8.5420000000000007e-08, 6.1349738729499999e+00, 7.3587651353180001e+02 },
    { 7.3230000000000001e-08, 3.2441886121000002e-01, 9.5628915597059995e+02 },
    { 6.9499999999999994e-08, 5.4583246738200000e+00, 6.2567019231239999e+02 },
    { 6.2299999999999995e-08, 3.8849150862099999e+00, 4.3371173787679999e+02 },
    { 5.4520000000000001e-08, 3.3919470398099998e+00, 2.2041264243879999e+02 },
    { 5.2460000000000001e-08, 6.3975690502000004e-01, 1.5819593482830001e+03 },
    { 5.0920000000000002e-08, 2.3832371723799999e+00, 6.3989728631399998e+02 },
    { 5.0069999999999998e-08, 2.6577085110200001e+00, 1.1695882514085999e+03 },
    { 4.7479999999999999e-08, 4.7097730019200004e+00, 9.4206206196899996e+02 },
    { 3.8140000000000003e-08, 4.4086899009499998e+00, 3.3061896365820002e+02 },
    { 3.5810000000000001e-08, 3.6169940075300002e+00, 1.0736090241908000e+03 },
    { 3.3540000000000001e-08, 5.5281492960199996e+00, 8.5319638175199998e+02 },
    { 3.1170000000000001e-08, 3.4635863812999999e+00, 1.9907200143640000e+02 },
    { 3.0680000000000001e-08, 1.2997131879099999e+00, 5.0835032409220003e+02 },
    { 3.0180000000000003e-08, 4.0383801301000000e-01, 1.1553611574070001e+03 },
    { 2.6930000000000000e-08, 6.2528343760300000e+00, 7.2876296653099996e+02 },
    { 2.3339999999999999e-08, 1.7451646715100000e+00, 4.1237109687440000e+02 },
    { 2.1880000000000001e-08, 3.8424489785499998e+00, 3.0927832265580003e+02 },
    { 1.9029999999999999e-08, 1.6889571387000000e+00, 5.1939602435610004e+02 },
    { 1.8080000000000000e-08, 3.2345886598800000e+00, 1.2655674786264001e+03 },
    { 1.6720000000000000e-08, 3.0499479760199999e+00, 5.5103160609700001e+02 },
    { 1.6470000000000001e-08, 2.1181186036999999e-01, 2.1329909543799999e+02 },
    { 1.5110000000000001e-08, 2.6253123077599998e+00, 5.3998590583309999e+02 },
    { 1.3960000000000001e-08, 4.2075445591699996e+00, 8.4608283475120004e+02 },
    { 1.2480000000000000e-08, 4.4561133808099997e+00, 3.1813937377000001e+00 },
    { 1.2470000000000000e-08, 4.6020553529099999e+00, 2.1340641002400002e+01 },
    { 1.2380000000000001e-08, 1.0926731086999999e+00, 8.3896928775039999e+02 },
    { 1.0929999999999999e-08, 3.3181830883000001e+00, 1.0625633239269000e+03 },
    { 1.0440000000000000e-08, 3.4970411767799998e+00, 2.2752618943959999e+02 },
    { 1.0099999999999999e-08, 1.6623283133700000e+00, 1.6921656695024001e+03 },
    { 9.8299999999999993e-09, 4.3186109556799996e+00, 7.4299006053259995e+02 },
    { 8.5199999999999995e-09, 2.6789355510299999e+00, 4.4082528487759998e+02 },
    { 8.3099999999999996e-09, 1.3795536126000001e+00, 1.2584539316256000e+03 },
    { 7.6899999999999997e-09, 2.3777754179500001e+00, 1.4859801210651999e+03 },
};

static const VSOP87Term VSOP87A_JUPITER_Z3[27] = {
    { 9.6435500000000002e-06, 4.7922841203199997e+00, 5.2969096509459996e+02 },
    { 4.4324400000000004e-06, 1.3996995299799999e+00, 5.3680451209540001e+02 },
    { 2.9560000000000002e-06, 2.8128140637299999e+00, 5.2257741809380002e+02 },
    { 1.1295200000000001e-06, 0.0000000000000000e+00, 0.0000000000000000e+00 },
    { 5.0664999999999995e-07, 6.2606246590400003e+00, 1.0593819301891999e+03 },
    { 2.9345000000000002e-07, 2.8518849146599998e+00, 7.1135470007999997e+00 },
    { 2.9100000000000000e-07, 2.7523525268100002e+00, 5.1546387109299997e+02 },
    { 2.8110000000000000e-07, 2.0438340627099998e+00, 1.0664954771900000e+03 },
    { 2.5368999999999999e-07, 1.2917522803000001e+00, 5.4391805909619995e+02 },
    { 1.3030000000000000e-07, 2.5479274356700001e+00, 1.0522683831884001e+03 },
    { 7.8300000000000006e-08, 2.8330931148400000e+00, 1.4227094001599999e+01 },
    { 7.1740000000000006e-08, 8.6931032543999998e-01, 1.5890728952838001e+03 },
    { 3.0799999999999998e-08, 1.8910749784500001e+00, 1.0451548361876000e+03 },
    { 2.8629999999999999e-08, 3.4329188678999998e+00, 1.1020632121940000e+02 },
    { 2.2420000000000002e-08, 4.0743942406800002e+00, 4.1948464387519999e+02 },
    { 1.9270000000000000e-08, 2.5807207130299998e+00, 3.2350541665740002e+02 },
    { 1.7570000000000001e-08, 3.9188139800499999e+00, 1.1731986822019999e+02 },
    { 1.7500000000000001e-08, 2.7491262351699999e+00, 1.5961864422845999e+03 },
    { 1.3000000000000001e-08, 3.0124653574000000e+00, 5.0835032409220003e+02 },
    { 1.2029999999999999e-08, 4.3021725351100004e+00, 9.5979227217800002e+01 },
    { 1.1940000000000000e-08, 1.7658581671000000e+00, 7.3587651353180001e+02 },
    { 1.1860000000000000e-08, 6.2664349222100002e+00, 6.3278373931320004e+02 },
    { 1.1310000000000000e-08, 5.4139327470699996e+00, 1.0309277421860000e+02 },
    { 1.0519999999999999e-08, 2.0079209841900001e+00, 1.0736090241908000e+03 },
    { 1.0449999999999999e-08, 9.4551854960000001e-02, 6.3989728631399998e+02 },
    { 9.2199999999999992e-09, 2.7894425978999998e-01, 9.4206206196899996e+02 },
    { 7.9699999999999996e-09, 1.5503588900400000e+00, 6.2567019231239999e+02 },
};

static const VSOP87Term VSOP87A_JUPITER_Z4[10] = {
    { 4.6366000000000001e-07, 5.9369539858099998e+00, 5.3680451209540001e+02 },
    { 2.7761000000000001e-07, 4.8028744146699998e+00, 5.2257741809380002e+02 },
    { 5.8579999999999999e-08, 4.5362202453400000e+00, 5.1546387109299997e+02 },
    { 5.5560000000000003e-08, 5.8167085902300002e+00, 5.4391805909619995e+02 },
    { 4.1829999999999998e-08, 1.5089481203799999e+00, 5.2969096509459996e+02 },
    { 2.6379999999999999e-08, 4.8959557023000000e-01, 1.0664954771900000e+03 },
    { 1.6140000000000001e-08, 9.0503544736999997e-01, 7.1135470007999997e+00 },
    { 1.5489999999999998e-08, 4.5869005971499996e+00, 1.0593819301891999e+03 },
    { 1.4899999999999999e-08, 4.5139084174299997e+00, 1.0522683831884001e+03 },
    { 1.1300000000000000e-08, 1.3299467026800000e+00, 1.4227094001599999e+01 },
};

static const VSOP87Term VSOP87A_JUPITER_Z5[3] = {
    { 3.4459999999999999e-08, 4.1041362646200001e+00, 5.3680451209540001e+02 },
    { 2.2430000000000000e-08, 8.2614663721000003e-01, 5.2257741809380002e+02 },
    { 1.0810000000000001e-08, 5.4984577690000001e-02, 5.1546387109299997e+02 },
};

static const VSOP87Block VSOP87A_JUPITER_X[6] = {
    { VSOP87A_JUPITER_X0, 1055 },
    { VSOP87A_JUPITER_X1, 488 },
    { VSOP87A_JUPITER_X2, 255 },
    { VSOP87A_JUPITER_X3, 140 },
    { VSOP87A_JUPITER_X4, 58 },
    { VSOP87A_JUPITER_X5, 11 },
};

static const VSOP87Block VSOP87A_JUPITER_Y[6] = {
    { VSOP87A_JUPITER_Y0, 1037 },
    { VSOP87A_JUPITER_Y1, 499 },
    { VSOP87A_JUPITER_Y2, 259 },
    { VSOP87A_JUPITER_Y3, 136 },
    { VSOP87A_JUPITER_Y4, 60 },
    { VSOP87A_JUPITER_Y5, 11 },
};

static const VSOP87Block VSOP87A_JUPITER_Z[6] = {
    { VSOP87A_JUPITER_Z0, 216 },
    { VSOP87A_JUPITER_Z1, 104 },
    { VSOP87A_JUPITER_Z2, 65 },
    { VSOP87A_JUPITER_Z3, 27 },
    { VSOP87A_JUPITER_Z4, 10 },
    { VSOP87A_JUPITER_Z5, 3 },
};

static const int32 VSOP87A_JUPITER_TRUNC_COUNT[11][3][6] = {
    {
        { 1055, 488, 255, 140, 58, 11 },
        { 1037, 499, 259, 136, 60, 11 },
        { 216, 104, 65, 27, 10, 3 },
    },
    {
        { 739, 352, 179, 97, 49, 8 },
        { 740, 351, 177, 103, 49, 8 },
        { 189, 95, 61, 25, 10, 3 },
    },
    {
        { 412, 191, 105, 59, 23, 5 },
        { 415, 189, 103, 60, 22, 5 },
        { 99, 57, 39, 12, 4, 0 },
    },
    {
        { 329, 138, 92, 48, 15, 5 },
        { 328, 140, 90, 51, 17, 5 },
        { 80, 49, 27, 10, 2, 0 },
    },
    {
        { 171, 88, 58, 21, 10, 2 },
        { 169, 90, 58, 20, 10, 2 },
        { 49, 30, 11, 5, 0, 0 },
    },
    {
        { 136, 75, 51, 13, 6, 2 },
        { 137, 76, 48, 13, 5, 2 },
        { 39, 22, 10, 4, 0, 0 },
    },
    {
        { 77, 47, 24, 9, 2, 0 },
        { 76, 46, 23, 9, 2, 0 },
        { 22, 11, 5, 1, 0, 0 },
    },
    {
        { 56, 37, 14, 8, 2, 0 },
        { 56, 38, 14, 8, 2, 0 },
        { 18, 7, 4, 0, 0, 0 },
    },
    {
        { 37, 16, 9, 2, 0, 0 },
        { 37, 16, 9, 2, 0, 0 },
        { 9, 5, 1, 0, 0, 0 },
    },
    {
        { 28, 13, 6, 2, 0, 0 },
        { 28, 13, 6, 2, 0, 0 },
        { 6, 5, 1, 0, 0, 0 },
    },
    {
        { 14, 4, 2, 0, 0, 0 },
        { 14, 4, 2, 0, 0, 0 },
        { 3, 1, 0, 0, 0, 0 },
    },
};

