// Copyright Epic Games, Inc. All Rights Reserved.

// Auto-generated VSOP87A header for MARS (rectangular XYZ, J2000)
// Source: IMCCE/BDL VSOP87A.mars file
// Generated by vsop87a_header_gen_imcce.py

#pragma once

#include "VSOP87A_Common.h"

static const VSOP87Term VSOP87A_MARS_X0[1584] = {
    { 1.5176993638300000e+00, 6.2040334654800002e+00, 3.3406124266997999e+03 },
    { 1.9502945246000000e-01, 3.1415926535900001e+00, 0.0000000000000000e+00 },
    { 7.0709196550000006e-02, 2.5870338558000000e-01, 6.6812248533995999e+03 },
    { 4.9419691400000004e-03, 5.9669127767999997e-01, 1.0021837280099400e+04 },
    { 4.0938237000000000e-04, 9.3473307418999996e-01, 1.3362449706799200e+04 },
    { 2.1067199000000000e-04, 1.8043565615399999e+00, 3.3370893083507999e+03 },
    { 2.1041625999999999e-04, 1.1789561947400000e+00, 3.3441355450487999e+03 },
    { 1.3527976000000000e-04, 6.3010765169000005e-01, 5.2969096509459996e+02 },
    { 1.1370375000000000e-04, 4.8326521110899998e+00, 1.0593819301891999e+03 },
    { 8.2260689999999996e-05, 1.8684351953500000e+00, 2.2812304965106000e+03 },
    { 6.7741070000000006e-05, 3.6178504828200002e+00, 3.3405951730475999e+03 },
    { 6.7740600000000000e-05, 5.6486221143100002e+00, 3.3406296803519999e+03 },
    { 5.4690459999999998e-05, 1.1332442900299999e+00, 2.9424634232916001e+03 },
    { 5.2762600000000002e-05, 2.3314808311599999e+00, 6.1515338883049999e+03 },
    { 4.9375790000000002e-05, 4.4324144065400004e+00, 5.6218429232103999e+03 },
    { 4.8171340000000000e-05, 1.8509104553600000e+00, 3.7387614301079998e+03 },
    { 3.7297459999999997e-05, 1.2139832363700001e+00, 3.9814900340819997e+02 },
    { 3.7258229999999997e-05, 1.2728018294300001e+00, 1.6703062133498999e+04 },
    { 3.6366670000000003e-05, 6.1139759210599998e+00, 7.9629800681639995e+02 },
    { 2.3978650000000001e-05, 6.3553674053999998e-01, 3.1491641605882000e+03 },
    { 2.3685130000000000e-05, 2.9684189536000001e+00, 2.5443144198834002e+03 },
    { 2.2746460000000000e-05, 2.3570832885300002e+00, 3.5320606928113998e+03 },
    { 2.2410100000000001e-05, 4.8221865531099999e+00, 8.9624553499102003e+03 },
    { 2.2291759999999999e-05, 1.6958896251300000e+00, 3.3405451163970001e+03 },
    { 2.2291169999999999e-05, 1.2873932382100000e+00, 3.3406797370026002e+03 },
    { 2.1822060000000000e-05, 1.6965511296900000e+00, 6.2830758499914000e+03 },
    { 1.9775790000000001e-05, 2.1408782611000001e+00, 6.6777017350506003e+03 },
    { 1.9581620000000000e-05, 1.5191454455500000e+00, 6.6847479717486003e+03 },
    { 1.6776930000000001e-05, 3.1444261204599999e+00, 5.8849268465832001e+03 },
    { 1.6304820000000002e-05, 2.4117974845000001e-01, 4.1369104335162001e+03 },
    { 1.5725400000000000e-05, 4.8480992178899998e+00, 1.5890728952838001e+03 },
    { 1.4681240000000000e-05, 1.8786973054300000e+00, 3.8703033917943999e+03 },
    { 1.3784700000000000e-05, 2.1801190002099999e+00, 1.7515395314160000e+03 },
    { 1.2901700000000000e-05, 4.3596325295999999e-01, 2.8109214616052000e+03 },
    { 1.2898040000000000e-05, 4.7097077862100001e+00, 1.1944470102246000e+03 },
    { 9.5675199999999992e-06, 5.3699422739199996e+00, 4.2659819087599999e+02 },
    { 8.1945799999999997e-06, 5.1588416764899998e+00, 4.3999943568890003e+03 },
    { 7.2012099999999996e-06, 2.6553906786199999e+00, 9.4921463150048003e+03 },
    { 7.1904800000000003e-06, 2.9114534041200000e+00, 1.9144826611159999e+02 },
    { 7.0913100000000000e-06, 9.2869188035000005e-01, 2.1329909543799999e+02 },
    { 7.0871200000000000e-06, 4.6956271336900004e+00, 5.4867778431750003e+03 },
    { 7.0284800000000000e-06, 4.6759000372199999e+00, 2.1461654164751999e+03 },
    { 6.3118600000000002e-06, 3.9556967973699999e+00, 6.6812075997474003e+03 },
    { 6.3118600000000002e-06, 5.9864684288700003e+00, 6.6812421070518003e+03 },
    { 5.8228700000000002e-06, 3.9529596777700000e+00, 3.1851920272655998e+03 },
    { 5.5556800000000003e-06, 5.3201460407700001e+00, 3.4960328261340001e+03 },
    { 5.3512200000000002e-06, 3.3031012013900001e+00, 1.5925960136327999e+03 },
    { 4.9219000000000003e-06, 2.1760653080800001e+00, 7.0793738568077997e+03 },
    { 4.1776900000000000e-06, 3.9648279691899999e+00, 3.3396321056316001e+03 },
    { 4.1765200000000003e-06, 5.3016360108300002e+00, 3.3415927477680002e+03 },
    { 3.7075200000000002e-06, 5.0812714818800000e+00, 8.4327643848156004e+03 },
    { 3.6002799999999999e-06, 1.6108912290099999e+00, 2.0043674560198800e+04 },
    { 3.5303599999999999e-06, 3.8867842964900001e+00, 6.2546266625236003e+03 },
    { 3.5207900000000000e-06, 5.1704371792900004e+00, 1.2303067776610000e+04 },
    { 3.4604200000000001e-06, 6.1640411933100001e+00, 5.0886288397668004e+03 },
    { 3.2157800000000001e-06, 1.9464428319600000e+00, 3.5539115221378001e+03 },
    { 3.1387499999999998e-06, 5.1065196659999996e+00, 4.5350594369244000e+03 },
    { 3.0705999999999999e-06, 9.6613024299999992e-03, 1.7480164130670000e+03 },
    { 3.0192400000000001e-06, 9.8460899560000004e-01, 3.1273133312618002e+03 },
    { 2.8721099999999998e-06, 2.0753130341500001e+00, 7.1135470007999997e+00 },
    { 2.7181500000000000e-06, 1.9492569199500001e+00, 6.4679257579615996e+03 },
    { 2.6011599999999999e-06, 1.8958734890200000e+00, 1.9907450170410000e+03 },
    { 2.5306800000000001e-06, 1.3982886388700000e+00, 2.9140142358237999e+03 },
    { 2.3405800000000000e-06, 2.0375013092100001e+00, 9.6236882766912004e+03 },
    { 2.2335899999999998e-06, 3.5535456306800000e+00, 3.3198370312073998e+03 },
    { 2.2161499999999999e-06, 5.7128207678400003e+00, 3.3613878221922000e+03 },
    { 2.1871799999999999e-06, 9.7547090193999997e-01, 6.4897765872880000e+03 },
    { 2.0785099999999999e-06, 2.4784199643500000e+00, 1.0018314161750401e+04 },
    { 2.0770799999999999e-06, 2.0337320518099999e+00, 6.6811575430967996e+03 },
    { 2.0770799999999999e-06, 1.6252356649599999e+00, 6.6812921637024001e+03 },
    { 2.0521700000000000e-06, 1.8579083568500001e+00, 1.0025360398448400e+04 },
    { 1.9325699999999999e-06, 4.9322464690699999e+00, 2.1187638603783998e+03 },
    { 1.9038099999999999e-06, 1.2852735599800000e+00, 4.6904798363585996e+03 },
    { 1.7881399999999999e-06, 2.5477608418100002e+00, 1.2218485663214001e+03 },
    { 1.7717699999999999e-06, 4.2708102305500004e+00, 2.9577158944766002e+03 },
    { 1.6844999999999999e-06, 5.0692903407700003e+00, 3.7235089589230001e+03 },
    { 1.6632300000000000e-06, 3.1690536803199998e+00, 1.0213285546211000e+04 },
    { 1.6471000000000001e-06, 5.7591122271999995e-01, 7.4775228602159996e+03 },
    { 1.6123100000000001e-06, 1.5311823435800000e+00, 1.3498674096587999e+03 },
    { 1.5730700000000000e-06, 4.8852895156999998e-01, 2.3888940204492001e+03 },
    { 1.5394400000000001e-06, 4.4447311943200001e+00, 6.3989728631399998e+02 },
    { 1.4771100000000001e-06, 2.4758865312400000e+00, 7.2109158184941998e+03 },
    { 1.4684899999999999e-06, 4.4161440759000001e+00, 3.3370219980480001e+03 },
    { 1.4621100000000000e-06, 4.8472356334100004e+00, 3.3442028553516002e+03 },
    { 1.4409799999999999e-06, 5.3630524838599998e+00, 2.7870430238573999e+03 },
    { 1.3514600000000001e-06, 1.2922395244000001e-01, 7.9030734197210004e+03 },
    { 1.2616600000000001e-06, 2.6430900833400002e+00, 4.2923308329503998e+03 },
    { 1.2511599999999999e-06, 3.9742138047200002e+00, 3.8941818295421999e+03 },
    { 1.2031899999999999e-06, 3.4096606615699998e+00, 9.2255392732830005e+03 },
    { 1.1942500000000000e-06, 4.6554564322000003e+00, 1.1773376811515400e+04 },
    { 1.1812499999999999e-06, 5.1857086609699996e+00, 4.9296853219836003e+03 },
    { 1.1524000000000000e-06, 5.8655429701499999e+00, 1.5542039943419999e+02 },
    { 1.1232800000000000e-06, 7.4982073090000004e-02, 3.2055473466643998e+03 },
    { 1.1230500000000000e-06, 3.4570146262799999e+00, 3.3334988796990001e+03 },
    { 1.0678800000000001e-06, 2.9105087874800000e+00, 3.4756775067352000e+03 },
    { 1.0651500000000001e-06, 5.5016281536499996e+00, 3.8289653222319998e+02 },
    { 9.7207000000000000e-07, 2.9986567656099998e+00, 9.5171840625059997e+02 },
    { 9.6089999999999998e-07, 5.8181922452699997e+00, 3.3477259737006002e+03 },
    { 9.2022999999999998e-07, 4.2611387039900004e+00, 9.5952390892233998e+03 },
    { 8.8762999999999996e-07, 2.9869201505700000e+00, 1.2832758741704600e+04 },
    { 8.8423999999999998e-07, 4.6788407861999998e-01, 3.0978838227257902e+03 },
    { 8.5394000000000004e-07, 3.6855358093500001e+00, 4.9332084403325998e+03 },
    { 8.4575000000000001e-07, 9.8139044514999996e-01, 6.6741113063987996e+03 },
    { 8.0693000000000004e-07, 2.5426249989700000e+00, 3.5833410306738001e+03 },
    { 7.3967000000000001e-07, 5.8900473802200004e+00, 3.7672106175757999e+03 },
    { 7.3608000000000001e-07, 5.4857003299100002e+00, 7.7406067835887998e+03 },
    { 7.3165000000000002e-07, 5.0316529230200002e+00, 1.0522683831884001e+03 },
    { 7.2878999999999995e-07, 4.4389584642799997e+00, 5.5356940284240000e+02 },
    { 6.9810000000000004e-07, 5.7367795967199999e+00, 5.0921519581158000e+03 },
    { 6.6212999999999996e-07, 2.3447226992600001e+00, 2.7007151403858002e+03 },
    { 6.6184000000000000e-07, 4.1319201520000001e-02, 1.0021854533751601e+04 },
    { 6.6184000000000000e-07, 4.2937328772100001e+00, 1.0021820026447200e+04 },
    { 6.6160000000000005e-07, 9.4815285206000000e-01, 3.5231183490000002e+00 },
    { 6.6069999999999998e-07, 1.6038851242900001e+00, 1.0664954771900000e+03 },
    { 5.7874999999999996e-07, 5.0574412452999997e+00, 6.0413275670856001e+03 },
    { 5.4166999999999995e-07, 4.2929295460999999e+00, 6.5258044539654002e+03 },
    { 5.3323999999999998e-07, 2.5086303575800000e+00, 1.0419986283507600e+04 },
    { 5.0165000000000000e-07, 2.1473698982000000e+00, 4.5624609930212000e+03 },
    { 4.9918999999999996e-07, 5.0929980081300004e+00, 3.3121632392319998e+03 },
    { 4.9213000000000000e-07, 4.1730614060500004e+00, 3.3690616141676001e+03 },
    { 4.8698000000000004e-07, 5.0989994191000001e+00, 2.2741169495098002e+03 },
    { 4.7370000000000001e-07, 5.5150727032300004e+00, 1.5643680203309799e+04 },
    { 4.6382999999999999e-07, 1.7371707301999999e+00, 2.2883440435113998e+03 },
    { 4.4704000000000001e-07, 4.3107220168799998e+00, 3.0775285033270002e+03 },
    { 4.3268999999999999e-07, 1.9463287572400001e+00, 1.4956319713459999e+02 },
    { 4.3108999999999998e-07, 2.7131907511800000e+00, 3.3253599555147998e+03 },
    { 4.2866999999999999e-07, 2.7117433127000001e-01, 3.3558648978848000e+03 },
    { 4.1472000000000000e-07, 3.6089396585400002e+00, 5.3680451209540001e+02 },
    { 4.0051999999999999e-07, 4.9536041243400000e+00, 3.6036963500726001e+03 },
    { 3.9073000000000001e-07, 4.3037910913499999e+00, 6.6802445323313996e+03 },
    { 3.9049000000000000e-07, 7.4753075599999996e-01, 5.2257741809380002e+02 },
    { 3.8939999999999998e-07, 5.6397386293900000e+00, 6.6822051744678001e+03 },
    { 3.8513000000000001e-07, 2.3513022527700000e+00, 6.8945239488376001e+03 },
    { 3.6267000000000000e-07, 1.9490292884100000e+00, 2.3384286986898602e+04 },
    { 3.6133999999999998e-07, 2.2759272728800002e+00, 9.8085381846614000e+03 },
    { 3.3826000000000001e-07, 2.1428099910000000e+00, 2.4272860397400001e+02 },
    { 3.2385999999999999e-07, 5.4405778061700003e+00, 7.8756718636242003e+03 },
    { 3.1936000000000002e-07, 4.2284775201000002e+00, 5.6289564702112002e+03 },
    { 3.1693000000000002e-07, 5.1148301893600001e+00, 2.6484548254729998e+03 },
    { 3.1613000000000001e-07, 1.9251888841800000e+00, 1.0309277421860000e+02 },
    { 3.1516000000000002e-07, 2.2648792721700000e+00, 5.3313574437407997e+03 },
    { 2.9903000000000000e-07, 5.0914323278999996e-01, 7.3733824546264004e+03 },
    { 2.9766999999999999e-07, 5.4199108322700003e+00, 3.1781457905676002e+03 },
    { 2.9415000000000002e-07, 1.4170396640600000e+00, 5.6147293762095996e+03 },
    { 2.7780000000000001e-07, 3.7668482030099999e+00, 3.5030790628320001e+03 },
    { 2.7674999999999999e-07, 2.3800762219600000e+00, 1.2964300703391000e+04 },
    { 2.7623999999999999e-07, 1.5159825300300001e+00, 8.4292412664666008e+03 },
    { 2.5193000000000001e-07, 3.5005183306999998e-01, 3.3350895023923999e+03 },
    { 2.5134999999999998e-07, 2.6304582682400000e+00, 3.3461353510071999e+03 },
    { 2.4954000000000000e-07, 2.4767694694800002e+00, 1.6484467571974001e+03 },
    { 2.4892999999999999e-07, 2.9480255656000001e+00, 6.9215760122680001e+02 },
    { 2.4400000000000001e-07, 1.6200955679200000e+00, 7.6266071275599998e+01 },
    { 2.3774000000000001e-07, 3.7928368887100001e+00, 3.3202571073009999e+03 },
    { 2.3600000000000000e-07, 6.0570505093899998e+00, 1.0713994881326200e+04 },
    { 2.3575999999999999e-07, 5.4725418573800004e+00, 3.3609677460985899e+03 },
    { 2.3244999999999999e-07, 2.8133578070200000e+00, 1.7586530784168001e+03 },
    { 2.3113999999999999e-07, 4.2439967131699996e+00, 1.6921656695024001e+03 },
    { 2.3013999999999999e-07, 2.8146110666399999e+00, 1.3358926588450200e+04 },
    { 2.2753000000000000e-07, 1.3129932920200000e+00, 9.8303890139878004e+03 },
    { 2.2690000000000000e-07, 2.1946930161200000e+00, 1.3365972825148199e+04 },
    { 2.2490999999999999e-07, 3.0508277161700001e+00, 8.0310922630584000e+03 },
    { 2.2361000000000000e-07, 4.8611313364299997e+00, 1.5113989238215199e+04 },
    { 2.1831000000000000e-07, 1.9616467261099999e+00, 1.0021904590402200e+04 },
    { 2.1831000000000000e-07, 2.3701431129500001e+00, 1.0021769969796600e+04 },
    { 2.1451999999999999e-07, 3.9048500211600001e+00, 6.6604494579071998e+03 },
    { 2.1283000000000001e-07, 3.6889274817000001e-01, 2.7078286873866000e+03 },
    { 2.0716000000000001e-07, 6.0488255559799997e+00, 6.7020002488919999e+03 },
    { 2.0657000000000000e-07, 4.0904852768800000e+00, 1.5819593482830001e+03 },
    { 2.0575000000000000e-07, 9.0645437334000001e-01, 6.4384962494255997e+03 },
    { 1.9863000000000000e-07, 5.2627766252899999e+00, 9.3819399937853996e+03 },
    { 1.9621999999999999e-07, 3.5163580032300001e+00, 8.5319638175199998e+02 },
    { 1.9593999999999999e-07, 1.2029077039100000e+00, 2.8180350086059998e+03 },
    { 1.9324000000000000e-07, 9.7030055777000002e-01, 3.3299757613500001e+03 },
    { 1.9312000000000000e-07, 5.6713110463599996e+00, 6.8366452528338004e+03 },
    { 1.9240999999999999e-07, 2.0119745214700000e+00, 3.3512490920496002e+03 },
    { 1.8911000000000000e-07, 1.7083829771900001e+00, 3.8631898447936001e+03 },
    { 1.7704999999999999e-07, 9.0939692872000000e-01, 1.0818135286915800e+04 },
    { 1.7630000000000001e-07, 7.7500497806000002e-01, 3.0740053849780002e+03 },
    { 1.7286000000000000e-07, 9.0425997403000002e-01, 6.8726731195111997e+03 },
    { 1.7186999999999999e-07, 5.0209158436200001e+00, 1.1243685846420800e+04 },
    { 1.6780999999999999e-07, 2.4483108149400001e+00, 3.9733961660129999e+03 },
    { 1.6775999999999999e-07, 3.2191514396700001e+00, 4.0327700279266001e+03 },
    { 1.6533000000000001e-07, 2.1479684623400002e+00, 6.1586474353058002e+03 },
    { 1.6341999999999999e-07, 4.6098233571699998e+00, 6.2983283211764001e+03 },
    { 1.6318000000000001e-07, 2.9086034890299999e+00, 1.0551528245194000e+04 },
    { 1.6254000000000000e-07, 5.6386262807799996e+00, 6.1444203413041996e+03 },
    { 1.6241999999999999e-07, 2.2126590274600000e+00, 3.6072194684216001e+03 },
    { 1.6212000000000000e-07, 8.4608813574999997e-01, 5.7295064471490005e+03 },
    { 1.5669999999999999e-07, 4.4663183819699999e+00, 3.3335661900017999e+03 },
    { 1.5437000000000000e-07, 5.3788824544400002e+00, 7.4781598567299994e+01 },
    { 1.5246000000000000e-07, 5.2763761753800003e+00, 5.4593762870782002e+03 },
    { 1.4994000000000000e-07, 1.0398521920199999e+00, 6.5316616562649997e+03 },
    { 1.4286000000000000e-07, 5.7434628738000004e-01, 1.0404733812322600e+04 },
    { 1.3969999999999999e-07, 2.3721491539300001e+00, 4.1948464387519999e+02 },
    { 1.3932000000000001e-07, 4.3513047925999998e-01, 6.5461597733642002e+03 },
    { 1.3925000000000001e-07, 4.6108115878799998e+00, 1.2935851515923199e+04 },
    { 1.3778999999999999e-07, 5.2489867818800002e+00, 4.3371173787679999e+02 },
    { 1.3766999999999999e-07, 4.7509069731500002e+00, 6.6776344247478000e+03 },
    { 1.3736999999999999e-07, 5.7095487398499998e+00, 6.1276554505572003e+03 },
    { 1.3694000000000001e-07, 4.6514554679400000e+00, 3.3476586633978000e+03 },
    { 1.3658000000000000e-07, 9.5367846072999995e-01, 7.0641213856227996e+03 },
    { 1.3640000000000000e-07, 5.5941167431799999e+00, 3.4153940252671000e+03 },
    { 1.3547000000000001e-07, 5.1859269174799998e+00, 6.6848152820513997e+03 },
    { 1.3442999999999999e-07, 1.5824370746800001e+00, 1.5961864422845999e+03 },
    { 1.3330000000000001e-07, 1.7363396784900000e+00, 8.9553418029093991e+03 },
    { 1.3281000000000001e-07, 3.7032514851000000e+00, 3.2658308281324998e+03 },
    { 1.3162999999999999e-07, 6.2176255465499999e+00, 6.6883384004004001e+03 },
    { 1.3026000000000000e-07, 3.9304758355099998e+00, 1.1769853693166400e+04 },
    { 1.2994000000000001e-07, 3.5051498776600001e+00, 8.8273902698748007e+03 },
    { 1.2966000000000001e-07, 4.6663264608499997e+00, 8.9695688969109997e+03 },
    { 1.2925000000000000e-07, 4.6785432133900002e+00, 7.6329432596502002e+03 },
    { 1.2720000000000000e-07, 2.2848321237600002e+00, 3.3367310913418000e+03 },
    { 1.2702999999999999e-07, 6.9084968318999995e-01, 3.3444937620577998e+03 },
    { 1.2310000000000001e-07, 3.3281342248099999e+00, 2.4874160449477999e+03 },
    { 1.2214999999999999e-07, 1.3087307706700000e+00, 1.0014723733098601e+04 },
    { 1.2132999999999999e-07, 4.8750965747199997e+00, 3.9805097130138001e+03 },
    { 1.2130000000000001e-07, 2.0065143304099999e+00, 3.4437052009183999e+03 },
    { 1.2109999999999999e-07, 4.6378863329600000e+00, 3.2812385647862002e+03 },
    { 1.1916000000000000e-07, 5.4442218338300004e+00, 1.7444259844152000e+03 },
    { 1.1758000000000000e-07, 4.6178231150700002e+00, 3.3999862886134001e+03 },
    { 1.1535000000000000e-07, 3.3825070567700002e+00, 3.0242205570432002e+03 },
    { 1.1417999999999999e-07, 8.8286540120000001e-01, 3.2375196524812000e+03 },
    { 1.1412000000000001e-07, 3.5548817306300000e+00, 1.1371704689758200e+04 },
    { 1.1351000000000000e-07, 4.9805741503499998e+00, 4.9890591838971995e+03 },
    { 1.1152999999999999e-07, 3.2580778978099998e+00, 6.8162899334350004e+03 },
    { 1.1148000000000000e-07, 5.5227271177599997e+00, 8.2702977486834006e+03 },
    { 1.0902000000000001e-07, 2.9090017645600001e+00, 1.1020632121940000e+02 },
    { 1.0824000000000000e-07, 8.8978834813999996e-01, 3.3167339889519999e+03 },
    { 1.0768000000000000e-07, 3.3254904940099999e+00, 1.6173371168404399e+04 },
    { 1.0722000000000000e-07, 2.0865018059799998e+00, 3.3644908644476000e+03 },
    { 1.0563000000000000e-07, 4.7608815746099999e+00, 1.0973555686350001e+04 },
    { 1.0474999999999999e-07, 5.8713938994300001e+00, 3.6570042963564001e+03 },
    { 1.0423000000000000e-07, 2.3209650357800000e+00, 1.2168002696574600e+04 },
    { 9.7279999999999995e-08, 2.2682815296499999e+00, 1.3506508003540000e+02 },
    { 9.3789999999999995e-08, 7.0747070827000003e-01, 6.3278373931320004e+02 },
    { 9.3139999999999999e-08, 3.9104715017300000e+00, 1.4584298273120599e+04 },
    { 9.2869999999999997e-08, 6.0398832223900003e+00, 3.3045845600224002e+03 },
    { 9.1600000000000000e-08, 3.2186372146600002e+00, 3.3766402933772001e+03 },
    { 9.0079999999999997e-08, 4.8860428096000001e+00, 6.7843176276182003e+03 },
    { 8.9260000000000007e-08, 4.0185136124799996e+00, 8.2738208670324002e+03 },
    { 8.9060000000000005e-08, 9.3197483563000005e-01, 3.3410325027934000e+03 },
    { 8.9060000000000005e-08, 2.0437143851599999e+00, 3.3401923506061898e+03 },
    { 8.5790000000000003e-08, 5.8455172082500004e+00, 5.0992655051166003e+03 },
    { 8.3879999999999994e-08, 3.3590234187800001e+00, 1.2566151699982800e+04 },
    { 8.2730000000000000e-08, 1.3601812025100000e+00, 6.6049587821240002e+03 },
    { 8.1489999999999997e-08, 2.7297483890500001e+00, 2.9211277828246002e+03 },
    { 8.1250000000000003e-08, 8.6848770670000006e-02, 5.8280284716475999e+03 },
    { 7.8300000000000006e-08, 3.3719678113399998e+00, 1.6246663613219999e+02 },
    { 7.6560000000000005e-08, 5.8160484146000000e+00, 1.1081219210288600e+04 },
    { 7.6090000000000002e-08, 5.0837114755600004e+00, 3.3040070613956000e+03 },
    { 7.5670000000000002e-08, 3.4402990733499998e+00, 3.2090704650133998e+03 },
    { 7.5460000000000002e-08, 4.1710431717600001e+00, 3.1344268782626000e+03 },
    { 7.5160000000000006e-08, 2.8808930355900002e+00, 6.9239534573736000e+03 },
    { 7.4750000000000004e-08, 4.1783865282099999e+00, 3.3772177920039999e+03 },
    { 7.4470000000000004e-08, 4.2866271339799997e+00, 1.8617458526354001e+03 },
    { 7.3650000000000001e-08, 7.4165927300000001e-01, 3.7600970705750001e+03 },
    { 7.3199999999999994e-08, 4.6324162025800000e+00, 1.3362432453146999e+04 },
    { 7.3000000000000005e-08, 3.7690059591000002e-01, 1.3362466960451400e+04 },
    { 7.2359999999999994e-08, 5.0848366286399997e+00, 3.5467979751369999e+03 },
    { 6.9440000000000005e-08, 5.4382000601899998e+00, 8.3296716105970008e+03 },
    { 6.8610000000000004e-08, 3.8005044620800001e+00, 3.1699395560806001e+03 },
    { 6.8499999999999998e-08, 5.8261428710600001e+00, 3.4721543883862000e+03 },
    { 6.8320000000000006e-08, 3.8727752248399998e+00, 6.6064432548323002e+03 },
    { 6.8070000000000000e-08, 1.3521095675000000e-01, 7.2347942562420003e+03 },
    { 6.6689999999999997e-08, 5.3468831926300000e+00, 3.8133035637799999e+01 },
    { 6.6460000000000001e-08, 1.4170826428700001e+00, 1.3757737998458001e+03 },
    { 6.5140000000000000e-08, 2.7864543204499999e+00, 1.1187557921027999e+03 },
    { 6.4949999999999996e-08, 1.9778717019999999e-02, 7.1078230442756003e+03 },
    { 6.4669999999999996e-08, 2.2495366140000000e-02, 1.0184303916231600e+04 },
    { 6.4529999999999996e-08, 5.4630874115500001e+00, 3.5112852973190002e+03 },
    { 6.4459999999999996e-08, 5.7838418440600003e+00, 3.3410423098265001e+03 },
    { 6.4459999999999996e-08, 3.4815177281700000e+00, 3.3401825435730998e+03 },
    { 6.4389999999999996e-08, 2.3364197957099999e+00, 1.4788665740644001e+03 },
    { 6.4329999999999994e-08, 5.9649022156699996e+00, 1.1624747044077999e+03 },
    { 6.4319999999999996e-08, 4.8636537012699996e+00, 3.2968935143948001e+03 },
    { 6.4090000000000000e-08, 6.1867710678499996e+00, 1.0575406682941801e+04 },
    { 6.3619999999999997e-08, 4.3985067839100003e+00, 3.3843313390048002e+03 },
    { 6.2940000000000006e-08, 4.2789981333000000e+00, 2.2218566345969998e+03 },
    { 6.2660000000000007e-08, 6.6358217013999998e-01, 2.2041264243879999e+02 },
    { 6.2629999999999999e-08, 5.3141429715899999e+00, 1.9648386268540000e+03 },
    { 6.2499999999999997e-08, 2.8430990831099998e+00, 9.5628915597059995e+02 },
    { 6.2289999999999997e-08, 4.4006299976700003e+00, 2.6087903141574199e+04 },
    { 6.2139999999999999e-08, 5.7694497613999998e+00, 8.4398779318163997e+03 },
    { 6.0390000000000000e-08, 7.2624360734000004e-01, 6.4087760738220004e+02 },
    { 6.0370000000000004e-08, 5.8526872521500000e+00, 1.8984292630009601e+04 },
    { 5.9740000000000004e-08, 2.8402549551199998e+00, 1.3760598710207400e+04 },
    { 5.8649999999999999e-08, 3.9474583455199999e+00, 2.0618554843720000e+02 },
    { 5.7779999999999999e-08, 3.0366156776300000e+00, 2.1781377222920000e+03 },
    { 5.6550000000000000e-08, 4.6327761096700000e+00, 9.8664168806651996e+03 },
    { 5.6179999999999997e-08, 2.0497353801800000e+00, 2.6997348193175999e+03 },
    { 5.5560000000000003e-08, 4.5484711164900000e+00, 9.4917560896980001e+02 },
    { 5.3830000000000000e-08, 3.1833136053900000e+00, 1.2289621133221999e+03 },
    { 5.3809999999999997e-08, 1.7428941000100000e+00, 2.7662676283649998e+03 },
    { 5.2889999999999999e-08, 7.8643508759999994e-02, 1.3916019109641600e+04 },
    { 5.2399999999999999e-08, 4.4359951597300000e+00, 2.2434479570190001e+02 },
    { 5.1779999999999998e-08, 3.3828437313599999e+00, 3.2741250177853999e+03 },
    { 5.1319999999999999e-08, 1.4012731748500000e+00, 3.9149572250346000e+03 },
    { 5.0379999999999998e-08, 4.0080621409799999e+00, 6.2475131155228000e+03 },
    { 5.0300000000000000e-08, 5.8735666621000000e+00, 3.4070998356141999e+03 },
    { 4.8680000000000003e-08, 5.4447862706199999e+00, 4.3928808098882000e+03 },
    { 4.8350000000000000e-08, 5.0715170814099997e+00, 1.4314168113049800e+04 },
    { 4.8000000000000000e-08, 1.9353818750600000e+00, 4.4071079038897997e+03 },
    { 4.7820000000000001e-08, 7.6221170409000005e-01, 6.2617402095243997e+03 },
    { 4.7010000000000002e-08, 4.0232702700500003e+00, 2.1116503133776000e+03 },
    { 4.6269999999999997e-08, 3.6650549185000001e+00, 2.8038079146044001e+03 },
    { 4.5860000000000002e-08, 4.5071734212700001e+00, 6.7096740408674004e+03 },
    { 4.5669999999999998e-08, 5.4734598131599999e+00, 1.3524916342931399e+04 },
    { 4.5569999999999997e-08, 1.5370234861800001e+00, 2.8073983432562000e+03 },
    { 4.3889999999999998e-08, 2.6034298544799999e+00, 1.3149150611361199e+04 },
    { 4.3830000000000003e-08, 3.0606053718799999e+00, 6.6659723822145997e+03 },
    { 4.3719999999999997e-08, 6.1246867826900004e+00, 9.1686408983473993e+03 },
    { 4.3520000000000002e-08, 2.7100258636599999e+00, 1.0235136375537400e+04 },
    { 4.3159999999999997e-08, 4.8467947980000003e+00, 3.8774169387952002e+03 },
    { 4.1369999999999999e-08, 1.7492105394199999e+00, 5.7434479833479998e+02 },
    { 4.1070000000000003e-08, 4.6413784389200003e+00, 1.0020856959031200e+04 },
    { 4.1050000000000000e-08, 6.2939165403999997e-01, 2.6308392337279997e+02 },
    { 4.0949999999999999e-08, 5.9773899642200004e+00, 1.0022817601167601e+04 },
    { 4.0490000000000001e-08, 1.1267632934700000e+00, 3.9814900340820000e+03 },
    { 4.0189999999999998e-08, 6.0853100264000004e-01, 6.6964773245846000e+03 },
    { 3.9620000000000000e-08, 1.8339063684100001e+00, 6.0484411140864004e+03 },
    { 3.9540000000000002e-08, 5.4043111886600004e+00, 6.6527756659318002e+03 },
    { 3.9260000000000002e-08, 1.7526089266900000e+00, 2.6660704172179999e+02 },
    { 3.9039999999999998e-08, 3.7715899197999998e+00, 2.3843232707292000e+03 },
    { 3.8220000000000001e-08, 2.2162669351700002e+00, 3.4901756238344001e+03 },
    { 3.7809999999999999e-08, 1.5148402580600000e+00, 2.9069006888230001e+03 },
    { 3.7669999999999999e-08, 2.2871505029099999e+00, 2.6724899413598399e+04 },
    { 3.7660000000000001e-08, 5.9863161124399999e+00, 1.4712317116458000e+04 },
    { 3.6909999999999998e-08, 7.7943306000000004e-02, 6.5710185321802001e+03 },
    { 3.5910000000000002e-08, 1.5548337143400000e+00, 3.8738265101433999e+03 },
    { 3.5439999999999998e-08, 1.4200659023200000e+00, 1.3517870106233400e+04 },
    { 3.4959999999999997e-08, 5.7717570748700000e+00, 1.1216284290324000e+04 },
    { 3.3899999999999999e-08, 5.1509151199099996e+00, 1.8454601664915001e+04 },
    { 3.3489999999999997e-08, 8.7562878675000000e-01, 1.0177257679533601e+04 },
    { 3.3250000000000003e-08, 3.9339640639000001e-01, 6.8436914895317996e+03 },
    { 3.3109999999999997e-08, 2.5990952580300002e+00, 8.6719698704406001e+03 },
    { 3.2929999999999998e-08, 1.7053788502600000e+00, 9.7791086761253991e+03 },
    { 3.2829999999999997e-08, 3.5730649733700002e+00, 1.7277406931833801e+04 },
    { 3.2689999999999997e-08, 2.4757822762699999e+00, 6.0403472460173998e+03 },
    { 3.2679999999999999e-08, 2.7186588237899998e+00, 1.6304913130090799e+04 },
    { 3.2660000000000003e-08, 3.2378424427899999e+00, 1.3936794505133999e+04 },
    { 3.2600000000000001e-08, 2.0651941444499999e+00, 8.4256508378147992e+03 },
    { 3.2339999999999997e-08, 3.2364232717700001e+00, 5.5187501489918004e+03 },
    { 3.2299999999999998e-08, 4.3313976511499996e+00, 6.4701083331480004e+02 },
    { 3.1540000000000003e-08, 6.4171873872999996e-01, 5.2023582793351998e+03 },
    { 3.1149999999999998e-08, 1.5406669426099999e+00, 5.3054510535538002e+03 },
    { 3.0090000000000000e-08, 3.3769522415200002e+00, 2.0085575391590000e+03 },
    { 2.9499999999999999e-08, 5.2530894648400004e+00, 6.9443087767724001e+03 },
    { 2.9480000000000000e-08, 1.1536251119600001e+00, 6.4146178116778001e+03 },
    { 2.8839999999999999e-08, 6.2526874705599997e+00, 6.6123292678099995e+02 },
    { 2.8609999999999999e-08, 1.9890536352099999e+00, 3.3420968994080999e+03 },
    { 2.8460000000000001e-08, 3.3618989187100001e+00, 1.3320548875408001e+03 },
    { 2.8360000000000001e-08, 9.8608001394000000e-01, 3.3391279539914999e+03 },
    { 2.7409999999999999e-08, 1.3674165608400000e+00, 9.8722740829647992e+03 },
    { 2.7400000000000001e-08, 7.8084432440000004e-02, 7.9101869667217998e+03 },
    { 2.7170000000000001e-08, 5.7632278355200004e+00, 6.5187582172674001e+03 },
    { 2.7070000000000000e-08, 1.8076821274799999e+00, 1.6858482532933202e+04 },
    { 2.6619999999999999e-08, 3.5486844473299999e+00, 8.4608283475120004e+02 },
    { 2.6580000000000000e-08, 4.9068525388199999e+00, 3.1910492295652002e+03 },
    { 2.6449999999999999e-08, 5.4677728362700000e+00, 5.9890672521728002e+03 },
    { 2.6230000000000001e-08, 3.1543470481700000e+00, 1.6699539015149989e+04 },
    { 2.5819999999999999e-08, 2.5348896569599999e+00, 1.6706585251847999e+04 },
    { 2.5320000000000001e-08, 4.8633360744300003e+00, 6.4750393049623999e+03 },
    { 2.5310000000000000e-08, 3.9831557175099999e+00, 1.5110466119866200e+04 },
    { 2.5279999999999999e-08, 1.6534731716900000e+00, 1.3171001440687600e+04 },
    { 2.4900000000000001e-08, 5.1390890871800003e+00, 1.2722552420485201e+04 },
    { 2.4830000000000001e-08, 3.8726006509199999e+00, 3.3132108706029999e+03 },
    { 2.4769999999999999e-08, 4.7566668551799998e+00, 5.7320492444297997e+03 },
    { 2.4560000000000000e-08, 5.3928348791399996e+00, 3.3680139827966000e+03 },
    { 2.4480000000000001e-08, 4.5908069727100003e+00, 6.4181409300267997e+03 },
    { 2.4369999999999999e-08, 9.9170531280000002e-02, 5.7249356974290004e+03 },
    { 2.4330000000000000e-08, 3.9761796252699999e+00, 3.1710322435667999e+03 },
    { 2.4200000000000002e-08, 1.7794787048400000e+00, 3.5610250691386000e+03 },
    { 2.4159999999999999e-08, 2.3012084023099999e+00, 1.3362517017102000e+04 },
    { 2.4159999999999999e-08, 2.7097047891599999e+00, 1.3362382396496399e+04 },
    { 2.3989999999999998e-08, 2.6342059651200000e+00, 1.5508615123274400e+04 },
    { 2.3549999999999999e-08, 2.2072753157600000e+00, 6.4608122109608003e+03 },
    { 2.3520000000000001e-08, 6.8899979183000004e-01, 6.6757019290921999e+03 },
    { 2.3339999999999999e-08, 6.0009467425200000e+00, 9.4850327680040009e+03 },
    { 2.3050000000000001e-08, 4.0965524246999996e+00, 1.7924910699820401e+04 },
    { 2.3000000000000001e-08, 1.0545713786899999e+00, 3.1201997842609999e+03 },
    { 2.2810000000000000e-08, 3.8579047081500000e+00, 6.6430918177617996e+03 },
    { 2.2740000000000000e-08, 2.9662955093900001e+00, 6.6867477777069998e+03 },
    { 2.2729999999999999e-08, 4.2479588721699999e+00, 1.0001061884606999e+04 },
    { 2.2469999999999999e-08, 2.2220960681399999e+00, 3.3433639373107999e+03 },
    { 2.2469999999999999e-08, 7.6007819691000000e-01, 3.3378609160888000e+03 },
    { 2.2209999999999998e-08, 4.1309410037500003e+00, 6.6608695340007998e+03 },
    { 2.2150000000000000e-08, 2.4868510284599998e+00, 9.4992598620055996e+03 },
    { 2.2099999999999999e-08, 9.8734172819999999e-01, 5.3321408344359997e+02 },
    { 2.2020000000000001e-08, 5.0601130671799996e+00, 2.0775395492400001e+01 },
    { 2.1940000000000000e-08, 5.2515749770899998e+00, 3.5101926098327999e+03 },
    { 2.1850000000000000e-08, 1.0437662983000000e-01, 1.0042612675591799e+04 },
    { 2.1710000000000001e-08, 2.8265295374799999e+00, 7.3140085927128002e+03 },
    { 2.1670000000000001e-08, 1.6887163810500001e+00, 2.1258774073792001e+03 },
    { 2.1559999999999999e-08, 5.2595237681200002e+00, 1.7952584437210000e+03 },
    { 2.1520000000000000e-08, 1.9325513362100000e+00, 1.4227094001599999e+01 },
    { 2.1500000000000001e-08, 4.4492585630999998e-01, 1.7256631536341400e+04 },
    { 2.1360000000000001e-08, 4.6897595623999999e-01, 2.8449187467800002e+01 },
    { 2.1089999999999999e-08, 2.1150455497300000e+00, 7.2038022714934004e+03 },
    { 2.1019999999999999e-08, 3.2313760513399998e+00, 1.3745346239022399e+04 },
    { 2.0710000000000001e-08, 2.9521892642899998e+00, 1.0596182078434200e+04 },
    { 2.0660000000000001e-08, 2.7033971471400000e+00, 1.3119721102825190e+04 },
    { 2.0570000000000002e-08, 1.5539657221000001e+00, 1.5453539829788001e+03 },
    { 2.0510000000000000e-08, 5.8063556570200001e+00, 1.2147350193206000e+03 },
    { 2.0160000000000000e-08, 5.8033311682099997e+00, 6.7015801727983999e+03 },
    { 1.9989999999999999e-08, 3.7840802332200001e+00, 3.2304061054804001e+03 },
    { 1.9890000000000001e-08, 1.2493740913700000e+00, 1.4158747713615599e+04 },
    { 1.9810000000000000e-08, 5.0323747595799997e+00, 1.2310181323610799e+04 },
    { 1.9799999999999999e-08, 3.1899330628300002e+00, 1.6460333529524989e+04 },
    { 1.9630000000000001e-08, 9.7426402219999997e-01, 8.5429707060349992e+03 },
    { 1.9630000000000001e-08, 4.8687980856399999e+00, 6.4568800576977001e+03 },
    { 1.9510000000000001e-08, 1.1054938591100001e+00, 7.9982112516539996e+02 },
    { 1.9210000000000001e-08, 2.2581970273600001e+00, 2.6758563815697999e+03 },
    { 1.9180000000000000e-08, 2.8481015401900001e+00, 2.9459865416406001e+03 },
    { 1.8940000000000000e-08, 1.5474780851100001e+00, 7.9277488846740005e+02 },
    { 1.8870000000000000e-08, 5.7275528813000003e+00, 7.7999806455024000e+03 },
    { 1.8839999999999999e-08, 4.3882284635000002e-01, 1.3553897972910800e+04 },
    { 1.8699999999999999e-08, 5.8471003637999996e-01, 3.1131362939107998e+03 },
    { 1.8670000000000001e-08, 3.3273301739700001e+00, 1.3892140671893800e+04 },
    { 1.8660000000000000e-08, 5.5984679043999996e+00, 1.0390266107903999e+03 },
    { 1.8620000000000001e-08, 1.3168618932300000e+00, 6.6705881880498000e+03 },
    { 1.8559999999999999e-08, 1.9705556739700001e+00, 3.4168784979754000e+03 },
    { 1.8439999999999998e-08, 4.9541177551600004e+00, 1.6276463942623001e+04 },
    { 1.8270000000000001e-08, 4.4138733126399998e+00, 2.7515475996916002e+03 },
    { 1.8270000000000001e-08, 2.1379248307399998e+00, 1.2295954229609200e+04 },
    { 1.8050000000000000e-08, 2.3500930635500001e+00, 6.6918615187493997e+03 },
    { 1.7789999999999999e-08, 6.7914472160999995e-01, 3.2350541665740002e+02 },
    { 1.7479999999999998e-08, 4.5708682258000000e+00, 5.0327780962021998e+03 },
    { 1.7090000000000000e-08, 1.3173287359100001e+00, 2.3914368177299998e+03 },
    { 1.7000000000000000e-08, 5.6589045382499998e+00, 1.4351476617594001e+03 },
    { 1.7000000000000000e-08, 2.3969990747400001e+00, 3.5680885594888000e+03 },
    { 1.6829999999999999e-08, 1.9768885489000001e+00, 8.6460634802536006e+03 },
    { 1.6720000000000000e-08, 2.5507577116500002e+00, 6.9478318951213996e+03 },
    { 1.6630000000000001e-08, 3.8339196679200001e+00, 2.2777073781616000e+03 },
    { 1.6600000000000000e-08, 4.9456126793599999e+00, 9.6389407478762005e+03 },
    { 1.6510000000000000e-08, 5.9104936829600003e+00, 6.7560064519669004e+03 },
    { 1.6470000000000001e-08, 1.1863086011599999e+00, 9.0701188738487999e+03 },
    { 1.6040000000000000e-08, 3.4810627905499998e+00, 6.9927114822759995e+02 },
    { 1.6000000000000001e-08, 1.4071728048200001e+00, 1.9054647649404001e+03 },
    { 1.5990000000000000e-08, 5.0245184023800000e+00, 3.2125935833623998e+03 },
    { 1.5959999999999999e-08, 4.2136250061400000e+00, 3.9321532631000000e+00 },
    { 1.5650000000000001e-08, 5.3186010464300004e+00, 7.4299006053259995e+02 },
    { 1.5630000000000002e-08, 4.7403323478399999e+00, 6.6741786167015998e+03 },
    { 1.5609999999999999e-08, 5.1781061390899996e+00, 3.4508187479192002e+03 },
    { 1.5559999999999998e-08, 3.3848392635800000e+00, 7.8959598727202001e+03 },
    { 1.5550000000000000e-08, 2.7389127072000002e-01, 1.0028950827100200e+04 },
    { 1.5530000000000001e-08, 3.8995044543300001e+00, 4.1938088084518004e+03 },
    { 1.5519999999999999e-08, 7.5522820685000003e-01, 9.8867722000640006e+03 },
    { 1.5480000000000000e-08, 1.5343691144400000e+00, 6.5781320791810003e+03 },
    { 1.5469999999999999e-08, 1.6410092081300001e+00, 1.3355336159798400e+04 },
    { 1.5309999999999999e-08, 7.8845566079999996e-01, 4.0053684718298000e+03 },
    { 1.5259999999999999e-08, 3.9781988920000000e-01, 6.6475604512999996e+02 },
    { 1.5220000000000000e-08, 4.2424735843999999e+00, 3.4686312700372000e+03 },
    { 1.5150000000000000e-08, 4.4084121595800001e+00, 4.9225717749828000e+03 },
    { 1.5030000000000000e-08, 2.6778913449000002e-01, 3.3787454623376002e+03 },
    { 1.5020000000000001e-08, 3.7934148618600001e+00, 3.1639186965660002e+02 },
    { 1.4920000000000001e-08, 3.5940057944100001e+00, 6.1550570066540004e+03 },
    { 1.4820000000000000e-08, 5.7789503720699997e+00, 2.6047359131680000e+03 },
    { 1.4690000000000000e-08, 2.6252458221500001e+00, 1.0156630178842000e+03 },
    { 1.4680000000000000e-08, 2.4859840621000000e+00, 2.6936015933849999e+03 },
    { 1.4629999999999999e-08, 2.7191881985999999e-01, 3.3187615973734000e+03 },
    { 1.4629999999999999e-08, 2.7102554451900001e+00, 3.3624632560261998e+03 },
    { 1.4620000000000000e-08, 5.6988848151799996e+00, 1.2995225377836799e+04 },
    { 1.4590000000000000e-08, 5.5252672785400003e+00, 2.6793794999187999e+03 },
    { 1.4550000000000000e-08, 5.6208803778499998e+00, 8.7999887137780006e+03 },
    { 1.4440000000000001e-08, 5.0905862521899996e+00, 1.0018246851447600e+04 },
    { 1.4400000000000000e-08, 3.9017527963799998e+00, 2.6413412784722000e+03 },
    { 1.4340000000000000e-08, 5.7369896245099996e+00, 3.2837140517642001e+03 },
    { 1.4300000000000001e-08, 5.5256082023299999e+00, 1.0025427708751200e+04 },
    { 1.4279999999999999e-08, 6.0514586094400000e+00, 9.4682678772569998e+03 },
    { 1.4240000000000000e-08, 1.8796442003400000e+00, 1.7654780539749601e+04 },
    { 1.4040000000000000e-08, 3.5939545513100000e+00, 1.9800945956224801e+04 },
    { 1.4030000000000001e-08, 3.5279294223800002e+00, 3.3975108016354002e+03 },
    { 1.4000000000000000e-08, 3.1722893063500002e+00, 5.8906482700820004e+02 },
    { 1.3980000000000000e-08, 4.3713851288800001e+00, 9.5881255422226004e+03 },
    { 1.3510000000000000e-08, 1.6929902748600001e+00, 9.9455712088237997e+03 },
    { 1.3469999999999999e-08, 3.6330929776500001e+00, 1.9513983595104201e+04 },
    { 1.3440000000000000e-08, 5.4074511103600003e+00, 4.6726673142405998e+03 },
    { 1.3440000000000000e-08, 3.5411465519699998e+00, 5.9373861913600003e+01 },
    { 1.3350000000000000e-08, 5.0214739251899996e+00, 4.4593682188025996e+03 },
    { 1.3290000000000000e-08, 3.1968044550500001e+00, 3.3472960905739001e+03 },
    { 1.3290000000000000e-08, 6.0685551171799998e+00, 3.3339287628256998e+03 },
    { 1.3259999999999999e-08, 5.5583661417099997e+00, 5.8351420186484002e+03 },
    { 1.3230000000000000e-08, 1.6177637471000000e-01, 1.1909238918756000e+03 },
    { 1.3119999999999999e-08, 5.1713551158000000e+00, 3.3111829181637900e+03 },
    { 1.3119999999999999e-08, 4.0940044564300004e+00, 3.3700419352357999e+03 },
    { 1.2930000000000001e-08, 5.2414987914799998e+00, 1.0124930054318000e+04 },
    { 1.2860000000000001e-08, 1.8746379601100001e+00, 3.1526872789372101e+03 },
    { 1.2730000000000001e-08, 5.6703758771099997e+00, 5.6253660415594004e+03 },
    { 1.2720000000000001e-08, 6.2571869728399996e+00, 6.9976167230561996e+03 },
    { 1.2690000000000000e-08, 5.1993170462300000e+00, 3.1887151456145998e+03 },
    { 1.2660000000000001e-08, 5.6156002868700003e+00, 6.8874104018367998e+03 },
    { 1.2590000000000001e-08, 3.2653541699000002e+00, 3.7422845484569998e+03 },
    { 1.2400000000000000e-08, 2.5457020646299999e+00, 3.3024793910620001e+03 },
    { 1.2330000000000000e-08, 3.4901851099600001e+00, 6.6487408914400007e+01 },
    { 1.2290000000000000e-08, 2.5376487968300001e+00, 3.1456410422392000e+03 },
    { 1.2229999999999999e-08, 5.7478209685700001e+00, 1.1670284037296800e+04 },
    { 1.2220000000000000e-08, 4.5420343913499996e+00, 3.1842117061974000e+03 },
    { 1.2200000000000000e-08, 5.1770874347599998e+00, 6.0342140200847998e+03 },
    { 1.2180000000000001e-08, 3.5965509902699999e+00, 1.0156902360134800e+04 },
    { 1.2050000000000000e-08, 4.8978723418300003e+00, 6.6882710900975999e+03 },
    { 1.2000000000000000e-08, 5.4993829038599999e+00, 2.9709126107594002e+03 },
    { 1.1930000000000000e-08, 1.0322229583400000e+00, 6.6851061887575997e+03 },
    { 1.1930000000000000e-08, 2.6261998929799999e+00, 6.6773435180416000e+03 },
    { 1.1860000000000000e-08, 3.1059456143499999e+00, 2.2847536148596000e+03 },
    { 1.1830000000000001e-08, 3.1546183831899999e+00, 1.7395219734725801e+04 },
    { 1.1770000000000001e-08, 3.6725310709499999e+00, 2.3015858159093900e+03 },
    { 1.1749999999999999e-08, 3.4687340494400001e+00, 4.0764889402315998e+03 },
    { 1.1710000000000000e-08, 5.2071377934500003e+00, 2.7717905526723998e+03 },
    { 1.1670000000000000e-08, 9.9889492368999999e-01, 9.6023526362241992e+03 },
    { 1.1660000000000000e-08, 3.8670901082800002e+00, 4.8859664096786000e+03 },
    { 1.1530000000000000e-08, 5.4187936190199997e+00, 6.7405987153132000e+03 },
    { 1.1460000000000000e-08, 4.7238424047600001e+00, 3.4970131472021999e+03 },
    { 1.1280000000000001e-08, 4.9800306987000003e+00, 6.6218509914859997e+03 },
    { 1.1210000000000001e-08, 3.9414293181999999e-01, 9.0655481241287998e+03 },
    { 1.1199999999999999e-08, 6.0978593721600003e+00, 1.5252471184999999e+01 },
    { 1.1160000000000000e-08, 6.0645202613700002e+00, 5.6183198048614004e+03 },
    { 1.1160000000000000e-08, 5.8619255859199999e+00, 1.1610910175383200e+04 },
    { 1.1150000000000001e-08, 4.6296273124600003e+00, 1.6062184526116800e+04 },
    { 1.1110000000000000e-08, 5.0248550885099998e+00, 1.9402796952816600e+04 },
    { 1.1020000000000000e-08, 3.9844310390100000e+00, 6.3648329837430001e+03 },
    { 1.1020000000000000e-08, 5.2642003545899998e+00, 7.3211221397135996e+03 },
    { 1.1010000000000001e-08, 5.7899085134600003e+00, 3.7743241645766002e+03 },
    { 1.0870000000000001e-08, 3.9730793434999998e+00, 5.6421982426092000e+03 },
    { 1.0859999999999999e-08, 4.3209744418099998e+00, 2.3249494088155998e+03 },
    { 1.0719999999999999e-08, 3.2894340773100001e+00, 6.1480107699560003e+03 },
    { 1.0710000000000000e-08, 3.0090906484200000e+00, 7.2555696517344004e+03 },
    { 1.0690000000000000e-08, 2.2785705956900002e+00, 9.3890535407862008e+03 },
    { 1.0660000000000001e-08, 5.9437924226299996e+00, 1.1979701285736001e+03 },
    { 1.0620000000000000e-08, 1.6376166836299999e+00, 1.3154196168639999e+02 },
    { 1.0530000000000001e-08, 4.9996563419999998e+00, 1.4054607308025999e+04 },
    { 1.0430000000000000e-08, 4.0675602498399996e+00, 3.4925097077850000e+03 },
    { 1.0390000000000001e-08, 1.9243046130999999e+00, 4.9367988689843996e+03 },
    { 1.0300000000000000e-08, 4.1783129345100001e+00, 1.2721572099417001e+04 },
    { 1.0300000000000000e-08, 3.7656303533900002e+00, 3.7103122426402001e+03 },
    { 1.0290000000000000e-08, 1.8060024870200000e+00, 1.4844727083000000e+00 },
    { 1.0179999999999999e-08, 1.3969298352499999e+00, 2.5976223661672002e+03 },
    { 1.0140000000000000e-08, 5.9363275107700000e+00, 8.8593625756916008e+03 },
    { 1.0090000000000000e-08, 1.2310553151300001e+00, 6.6573464156518003e+03 },
    { 1.0060000000000000e-08, 3.2808498663000002e-01, 9.6546129511370000e+03 },
    { 1.0039999999999999e-08, 4.8611137051800002e+00, 1.6335837804536601e+04 },
    { 9.9100000000000007e-09, 1.4957420160199999e+00, 4.1404335518651997e+03 },
    { 9.8999999999999993e-09, 5.9541108248199999e+00, 3.1816689089165998e+03 },
    { 9.7800000000000006e-09, 1.6317214381600000e+00, 1.1766263264514600e+04 },
    { 9.7800000000000006e-09, 3.6940983228299999e+00, 2.5382485042536000e+03 },
    { 9.7700000000000008e-09, 1.0544926158800001e+00, 3.3619480671667998e+03 },
    { 9.7700000000000008e-09, 1.9276816491700000e+00, 3.3192767862328001e+03 },
    { 9.6799999999999997e-09, 6.0740605942199997e+00, 1.2726810256271999e+03 },
    { 9.6699999999999999e-09, 4.3496056131199996e+00, 1.1614433293732200e+04 },
    { 9.6600000000000001e-09, 4.3635654361999998e-01, 8.9589322315612008e+03 },
    { 9.5999999999999999e-09, 4.0556509629799997e+00, 3.9094343007272000e+03 },
    { 9.5800000000000004e-09, 3.0786416617599999e+00, 2.4092493398480001e+03 },
    { 9.5700000000000007e-09, 1.0887247031000000e-01, 6.6451969867221997e+03 },
    { 9.5599999999999992e-09, 1.8458081832000001e-01, 2.0809467624645200e+04 },
    { 9.5000000000000007e-09, 2.1842803605099999e+00, 2.0199094959632999e+04 },
    { 9.4300000000000007e-09, 6.2298659929000000e-01, 3.3441527987009999e+03 },
    { 9.4300000000000007e-09, 5.5007803193499996e+00, 3.3370720546985999e+03 },
    { 9.4300000000000007e-09, 1.2483666436600001e+00, 3.3371065620029999e+03 },
    { 9.4300000000000007e-09, 4.8754002749799996e+00, 3.3441182913966099e+03 },
    { 9.3800000000000003e-09, 3.5669339272400000e+00, 9.3809596727171993e+03 },
    { 9.2900000000000008e-09, 3.6473319661599999e+00, 4.0018453534808000e+03 },
    { 9.2500000000000001e-09, 3.3979229922299998e+00, 1.4859801210651999e+03 },
    { 9.1000000000000004e-09, 5.9129709709800000e+00, 9.3136308685179995e+02 },
    { 9.0799999999999993e-09, 3.6388340705200002e+00, 3.3263853326981998e+03 },
    { 8.9000000000000003e-09, 4.4759752719999997e-02, 6.7477122623140003e+03 },
    { 8.9000000000000003e-09, 6.0815333953100001e+00, 8.9659784682591999e+03 },
    { 8.8800000000000008e-09, 4.5676454811000000e-01, 3.6605365304199999e+01 },
    { 8.8800000000000008e-09, 6.0293498356899997e+00, 1.7067287061920001e+02 },
    { 8.8599999999999996e-09, 2.8693484139999999e-02, 8.0132797409404002e+03 },
    { 8.7899999999999996e-09, 5.6644153137400002e+00, 3.3548395207014000e+03 },
    { 8.7799999999999999e-09, 3.6195363254399999e+00, 1.6958018313299999e+02 },
    { 8.7299999999999994e-09, 5.0718226570000002e-01, 2.0679314010725998e+03 },
    { 8.7199999999999997e-09, 3.4701095150499999e+00, 3.2329489027611999e+03 },
    { 8.6800000000000006e-09, 4.2272384725099998e+00, 9.9470556815320997e+03 },
    { 8.6699999999999992e-09, 7.1131990195999995e-01, 7.3804960016271998e+03 },
    { 8.6399999999999999e-09, 3.5600439966600002e+00, 6.7172527200770000e+03 },
    { 8.5600000000000002e-09, 2.3800603119699999e+00, 6.7051032911474003e+03 },
    { 8.5400000000000007e-09, 2.6854522989300000e+00, 2.0750449480734001e+03 },
    { 8.5299999999999993e-09, 5.2003136039999998e-01, 2.4150080051345001e+04 },
    { 8.4800000000000005e-09, 5.3653037020200003e+00, 1.1780490358516199e+04 },
    { 8.4100000000000005e-09, 5.8083538941999997e+00, 3.4482759506384000e+03 },
    { 8.3500000000000003e-09, 4.6463627187499998e+00, 3.9296772537080001e+03 },
    { 8.3400000000000006e-09, 2.3850812004300002e+00, 6.6808047773059998e+03 },
    { 8.3400000000000006e-09, 1.2733416509000000e+00, 6.6816449294931999e+03 },
    { 8.2499999999999994e-09, 5.8410920259300001e+00, 3.2533042221599999e+03 },
    { 8.2100000000000004e-09, 1.4553263571799999e+00, 3.3584249488178002e+03 },
    { 8.2100000000000004e-09, 1.5268479078799999e+00, 3.3227999045818001e+03 },
    { 8.1899999999999992e-09, 3.4053702800700001e+00, 8.0236392244620004e+02 },
    { 8.1400000000000004e-09, 4.9697875608200004e+00, 1.6703044879846799e+04 },
    { 8.1400000000000004e-09, 7.1737388513000000e-01, 1.6703079387151189e+04 },
    { 8.1400000000000004e-09, 5.2586126168299998e+00, 1.0451548361876000e+03 },
    { 7.9900000000000007e-09, 1.7160914262400000e+00, 2.5407913015344002e+03 },
    { 7.9500000000000001e-09, 3.4243105811899999e+00, 3.4279206312396000e+03 },
    { 7.9200000000000008e-09, 9.6216543925999998e-01, 7.1007094972748000e+03 },
    { 7.8999999999999996e-09, 3.2139880814100001e+00, 1.0264565884073400e+04 },
    { 7.8800000000000001e-09, 6.1506464200900002e+00, 1.4421831636988400e+04 },
    { 7.8700000000000003e-09, 3.9648944141600002e+00, 3.3725847325166001e+03 },
    { 7.8700000000000003e-09, 5.3004651580699997e+00, 3.3086401208829998e+03 },
    { 7.7900000000000006e-09, 3.2880315595099998e+00, 3.4995559444830001e+03 },
    { 7.7900000000000006e-09, 6.2044693919200000e+00, 2.2324905056709398e+04 },
    { 7.7499999999999999e-09, 2.6356906270899998e+00, 1.1045700263900001e+01 },
    { 7.4000000000000001e-09, 5.3785241840699998e+00, 1.0629050485381999e+03 },
    { 7.3000000000000000e-09, 2.5778708519800002e+00, 6.0164688082696002e+03 },
    { 7.2900000000000003e-09, 5.3981003636200002e+00, 2.2743409379516401e+04 },
    { 7.2900000000000003e-09, 3.8300398364300001e+00, 1.5906764126682599e+04 },
    { 7.2600000000000002e-09, 2.9183271803399999e+00, 6.0199919266185998e+03 },
    { 7.2500000000000004e-09, 6.2061537509000004e+00, 3.2294257844121998e+03 },
    { 7.2300000000000001e-09, 5.4972255114099999e+00, 3.7352383117590002e+03 },
    { 7.2000000000000000e-09, 5.3720332172700003e+00, 3.2643463554241998e+03 },
    { 7.1500000000000003e-09, 1.5324053750800000e+00, 4.5030871311075998e+03 },
    { 7.0699999999999998e-09, 3.9644418483399999e+00, 2.0597243963041201e+04 },
    { 7.0600000000000000e-09, 4.5195400360200004e+00, 6.7178302187037998e+03 },
    { 7.0399999999999997e-09, 2.6136688458499999e+00, 3.5473131639963999e+03 },
    { 7.0200000000000002e-09, 6.0460060375899998e+00, 4.3718912305000003e+01 },
    { 6.9999999999999998e-09, 3.9905678074100002e+00, 2.0426571092422000e+04 },
    { 6.9399999999999996e-09, 3.0591758083000000e+00, 3.4517990689874000e+03 },
    { 6.9200000000000001e-09, 3.9396978789400001e+00, 3.0679394693482000e+03 },
    { 6.9100000000000003e-09, 5.2160744839900000e+00, 4.3562754445840001e+03 },
    { 6.9100000000000003e-09, 1.1638642312100000e+00, 6.6797403806912998e+03 },
    { 6.8599999999999999e-09, 4.1801946402599999e+00, 1.4371756141986000e+03 },
    { 6.8299999999999998e-09, 6.0633347088800003e+00, 3.3366802734367002e+03 },
    { 6.8299999999999998e-09, 3.2020248633500001e+00, 3.3445445799629001e+03 },
    { 6.7599999999999998e-09, 1.5809413709000000e-01, 1.9004647949408401e+04 },
    { 6.7500000000000001e-09, 5.9888043708499996e+00, 1.5664035522708589e+04 },
    { 6.7200000000000000e-09, 1.1216570797100001e+00, 6.6669977593980002e+03 },
    { 6.6700000000000003e-09, 3.7126610848000002e-01, 7.2702896804078000e+03 },
    { 6.5899999999999998e-09, 4.9287296501000000e-01, 2.2345260376108199e+04 },
    { 6.5199999999999998e-09, 3.1293596677400002e+00, 3.2126608936652001e+03 },
    { 6.5100000000000001e-09, 6.0902053533000000e+00, 1.7468855197945399e+04 },
    { 6.4899999999999997e-09, 4.0581443460399997e+00, 5.9382347928669997e+03 },
    { 6.4800000000000000e-09, 6.1706905415700000e+00, 1.4807872442630000e+02 },
    { 6.4800000000000000e-09, 2.6488545251300000e+00, 4.1333873151671996e+03 },
    { 6.4599999999999996e-09, 2.8900266866800002e+00, 4.8597205278960001e+02 },
    { 6.4400000000000001e-09, 1.9449373107800001e+00, 4.2897880356695996e+03 },
    { 6.4400000000000001e-09, 1.1794469628199999e+00, 5.2175806283148399e+04 },
    { 6.4000000000000002e-09, 5.6222440269199998e+00, 1.6413332101966000e+03 },
    { 6.3899999999999996e-09, 2.2461164665200002e+00, 3.1162676309979001e+03 },
    { 6.3700000000000001e-09, 4.3174002743000001e-01, 3.1339116894031999e+03 },
    { 6.3600000000000004e-09, 8.6288201968000000e-01, 1.5104766984290001e+02 },
    { 6.3400000000000000e-09, 4.1400524489999997e+00, 6.5105519827804001e+03 },
    { 6.3300000000000003e-09, 3.1723201858199999e+00, 1.7101211136907201e+04 },
    { 6.3300000000000003e-09, 2.6746331671200001e+00, 5.0850384111149997e+03 },
    { 6.3300000000000003e-09, 3.3861758808999998e-01, 2.9632388187840002e+03 },
    { 6.3099999999999999e-09, 5.3443118025800000e+00, 3.3295667264358999e+03 },
    { 6.3099999999999999e-09, 3.9210477696499999e+00, 3.3516581269636999e+03 },
    { 6.3099999999999999e-09, 1.5035258857000000e-01, 3.3431552239806001e+03 },
    { 6.2799999999999998e-09, 5.6051133787699996e+00, 3.1938007401762002e+03 },
    { 6.2099999999999999e-09, 4.0499338311999999e-01, 1.0448435470975401e+04 },
    { 6.2000000000000001e-09, 2.7356545183900001e+00, 2.1496885348241999e+03 },
    { 6.1900000000000003e-09, 6.1339750080500002e+00, 3.4685639597344002e+03 },
    { 6.1900000000000003e-09, 4.9680508897899998e+00, 1.3207029307364999e+04 },
    { 6.1600000000000002e-09, 2.1150369802200002e+00, 1.1695882514085999e+03 },
    { 6.1600000000000002e-09, 6.0114801896300003e+00, 2.0355319398799999e+01 },
    { 6.1300000000000001e-09, 3.8966070047999999e+00, 6.5496828917131998e+03 },
    { 6.1000000000000000e-09, 5.1823731379000004e-01, 9.9116309588799995e+03 },
    { 6.0699999999999999e-09, 4.4356393925899997e+00, 1.8451078546565990e+04 },
    { 6.0600000000000002e-09, 2.4551855135600000e+00, 4.0167212175719999e+02 },
    { 6.0399999999999998e-09, 3.3192898022000000e-01, 2.3736415492642000e+03 },
    { 6.0399999999999998e-09, 4.3403131324200004e+00, 3.3701042450032000e+03 },
    { 6.0399999999999998e-09, 4.9250464398100000e+00, 3.3111206083963998e+03 },
    { 6.0200000000000003e-09, 6.1219661371900003e+00, 6.6816547365263004e+03 },
    { 6.0200000000000003e-09, 4.2089759854200004e+00, 2.8546403739102002e+03 },
    { 6.0200000000000003e-09, 3.8196420213100000e+00, 6.6807949702729002e+03 },
    { 6.0000000000000000e-09, 2.6492252892999999e+00, 3.3380696294190002e+03 },
    { 6.0000000000000000e-09, 4.7358394011599998e+00, 6.7249437657046001e+03 },
    { 5.8800000000000004e-09, 2.2265425284300000e+00, 6.3577194367421998e+03 },
    { 5.8699999999999998e-09, 5.0164867642699997e+00, 2.1710241752912002e+03 },
    { 5.8299999999999999e-09, 4.4729896159500004e+00, 8.2265788363783995e+03 },
    { 5.8100000000000004e-09, 2.9881374430099998e+00, 5.4903009615239998e+03 },
    { 5.7699999999999997e-09, 3.2892643930000000e-02, 7.3587651353180001e+02 },
    { 5.7500000000000002e-09, 1.1092304034100000e+00, 3.5285375744623998e+03 },
    { 5.6100000000000003e-09, 7.7276617470000003e-02, 4.2719755135515998e+03 },
    { 5.5800000000000002e-09, 5.3190726352100004e+00, 4.5553474460203997e+03 },
    { 5.5500000000000001e-09, 2.6420406917100001e+00, 3.7179860346156001e+03 },
    { 5.5400000000000003e-09, 2.3355366962500002e+00, 3.9462588505920002e+02 },
    { 5.5299999999999997e-09, 3.6926554150099999e+00, 3.4874241132234001e+03 },
    { 5.4800000000000001e-09, 2.3222074239500001e+00, 1.2655674786264001e+03 },
    { 5.4599999999999998e-09, 4.8195780035800002e+00, 6.6375059410945996e+03 },
    { 5.4299999999999997e-09, 4.5263098618199997e+00, 1.5961191319817999e+03 },
    { 5.3899999999999998e-09, 2.1659386197199999e+00, 3.1852593375684000e+03 },
    { 5.3599999999999997e-09, 5.2587705247000001e+00, 3.6132853840513999e+03 },
    { 5.3599999999999997e-09, 2.2943192463000001e-01, 3.1984337996636000e+03 },
    { 5.3599999999999997e-09, 3.3938242469900000e+00, 8.6030992875280003e+02 },
    { 5.3499999999999999e-09, 4.9683953778200003e+00, 9.0726616711295992e+03 },
    { 5.3300000000000004e-09, 5.2491431277500000e+00, 4.0398835749273999e+03 },
    { 5.3199999999999998e-09, 1.6661442068800001e+00, 1.0554497769261000e+03 },
    { 5.3000000000000003e-09, 1.0669327149500001e+00, 1.6992792165031999e+03 },
    { 5.2700000000000002e-09, 6.0096132763899996e+00, 3.2893320888373901e+03 },
    { 5.2700000000000002e-09, 3.2557462958399999e+00, 3.3918927645622098e+03 },
    { 5.2199999999999998e-09, 3.8405754764900002e+00, 3.3417930693519002e+03 },
    { 5.2199999999999998e-09, 5.4247840957399998e+00, 3.3394317840477001e+03 },
    { 5.2000000000000002e-09, 9.2503823922999995e-01, 3.4959655158311998e+03 },
    { 5.1700000000000001e-09, 4.0478673928999998e-01, 3.5355838111603998e+03 },
    { 5.1300000000000003e-09, 6.4644878070000000e-01, 2.9912639426919998e+02 },
    { 5.1099999999999999e-09, 2.8447502585900000e+00, 1.6489763038060999e+04 },
    { 5.0899999999999996e-09, 2.2325136820300000e+00, 8.1163725151589997e+03 },
    { 5.0899999999999996e-09, 5.9780163391099999e+00, 6.8127668150860000e+03 },
    { 5.0600000000000003e-09, 2.7525341180099998e+00, 3.4827910537359999e+03 },
    { 5.0499999999999997e-09, 4.8400521549900004e+00, 1.0633140834523001e+03 },
    { 5.0199999999999996e-09, 4.6239468843299996e+00, 5.5624690612967997e+03 },
    { 5.0199999999999996e-09, 2.8998539428100001e+00, 1.7657666254175999e+03 },
    { 5.0099999999999999e-09, 3.1933378160800001e+00, 2.9733880977374001e+03 },
    { 5.0000000000000001e-09, 5.7256979156099996e+00, 1.1250799393421599e+04 },
    { 5.0000000000000001e-09, 5.1340337344200000e+00, 3.5904286517999999e+00 },
    { 5.0000000000000001e-09, 2.5382345754100002e+00, 4.8194790007641996e+03 },
    { 4.9799999999999998e-09, 5.6786836017500004e+00, 1.2509253325047201e+04 },
    { 4.9099999999999998e-09, 1.4144764581000000e+00, 3.2883521117870000e+03 },
    { 4.9099999999999998e-09, 1.5676978069500001e+00, 3.3928727416125998e+03 },
    { 4.8900000000000003e-09, 3.0670252315000002e+00, 1.3575748802237200e+04 },
    { 4.8900000000000003e-09, 2.8186198219100000e+00, 2.3878437747800000e+01 },
    { 4.8699999999999999e-09, 2.2851711564200001e+00, 1.6555603041981999e+03 },
    { 4.8600000000000002e-09, 3.2241612841100000e+00, 3.8287927857099999e+02 },
    { 4.8600000000000002e-09, 6.1598534982300004e+00, 3.0171070100423999e+03 },
    { 4.8600000000000002e-09, 4.8470011689800003e+00, 1.0050286467567201e+04 },
    { 4.8300000000000001e-09, 3.9219539685500000e+00, 3.3345465110700002e+03 },
    { 4.8300000000000001e-09, 5.3434056036799999e+00, 3.3466783423296001e+03 },
    { 4.8099999999999997e-09, 6.2007394918200003e+00, 2.4945295919485998e+03 },
    { 4.8099999999999997e-09, 1.0309649722400001e+00, 6.6851570066627000e+03 },
    { 4.8099999999999997e-09, 5.1288697806299997e+00, 9.7974924843975004e+03 },
    { 4.7799999999999996e-09, 5.7735176383700004e+00, 1.1439910691340000e+02 },
    { 4.7699999999999999e-09, 5.9964504582400000e+00, 4.3796390374902003e+03 },
    { 4.7699999999999999e-09, 3.7213370866800002e+00, 6.6147374444852003e+03 },
    { 4.7699999999999999e-09, 4.5580534738900003e+00, 1.3295120902599999e+03 },
    { 4.7600000000000001e-09, 2.2242794162999999e-01, 6.8508050365325998e+03 },
    { 4.7500000000000003e-09, 5.6296459350700001e+00, 1.2323423096008801e+04 },
    { 4.7300000000000000e-09, 3.2254608648600001e+00, 4.2969015826703999e+03 },
    { 4.7300000000000000e-09, 1.5827029922900000e+00, 2.2954575905122001e+03 },
    { 4.6999999999999999e-09, 5.4512010478399997e+00, 2.1795214091614791e+04 },
    { 4.6800000000000004e-09, 2.0760965689600002e+00, 2.2747613278899999e+02 },
    { 4.6800000000000004e-09, 3.6393313778200000e+00, 1.8546323056346000e+03 },
    { 4.6800000000000004e-09, 5.3242128672800000e+00, 4.1866952614510001e+03 },
    { 4.6800000000000004e-09, 5.4157729514000001e+00, 5.2440492392010001e+03 },
    { 4.6699999999999998e-09, 3.4722889730299999e+00, 5.7498617665477996e+03 },
    { 4.5999999999999998e-09, 1.3016281115499999e+00, 2.2812477501628000e+03 },
    { 4.5999999999999998e-09, 5.5540417872400001e+00, 2.2812132428584000e+03 },
    { 4.5699999999999997e-09, 1.9517559436200000e+00, 2.1947111372700001e+04 },
    { 4.5600000000000000e-09, 4.2847890431300000e+00, 8.9624726035624008e+03 },
    { 4.5600000000000000e-09, 2.2540174116400000e+00, 8.9624380962579999e+03 },
    { 4.5600000000000000e-09, 5.3722488378400000e+00, 1.1140593072202200e+04 },
    { 4.5600000000000000e-09, 3.3976368769700001e+00, 1.0006584808914400e+04 },
    { 4.5299999999999999e-09, 3.5922501634600001e+00, 2.9577331481288002e+03 },
    { 4.5200000000000001e-09, 2.0788049151800001e+00, 6.1068800550648002e+03 },
    { 4.5200000000000001e-09, 2.6436462673499999e+00, 4.3075833041353999e+03 },
    { 4.4800000000000002e-09, 4.9794712219399999e+00, 1.3361469385730999e+04 },
    { 4.4699999999999997e-09, 3.2312320620000001e-02, 1.3363430027867400e+04 },
    { 4.4100000000000003e-09, 1.8287444779699999e+00, 2.2772983432474998e+03 },
    { 4.4100000000000003e-09, 5.3154188127399999e+00, 4.1429763491459998e+03 },
    { 4.3999999999999997e-09, 4.6926165596600002e+00, 3.5649572224017002e+03 },
    { 4.3699999999999996e-09, 1.9026724958500001e+00, 5.8814037282341997e+03 },
    { 4.3599999999999998e-09, 5.3166775500599996e+00, 1.9034368125011999e+03 },
    { 4.3400000000000003e-09, 1.8964098060000001e+00, 7.2144389368432003e+03 },
    { 4.3299999999999997e-09, 5.1972289845799997e+00, 3.3050496922312000e+03 },
    { 4.3299999999999997e-09, 4.0681305876499998e+00, 3.3761751611683999e+03 },
    { 4.3200000000000000e-09, 6.0002691036300000e+00, 1.0721108428326999e+04 },
    { 4.2999999999999996e-09, 1.7406839234000000e+00, 4.7163862265456000e+03 },
    { 4.2999999999999996e-09, 1.8545210237900001e+00, 2.6555683724738001e+03 },
    { 4.2899999999999999e-09, 5.0427849031300003e+00, 2.2851626497737002e+03 },
    { 4.2700000000000004e-09, 3.9334712222600001e+00, 1.5748458012822000e+03 },
    { 4.2700000000000004e-09, 5.9215487068100003e+00, 9.0545669666480001e+02 },
    { 4.2500000000000000e-09, 5.5714873949700001e+00, 7.2180293654950001e+03 },
    { 4.2299999999999997e-09, 3.8649506619600000e+00, 7.3662689076256001e+03 },
    { 4.2299999999999997e-09, 9.4668022804999996e-01, 1.0037089751284400e+04 },
    { 4.2299999999999997e-09, 3.5834238020300000e+00, 5.4832547248259998e+03 },
    { 4.2199999999999999e-09, 5.4629345180800000e+00, 9.3748264467846002e+03 },
    { 4.2199999999999999e-09, 3.9128809762399999e+00, 2.3141558382924599e+04 },
    { 4.2000000000000004e-09, 3.4695775435499998e+00, 5.7533848848968000e+03 },
    { 4.1499999999999999e-09, 4.7802228667000000e+00, 9.8032106819999998e-01 },
    { 4.1400000000000002e-09, 2.2688191466699998e+00, 7.7477203305896001e+03 },
    { 4.1199999999999998e-09, 3.4427994553999999e+00, 2.4803024979470001e+03 },
    { 4.1199999999999998e-09, 1.6244075037000000e-01, 2.5478375382324002e+03 },
    { 4.0800000000000000e-09, 1.3929972196600000e+00, 1.1883583132734801e+04 },
    { 4.0400000000000001e-09, 5.2264261794299998e+00, 6.6446194880953999e+03 },
    { 4.0300000000000004e-09, 5.6442915904700000e+00, 3.7234917052708001e+03 },
    { 4.0199999999999998e-09, 1.7926956610600000e+00, 6.1515511419572003e+03 },
    { 4.0199999999999998e-09, 6.0451093367500004e+00, 6.1515166346528003e+03 },
    { 4.0199999999999998e-09, 4.3127729791400000e+00, 1.6850521225016000e+03 },
    { 4.0000000000000002e-09, 2.7918639423600000e+00, 1.8849227549974199e+04 },
    { 3.9899999999999997e-09, 9.0410764394999998e-01, 5.1546387109299997e+02 },
    { 3.9799999999999999e-09, 4.9515591262200003e+00, 1.5806146839442001e+04 },
    { 3.9799999999999999e-09, 5.8875142042000004e+00, 2.3539707386332801e+04 },
    { 3.9799999999999999e-09, 4.3921396502799999e+00, 2.1265523126520198e+04 },
    { 3.9799999999999999e-09, 5.0692169598500003e+00, 3.8265844794894001e+03 },
    { 3.9499999999999998e-09, 5.7385620283799996e+00, 9.9933880926315996e+03 },
    { 3.9499999999999998e-09, 3.3243831383000000e+00, 3.3371566186536002e+03 },
    { 3.9300000000000003e-09, 3.0542613216699999e+00, 1.9645525556790599e+04 },
    { 3.9099999999999999e-09, 3.4011357299600000e+00, 2.1426422981261999e+03 },
    { 3.8899999999999996e-09, 2.6252724536800001e+00, 3.0065511840298201e+04 },
    { 3.8799999999999998e-09, 1.5341707154499999e+00, 1.8606498946000200e+04 },
    { 3.8799999999999998e-09, 4.4489650907899998e+00, 5.6882187402739999e+02 },
    { 3.8600000000000003e-09, 1.0712679690200000e+00, 5.5116366019910001e+03 },
    { 3.8400000000000000e-09, 5.8724975763799998e+00, 7.7334932365880004e+03 },
    { 3.8300000000000002e-09, 4.8573266075500001e+00, 2.9216880277992000e+03 },
    { 3.8199999999999996e-09, 2.7507075563100001e+00, 3.2673153008407999e+03 },
    { 3.8199999999999996e-09, 2.3146670874000000e-01, 3.4139095525588000e+03 },
    { 3.8000000000000001e-09, 3.8562809731300001e+00, 1.4610540519464000e+03 },
    { 3.7799999999999998e-09, 2.6477908784899999e+00, 9.8014246376606006e+03 },
    { 3.7399999999999999e-09, 2.3016743066199998e+00, 2.5287723799399799e+04 },
    { 3.7200000000000000e-09, 1.4343586792500000e+00, 6.3140005244371996e+03 },
    { 3.6800000000000001e-09, 1.7551235252799999e+00, 9.1257024366559995e+02 },
    { 3.6699999999999999e-09, 5.9873275161599997e+00, 8.5643063465020005e+03 },
    { 3.6600000000000002e-09, 5.2429055859400000e+00, 8.9828106693089994e+03 },
    { 3.6399999999999998e-09, 2.2865167473399999e+00, 6.6827093261078999e+03 },
    { 3.6300000000000001e-09, 1.4190318215200000e+00, 5.8884499649321997e+03 },
    { 3.6300000000000001e-09, 3.4307283217900002e+00, 2.7682140744156401e+04 },
    { 3.6000000000000000e-09, 5.6200393843400001e+00, 7.6968878712838005e+03 },
    { 3.6000000000000000e-09, 6.1819408524000004e+00, 6.8504405422599996e+02 },
    { 3.5899999999999998e-09, 5.0148752223499997e+00, 5.0851057214177999e+03 },
    { 3.5499999999999999e-09, 1.4598055886500001e+00, 7.3221024607817999e+03 },
    { 3.5400000000000002e-09, 2.2131478706899999e+00, 4.6061799053262002e+03 },
    { 3.5199999999999998e-09, 5.7728843443700004e+00, 1.4128242771245599e+04 },
    { 3.5100000000000001e-09, 5.7064274655800000e+00, 6.8518977240187996e+03 },
    { 3.5100000000000001e-09, 4.3755110066500000e+00, 5.4522627400773999e+03 },
    { 3.5100000000000001e-09, 1.6978624950100001e+00, 1.3212886509664600e+04 },
    { 3.4900000000000001e-09, 2.6632321842100000e+00, 5.4085438277723997e+03 },
    { 3.4600000000000000e-09, 3.4624877762900002e+00, 2.0735832161425591e+04 },
    { 3.4600000000000000e-09, 5.3442695577799997e+00, 1.8052929543157799e+04 },
    { 3.4400000000000001e-09, 6.5068647382000000e-01, 1.6894510399610601e+04 },
    { 3.4400000000000001e-09, 3.9543642736600000e+00, 5.6360700172119996e+03 },
    { 3.4400000000000001e-09, 6.1356495238099997e+00, 1.4681168652360000e+02 },
    { 3.3900000000000001e-09, 1.8301138569800000e+00, 3.8560762977927998e+03 },
    { 3.3900000000000001e-09, 1.1520604080700001e+00, 2.8251485556068001e+03 },
    { 3.3799999999999999e-09, 6.0821863759400001e+00, 1.4556896717023799e+04 },
    { 3.3700000000000001e-09, 3.3951726613700002e+00, 5.4391805909619995e+02 },
    { 3.3499999999999998e-09, 4.3245705881000003e+00, 3.0608259223474001e+03 },
    { 3.3299999999999999e-09, 3.0171329630199999e+00, 1.8830814931023999e+03 },
    { 3.3299999999999999e-09, 2.0447273206399998e+00, 9.5979227217800002e+01 },
    { 3.3099999999999999e-09, 8.5002011574000003e-01, 2.5685872802808000e+04 },
    { 3.3099999999999999e-09, 5.0243555994999998e+00, 1.6063164847185000e+04 },
    { 3.3000000000000002e-09, 1.4731578725500001e+00, 1.0736090241908000e+03 },
    { 3.2900000000000000e-09, 3.1354958913900002e+00, 1.0706881334325401e+04 },
    { 3.2799999999999998e-09, 1.4384036268500000e+00, 2.4351557300350000e+03 },
    { 3.2799999999999998e-09, 3.6099043287599999e+00, 3.3974156888696002e+03 },
    { 3.2799999999999998e-09, 5.6554552434699996e+00, 3.2838091645300001e+03 },
    { 3.2700000000000001e-09, 5.6196006944399999e+00, 3.3538541990977901e+03 },
    { 3.2700000000000001e-09, 3.6457588777900001e+00, 3.3273706543018002e+03 },
    { 3.2700000000000001e-09, 6.6722093799000004e-01, 5.3362311835770004e+02 },
    { 3.2599999999999999e-09, 5.7758421041799997e+00, 3.7078367556622002e+03 },
    { 3.2500000000000002e-09, 5.0964785430199999e+00, 2.2670034025089999e+03 },
    { 3.2500000000000002e-09, 5.1916133816999999e+00, 5.2236939198022001e+03 },
    { 3.2299999999999998e-09, 1.9078991313999999e-01, 1.7085958665722199e+04 },
    { 3.2099999999999999e-09, 3.3443990553000003e-01, 1.1904918773201800e+04 },
    { 3.2099999999999999e-09, 4.3994856691799997e+00, 3.7595368256003999e+03 },
    { 3.2000000000000001e-09, 3.7539299929599999e+00, 5.2575881183150000e+02 },
    { 3.1899999999999999e-09, 5.1618890824000001e+00, 9.8156517316621994e+03 },
    { 3.1599999999999998e-09, 2.1030690015400002e+00, 3.2410427708302000e+03 },
    { 3.1599999999999998e-09, 8.7910526350999996e-01, 3.4401820825693999e+03 },
    { 3.1399999999999999e-09, 4.2185433273499999e+00, 9.9837042444616000e+03 },
    { 3.1300000000000002e-09, 2.8745969027600000e+00, 6.2865989683403996e+03 },
    { 3.1200000000000000e-09, 5.7937929081000004e+00, 4.4035174752379999e+03 },
    { 3.1099999999999998e-09, 2.8823267196800000e+00, 3.6027866677399999e+01 },
    { 3.1099999999999998e-09, 5.3215369107499999e+00, 2.0608178540718000e+03 },
    { 3.1099999999999998e-09, 1.2835911879999999e+00, 3.3914448860056000e+03 },
    { 3.1099999999999998e-09, 1.6985830770499999e+00, 3.2897799673939999e+03 },
    { 3.1000000000000000e-09, 9.3049157656000003e-01, 6.4537487206105998e+03 },
    { 3.0899999999999999e-09, 5.0115666855200001e+00, 1.7444932947180000e+03 },
    { 3.0699999999999999e-09, 1.5105690406600001e+00, 9.7552302383776005e+03 },
    { 3.0600000000000002e-09, 6.0459340811000004e+00, 8.7562698014729995e+03 },
    { 3.0500000000000000e-09, 2.9335138120100002e+00, 1.2012582297140400e+04 },
    { 3.0399999999999998e-09, 4.9429057729499997e+00, 3.6203989310522002e+03 },
    { 3.0399999999999998e-09, 3.8378660438400001e+00, 5.6218601768626004e+03 },
    { 3.0399999999999998e-09, 1.8070944123499999e+00, 5.6218256695582004e+03 },
    { 3.0199999999999999e-09, 2.0821199425000000e-01, 1.2825645194703800e+04 },
    { 3.0100000000000002e-09, 3.9236366038499999e+00, 9.4956694333537998e+03 },
    { 3.0000000000000000e-09, 3.7883330633300001e+00, 1.9872218986920000e+03 },
    { 2.9899999999999998e-09, 5.6300816721000002e+00, 1.0284921203472200e+04 },
    { 2.9600000000000001e-09, 1.1282966281000000e+00, 1.5265886519300400e+04 },
    { 2.9400000000000002e-09, 3.4925307219800001e+00, 2.0040151441849790e+04 },
    { 2.9199999999999998e-09, 9.9823300731999998e-01, 2.0618019358533598e+04 },
    { 2.9199999999999998e-09, 9.4159319003999997e-01, 3.0501269472302001e+03 },
    { 2.9100000000000001e-09, 2.4782245131699998e+00, 1.1986675906953400e+04 },
    { 2.8999999999999999e-09, 2.8728067677600002e+00, 2.0047197678547800e+04 },
    { 2.8999999999999999e-09, 3.7772854174699999e+00, 2.4889574795991601e+04 },
    { 2.8900000000000002e-09, 3.0345249722499998e+00, 2.4341528317456599e+04 },
    { 2.8900000000000002e-09, 2.2289232672999999e+00, 6.9016374958384004e+03 },
    { 2.8600000000000000e-09, 2.5293518356200000e+00, 2.7978650435240002e+02 },
    { 2.8600000000000000e-09, 1.9313178141000001e-01, 1.8983512179396000e+03 },
    { 2.8299999999999999e-09, 2.7833544032500002e+00, 1.2839872288705399e+04 },
    { 2.8299999999999999e-09, 6.1027690621500001e+00, 9.8593706439672005e+03 },
    { 2.8200000000000002e-09, 2.3015481658199999e+00, 5.9617837400899998e+02 },
    { 2.7999999999999998e-09, 1.9908607752600000e+00, 1.6511613867387401e+04 },
    { 2.7900000000000001e-09, 6.0103521825200001e+00, 7.1671969061891996e+03 },
    { 2.7700000000000002e-09, 5.8278426448099996e+00, 1.2465534412742199e+04 },
    { 2.7600000000000000e-09, 2.5330294459000002e-01, 6.6701791531357003e+03 },
    { 2.7600000000000000e-09, 4.1768165431300002e+00, 1.9146759266141798e+04 },
    { 2.7400000000000001e-09, 8.2188790950000007e-02, 1.2306590894958999e+04 },
    { 2.7200000000000001e-09, 5.0880724420299996e+00, 2.4280421830342002e+03 },
    { 2.7099999999999999e-09, 5.4178786494900004e+00, 3.3360416769797998e+03 },
    { 2.7099999999999999e-09, 3.8474809227400000e+00, 3.3451831764198000e+03 },
    { 2.7099999999999999e-09, 5.4635712182800003e+00, 1.8795583747533999e+03 },
    { 2.6900000000000000e-09, 4.4603869982899997e+00, 2.7444340526907999e+03 },
    { 2.6900000000000000e-09, 3.4094981100899999e+00, 3.8667802734453999e+03 },
    { 2.6799999999999998e-09, 3.0478286498400000e+00, 1.6702994823196201e+04 },
    { 2.6799999999999998e-09, 2.6393322630000000e+00, 1.6703129443801801e+04 },
    { 2.6700000000000001e-09, 2.5311564012700001e+00, 6.8307880505342000e+03 },
    { 2.6599999999999999e-09, 5.4839812210899996e+00, 1.6865528769631201e+04 },
    { 2.6500000000000002e-09, 4.4770655081199999e+00, 3.0536500655792001e+03 },
    { 2.6400000000000000e-09, 4.1834604418500003e+00, 2.9389403049426001e+03 },
    { 2.6299999999999998e-09, 6.1040728538000000e-01, 4.5315363185754004e+03 },
    { 2.6200000000000001e-09, 4.9310152273599996e+00, 3.9367908007087999e+03 },
    { 2.6200000000000001e-09, 2.0401875422900000e+00, 3.6310979061694002e+03 },
    { 2.5800000000000002e-09, 4.0578192349900002e+00, 5.1952447323344004e+03 },
    { 2.5800000000000002e-09, 4.2601797297499999e+00, 9.4886231966558007e+03 },
    { 2.5599999999999998e-09, 3.1476131187199998e+00, 1.7276610936682000e+03 },
    { 2.5500000000000001e-09, 1.4020434374899999e+00, 8.1661573430937997e+03 },
    { 2.5500000000000001e-09, 3.1246170204600001e+00, 1.9942681353900000e+03 },
    { 2.5500000000000001e-09, 5.7687651968999998e-01, 2.9424806769438001e+03 },
    { 2.5500000000000001e-09, 4.8292901953699996e+00, 2.9424461696394001e+03 },
    { 2.5200000000000000e-09, 3.9861936275999998e-01, 1.4422612087601999e+03 },
    { 2.5200000000000000e-09, 4.6107673270299996e+00, 3.3821632176846001e+03 },
    { 2.5200000000000000e-09, 4.6545922451999999e+00, 3.2990616357150002e+03 },
    { 2.5099999999999998e-09, 3.7259529798800002e+00, 3.3375093844444000e+03 },
    { 2.5099999999999998e-09, 5.5394065923499998e+00, 3.3437154689551999e+03 },
    { 2.5099999999999998e-09, 4.9426246791299997e+00, 9.7587533567266000e+03 },
    { 2.5099999999999998e-09, 5.0925820165499998e+00, 4.9961727308979998e+03 },
    { 2.5099999999999998e-09, 1.6937869851600000e+00, 1.2797945726280000e+03 },
    { 2.5099999999999998e-09, 3.0797289314999998e-01, 2.7905661422063999e+03 },
    { 2.5099999999999998e-09, 2.0069909031800002e+00, 1.1236572299420000e+04 },
    { 2.5000000000000001e-09, 2.5250967330499998e+00, 9.6983318634419993e+03 },
    { 2.5000000000000001e-09, 1.9079279820299999e+00, 2.0670073729660001e+02 },
    { 2.5000000000000001e-09, 4.1624898717500001e+00, 1.2508301963008000e+03 },
    { 2.5000000000000001e-09, 3.5926935365000001e-01, 4.5695745400220003e+03 },
    { 2.4899999999999999e-09, 4.1541567764899998e+00, 3.0846420503278000e+03 },
    { 2.4800000000000001e-09, 5.1102888596799998e+00, 5.4303946570988001e+03 },
    { 2.4800000000000001e-09, 3.3550763123900000e+00, 2.1548962369291799e+04 },
    { 2.4800000000000001e-09, 2.8758278931999998e-01, 4.0256564809258002e+03 },
    { 2.4699999999999999e-09, 1.0271190998199999e+00, 1.0016314355791999e+04 },
    { 2.4699999999999999e-09, 3.7528774009000001e+00, 3.1022753170856198e+04 },
    { 2.4600000000000002e-09, 5.4704003639299996e+00, 1.5650793750310600e+04 },
    { 2.4500000000000000e-09, 7.2828250886000001e-01, 5.4156573747732000e+03 },
    { 2.4399999999999998e-09, 3.7861759241400001e+00, 5.3491699658588004e+03 },
    { 2.4399999999999998e-09, 2.1530667993699999e+00, 4.9819456368964002e+03 },
    { 2.4300000000000001e-09, 5.8072704618700000e+00, 9.3296796788726006e+03 },
    { 2.4199999999999999e-09, 3.3214068372100001e+00, 2.3958631785233400e+04 },
    { 2.4000000000000000e-09, 3.3047732631500000e+00, 1.0027360204406799e+04 },
    { 2.3899999999999998e-09, 5.4892413543699998e+00, 4.6940029547076001e+03 },
    { 2.3899999999999998e-09, 3.2096916594499998e+00, 3.3440682347460001e+03 },
    { 2.3800000000000001e-09, 3.2463721569600001e+00, 6.5668257464861999e+03 },
    { 2.3699999999999999e-09, 4.7882885612399999e+00, 3.6275747878203902e+03 },
    { 2.3600000000000001e-09, 4.4001421831699998e+00, 1.5189728108520001e+02 },
    { 2.3499999999999999e-09, 3.6739649140299999e+00, 5.9453483398678000e+03 },
    { 2.3400000000000002e-09, 3.1951804186700001e+00, 5.5229243074000003e+00 },
    { 2.3400000000000002e-09, 1.6311011003600000e+00, 1.7499360140315399e+04 },
    { 2.3300000000000000e-09, 1.2981792671700001e+00, 2.8144445799542000e+03 },
    { 2.3199999999999998e-09, 5.0352484055800000e+00, 3.1639186965660001e+03 },
    { 2.3199999999999998e-09, 4.2301111666500004e+00, 3.5173061568336002e+03 },
    { 2.3100000000000001e-09, 4.2106373031700004e+00, 6.6538232973027998e+03 },
    { 2.3100000000000001e-09, 4.5799436052700004e+00, 2.0111003364398000e+03 },
    { 2.3100000000000001e-09, 4.0919530645900002e+00, 9.0939973115966004e+03 },
    { 2.3100000000000001e-09, 4.6919365458600000e+00, 5.3517127631395997e+03 },
    { 2.2999999999999999e-09, 3.1967360449200002e+00, 1.0654621019412600e+04 },
    { 2.2900000000000002e-09, 3.4384196410099999e+00, 7.5866815500643997e+03 },
    { 2.2600000000000001e-09, 4.1352571825200002e+00, 1.8688593996361999e+03 },
    { 2.2600000000000001e-09, 1.4763305925000001e+00, 2.6482170809624400e+04 },
    { 2.2600000000000001e-09, 4.1103690637000003e+00, 9.2788472195668000e+03 },
    { 2.2600000000000001e-09, 5.1112213064600001e+00, 3.5965828030717998e+03 },
    { 2.2499999999999999e-09, 3.0857206146100000e+00, 8.7491562544722001e+03 },
    { 2.2400000000000001e-09, 1.5101544584200000e+00, 3.5428658218739001e+03 },
    { 2.2400000000000001e-09, 1.4720198066300001e+00, 3.1383590315257002e+03 },
    { 2.2400000000000001e-09, 3.6865457996400002e+00, 7.7750543983900002e+01 },
    { 2.2400000000000001e-09, 3.0949922047300000e+00, 6.5331461289732997e+03 },
    { 2.2300000000000000e-09, 4.7059658135399998e+00, 3.3362503903100001e+03 },
    { 2.2300000000000000e-09, 4.5593937586899997e+00, 3.3449744630895998e+03 },
    { 2.2300000000000000e-09, 4.5836989909400003e+00, 1.3341674311306801e+04 },
    { 2.2200000000000002e-09, 2.7757194390200000e+00, 1.8353246176068001e+03 },
    { 2.2200000000000002e-09, 1.2949050573700001e+00, 3.0704149563261999e+03 },
    { 2.2100000000000000e-09, 1.4023240804000001e-01, 5.8209149246467996e+03 },
    { 2.2100000000000000e-09, 2.6531703537400002e+00, 2.8628336226099600e+04 },
    { 2.1999999999999998e-09, 4.3159117449000002e+00, 6.5116446702665999e+03 },
    { 2.1999999999999998e-09, 4.7679528966199998e+00, 3.9876232600146000e+03 },
    { 2.1999999999999998e-09, 6.1656223584500003e+00, 1.0081211142013000e+04 },
    { 2.1999999999999998e-09, 2.1190033731200000e+00, 6.5301771835566997e+03 },
    { 2.1999999999999998e-09, 5.6143806427599996e+00, 3.6722432896240002e+02 },
    { 2.1900000000000001e-09, 5.8765580570299996e+00, 3.0704822666290002e+03 },
    { 2.1799999999999999e-09, 1.1756984279700000e+00, 6.2830931036436004e+03 },
    { 2.1799999999999999e-09, 5.4281121036600002e+00, 6.2830585963392004e+03 },
    { 2.1600000000000000e-09, 4.7318662036100001e+00, 1.4849324896942001e+03 },
    { 2.1499999999999998e-09, 4.4293995072999998e-01, 1.3383225102291601e+04 },
    { 2.1400000000000001e-09, 3.9480943901600001e+00, 1.3858952763361999e+03 },
    { 2.1299999999999999e-09, 1.8586302515199999e+00, 8.8522490286907996e+03 },
    { 2.1299999999999999e-09, 1.2531431635300001e+00, 5.6257750764735001e+03 },
    { 2.1299999999999999e-09, 7.5884375260000003e-01, 1.1925274092600601e+04 },
    { 2.1200000000000001e-09, 1.7164024159200000e+00, 2.3924171387982001e+03 },
    { 2.1200000000000001e-09, 1.2583581747000000e-01, 4.5385825552734004e+03 },
    { 2.1100000000000000e-09, 3.2806954610000000e-01, 1.2199975002391400e+04 },
    { 2.1000000000000002e-09, 1.0982024900500000e+00, 6.6784733427886003e+03 },
    { 2.1000000000000002e-09, 4.1123256143200004e+00, 2.8230187222691391e+04 },
    { 2.1000000000000002e-09, 2.5602203612699999e+00, 6.6839763640106003e+03 },
    { 2.0900000000000000e-09, 6.1702908360999997e-01, 4.8459002357928002e+03 },
    { 2.0700000000000001e-09, 4.4690466874199997e+00, 1.0001481960700610e+04 },
    { 2.0599999999999999e-09, 3.3717506562900001e+00, 7.0828969751568002e+03 },
    { 2.0599999999999999e-09, 5.9691591289300003e+00, 1.1553611574070001e+03 },
    { 2.0599999999999999e-09, 1.6998890240300000e+00, 1.8792514776260001e+02 },
    { 2.0500000000000002e-09, 2.2487089415699999e+00, 1.0593646765369999e+03 },
    { 2.0500000000000002e-09, 4.2794805730599998e+00, 1.0593991838413999e+03 },
    { 2.0400000000000000e-09, 2.4323540074899999e+00, 1.5636566656309000e+04 },
    { 2.0400000000000000e-09, 5.6357547346899999e+00, 5.1063790521173996e+03 },
    { 2.0200000000000001e-09, 4.0769539307800002e+00, 4.2531826703653996e+03 },
    { 2.0099999999999999e-09, 6.4407263169999995e-01, 9.3606043533184002e+03 },
    { 2.0099999999999999e-09, 5.5622328906799998e+00, 3.7387441764557998e+03 },
    { 2.0099999999999999e-09, 1.3098192150000001e+00, 3.7387786837601998e+03 },
    { 2.0000000000000001e-09, 4.3664725059700000e+00, 5.6179107699472997e+03 },
    { 2.0000000000000001e-09, 1.7715883007800000e+00, 8.7420427074714007e+03 },
    { 2.0000000000000001e-09, 1.2996948445000001e+00, 1.9497138446060001e+02 },
    { 2.0000000000000001e-09, 2.0626433512800002e+00, 2.7357626859950001e+03 },
    { 2.0000000000000001e-09, 1.6872027711199999e+00, 3.6108098970734000e+03 },
    { 1.9900000000000000e-09, 4.8322573085600000e+00, 3.0297716278314001e+03 },
    { 1.9900000000000000e-09, 2.6222962260200000e+00, 3.8906587111931899e+03 },
    { 1.9900000000000000e-09, 5.6855641235999999e+00, 2.6084021806216198e+04 },
    { 1.9900000000000000e-09, 2.4024675684200001e+00, 8.0061661939395999e+03 },
    { 1.9800000000000002e-09, 5.4921865904100002e+00, 3.0943607043767902e+03 },
    { 1.9800000000000002e-09, 4.0637838098800003e+00, 4.1064054911461999e+03 },
    { 1.9700000000000000e-09, 2.7248886488999999e+00, 3.4365916539176001e+03 },
    { 1.9700000000000000e-09, 2.5728561615000001e-01, 3.2446331994819998e+03 },
    { 1.9599999999999998e-09, 3.3923431690500001e+00, 3.6107425867706002e+03 },
    { 1.9500000000000001e-09, 2.6504261874999999e+00, 1.1731986822019999e+02 },
    { 1.9300000000000002e-09, 4.0028123814100001e+00, 7.4171013669313998e+03 },
    { 1.9300000000000002e-09, 2.2143365956000000e-01, 3.0313341040440000e+03 },
    { 1.9200000000000000e-09, 3.5787880363000002e+00, 3.3157536678838001e+03 },
    { 1.9200000000000000e-09, 5.6865715359299998e+00, 3.3654711855157998e+03 },
    { 1.9200000000000000e-09, 5.1857005464599997e+00, 1.1567191263078201e+04 },
    { 1.9099999999999998e-09, 7.9284041095000002e-01, 3.7529835235741998e+03 },
    { 1.9099999999999998e-09, 2.1893338541000000e+00, 2.9282413298254000e+03 },
    { 1.9099999999999998e-09, 2.4535621929899998e+00, 1.0544414698193201e+04 },
    { 1.9099999999999998e-09, 7.8696608876999996e-01, 3.4792006250842001e+03 },
    { 1.9099999999999998e-09, 2.1952081762900000e+00, 3.2020242283153998e+03 },
    { 1.8899999999999999e-09, 2.9464055232400002e+00, 1.8208349942592002e+04 },
    { 1.8899999999999999e-09, 1.0606716159600000e+00, 2.7835199055083999e+03 },
    { 1.8899999999999999e-09, 6.1414620036200001e+00, 1.0042192599498199e+04 },
    { 1.8800000000000001e-09, 1.7601899459999999e+00, 3.3633809233092002e+03 },
    { 1.8800000000000001e-09, 1.2219843190499999e+00, 3.3178439300904001e+03 },
    { 1.8800000000000001e-09, 5.5005797125599996e+00, 2.5748193622534000e+03 },
    { 1.8800000000000001e-09, 2.4489339270100001e+00, 2.3853709021002001e+03 },
    { 1.8800000000000001e-09, 1.2656227120000000e-02, 4.6869567180095901e+03 },
    { 1.8700000000000000e-09, 2.6182647147900000e+00, 2.6194731954936001e+03 },
    { 1.8400000000000001e-09, 2.5169651319000003e-01, 1.1353892167640201e+04 },
    { 1.8400000000000001e-09, 1.3995880186400000e+00, 1.2942965062924000e+04 },
    { 1.8300000000000001e-09, 3.7676347949000000e-01, 6.7914311746189996e+03 },
    { 1.8199999999999999e-09, 2.9019974278400000e+00, 5.9600856221934000e+03 },
    { 1.8199999999999999e-09, 1.4773242324800000e+00, 4.2888077146014002e+03 },
    { 1.8199999999999999e-09, 5.6187224785099996e+00, 2.7832038219283200e+04 },
    { 1.8100000000000000e-09, 3.4407602574000000e+00, 1.7550626497650001e+03 },
    { 1.8100000000000000e-09, 4.5144430179000000e-01, 1.0191417463232399e+04 },
    { 1.8100000000000000e-09, 2.1520482955200002e+00, 1.3686602528450001e+03 },
    { 1.8000000000000000e-09, 5.8769217206000002e-01, 3.3325185586307998e+03 },
    { 1.8000000000000000e-09, 2.3944820929900001e+00, 3.3487062947688000e+03 },
    { 1.8000000000000000e-09, 3.0668855601999998e+00, 3.6498907493555998e+03 },
    { 1.7900000000000000e-09, 2.2935109960300002e+00, 6.3820984591304004e+03 },
    { 1.7900000000000000e-09, 3.7730626140800001e+00, 3.5868641490228001e+03 },
    { 1.7900000000000000e-09, 3.7144729217100001e+00, 1.7232753098593599e+04 },
    { 1.7900000000000000e-09, 3.7975631676999999e-01, 2.7149422343873998e+03 },
    { 1.7900000000000000e-09, 2.6024179482799998e+00, 3.9662826190122000e+03 },
    { 1.7800000000000001e-09, 1.7581582678000001e+00, 1.5106875691214400e+04 },
    { 1.7700000000000001e-09, 4.4329615372699998e+00, 3.6514532255682002e+03 },
    { 1.7700000000000001e-09, 4.7114376996800003e+00, 1.2928737968922400e+04 },
    { 1.7700000000000001e-09, 6.0448814178999999e-01, 2.5727636073511999e+03 },
    { 1.7700000000000001e-09, 4.5733288585600000e+00, 2.9784912899689998e+03 },
    { 1.7700000000000001e-09, 1.9614739112799999e+00, 8.3225580635961996e+03 },
    { 1.7599999999999999e-09, 5.4743309467000003e+00, 1.3465542481017799e+04 },
    { 1.7500000000000000e-09, 9.8299291721000004e-01, 1.2299544658261000e+04 },
    { 1.7500000000000000e-09, 5.7036127138300001e+00, 6.7086264094963999e+03 },
    { 1.7500000000000000e-09, 1.2040583374600000e+00, 2.7576135153214000e+03 },
    { 1.7500000000000000e-09, 5.4259652428300003e+00, 5.2201708014531996e+03 },
    { 1.7400000000000000e-09, 6.1289404578299997e+00, 1.0096618878666701e+04 },
    { 1.7400000000000000e-09, 1.2435272208000001e-01, 3.1557625187295998e+03 },
    { 1.7300000000000000e-09, 6.0711820444100004e+00, 1.0867834862860000e+03 },
    { 1.7200000000000001e-09, 6.9353998666000005e-01, 1.5052878090929901e+03 },
    { 1.7200000000000001e-09, 5.3948727537799996e+00, 1.9617076369322789e+04 },
    { 1.7100000000000001e-09, 1.6473676403399999e+00, 1.0011200614749599e+04 },
    { 1.6999999999999999e-09, 5.7420900767000003e+00, 6.6776844813983998e+03 },
    { 1.6999999999999999e-09, 1.4896764010200001e+00, 6.6777189887027898e+03 },
    { 1.6900000000000000e-09, 5.1181324760000004e+00, 6.6847307180964099e+03 },
    { 1.6900000000000000e-09, 8.6571880031000004e-01, 6.6847652254007999e+03 },
    { 1.6900000000000000e-09, 6.1934071867800000e+00, 7.0641041319706001e+03 },
    { 1.6800000000000000e-09, 5.9425226939300000e+00, 1.1116422451020001e+03 },
    { 1.6800000000000000e-09, 2.0020972663299998e+00, 6.1657609823066005e+03 },
    { 1.6800000000000000e-09, 5.9712649681299999e+00, 2.7401556096800000e+01 },
    { 1.6800000000000000e-09, 2.6882131521599999e+00, 1.0032473945449199e+04 },
    { 1.6800000000000000e-09, 1.7015651337000001e-01, 4.2958539512994003e+03 },
    { 1.6700000000000000e-09, 1.9635882558500000e+00, 4.7757600884592002e+03 },
    { 1.6700000000000000e-09, 3.8551119236999998e-01, 1.3463442913097999e+03 },
    { 1.6700000000000000e-09, 2.8918149686599999e+00, 1.0288444321821200e+04 },
    { 1.6700000000000000e-09, 5.2831428354099996e+00, 8.3367851575978002e+03 },
    { 1.6700000000000000e-09, 5.4377772995500004e+00, 1.0766352393859999e+02 },
    { 1.6600000000000001e-09, 5.0460215001000002e+00, 1.5121102785216000e+04 },
    { 1.6600000000000001e-09, 5.6644238997100000e+00, 2.0995392966449399e+04 },
    { 1.6600000000000001e-09, 7.7365213441000003e-01, 6.7193578890374001e+03 },
    { 1.6600000000000001e-09, 2.0319480326899999e+00, 5.4664898340789996e+03 },
    { 1.6600000000000001e-09, 2.7354392711400002e+00, 6.9087009861885999e+03 },
    { 1.6600000000000001e-09, 5.1363600542599999e+00, 6.5532060100622002e+03 },
    { 1.6500000000000001e-09, 5.1846212979799997e+00, 2.4491425792583399e+04 },
    { 1.6500000000000001e-09, 1.1393406134599999e+00, 7.3459808985296004e+03 },
    { 1.6399999999999999e-09, 2.0710975118800001e+00, 2.1258100970763999e+03 },
    { 1.6300000000000000e-09, 2.8838717250000001e-01, 1.5535308913140000e+02 },
    { 1.6300000000000000e-09, 9.1027741450000005e-02, 2.1393541969857601e+04 },
    { 1.6300000000000000e-09, 1.7201446799300000e+00, 3.3458116178656001e+03 },
    { 1.6300000000000000e-09, 1.2620295851200001e+00, 3.3354132355339998e+03 },
    { 1.6300000000000000e-09, 2.0423953877900001e+00, 1.3286183635523599e+04 },
    { 1.6300000000000000e-09, 1.4206269266500000e+00, 2.9648466951296000e+03 },
    { 1.6200000000000000e-09, 1.9724355080700000e+00, 1.6695948586498191e+04 },
    { 1.6100000000000000e-09, 3.5424742777699998e+00, 2.7267295735160002e+02 },
    { 1.6000000000000001e-09, 5.4692758274399997e+00, 3.2873044804159999e+03 },
    { 1.6000000000000001e-09, 3.7960837447900002e+00, 3.3939203729835999e+03 },
    { 1.6000000000000001e-09, 2.6369580714600001e+00, 5.4014302807716003e+03 },
    { 1.5900000000000001e-09, 3.3316888781999998e-01, 4.0617516579060002e+03 },
    { 1.5900000000000001e-09, 2.9519714747000001e+00, 3.3464696289993999e+03 },
    { 1.5900000000000001e-09, 3.0202790350000001e-02, 3.3347552244001999e+03 },
    { 1.5900000000000001e-09, 4.8071361649600002e+00, 8.9585231966471001e+03 },
    { 1.5900000000000001e-09, 1.7365043621200000e+00, 8.9663875031733005e+03 },
    { 1.5799999999999999e-09, 4.7426759575400004e+00, 6.0921600263913997e+03 },
    { 1.5700000000000000e-09, 6.0418624583399998e+00, 3.1172650645983002e+04 },
    { 1.5700000000000000e-09, 5.6952639011999995e-01, 8.5846616659007996e+03 },
    { 1.5500000000000000e-09, 5.5563718563700002e+00, 8.7710070837986004e+03 },
    { 1.5500000000000000e-09, 5.6821319549000000e+00, 2.5135826518314600e+04 },
    { 1.5400000000000001e-09, 1.5119547437800001e+00, 5.6076158292088003e+03 },
    { 1.5300000000000001e-09, 4.6919984012600002e+00, 3.7027335634306000e+03 },
    { 1.5199999999999999e-09, 5.0926550908100001e+00, 3.3406469340041999e+03 },
    { 1.5199999999999999e-09, 1.0311118278300000e+00, 3.3405779193953999e+03 },
    { 1.5100000000000000e-09, 6.0266774651899997e+00, 1.5010896463996600e+04 },
    { 1.5100000000000000e-09, 5.9018570129999999e+00, 1.0228022828536599e+04 },
    { 1.5100000000000000e-09, 2.8294153865800000e+00, 3.5254623346700000e+03 },
    { 1.5100000000000000e-09, 2.4888093798900002e+00, 1.8592030553546001e+03 },
    { 1.5100000000000000e-09, 3.2235874552400001e+00, 2.2812978068133998e+03 },
    { 1.5100000000000000e-09, 6.2062793633500002e+00, 8.9625226602130006e+03 },
    { 1.5100000000000000e-09, 3.3159044302000001e-01, 8.9623880396074001e+03 },
    { 1.5100000000000000e-09, 3.6320838420900001e+00, 2.2811631862078002e+03 },
    { 1.5000000000000000e-09, 5.2028730106200003e+00, 2.9221081038928000e+03 },
    { 1.5000000000000000e-09, 1.8479374762500000e+00, 3.8977049478911999e+03 },
    { 1.5000000000000000e-09, 5.9825213496299998e+00, 8.4469914788171991e+03 },
    { 1.4900000000000000e-09, 1.1653424147500000e+00, 1.4577184726119800e+04 },
    { 1.4900000000000000e-09, 1.8955163235400001e+00, 3.6641178433571999e+03 },
    { 1.4800000000000001e-09, 3.9769199767200001e+00, 7.0509246693400000e+03 },
    { 1.4700000000000001e-09, 2.1840066498000001e+00, 1.8983339642874000e+03 },
    { 1.4700000000000001e-09, 1.8760887836300000e+00, 9.9187445058808007e+03 },
    { 1.4599999999999999e-09, 6.2125880196000005e-01, 1.3369563253800001e+04 },
    { 1.4599999999999999e-09, 1.9127804910299999e+00, 2.7149872905026000e+03 },
    { 1.4599999999999999e-09, 3.2971724510299998e+00, 3.2827337306959998e+03 },
    { 1.4599999999999999e-09, 5.9681871211999997e+00, 3.3984911227036000e+03 },
    { 1.4500000000000000e-09, 4.9061642456500003e+00, 8.3733905229020002e+03 },
    { 1.4400000000000000e-09, 9.6134974857000000e-01, 3.9454621674046002e+03 },
    { 1.4400000000000000e-09, 8.5144131454000005e-01, 1.8023719907218001e+03 },
    { 1.4300000000000000e-09, 1.1016608374700001e+00, 1.3227384626763800e+04 },
    { 1.4300000000000000e-09, 9.3308559393000001e-01, 9.1539036160218002e+03 },
    { 1.4300000000000000e-09, 4.8612564650500003e+00, 1.0014791043401399e+04 },
    { 1.4300000000000000e-09, 5.9268116895300000e+00, 3.0414860324306001e+03 },
    { 1.4300000000000000e-09, 2.3367458533400001e+00, 2.0206141196330998e+04 },
    { 1.4300000000000000e-09, 1.0997427839600000e+00, 2.2289701815978001e+03 },
    { 1.4300000000000000e-09, 5.5126029248700004e+00, 1.0787630344545800e+04 },
    { 1.4100000000000001e-09, 3.8374273729100001e+00, 6.2795527316424004e+03 },
    { 1.4100000000000001e-09, 1.5639801411000001e+00, 3.7163781582699999e+03 },
    { 1.3999999999999999e-09, 4.8645154634300001e+00, 1.8461715211915791e+04 },
    { 1.3999999999999999e-09, 5.7989906209499997e+00, 1.5382404359780001e+03 },
    { 1.3999999999999999e-09, 3.9711511869999999e-01, 3.4554813863667601e+04 },
    { 1.3999999999999999e-09, 1.6937167106100000e+00, 1.1456984941858800e+04 },
    { 1.3999999999999999e-09, 1.3670858619199999e+00, 2.2334736210149999e+03 },
    { 1.3900000000000000e-09, 9.8177816583999999e-01, 4.8788528626777997e+03 },
    { 1.3900000000000000e-09, 2.5367557198899999e+00, 1.4867737515892190e+04 },
    { 1.3800000000000000e-09, 5.3531085594000005e-01, 6.0484974070479996e+02 },
    { 1.3800000000000000e-09, 4.8108736791100002e+00, 3.4803105662225998e+03 },
    { 1.3800000000000000e-09, 4.4544858931200002e+00, 3.2009142871770000e+03 },
    { 1.3700000000000000e-09, 3.0483797383300000e+00, 6.7030756827260002e+03 },
    { 1.3700000000000000e-09, 5.8659671311299997e+00, 6.1373067943034002e+03 },
    { 1.3700000000000000e-09, 5.4626304699999997e+00, 1.4580775154771600e+04 },
    { 1.3700000000000000e-09, 4.7926471095599998e+00, 1.4587821391469601e+04 },
    { 1.3700000000000000e-09, 1.6482953196300001e+00, 1.0636665349799999e+01 },
    { 1.3700000000000000e-09, 4.2931712036800000e+00, 7.6579306444639997e+02 },
    { 1.3600000000000001e-09, 2.4496168440899999e+00, 1.2729665967486000e+04 },
    { 1.3600000000000001e-09, 5.5936584915200003e+00, 8.6958483081884006e+03 },
    { 1.3500000000000001e-09, 5.1571141766700004e+00, 1.4481205498902000e+04 },
    { 1.3399999999999999e-09, 4.2853576031099996e+00, 2.4726787626221999e+03 },
    { 1.3399999999999999e-09, 6.1822344740300004e+00, 2.9541927761276002e+03 },
    { 1.3399999999999999e-09, 5.9423378960999995e-01, 6.6593740240732004e+03 },
    { 1.3300000000000000e-09, 4.8275976959399998e+00, 2.1150813365883601e+04 },
    { 1.3300000000000000e-09, 6.2694970495300000e+00, 1.2171525814923600e+04 },
    { 1.3300000000000000e-09, 6.0753228315800003e+00, 6.6243264784639996e+03 },
    { 1.3300000000000000e-09, 1.7711648158200000e+00, 3.9236113380781999e+03 },
    { 1.3300000000000000e-09, 2.3234778886699998e+00, 6.1476017350418997e+03 },
    { 1.3300000000000000e-09, 3.8662853269199999e+00, 6.7381232283352001e+03 },
    { 1.3200000000000000e-09, 2.6191863767100001e+00, 4.7828736354599996e+03 },
    { 1.3000000000000001e-09, 4.3553212208699996e+00, 5.2967371144239996e+02 },
    { 1.3000000000000001e-09, 4.6593430175100004e+00, 7.5937950970652000e+03 },
    { 1.3000000000000001e-09, 1.0290754518000000e-01, 5.2970821874679996e+02 },
    { 1.3000000000000001e-09, 2.3776470567399999e+00, 4.1084612460484004e+03 },
    { 1.3000000000000001e-09, 5.5149424220799999e+00, 6.1554660415681001e+03 },
    { 1.2900000000000001e-09, 4.6665620691400003e+00, 3.1014069410748002e+03 },
    { 1.2900000000000001e-09, 4.5987975031000001e+00, 3.5798179123248001e+03 },
    { 1.2900000000000001e-09, 3.7651729640300000e+00, 3.7199858405739901e+03 },
    { 1.2900000000000001e-09, 5.5001866081999999e+00, 2.9612390128256002e+03 },
    { 1.2900000000000001e-09, 4.5055093861200000e+00, 4.5914426230006002e+03 },
    { 1.2799999999999999e-09, 6.6575049629000005e-01, 3.6001732317236001e+03 },
    { 1.2799999999999999e-09, 2.3164237687600000e+00, 3.0810516216760002e+03 },
    { 1.2700000000000000e-09, 1.9209373338500000e+00, 4.4477512323846004e+03 },
    { 1.2700000000000000e-09, 4.0624750471200004e+00, 3.7591167495067998e+03 },
    { 1.2700000000000000e-09, 5.9584670529499997e+00, 2.6555010621709998e+03 },
    { 1.2600000000000000e-09, 1.8544965476299999e+00, 1.8447488117914188e+04 },
    { 1.2600000000000000e-09, 8.8452034949000002e-01, 1.2406160550828599e+04 },
    { 1.2600000000000000e-09, 5.7223481254999999e+00, 4.0836024872324001e+03 },
    { 1.2500000000000000e-09, 7.9695675334000005e-01, 9.1249219860424000e+03 },
    { 1.2500000000000000e-09, 3.4150053018300000e+00, 3.0227978476430399e+04 },
    { 1.2400000000000001e-09, 3.5349287481900000e+00, 6.6879085172737005e+03 },
    { 1.2400000000000001e-09, 2.9668251079600001e+00, 2.0897822303990001e+03 },
    { 1.2300000000000001e-09, 3.3608348095500000e+00, 2.9424634232916000e+04 },
    { 1.2300000000000001e-09, 4.4321287495600004e+00, 6.7106543619355998e+03 },
    { 1.2199999999999999e-09, 1.0822164206800000e+00, 2.2573520587628000e+03 },
    { 1.2199999999999999e-09, 3.2659416472100000e+00, 1.2360756603230000e+03 },
    { 1.2199999999999999e-09, 6.1324340709000003e-01, 9.1615273513466000e+03 },
    { 1.2199999999999999e-09, 2.6938901018400001e+00, 2.7676617819849002e+04 },
    { 1.2100000000000000e-09, 1.0620420659999999e-02, 3.0907079659576002e+03 },
    { 1.2100000000000000e-09, 2.9715538443899998e+00, 3.5905168874420001e+03 },
    { 1.2000000000000000e-09, 1.3931733509399999e+00, 3.2020949298290998e+03 },
    { 1.2000000000000000e-09, 1.5890009141100001e+00, 3.4791299235705001e+03 },
    { 1.1900000000000000e-09, 9.2051706256999999e-01, 2.3937856389740999e+04 },
    { 1.1900000000000000e-09, 2.4542161032599998e+00, 1.4717530270636000e+03 },
    { 1.1900000000000000e-09, 2.5257145975000000e-01, 5.8132911893219998e+03 },
    { 1.1900000000000000e-09, 4.2489612456800003e+00, 1.2801884333740000e+02 },
    { 1.1800000000000001e-09, 4.2452222640199997e+00, 7.5344212351515998e+03 },
    { 1.1800000000000001e-09, 5.1426876683899998e+00, 1.9676450231236398e+04 },
    { 1.1800000000000001e-09, 2.5972174518100002e+00, 2.1852512692927999e+03 },
    { 1.1800000000000001e-09, 2.3028176098399999e+00, 3.3376434812831999e+03 },
    { 1.1800000000000001e-09, 6.7935665520999999e-01, 3.3435813721164000e+03 },
    { 1.1800000000000001e-09, 3.3308541328000002e-01, 2.5239591004846002e+03 },
    { 1.1800000000000001e-09, 5.6476441145000003e+00, 3.3969956127760001e+03 },
    { 1.1800000000000001e-09, 3.6177154577300001e+00, 3.2842292406236002e+03 },
    { 1.1700000000000001e-09, 4.3472889272400002e+00, 6.3429821544165998e+03 },
    { 1.1599999999999999e-09, 3.1023473754999997e-01, 1.0338229149756000e+04 },
    { 1.1599999999999999e-09, 3.2127791252200000e+00, 7.2113923120619995e+02 },
    { 1.1599999999999999e-09, 3.0829298746400000e+00, 3.7270320772720002e+03 },
    { 1.1599999999999999e-09, 2.7649751674499998e+00, 2.1000915890756802e+04 },
    { 1.1599999999999999e-09, 7.7977053917000005e-01, 1.4694504594340000e+04 },
    { 1.1500000000000000e-09, 4.7098186144799996e+00, 2.5016431389494001e+03 },
    { 1.1500000000000000e-09, 4.5472680549900000e+00, 8.9766824439118009e+03 },
    { 1.1500000000000000e-09, 3.0529427366599999e+00, 7.7680083396855998e+03 },
    { 1.1500000000000000e-09, 6.2413866466099996e+00, 2.6316226244231998e+03 },
    { 1.1500000000000000e-09, 1.8434202398299999e+00, 9.4819528790159995e+02 },
    { 1.1400000000000000e-09, 3.3656365178900001e+00, 6.5344131668760001e+03 },
    { 1.1400000000000000e-09, 1.0693680112199999e+00, 3.9662375628969999e+03 },
    { 1.1400000000000000e-09, 1.1788899816200000e+00, 3.0774501642574800e+04 },
    { 1.1400000000000000e-09, 2.8573889073399998e+00, 3.0635774329584001e+03 },
    { 1.1400000000000000e-09, 1.2478535771000000e-01, 3.6176474204412002e+03 },
    { 1.1200000000000001e-09, 5.6327622302200000e+00, 1.3533905280078000e+03 },
    { 1.1200000000000001e-09, 1.0814072390200000e+00, 6.4745241161029999e+03 },
    { 1.1200000000000001e-09, 6.1908290346999995e-01, 4.8255449163940002e+03 },
    { 1.1200000000000001e-09, 3.2463123490800001e+00, 1.4900690142777201e+04 },
    { 1.1200000000000001e-09, 5.3187318578899996e+00, 5.2953295770634004e+03 },
    { 1.1100000000000001e-09, 5.0966952807999997e+00, 7.2774032274086003e+03 },
    { 1.1100000000000001e-09, 5.0617258843400004e+00, 6.8376255739019998e+03 },
    { 1.0999999999999999e-09, 4.6677420435699997e+00, 4.1795817144501998e+03 },
    { 1.0999999999999999e-09, 4.0138567885800001e+00, 2.7417057014825201e+04 },
    { 1.0900000000000000e-09, 7.5405687886999995e-01, 2.7433889215874991e+04 },
    { 1.0900000000000000e-09, 5.7082086385900004e+00, 2.4990446074220000e+02 },
    { 1.0900000000000000e-09, 5.5983628720200000e+00, 1.0610902107107600e+04 },
    { 1.0800000000000000e-09, 4.4027259723599999e+00, 2.7490692478044799e+04 },
    { 1.0700000000000000e-09, 3.7482524155600001e+00, 7.0758507384588002e+03 },
    { 1.0700000000000000e-09, 7.7642458908000000e-01, 4.8220217980449997e+03 },
    { 1.0700000000000000e-09, 4.0030330488299999e+00, 3.3695940566792001e+03 },
    { 1.0700000000000000e-09, 5.2623265233999996e+00, 3.3116307967204002e+03 },
    { 1.0600000000000001e-09, 1.3549826891700001e+00, 3.3824269898300003e+02 },
    { 1.0600000000000001e-09, 2.4792442986399998e+00, 7.8995503013719999e+03 },
    { 1.0600000000000001e-09, 4.8409497986000000e-01, 8.2194652893776001e+03 },
    { 1.0600000000000001e-09, 3.6753287734600000e+00, 1.0177190369230801e+04 },
    { 1.0500000000000001e-09, 5.4136314134200001e+00, 1.7821817925601790e+04 },
    { 1.0500000000000001e-09, 5.4936798339199999e+00, 6.9610113577519996e+03 },
    { 1.0500000000000001e-09, 5.5108532387900002e+00, 4.2388934936307000e+04 },
    { 1.0500000000000001e-09, 5.1378551909399999e+00, 4.4082528487759998e+02 },
    { 1.0399999999999999e-09, 3.2509428484899998e+00, 5.5004628449339998e+02 },
    { 1.0399999999999999e-09, 6.2604596992800001e+00, 3.1449827042751800e+04 },
    { 1.0399999999999999e-09, 1.8701787712000000e+00, 1.9719521738548001e+03 },
    { 1.0399999999999999e-09, 7.8744151949999994e-02, 3.1491814142404000e+03 },
    { 1.0399999999999999e-09, 4.3311578276400002e+00, 3.1491469069360001e+03 },
    { 1.0300000000000000e-09, 9.5690851610000002e-02, 3.1214201436967800e+04 },
    { 1.0300000000000000e-09, 3.2444738175999999e-01, 1.0593146198863999e+03 },
    { 1.0300000000000000e-09, 2.9439219908599998e+00, 3.6969981594040001e+02 },
    { 1.0300000000000000e-09, 5.9838093453100001e+00, 4.9535637597313998e+03 },
    { 1.0300000000000000e-09, 4.2138934691000003e+00, 2.0235122826310399e+04 },
    { 1.0300000000000000e-09, 6.1991363021000003e+00, 1.0594492404919999e+03 },
    { 1.0300000000000000e-09, 4.9828780365799998e+00, 6.7772040806174000e+03 },
    { 1.0200000000000000e-09, 4.7049061665799998e+00, 2.2147430875962000e+03 },
    { 1.0200000000000000e-09, 5.0642366593199997e+00, 2.7946747979919801e+04 },
    { 1.0200000000000000e-09, 3.1894357046900002e+00, 3.5244820136017902e+03 },
    { 1.0200000000000000e-09, 6.0759238675400002e+00, 3.1567428397978001e+03 },
    { 1.0100000000000000e-09, 4.7634733267199998e+00, 2.1791690973265791e+04 },
    { 1.0100000000000000e-09, 4.0518064917499999e+00, 3.0927832265580003e+02 },
    { 1.0000000000000001e-09, 4.4630679650999996e+00, 1.0101100013959699e+05 },
    { 1.0000000000000001e-09, 2.3672028852899998e+00, 2.6519779438219998e+03 },
    { 1.0000000000000001e-09, 7.3142842805999997e-01, 3.3406969906548002e+03 },
    { 1.0000000000000001e-09, 5.3923384905800003e+00, 3.3405278627448101e+03 },
    { 1.0000000000000001e-09, 5.3607905608399999e+00, 1.7554716846791000e+03 },
    { 1.0000000000000001e-09, 1.1399248149000001e+00, 3.3405623700492001e+03 },
    { 1.0000000000000001e-09, 4.9838421037399998e+00, 3.3406624833504002e+03 },
    { 1.0000000000000001e-09, 1.0647046345300000e+00, 1.4814093713452000e+03 },
    { 1.0000000000000001e-09, 1.7495352116500000e+00, 1.0817087655544799e+04 },
    { 9.9000000000000010e-10, 1.0703742953399999e+00, 5.3125646005545996e+03 },
    { 9.9000000000000010e-10, 5.6393373764900003e+00, 3.2227996238077999e+03 },
    { 9.9000000000000010e-10, 3.6260221957400001e+00, 3.4584252295918000e+03 },
    { 9.9000000000000010e-10, 4.6146801066799998e+00, 2.4606135553220000e+04 },
    { 9.9000000000000010e-10, 1.8728047073800000e+00, 7.9065965380700000e+03 },
    { 9.9000000000000010e-10, 3.6694261112900000e+00, 3.3050046361159998e+03 },
    { 9.9000000000000010e-10, 5.5959334609400004e+00, 3.3762202172836000e+03 },
    { 9.7999999999999992e-10, 2.5507546569400001e+00, 1.7840442797444000e+03 },
    { 9.7999999999999992e-10, 3.6071823126800000e+00, 7.3297125859000005e+01 },
    { 9.6999999999999996e-10, 3.3068612701100002e+00, 4.0257237912286000e+03 },
    { 9.6999999999999996e-10, 3.0785206187799998e+00, 8.6923251898393992e+03 },
    { 9.5999999999999999e-10, 1.4180527260800000e+00, 3.4363365597556003e+04 },
    { 9.5999999999999999e-10, 3.6911346347500000e+00, 1.5894351778320001e+02 },
    { 9.5999999999999999e-10, 5.2863235688300003e+00, 1.6975735090850601e+04 },
    { 9.5999999999999999e-10, 1.9219873461300001e+00, 1.0927293976764200e+04 },
    { 9.5999999999999999e-10, 2.3027300185999999e-01, 2.5442971662312002e+03 },
    { 9.5999999999999999e-10, 1.5285911243400001e+00, 3.3574446277495899e+03 },
    { 9.5999999999999999e-10, 1.4535831407099999e+00, 3.3237802256499999e+03 },
    { 9.5999999999999999e-10, 2.2610446333500001e+00, 2.5443316735356002e+03 },
    { 9.5999999999999999e-10, 6.1229196319000001e-01, 5.2168043612432397e+04 },
    { 9.5000000000000003e-10, 4.6955391477199999e+00, 3.3704944703100000e+03 },
    { 9.5000000000000003e-10, 4.5698204245099996e+00, 3.3107303830895999e+03 },
    { 9.5000000000000003e-10, 2.4914831219100000e+00, 1.1258693391035999e+03 },
    { 9.5000000000000003e-10, 2.4154401549500002e+00, 5.2962365479180005e+02 },
    { 9.5000000000000003e-10, 2.0069437681100002e+00, 5.2975827539739998e+02 },
    { 9.5000000000000003e-10, 4.3581203982299996e+00, 1.7810200939183800e+04 },
    { 9.5000000000000003e-10, 3.8841222545999998e+00, 2.1045367663768002e+03 },
    { 9.4000000000000006e-10, 3.1773440055800002e+00, 1.9577250798532000e+03 },
    { 9.4000000000000006e-10, 3.9146191955099998e+00, 3.3467456526323999e+03 },
    { 9.4000000000000006e-10, 5.3507403767200001e+00, 3.3344792007671999e+03 },
    { 9.4000000000000006e-10, 3.5253711014400002e+00, 7.9628075316419995e+02 },
    { 9.4000000000000006e-10, 5.5561427329299997e+00, 7.9631526046859994e+02 },
    { 9.2999999999999999e-10, 1.7716036300600000e+00, 1.1506769769793600e+04 },
    { 9.2999999999999999e-10, 4.5669211942899999e+00, 1.8617068301047198e+04 },
    { 9.2999999999999999e-10, 1.3553712224400001e+00, 8.9482282559085997e+03 },
    { 9.2999999999999999e-10, 3.6005599584499999e+00, 1.8263211299832999e+03 },
    { 9.2999999999999999e-10, 5.9722818645400002e+00, 1.2892132603618200e+04 },
    { 9.2000000000000003e-10, 3.2833988336500002e+00, 1.3828873468465999e+03 },
    { 9.2000000000000003e-10, 3.7931448031999998e-01, 2.4093276789175201e+04 },
    { 9.2000000000000003e-10, 1.9392142526100000e+00, 6.9538978107512003e+03 },
    { 9.2000000000000003e-10, 1.6074543836899999e+00, 3.4115114069274598e+04 },
    { 9.2000000000000003e-10, 8.7242130440999999e-01, 5.0021878195957201e+04 },
    { 9.2000000000000003e-10, 1.7629426634400001e+00, 3.2509208972941000e+04 },
    { 9.0999999999999996e-10, 5.7523735196299999e+00, 3.5320434391591998e+03 },
    { 9.0999999999999996e-10, 2.7168423656099998e+00, 5.0818176202773597e+04 },
    { 9.0999999999999996e-10, 1.4999598439399999e+00, 3.5320779464635998e+03 },
    { 8.9999999999999999e-10, 9.6686807912999995e-01, 1.5746772977528401e+04 },
    { 8.9999999999999999e-10, 4.9520840565700004e+00, 4.6701245169597996e+03 },
    { 8.9999999999999999e-10, 2.2831361243399999e+00, 5.1769894609024203e+04 },
    { 8.9999999999999999e-10, 5.6413397937500003e+00, 1.8028003474038989e+04 },
    { 8.9999999999999999e-10, 2.8361244439900002e+00, 1.6066065861474801e+04 },
    { 8.9999999999999999e-10, 1.5903939236900000e+00, 1.7476073781529001e+03 },
    { 8.9999999999999999e-10, 5.6870782000600002e+00, 2.4532976583568201e+04 },
    { 8.9999999999999999e-10, 4.8998482545600002e+00, 3.0639856638632999e+04 },
    { 8.9999999999999999e-10, 2.4447619185799998e+00, 1.1527125089192399e+04 },
    { 8.9999999999999999e-10, 1.6680193484100001e+00, 2.8857030601725000e+03 },
    { 8.9999999999999999e-10, 1.3141549166399999e+00, 3.7955217932270998e+03 },
    { 8.9000000000000003e-10, 2.2159810365500001e+00, 3.3680812930993902e+03 },
    { 8.9000000000000003e-10, 7.6619322850000005e-01, 3.3131435603002001e+03 },
    { 8.9000000000000003e-10, 3.3165898190299998e+00, 2.2531090605146590e+04 },
    { 8.9000000000000003e-10, 1.4610246806600000e+00, 3.3329386347243999e+03 },
    { 8.9000000000000003e-10, 1.5211495843900000e+00, 3.3482862186752000e+03 },
    { 8.9000000000000003e-10, 5.0241467007000002e+00, 5.3552358814886002e+03 },
    { 8.9000000000000003e-10, 5.7212660420799999e+00, 6.3115250374591997e+03 },
    { 8.7999999999999996e-10, 1.8461241907999999e+00, 2.7510151571800002e+03 },
    { 8.6999999999999999e-10, 2.0554759562000000e-01, 6.2567019231239999e+02 },
    { 8.6999999999999999e-10, 5.8329025766199996e+00, 1.5117512356564201e+04 },
    { 8.6000000000000003e-10, 2.1566882532300000e+00, 2.4336005393149200e+04 },
    { 8.6000000000000003e-10, 6.3892610093000002e-01, 3.7626398678557998e+03 },
    { 8.6000000000000003e-10, 2.3432481641299998e+00, 2.9185849855438000e+03 },
    { 8.6000000000000003e-10, 2.1663038321500001e+00, 9.8774627292800005e+02 },
    { 8.6000000000000003e-10, 1.3755838897499999e+00, 1.2410731300548599e+04 },
    { 8.6000000000000003e-10, 4.9872722543299997e+00, 2.3009949110935391e+04 },
    { 8.6000000000000003e-10, 2.6236583792900001e+00, 1.3517802795930600e+04 },
    { 8.6000000000000003e-10, 5.1830923830900000e+00, 3.3852662599400001e+03 },
    { 8.6000000000000003e-10, 4.0822671891400004e+00, 3.2959585934595998e+03 },
    { 8.6000000000000003e-10, 2.5993921458000002e-01, 1.6129513330315999e+03 },
    { 8.4999999999999996e-10, 4.3885427153099998e+00, 8.0947746944699998e+02 },
    { 8.4999999999999996e-10, 2.8699488643600000e+00, 1.6199975697295999e+03 },
    { 8.4999999999999996e-10, 1.5626665143999999e-01, 1.0558588118401999e+03 },
    { 8.4999999999999996e-10, 4.2532473360900003e+00, 5.4789936689978000e+04 },
    { 8.4999999999999996e-10, 1.2415625823000000e-01, 1.4715840234806999e+04 },
    { 8.4999999999999996e-10, 2.6452695245900002e+00, 3.1938179938284002e+03 },
    { 8.4999999999999996e-10, 3.3690474046000002e-01, 3.4874068595712001e+03 },
    { 8.4999999999999996e-10, 4.6001347353800002e+00, 1.9403777273884800e+04 },
    { 8.4999999999999996e-10, 4.3958268442700001e+00, 4.5494581429748789e+04 },
    { 8.4999999999999996e-10, 2.6490447234699999e+00, 4.1572657529149901e+03 },
    { 8.3999999999999999e-10, 4.5739279121000003e+00, 4.7777880408984001e+03 },
    { 8.3999999999999999e-10, 2.1856064264100001e+00, 1.1071388056848000e+03 },
    { 8.3999999999999999e-10, 1.3340696945499999e+00, 1.5524675299795999e+03 },
    { 8.3999999999999999e-10, 2.4988349177200000e+00, 2.9425307335943999e+03 },
    { 8.3999999999999999e-10, 4.4861845501800000e+00, 3.0835944189567999e+03 },
    { 8.3999999999999999e-10, 4.7791750220500004e+00, 3.5976304344427999e+03 },
    { 8.3999999999999999e-10, 2.9073313045600000e+00, 2.9423961129887998e+03 },
    { 8.3999999999999999e-10, 4.3029708710300003e+00, 4.8016664786461997e+03 },
    { 8.3999999999999999e-10, 1.0400772958500000e+00, 4.5331398359598599e+04 },
    { 8.3000000000000003e-10, 3.8045909975000001e+00, 2.0103048422112399e+04 },
    { 8.3000000000000003e-10, 5.5564877844000005e-01, 4.0292469095776000e+03 },
    { 8.3000000000000003e-10, 2.9026343555300000e+00, 4.0496022289764001e+03 },
    { 8.3000000000000003e-10, 4.2435626248700000e+00, 2.6369792420824001e+03 },
    { 8.1999999999999996e-10, 7.6471194062000003e-01, 3.3300430716527999e+03 },
    { 8.1999999999999996e-10, 2.2174623244300000e+00, 3.3511817817468000e+03 },
    { 8.1999999999999996e-10, 2.1556083117600000e+00, 2.2317791509708590e+04 },
    { 8.1999999999999996e-10, 5.5531735768799999e+00, 3.0220932239732399e+04 },
    { 8.1999999999999996e-10, 2.2380005807000000e-01, 3.4513263072682799e+04 },
    { 8.1999999999999996e-10, 3.8199497693900000e+00, 6.5735613294610002e+03 },
    { 8.0999999999999999e-10, 7.3729790565999997e-01, 3.1907149515729998e+03 },
    { 8.0999999999999999e-10, 2.2448763593900001e+00, 3.4905099018266001e+03 },
    { 8.0999999999999999e-10, 5.9122578904700003e+00, 3.3477432273528002e+03 },
    { 8.0999999999999999e-10, 3.3531016817600001e+00, 3.3334816260468001e+03 },
    { 8.0999999999999999e-10, 5.6197619486999995e-01, 4.7828908891122001e+03 },
    { 8.0999999999999999e-10, 4.5815752842000002e-01, 3.4043483250032000e+03 },
    { 8.0999999999999999e-10, 2.5240167366300001e+00, 3.2768765283963999e+03 },
    { 8.0999999999999999e-10, 5.6756110464200002e+00, 2.9175373541728000e+03 },
    { 8.0999999999999999e-10, 8.7939330800000007e-02, 2.9104911174747999e+03 },
    { 8.0999999999999999e-10, 2.9539669724500000e+00, 1.9573357457017799e+04 },
    { 8.0999999999999999e-10, 7.6982949413000001e-01, 9.4329775286197386e+04 },
    { 8.0999999999999999e-10, 5.5896658690700001e+00, 5.1358708704208002e+03 },
    { 8.0999999999999999e-10, 4.8602683013299997e+00, 1.7498312508944389e+04 },
    { 8.0000000000000003e-10, 4.4145582807799997e+00, 3.1570799649391200e+04 },
    { 8.0000000000000003e-10, 4.9528565243799996e+00, 1.9547171503636000e+03 },
    { 8.0000000000000003e-10, 5.7163515787900003e+00, 1.4061720855026801e+04 },
    { 8.0000000000000003e-10, 3.7092009022100001e+00, 6.1516011986078001e+03 },
    { 8.0000000000000003e-10, 4.1176972890499997e+00, 6.1514665780021996e+03 },
    { 8.0000000000000003e-10, 4.9696200967399999e+00, 9.7054454104428005e+03 },
    { 8.0000000000000003e-10, 3.2348561778899998e+00, 3.3772609896293002e+03 },
    { 8.0000000000000003e-10, 6.0305033943500002e+00, 3.3039638637703001e+03 },
    { 7.8999999999999996e-10, 1.8998788007800000e+00, 4.4238727946367999e+03 },
    { 7.8999999999999996e-10, 3.6416007842100000e+00, 4.8031133178916200e+04 },
    { 7.8999999999999996e-10, 2.5941566591099998e+00, 1.2988111830836000e+04 },
    { 7.8999999999999996e-10, 5.3091162599699997e+00, 1.5930050485469001e+03 },
    { 7.8999999999999996e-10, 5.0319979522200002e+00, 4.9225580189140797e+04 },
    { 7.7999999999999999e-10, 2.7944286144300001e+00, 2.6880319813032602e+04 },
    { 7.7999999999999999e-10, 5.9220694367000002e+00, 8.8309133882238002e+03 },
    { 7.7999999999999999e-10, 5.3245739122000000e+00, 4.2266603761290003e+02 },
    { 7.7999999999999999e-10, 5.2852867875699996e+00, 1.2979553174576000e+04 },
    { 7.7999999999999999e-10, 7.0009199829999993e-02, 1.7078827865069201e+04 },
    { 7.7000000000000003e-10, 2.2757647914899999e+00, 4.3053034413910001e+02 },
    { 7.7000000000000003e-10, 5.3097971729699998e+00, 1.3249413405400001e+03 },
    { 7.7000000000000003e-10, 5.6183061153500002e+00, 1.0085494708834000e+03 },
    { 7.7000000000000003e-10, 4.0573840944000000e-01, 4.2460691233646003e+03 },
    { 7.7000000000000003e-10, 4.9312229859599999e+00, 1.5423267560871000e+04 },
    { 7.7000000000000003e-10, 4.0849727815600003e+00, 7.8263709424722590e+04 },
    { 7.5999999999999996e-10, 1.1717082730899999e+00, 1.1663170490295999e+04 },
    { 7.5999999999999996e-10, 3.9009097956900001e+00, 2.1957680727747000e+04 },
    { 7.5999999999999996e-10, 3.3610051613300000e+00, 3.4126681600545999e+03 },
    { 7.5999999999999996e-10, 5.9043544108999999e+00, 3.2685566933450000e+03 },
    { 7.5999999999999996e-10, 9.8386664138000002e-01, 4.5554147563232000e+03 },
    { 7.5999999999999996e-10, 6.1024967372700001e+00, 6.6772927001364997e+03 },
    { 7.5999999999999996e-10, 4.5175592836799998e+00, 4.1892439890060001e+02 },
    { 7.5000000000000000e-10, 2.0240154886899999e+00, 1.4797597368558600e+04 },
    { 7.5000000000000000e-10, 1.8112965370400000e+00, 1.2148023296234001e+03 },
    { 7.5000000000000000e-10, 6.1176191784000000e-01, 9.0904741932476009e+03 },
    { 7.5000000000000000e-10, 1.6585280107100000e+00, 8.1604321851422384e+04 },
    { 7.5000000000000000e-10, 4.8199975849100003e+00, 3.2367097656207599e+04 },
    { 7.5000000000000000e-10, 1.6606738377800001e+00, 1.5851407420206999e+03 },
    { 7.5000000000000000e-10, 3.6822013083999998e+00, 6.8365779425310002e+03 },
    { 7.4000000000000003e-10, 2.6663963295499999e+00, 7.4764752288449999e+03 },
    { 7.4000000000000003e-10, 5.3813077753999998e+00, 1.3635122664150800e+04 },
    { 7.4000000000000003e-10, 5.5789856167799998e+00, 6.6034743094157002e+03 },
    { 7.4000000000000003e-10, 5.7290931993400003e+00, 3.1895647568569002e+03 },
    { 7.4000000000000003e-10, 4.4861274443999999e-01, 3.1925337022734998e+03 },
    { 7.2999999999999996e-10, 7.3049271630000001e-01, 3.1979518007846400e+04 },
    { 7.2999999999999996e-10, 5.8159014657999997e+00, 9.2783996850280005e+02 },
    { 7.2999999999999996e-10, 4.5797860910099999e+00, 6.8092436967370004e+03 },
    { 7.2999999999999996e-10, 8.0597242147000003e-01, 1.1869356038733200e+04 },
    { 7.2999999999999996e-10, 2.2178191226299999e+00, 6.4932997056370104e+03 },
    { 7.2999999999999996e-10, 6.3102381616000003e-01, 8.4362875031645999e+03 },
    { 7.2999999999999996e-10, 2.5554823896599999e+00, 3.0376352639166598e+04 },
    { 7.2999999999999996e-10, 2.8823539845399999e+00, 3.6105859086315599e+04 },
    { 7.2999999999999996e-10, 1.4914821776300000e+00, 7.2500467274270004e+03 },
    { 7.2999999999999996e-10, 6.0285267527500004e+00, 6.6954519474012004e+03 },
    { 7.2999999999999996e-10, 2.6749667915000003e-01, 3.0831304904744600e+04 },
    { 7.2999999999999996e-10, 2.9983141668000002e+00, 3.3716965065866403e+04 },
    { 7.2000000000000000e-10, 5.6973807889300003e+00, 1.5276455874347401e+04 },
    { 7.2000000000000000e-10, 2.1038644616100002e+00, 3.4142478899194002e+03 },
    { 7.2000000000000000e-10, 8.7830980343999998e-01, 3.2669769634802001e+03 },
    { 7.2000000000000000e-10, 4.2596499161599999e+00, 6.8241872144623194e+04 },
    { 7.2000000000000000e-10, 1.5172221329700000e+00, 3.3435134409896000e+03 },
    { 7.2000000000000000e-10, 1.4649521320800001e+00, 3.3377114124099999e+03 },
    { 7.1000000000000003e-10, 5.8800233450899997e+00, 2.3539640076029998e+04 },
    { 7.1000000000000003e-10, 5.7894923296599998e+00, 5.6219102335132002e+03 },
    { 7.1000000000000003e-10, 5.9570893273700003e+00, 1.2140601140477800e+04 },
    { 7.1000000000000003e-10, 1.8184674612900000e+00, 5.8857031136547987e+04 },
    { 7.1000000000000003e-10, 2.1520115657300001e+00, 3.3716044114483998e+03 },
    { 7.1000000000000003e-10, 8.3016269931999997e-01, 3.3096204419512001e+03 },
    { 7.1000000000000003e-10, 3.2865265020000001e+00, 2.3296978782358790e+04 },
    { 7.1000000000000003e-10, 6.1979887165000003e+00, 5.6217756129075997e+03 },
    { 7.1000000000000003e-10, 2.8713847950500000e+00, 6.4862534689390004e+03 },
    { 7.1000000000000003e-10, 4.6264065613899996e+00, 4.1850432280699999e+02 },
    { 6.9999999999999996e-10, 2.4258593488200000e+00, 9.3848410080752001e+03 },
    { 6.9999999999999996e-10, 5.8797411276100000e+00, 1.1247208964769799e+04 },
    { 6.9999999999999996e-10, 4.7322495639400000e+00, 1.3951514533807400e+04 },
    { 6.9999999999999996e-10, 1.1858547943900000e+00, 2.3320629558164001e+03 },
    { 6.9999999999999996e-10, 2.4983675011500002e+00, 7.8436995578074002e+03 },
    { 6.9999999999999996e-10, 1.7896485442800001e+00, 4.3491618975831998e+03 },
    { 6.9999999999999996e-10, 3.5176477230700001e+00, 1.0397603011669600e+04 },
    { 6.9999999999999996e-10, 4.4151264144400004e+00, 9.1757544453482005e+03 },
    { 6.9999999999999996e-10, 4.0641418836199996e+00, 1.1787603905517000e+04 },
    { 6.9000000000000000e-10, 3.0133584735999999e+00, 8.7308204539810006e+01 },
    { 6.9000000000000000e-10, 1.9722848416600001e+00, 5.7843095593425996e+03 },
    { 6.9000000000000000e-10, 1.9029200135600000e+00, 8.1865126624925997e+03 },
    { 6.9000000000000000e-10, 4.2259768309099996e+00, 2.1272636673520989e+04 },
    { 6.9000000000000000e-10, 4.6994043230300004e+00, 9.4206206196899996e+02 },
    { 6.9000000000000000e-10, 4.0405389930100002e+00, 8.0275691447093996e+03 },
    { 6.9000000000000000e-10, 2.6035417798100000e+00, 2.3553867170031601e+04 },
    { 6.9000000000000000e-10, 1.7328806644400001e+00, 3.1866871931753999e+03 },
    { 6.9000000000000000e-10, 1.2492936006099999e+00, 3.4945376602242000e+03 },
    { 6.9000000000000000e-10, 1.3110551149500000e+00, 1.4422784624123999e+03 },
    { 6.9000000000000000e-10, 2.5963353955900002e+00, 5.8849441002353897e+03 },
    { 6.9000000000000000e-10, 2.3118248795400000e+00, 6.7574909246752004e+03 },
    { 6.9000000000000000e-10, 5.6556376409999998e-01, 5.8849095929309997e+03 },
    { 6.9000000000000000e-10, 2.9377457284000003e-01, 1.1240162728071800e+04 },
    { 6.8000000000000003e-10, 8.2765945113999995e-01, 8.5500842530358004e+03 },
    { 6.8000000000000003e-10, 2.3109085268700000e+00, 8.5287436120334005e+03 },
    { 6.8000000000000003e-10, 4.2746649298000000e-01, 4.8971805736551996e+03 },
    { 6.8000000000000003e-10, 4.2728242049600000e+00, 5.8959726951979997e+02 },
    { 6.6999999999999996e-10, 5.5756472275800002e+00, 6.1124029793722002e+03 },
    { 6.6999999999999996e-10, 2.4314744223300000e+00, 1.2725453434775000e+04 },
    { 6.6999999999999996e-10, 2.5889701121100002e+00, 2.9048547946960002e+02 },
    { 6.6999999999999996e-10, 3.8624737536199998e+00, 1.9247376553382401e+04 },
    { 6.6999999999999996e-10, 5.4290227341200001e+00, 1.3358859278147400e+04 },
    { 6.6999999999999996e-10, 2.4708587345700002e+00, 1.0832603679370000e+03 },
    { 6.6999999999999996e-10, 7.3021872403999999e-01, 4.3964712385399998e+03 },
    { 6.6999999999999996e-10, 6.8969222601000002e-01, 9.5524152459959998e+02 },
    { 6.6999999999999996e-10, 1.5702766348699999e+00, 2.8997871418221998e+03 },
    { 6.6999999999999996e-10, 2.2642275646100001e+00, 5.0420027199365402e+04 },
    { 6.6999999999999996e-10, 5.8639673057700001e+00, 1.3366040135451000e+04 },
    { 6.6000000000000000e-10, 3.8795480082000000e+00, 1.6056051300184190e+04 },
    { 6.6000000000000000e-10, 2.0607455451500001e+00, 1.7669373013379999e+02 },
    { 6.6000000000000000e-10, 3.3626401992099999e+00, 2.2582926841020000e+02 },
    { 6.6000000000000000e-10, 9.3287145960999995e-01, 4.1555249061209997e+02 },
    { 6.6000000000000000e-10, 3.6402745231200000e+00, 3.7386941198052000e+03 },
    { 6.6000000000000000e-10, 5.7457022170399998e+00, 3.3487805632822001e+03 },
    { 6.6000000000000000e-10, 3.5196573551900001e+00, 3.3324442901174002e+03 },
    { 6.6000000000000000e-10, 4.3961606398199997e+00, 3.3776080173107998e+03 },
    { 6.6000000000000000e-10, 4.8691989324099998e+00, 3.3036168360888000e+03 },
    { 6.6000000000000000e-10, 3.2317781362800000e+00, 3.7388287404108000e+03 },
    { 6.5000000000000003e-10, 5.4357213906100004e+00, 3.3573773174468001e+03 },
    { 6.5000000000000003e-10, 3.8296381816200000e+00, 3.3238475359528002e+03 },
    { 6.5000000000000003e-10, 2.9117784734200001e+00, 2.6887366049730601e+04 },
    { 6.5000000000000003e-10, 5.3058763270000000e+00, 3.2268829871314001e+03 },
    { 6.5000000000000003e-10, 3.9594832452299999e+00, 3.4543418662682002e+03 },
    { 6.5000000000000003e-10, 3.4099472716500001e+00, 5.8704893915012603e+04 },
    { 6.5000000000000003e-10, 5.8318909662599996e+00, 3.8650173506198997e+04 },
    { 6.5000000000000003e-10, 2.8554604858400001e+00, 4.8273861782890199e+04 },
    { 6.3999999999999996e-10, 2.1384174843000001e+00, 2.7035740212466801e+04 },
    { 6.3999999999999996e-10, 3.3226555661000001e-01, 6.5639049851793998e+03 },
    { 6.3999999999999996e-10, 3.5182120443099998e+00, 4.1427486983178802e+04 },
    { 6.3999999999999996e-10, 1.2297614946200000e+00, 2.1258409579519390e+04 },
    { 6.3999999999999996e-10, 1.1739199931800000e+00, 3.2522518055641802e+04 },
    { 6.3999999999999996e-10, 3.2330328905800001e+00, 5.5258636959925998e+03 },
    { 6.3000000000000000e-10, 4.2237848148200001e+00, 1.5003782916995800e+04 },
    { 6.3000000000000000e-10, 4.2893866068199999e+00, 3.7937341387420003e+02 },
    { 6.3000000000000000e-10, 2.1393668487699999e+00, 1.7660303464057000e+04 },
    { 6.3000000000000000e-10, 2.6376681613900002e+00, 9.7227108858301999e+03 },
    { 6.3000000000000000e-10, 5.9743220418399998e+00, 4.1369276871683996e+03 },
    { 6.2000000000000003e-10, 9.7050370814999998e-01, 4.2307507252699997e+02 },
    { 6.2000000000000003e-10, 3.7724644441200001e+00, 6.2885158605009997e+02 },
    { 6.2000000000000003e-10, 2.9199965871500000e+00, 2.7117608406497002e+03 },
    { 6.2000000000000003e-10, 1.0946400020000000e+00, 1.3738215438369400e+04 },
    { 6.2000000000000003e-10, 1.4592537305700000e+00, 3.8252024502790802e+04 },
    { 6.2000000000000003e-10, 3.8427986095000000e-01, 4.3764389113990001e+02 },
    { 6.2000000000000003e-10, 3.7143941942100001e+00, 2.2487371692841600e+04 },
    { 6.0999999999999996e-10, 5.8227917543899999e+00, 3.8702861381421999e+03 },
    { 6.0999999999999996e-10, 1.5703780787099999e+00, 3.8703206454465999e+03 },
    { 6.0999999999999996e-10, 5.7003654685100003e+00, 5.4314209028907600e+04 },
    { 6.0999999999999996e-10, 3.7359186348900000e+00, 2.4076444588125400e+04 },
    { 6.0999999999999996e-10, 6.1805094413599999e+00, 1.1037500297983601e+04 },
    { 6.0999999999999996e-10, 5.3299793816300003e+00, 2.2286032299359999e+02 },
    { 6.0999999999999996e-10, 2.4588094237700000e+00, 7.6304004623694000e+03 },
    { 6.0999999999999996e-10, 5.6264363887200002e+00, 1.7373124374143999e+03 },
    { 6.0000000000000000e-10, 4.1562478944599999e+00, 4.4535100352782203e+04 },
    { 6.0000000000000000e-10, 5.2777509929199997e+00, 3.3481911059094000e+03 },
    { 6.0000000000000000e-10, 3.9876085793099998e+00, 3.3330337474901899e+03 },
    { 5.9000000000000003e-10, 5.5622129026999998e+00, 1.4071439509075790e+04 },
    { 5.9000000000000003e-10, 4.2350068792300002e+00, 7.0569905849697998e+03 },
    { 5.9000000000000003e-10, 1.9895701062500000e+00, 1.4319691037357199e+04 },
    { 5.9000000000000003e-10, 5.6832362884999998e+00, 9.9781183677943991e+03 },
    { 5.9000000000000003e-10, 4.0142786431399999e+00, 2.2854596021804002e+04 },
    { 5.7999999999999996e-10, 7.1565564537000004e-01, 2.9491818303399999e+01 },
    { 5.7999999999999996e-10, 7.3286492044999996e-01, 1.6070278394185800e+04 },
    { 5.7999999999999996e-10, 1.5981619740299999e+00, 4.7120241901558002e+03 },
    { 5.7999999999999996e-10, 1.3840122910199999e+00, 1.9692006632437999e+03 },
    { 5.7000000000000000e-10, 3.9227730989100000e+00, 2.7278468816440800e+04 },
    { 5.7000000000000000e-10, 3.4853560773500001e+00, 3.7057577492566197e+04 },
    { 5.7000000000000000e-10, 3.6311409415500000e+00, 3.9302096962196001e+03 },
    { 5.7000000000000000e-10, 2.4200304208199999e+00, 1.0227765648850000e+03 },
    { 5.7000000000000000e-10, 2.8924431308499998e+00, 1.5842752204746201e+04 },
    { 5.4999999999999996e-10, 3.3452508144699999e+00, 3.3258751443741999e+03 },
    { 5.4999999999999996e-10, 5.9201087577599996e+00, 3.3553497090254000e+03 },
    { 5.4999999999999996e-10, 2.8068048703200001e+00, 4.5659841113701996e+03 },
    { 5.4999999999999996e-10, 2.4006330302999999e+00, 1.1560077716077400e+04 },
    { 5.4999999999999996e-10, 4.3607609662600000e+00, 2.1850829326400000e+01 },
    { 5.4999999999999996e-10, 3.7443777882000001e-01, 2.5665517483409199e+04 },
};

static const VSOP87Term VSOP87A_MARS_X1[956] = {
    { 8.6144137400000001e-03, 3.1415926535900001e+00, 0.0000000000000000e+00 },
    { 5.5243794900000000e-03, 5.0956587289100002e+00, 6.6812248533995999e+03 },
    { 7.7184976999999997e-04, 5.4331563620900001e+00, 1.0021837280099400e+04 },
    { 2.0467293999999999e-04, 5.5705181236900003e+00, 3.3406124266997999e+03 },
    { 9.5895810000000005e-05, 5.7710723479099997e+00, 1.3362449706799200e+04 },
    { 2.6206100000000000e-05, 6.2244129512199997e+00, 3.3441355450487999e+03 },
    { 2.6205369999999999e-05, 3.0417215443600001e+00, 3.3370893083507999e+03 },
    { 1.1636120000000000e-05, 6.1090925709699997e+00, 1.6703062133498999e+04 },
    { 9.0117799999999993e-06, 3.3158554819399999e+00, 1.0593819301891999e+03 },
    { 4.2705799999999997e-06, 2.7400898016599999e+00, 2.9424634232916001e+03 },
    { 3.9500100000000002e-06, 2.0042939555000000e-01, 6.6847479717486003e+03 },
    { 3.8691599999999999e-06, 5.9423255261200003e+00, 5.6218429232103999e+03 },
    { 3.8142799999999999e-06, 2.2342431377999999e-01, 3.7387614301079998e+03 },
    { 3.3735599999999999e-06, 5.2141736107599996e+00, 3.1491641605882000e+03 },
    { 3.3420899999999999e-06, 2.4591027596499999e+00, 3.1851920272655998e+03 },
    { 3.1846299999999999e-06, 4.0514025189399998e+00, 3.5320606928113998e+03 },
    { 3.1842599999999999e-06, 5.2909208915999995e-01, 3.4960328261340001e+03 },
    { 3.1260499999999999e-06, 4.2291228061300004e+00, 2.5443144198834002e+03 },
    { 2.8925100000000001e-06, 4.6175934545399997e+00, 7.9629800681639995e+02 },
    { 2.6030800000000000e-06, 5.0263075485600002e+00, 4.1369104335162001e+03 },
    { 2.0673300000000001e-06, 3.2108322967300000e+00, 1.1944470102246000e+03 },
    { 1.5936600000000000e-06, 3.5776297758200002e+00, 1.5890728952838001e+03 },
    { 1.4056999999999999e-06, 1.6405101702000000e-01, 2.0043674560198800e+04 },
    { 1.3381900000000001e-06, 4.6435181455099999e+00, 5.8849268465832001e+03 },
    { 1.3232300000000001e-06, 3.7009451342499999e+00, 4.3999943568890003e+03 },
    { 1.2946900000000000e-06, 1.8047763795800000e+00, 1.5925960136327999e+03 },
    { 1.2712300000000000e-06, 2.8180607644700002e+00, 6.6777017350506003e+03 },
    { 1.1259099999999999e-06, 5.8904764702400003e+00, 2.1461654164751999e+03 },
    { 1.0279599999999999e-06, 6.0567204756599997e+00, 5.4867778431750003e+03 },
    { 1.0145700000000000e-06, 3.7253191382800002e+00, 3.3415927477680002e+03 },
    { 1.0142899999999999e-06, 5.5411587427500004e+00, 3.3396321056316001e+03 },
    { 8.9230999999999997e-07, 5.8826535095699999e+00, 3.8703033917943999e+03 },
    { 8.4359000000000000e-07, 3.9861148409000002e-01, 1.9907450170410000e+03 },
    { 8.1615000000000001e-07, 5.6530124835700004e+00, 5.0921519581158000e+03 },
    { 7.9681000000000000e-07, 3.9108038550700002e+00, 4.2659819087599999e+02 },
    { 7.7267000000000005e-07, 6.4237838595999996e-01, 7.0793738568077997e+03 },
    { 7.5596999999999996e-07, 3.6116152912800001e+00, 4.5350594369244000e+03 },
    { 7.5338000000000004e-07, 1.2529851175000000e+00, 5.0886288397668004e+03 },
    { 7.0673000000000003e-07, 3.8682011187400001e+00, 2.7870430238573999e+03 },
    { 6.4069000000000000e-07, 5.2756810039099999e+00, 2.3888940204492001e+03 },
    { 6.0976999999999999e-07, 1.3367677651400001e+00, 1.7480164130670000e+03 },
    { 6.0337999999999999e-07, 5.4464353123100002e+00, 3.8941818295421999e+03 },
    { 5.8111999999999995e-07, 4.0782428306000001e-01, 7.1135470007999997e+00 },
    { 5.7779000000000004e-07, 1.9532242483800000e+00, 3.0978838227257902e+03 },
    { 5.7349000000000004e-07, 5.0336630983999997e-01, 1.0025360398448400e+04 },
    { 5.7037999999999995e-07, 2.6909726912599998e+00, 4.6904798363585996e+03 },
    { 5.6626000000000004e-07, 1.0678933961799999e+00, 1.5542039943419999e+02 },
    { 5.5693000000000000e-07, 1.2210097014900001e+00, 9.4921463150048003e+03 },
    { 5.4143999999999996e-07, 2.7522016882500000e+00, 2.9577158944766002e+03 },
    { 5.3020000000000002e-07, 1.0499090543100000e+00, 3.5833410306738001e+03 },
    { 5.1266000000000004e-07, 4.5870536497599996e+00, 1.9144826611159999e+02 },
    { 4.9852999999999995e-07, 2.6909559157000001e-01, 3.7235089589230001e+03 },
    { 4.9312999999999997e-07, 2.5094127434300000e+00, 6.6812075997474003e+03 },
    { 4.9312999999999997e-07, 4.5401842436499997e+00, 6.6812421070518003e+03 },
    { 4.9103000000000002e-07, 4.0836196657899997e+00, 4.2923308329503998e+03 },
    { 4.7446999999999999e-07, 5.6433239797399999e+00, 6.4897765872880000e+03 },
    { 4.1874000000000000e-07, 2.9478478103999999e+00, 1.3498674096587999e+03 },
    { 3.9136999999999999e-07, 5.3781346608699998e+00, 7.4775228602159996e+03 },
    { 3.7362999999999999e-07, 7.3933165770000003e-02, 3.5539115221378001e+03 },
    { 3.6019000000000000e-07, 2.8492239983199998e+00, 3.1273133312618002e+03 },
    { 3.5314000000000000e-07, 2.8043091753599998e+00, 6.5258044539654002e+03 },
    { 3.2810999999999999e-07, 5.0574262906299996e+00, 3.3334988796990001e+03 },
    { 3.2356999999999998e-07, 4.4579053717299999e+00, 9.5171840625059997e+02 },
    { 2.9947999999999998e-07, 5.9171713920300002e+00, 5.5356940284240000e+02 },
    { 2.9433999999999997e-07, 5.3474041379799999e+00, 6.2546266625236003e+03 },
    { 2.9268000000000000e-07, 2.6579724270499998e+00, 6.3989728631399998e+02 },
    { 2.8209000000000001e-07, 4.2484974101599997e+00, 3.3477259737006002e+03 },
    { 2.7640000000000001e-07, 3.7511880366299999e+00, 1.2303067776610000e+04 },
    { 2.7491999999999998e-07, 2.1930801026100002e+00, 4.9332084403325998e+03 },
    { 2.4831999999999999e-07, 3.8659558744400000e+00, 2.1187638603783998e+03 },
    { 2.4653000000000002e-07, 7.1737818564000000e-01, 3.8289653222319998e+02 },
    { 2.4446000000000001e-07, 6.2277795992499998e+00, 1.0664954771900000e+03 },
    { 2.4027000000000002e-07, 2.9966699717999998e+00, 2.8109214616052000e+03 },
    { 2.2160000000000000e-07, 5.3804537836600002e+00, 2.2812304965106000e+03 },
    { 2.1283999999999999e-07, 3.8456339041400001e+00, 4.9296853219836003e+03 },
    { 1.9368000000000000e-07, 6.5233184103999997e-01, 2.4272860397400001e+02 },
    { 1.9156000000000000e-07, 5.9778646268699998e+00, 5.2969096509459996e+02 },
    { 1.8360999999999999e-07, 6.1249454437999995e-01, 9.6236882766912004e+03 },
    { 1.7709999999999999e-07, 4.0378634322400000e+00, 7.7406067835887998e+03 },
    { 1.7086999999999999e-07, 2.3090942371200001e+00, 3.3202571073009999e+03 },
    { 1.7058999999999999e-07, 2.7032268886200002e+00, 6.6741113063987996e+03 },
    { 1.6990000000000001e-07, 5.0198426833999998e-01, 2.3384286986898602e+04 },
    { 1.6973000000000000e-07, 6.7405360149000004e-01, 3.3609677460985899e+03 },
    { 1.6701000000000001e-07, 6.7061213764000005e-01, 7.2109158184941998e+03 },
    { 1.6240000000000000e-07, 5.8693834825000002e-01, 6.6811575430967996e+03 },
    { 1.6240000000000000e-07, 1.7847274470999999e-01, 6.6812921637024001e+03 },
    { 1.6065000000000000e-07, 4.2498281711599999e+00, 3.3253599555147998e+03 },
    { 1.6010000000000001e-07, 4.5122114069000002e-01, 1.0522683831884001e+03 },
    { 1.5982000000000001e-07, 5.0180023774900002e+00, 3.3558648978848000e+03 },
    { 1.4620000000000000e-07, 5.3426228575200003e+00, 4.5624609930212000e+03 },
    { 1.4616000000000001e-07, 1.9214881061000000e+00, 1.0018314161750401e+04 },
    { 1.4497000000000001e-07, 5.1986042995000004e-01, 2.2741169495098002e+03 },
    { 1.3777000000000000e-07, 1.5497074460500000e+00, 1.2832758741704600e+04 },
    { 1.3623000000000000e-07, 4.0619379179999998e-02, 2.2883440435113998e+03 },
    { 1.3051000000000000e-07, 4.4931048333900003e+00, 3.7672106175757999e+03 },
    { 1.2683000000000000e-07, 7.7128314603000003e-01, 5.3313574437407997e+03 },
    { 1.2504000000000001e-07, 1.0042335831800000e+00, 1.0419986283507600e+04 },
    { 1.2485000000000001e-07, 4.0946071267799997e+00, 6.6822051744678001e+03 },
    { 1.2263999999999999e-07, 1.9247209595899999e+00, 5.3680451209540001e+02 },
    { 1.1614000000000001e-07, 2.3675999625999999e+00, 6.4384962494255997e+03 },
    { 1.1399000000000000e-07, 2.4449011944699999e+00, 5.2257741809380002e+02 },
    { 1.1354000000000000e-07, 3.1954807550999997e-01, 6.0413275670856001e+03 },
    { 1.0642999999999999e-07, 3.0654456523400002e+00, 5.6147293762095996e+03 },
    { 1.0339000000000000e-07, 4.8744251170100004e+00, 1.0021854533751601e+04 },
    { 1.0339000000000000e-07, 2.8436602606300001e+00, 1.0021820026447200e+04 },
    { 1.0305000000000000e-07, 3.9575936940599998e+00, 7.8756718636242003e+03 },
    { 9.7170000000000003e-08, 7.3258032600000000e-01, 6.8366452528338004e+03 },
    { 8.8220000000000005e-08, 1.5372861724500000e+00, 3.2055473466643998e+03 },
    { 8.4060000000000000e-08, 1.4444779464199999e+00, 3.4756775067352000e+03 },
    { 8.2389999999999999e-08, 4.8547598906899996e+00, 8.9624553499102003e+03 },
    { 8.1069999999999997e-08, 8.1975962806000002e-01, 1.3365972825148199e+04 },
    { 8.0140000000000002e-08, 4.3511032486700003e+00, 2.7007151403858002e+03 },
    { 7.8750000000000000e-08, 5.6397798917899999e+00, 5.7295064471490005e+03 },
    { 7.8069999999999997e-08, 4.2207749483099999e+00, 6.1276554505572003e+03 },
    { 7.6360000000000003e-08, 4.0507143643200001e+00, 1.7515395314160000e+03 },
    { 7.3809999999999997e-08, 4.0738358190100001e+00, 1.5643680203309799e+04 },
    { 7.2689999999999998e-08, 6.3965217204000002e-01, 6.8945239488376001e+03 },
    { 7.0650000000000001e-08, 4.7520794158899999e+00, 2.1329909543799999e+02 },
    { 7.0350000000000005e-08, 2.5290437407500002e+00, 5.6289564702112002e+03 },
    { 6.7259999999999995e-08, 6.0242828409699998e+00, 9.8303890139878004e+03 },
    { 6.7090000000000001e-08, 4.6132420164900001e+00, 3.1781457905676002e+03 },
    { 6.5120000000000003e-08, 5.8243570825399997e+00, 6.6802445323313996e+03 },
    { 6.3580000000000004e-08, 5.5913422388800003e+00, 3.3167339889519999e+03 },
    { 6.3320000000000001e-08, 4.6500475347799997e+00, 3.5030790628320001e+03 },
    { 6.2839999999999999e-08, 3.6719236378300000e+00, 3.3644908644476000e+03 },
    { 6.2719999999999995e-08, 3.1058719015900000e+00, 6.2983283211764001e+03 },
    { 6.0380000000000002e-08, 3.2761933202000000e+00, 3.9814900340819997e+02 },
    { 6.0240000000000002e-08, 1.6419775725700001e+00, 8.5319638175199998e+02 },
    { 5.6029999999999999e-08, 5.7228588144000003e+00, 1.0818135286915800e+04 },
    { 5.5420000000000003e-08, 9.7325944065000003e-01, 1.7586530784168001e+03 },
    { 5.5379999999999997e-08, 5.7861281478400004e+00, 3.5231183490000002e+00 },
    { 5.5019999999999999e-08, 1.3901007292500001e+00, 6.9239534573736000e+03 },
    { 5.3139999999999998e-08, 4.2236041380299998e+00, 2.6484548254729998e+03 },
    { 5.1310000000000001e-08, 5.7581612160700004e+00, 8.4327643848156004e+03 },
    { 5.0610000000000001e-08, 6.2472843731900003e+00, 1.5961864422845999e+03 },
    { 5.0190000000000001e-08, 3.6002592892999998e+00, 4.3371173787679999e+02 },
    { 4.8410000000000001e-08, 4.6251219835000001e-01, 6.1586474353058002e+03 },
    { 4.8300000000000002e-08, 4.6448244952400000e+00, 6.6883384004004001e+03 },
    { 4.7829999999999999e-08, 1.0360877013800001e+00, 6.1444203413041996e+03 },
    { 4.7300000000000000e-08, 4.2231163012000001e+00, 2.9140142358237999e+03 },
    { 4.3329999999999998e-08, 9.4382312112000000e-01, 1.2964300703391000e+04 },
    { 4.3259999999999998e-08, 4.0903170450499999e+00, 1.1773376811515400e+04 },
    { 4.2879999999999998e-08, 5.2293998252599998e+00, 6.1515338883049999e+03 },
    { 4.2270000000000001e-08, 5.7275408134900001e+00, 2.8180350086059998e+03 },
    { 4.1159999999999999e-08, 3.1469829628100001e+00, 9.8664168806651996e+03 },
    { 4.0709999999999999e-08, 3.4603811309900001e+00, 3.8631898447936001e+03 },
    { 4.0660000000000001e-08, 4.9731026300699996e+00, 4.0327700279266001e+03 },
    { 4.0060000000000003e-08, 3.5017566203400001e+00, 2.6997348193175999e+03 },
    { 3.9699999999999998e-08, 3.4201626026999998e-01, 3.3121632392319998e+03 },
    { 3.9319999999999997e-08, 2.6418210392099999e+00, 3.3690616141676001e+03 },
    { 3.9279999999999998e-08, 3.4350440351599998e+00, 8.9553418029093991e+03 },
    { 3.9239999999999999e-08, 5.5217779444800001e+00, 6.4087760738220004e+02 },
    { 3.8180000000000002e-08, 1.6306973834100000e+00, 6.2830758499914000e+03 },
    { 3.8159999999999999e-08, 2.9889473006200000e+00, 8.9695688969109997e+03 },
    { 3.7219999999999998e-08, 4.6711475373700004e+00, 3.3350895023923999e+03 },
    { 3.7210000000000000e-08, 4.5953382352699998e+00, 3.3461353510071999e+03 },
    { 3.6290000000000003e-08, 1.2576224099700000e+00, 3.3045845600224002e+03 },
    { 3.6220000000000003e-08, 5.9979143121499998e+00, 1.5819593482830001e+03 },
    { 3.5770000000000002e-08, 1.7245702004400001e+00, 3.3766402933772001e+03 },
    { 3.5670000000000001e-08, 8.9902213786999996e-01, 1.7444259844152000e+03 },
    { 3.5649999999999998e-08, 2.5367024710699999e+00, 8.2738208670324002e+03 },
    { 3.4520000000000001e-08, 3.5803388480599998e+00, 9.2255392732830005e+03 },
    { 3.4270000000000002e-08, 9.2910700550000003e-01, 1.0021769969796600e+04 },
    { 3.4170000000000001e-08, 5.1900593402999995e-01, 1.0021904590402200e+04 },
    { 3.2469999999999999e-08, 5.0731005475000002e-01, 1.4956319713459999e+02 },
    { 3.1629999999999999e-08, 3.1909253630899999e+00, 3.9805097130138001e+03 },
    { 3.1489999999999999e-08, 4.0709066946699997e+00, 5.4593762870782002e+03 },
    { 3.1170000000000001e-08, 3.8181219620400002e+00, 3.3367310913418000e+03 },
    { 3.1039999999999999e-08, 5.4423247590599999e+00, 3.3444937620577998e+03 },
    { 3.0939999999999998e-08, 1.7897129151100000e+00, 1.3358926588450200e+04 },
    { 3.0180000000000003e-08, 1.2727537359900001e+00, 1.0551528245194000e+04 },
    { 3.0090000000000000e-08, 5.9193269284300003e+00, 3.9814900340820000e+03 },
    { 2.9720000000000001e-08, 4.0647783627500003e+00, 4.1948464387519999e+02 },
    { 2.9300000000000001e-08, 5.2939793791799996e+00, 2.4874160449477999e+03 },
    { 2.8590000000000000e-08, 4.1482444517000001e+00, 8.2702977486834006e+03 },
    { 2.7529999999999999e-08, 3.4642178913400001e+00, 7.0641213856227996e+03 },
    { 2.6689999999999999e-08, 3.6909955838199999e+00, 6.4679257579615996e+03 },
    { 2.6619999999999999e-08, 6.2485553998999999e-01, 7.6329432596502002e+03 },
    { 2.6180000000000000e-08, 4.5959259160199997e+00, 1.2218485663214001e+03 },
    { 2.5279999999999999e-08, 5.7875592971299996e+00, 3.3335661900017999e+03 },
    { 2.5139999999999999e-08, 1.8852628473499999e+00, 1.6173371168404399e+04 },
    { 2.4579999999999999e-08, 2.1740426116000000e-01, 2.7662676283649998e+03 },
    { 2.4529999999999999e-08, 5.6556300676499998e+00, 8.0310922630584000e+03 },
    { 2.4369999999999999e-08, 4.3726444956100003e+00, 1.1081219210288600e+04 },
    { 2.4240000000000001e-08, 8.2684290603999999e-01, 9.8085381846614000e+03 },
    { 2.4210000000000000e-08, 5.2295436827400001e+00, 8.4292412664666008e+03 },
    { 2.3280000000000001e-08, 8.4599472285999999e-01, 9.3819399937853996e+03 },
    { 2.2760000000000000e-08, 1.5666732435699999e+00, 5.8280284716475999e+03 },
    { 2.2480000000000000e-08, 2.2149789363600001e+00, 7.2347942562420003e+03 },
    { 2.2250000000000001e-08, 2.8856125174499998e+00, 3.9149572250346000e+03 },
    { 2.1920000000000000e-08, 3.1892596038999999e-01, 4.4071079038897997e+03 },
    { 2.1400000000000000e-08, 3.2205081102699999e+00, 3.3476586633978000e+03 },
    { 2.1190000000000000e-08, 1.7130154773999999e+00, 6.8726731195111997e+03 },
    { 2.1069999999999999e-08, 3.6327801078099999e+00, 1.5113989238215199e+04 },
    { 2.0590000000000001e-08, 8.4035144430999997e-01, 2.6724899413598399e+04 },
    { 2.0440000000000000e-08, 3.9178907955900000e+00, 6.0403472460173998e+03 },
    { 1.8870000000000000e-08, 1.7011382442000000e-01, 7.6266071275599998e+01 },
    { 1.8699999999999999e-08, 5.2584531074900003e+00, 2.2041264243879999e+02 },
    { 1.8670000000000001e-08, 1.3529897821900001e+00, 1.3760598710207400e+04 },
    { 1.8460000000000001e-08, 2.7900630290600001e+00, 1.6921656695024001e+03 },
    { 1.8240000000000000e-08, 5.6575943947800003e+00, 6.2475131155228000e+03 },
    { 1.8069999999999999e-08, 5.3705910769800003e+00, 6.6964773245846000e+03 },
    { 1.7879999999999999e-08, 2.4520427344100000e+00, 6.6604494579071998e+03 },
    { 1.7669999999999999e-08, 2.6499127861599998e+00, 6.6608695340007998e+03 },
    { 1.7439999999999999e-08, 1.7943418627500001e+00, 6.8162899334350004e+03 },
    { 1.7389999999999999e-08, 3.6490233546900002e+00, 3.2812385647862002e+03 },
    { 1.7170000000000001e-08, 5.2157611313799999e+00, 1.3362466960451400e+04 },
    { 1.7170000000000001e-08, 3.1849895060200000e+00, 1.3362432453146999e+04 },
    { 1.6980000000000001e-08, 5.6163763535399998e+00, 3.3999862886134001e+03 },
    { 1.6899999999999999e-08, 4.8993113275900004e+00, 7.1078230442756003e+03 },
    { 1.6569999999999999e-08, 5.4065020919900002e+00, 8.8273902698748007e+03 },
    { 1.6280000000000001e-08, 4.4515347068200004e+00, 1.0022817601167601e+04 },
    { 1.5930000000000001e-08, 5.8121882751399996e+00, 2.0618554843720000e+02 },
    { 1.5930000000000001e-08, 1.1130706036800000e+00, 8.6719698704406001e+03 },
    { 1.5690000000000000e-08, 3.2602802053000000e+00, 5.7434479833479998e+02 },
    { 1.5320000000000001e-08, 3.0402355936499998e+00, 3.8774169387952002e+03 },
    { 1.5290000000000000e-08, 5.6980933613999996e-01, 3.5467979751369999e+03 },
    { 1.5270000000000000e-08, 2.3997001334000001e+00, 3.1344268782626000e+03 },
    { 1.5150000000000000e-08, 4.6127877568300004e+00, 6.7020002488919999e+03 },
    { 1.4979999999999999e-08, 3.0422751726300001e+00, 1.0014723733098601e+04 },
    { 1.4730000000000000e-08, 3.9465602482800000e+00, 8.4398779318163997e+03 },
    { 1.4160000000000001e-08, 2.6199097490700001e+00, 6.4701083331480004e+02 },
    { 1.4030000000000001e-08, 4.4049572798099996e+00, 1.8984292630009601e+04 },
    { 1.4000000000000000e-08, 4.5232604378400003e+00, 5.0992655051166003e+03 },
    { 1.3840000000000000e-08, 4.2931013988500002e+00, 1.1216284290324000e+04 },
    { 1.3669999999999999e-08, 3.0238320459999999e+00, 9.7791086761253991e+03 },
    { 1.3599999999999999e-08, 2.2764191372600000e+00, 3.0740053849780002e+03 },
    { 1.3580000000000000e-08, 4.0902959468800004e+00, 1.0390266107903999e+03 },
    { 1.3399999999999999e-08, 5.4539265294400003e+00, 2.8038079146044001e+03 },
    { 1.3049999999999999e-08, 9.7554521193999999e-01, 6.7015801727983999e+03 },
    { 1.3020000000000000e-08, 4.5577384970799999e+00, 6.6659723822145997e+03 },
    { 1.2340000000000000e-08, 1.3592083236600001e+00, 1.2289621133221999e+03 },
    { 1.2299999999999999e-08, 2.9260201065000002e-01, 1.1371704689758200e+04 },
    { 1.2190000000000000e-08, 7.0589188231000000e-01, 3.6072194684216001e+03 },
    { 1.2029999999999999e-08, 1.9392260559500001e+00, 3.3198370312073998e+03 },
    { 1.1920000000000000e-08, 1.2746023938400000e+00, 1.1020632121940000e+02 },
    { 1.1570000000000000e-08, 1.0761522476100001e+00, 1.0235136375537400e+04 },
    { 1.1479999999999999e-08, 1.0485214856300000e+00, 3.3613878221922000e+03 },
    { 1.1240000000000000e-08, 3.2477868926800002e+00, 2.9069006888230001e+03 },
    { 1.1230000000000000e-08, 1.6441379783200001e+00, 2.9211277828246002e+03 },
    { 1.1210000000000001e-08, 1.1464357265100000e+00, 1.6706585251847999e+04 },
    { 1.0890000000000000e-08, 2.2851867505099999e+00, 3.1699395560806001e+03 },
    { 1.0859999999999999e-08, 3.2598075388100001e+00, 1.2935851515923199e+04 },
    { 1.0820000000000000e-08, 3.3207490981999999e+00, 6.6776344247478000e+03 },
    { 1.0780000000000000e-08, 1.8839207472699999e+00, 1.0575406682941801e+04 },
    { 1.0710000000000000e-08, 2.4542600962700001e+00, 6.5316616562649997e+03 },
    { 1.0680000000000000e-08, 3.7380138616499998e+00, 6.6848152820513997e+03 },
    { 1.0570000000000000e-08, 8.1408234263000001e-01, 1.0973555686350001e+04 },
    { 1.0270000000000000e-08, 5.3095385363999998e+00, 6.2617402095243997e+03 },
    { 1.0200000000000000e-08, 6.9684985470000005e-01, 3.5112852973190002e+03 },
    { 1.0099999999999999e-08, 1.0476311828300000e+00, 3.2533042221599999e+03 },
    { 1.0000000000000000e-08, 4.4812309473800003e+00, 9.5952390892233998e+03 },
    { 9.8199999999999996e-09, 2.5720172164999999e-01, 4.9890591838971995e+03 },
    { 9.7700000000000008e-09, 3.7638716752799999e+00, 8.4256508378147992e+03 },
    { 9.7399999999999999e-09, 1.9346198533400001e+00, 3.4279206312396000e+03 },
    { 9.3000000000000006e-09, 5.9814725316399997e+00, 9.0701188738487999e+03 },
    { 9.2500000000000001e-09, 5.0900486108800003e+00, 1.1243685846420800e+04 },
    { 9.2199999999999992e-09, 4.5644566559299999e+00, 9.4682678772569998e+03 },
    { 9.1800000000000001e-09, 1.0710061200000000e-01, 1.3171001440687600e+04 },
    { 9.1700000000000004e-09, 5.0158742129200000e+00, 3.3406296803519999e+03 },
    { 9.1700000000000004e-09, 2.9851026579300002e+00, 3.3405951730475999e+03 },
    { 9.1600000000000006e-09, 4.6529725520000002e-02, 3.5610250691386000e+03 },
    { 8.9600000000000005e-09, 5.1425875570699997e+00, 2.3015858159093900e+03 },
    { 8.8699999999999994e-09, 2.3591025112100001e+00, 3.7600970705750001e+03 },
    { 8.7999999999999994e-09, 2.4751675996200002e+00, 3.3132108706029999e+03 },
    { 8.7799999999999999e-09, 1.6941691762300000e+00, 9.5628915597059995e+02 },
    { 8.7199999999999997e-09, 2.7853882199900002e+00, 3.1201997842609999e+03 },
    { 8.7000000000000001e-09, 5.0700932773999996e-01, 3.3680139827966000e+03 },
    { 8.4499999999999996e-09, 8.3947970783000003e-01, 9.4992598620055996e+03 },
    { 8.4000000000000008e-09, 1.2611650681000000e-01, 2.1258774073792001e+03 },
    { 8.2200000000000002e-09, 2.5849507101300002e+00, 1.8617458526354001e+03 },
    { 8.0000000000000005e-09, 4.7915411503499996e+00, 6.5187582172674001e+03 },
    { 7.9699999999999996e-09, 1.2922445315000000e+00, 1.6304913130090799e+04 },
    { 7.8999999999999996e-09, 2.7725634462900000e+00, 3.4437052009183999e+03 },
    { 7.8999999999999996e-09, 4.0240007064099999e+00, 1.0404733812322600e+04 },
    { 7.7699999999999994e-09, 1.9060827732000001e+00, 9.1686408983473993e+03 },
    { 7.6899999999999997e-09, 3.4517037185900001e+00, 9.6389407478762005e+03 },
    { 7.6799999999999999e-09, 4.9332377028699996e+00, 1.1769853693166400e+04 },
    { 7.5900000000000005e-09, 6.0611251558900001e+00, 1.4158747713615599e+04 },
    { 7.5599999999999995e-09, 3.1748051746199999e+00, 6.3278373931320004e+02 },
    { 7.5300000000000003e-09, 5.4154692940100002e+00, 5.6421982426092000e+03 },
    { 7.4999999999999993e-09, 6.0808873002999997e+00, 3.2968935143948001e+03 },
    { 7.3700000000000000e-09, 3.1844872810700000e+00, 3.3843313390048002e+03 },
    { 7.3000000000000000e-09, 3.0210281670399999e+00, 6.7096740408674004e+03 },
    { 7.2500000000000004e-09, 3.1764144162400001e+00, 6.4750393049623999e+03 },
    { 7.2099999999999997e-09, 5.7683443129300001e+00, 7.9030734197210004e+03 },
    { 7.2099999999999997e-09, 3.3981901636799998e+00, 1.2310181323610799e+04 },
    { 7.0999999999999999e-09, 3.8949278551800002e+00, 6.4608122109608003e+03 },
    { 7.0100000000000004e-09, 3.8714381576000001e-01, 3.3111829181637900e+03 },
    { 7.0100000000000004e-09, 2.5950304492899998e+00, 3.3700419352357999e+03 },
    { 6.8900000000000000e-09, 4.6790039839900004e+00, 7.9101869667217998e+03 },
    { 6.8800000000000002e-09, 5.0039885543200002e+00, 1.0028950827100200e+04 },
    { 6.7400000000000003e-09, 2.9703674471800001e+00, 2.8073983432562000e+03 },
    { 6.6999999999999996e-09, 5.9456978750399996e+00, 6.6573464156518003e+03 },
    { 6.6000000000000004e-09, 1.3184997512900001e+00, 1.2147350193206000e+03 },
    { 6.4700000000000002e-09, 9.3841423908999999e-01, 4.3928808098882000e+03 },
    { 6.4599999999999996e-09, 5.7754565092299996e+00, 2.1116503133776000e+03 },
    { 6.3400000000000000e-09, 5.1369892158599999e+00, 6.9215760122680001e+02 },
    { 6.3300000000000003e-09, 2.8143284566300002e+00, 6.6049587821240002e+03 },
    { 6.2700000000000001e-09, 1.1103427652200000e+00, 1.3149150611361199e+04 },
    { 6.2300000000000002e-09, 1.7289289470700000e+00, 1.0264565884073400e+04 },
    { 6.1799999999999998e-09, 2.2150206709800000e+00, 1.0177257679533601e+04 },
    { 6.1300000000000001e-09, 4.4003699988299996e+00, 5.9890672521728002e+03 },
    { 6.1200000000000004e-09, 2.9523233871999999e-01, 3.2375196524812000e+03 },
    { 6.1099999999999998e-09, 2.7287007982199998e+00, 1.2566151699982800e+04 },
    { 6.0500000000000004e-09, 6.1666132113399996e+00, 9.4917560896980001e+02 },
    { 6.0000000000000000e-09, 3.7440291228899998e+00, 2.7078286873866000e+03 },
    { 5.8399999999999997e-09, 3.0094472591299999e+00, 5.0850384111149997e+03 },
    { 5.8399999999999997e-09, 2.1224143021200002e+00, 4.1938088084518004e+03 },
    { 5.7299999999999999e-09, 2.6951060013600001e+00, 1.4371756141986000e+03 },
    { 5.6999999999999998e-09, 4.2968768206299996e+00, 2.6936015933849999e+03 },
    { 5.5700000000000004e-09, 8.5462599458999999e-01, 1.3362517017102000e+04 },
    { 5.5700000000000004e-09, 1.2631493646900001e+00, 1.3362382396496399e+04 },
    { 5.5100000000000002e-09, 3.3721491313200000e+00, 2.6308392337279997e+02 },
    { 5.4400000000000002e-09, 1.9711695294100000e+00, 1.6699539015149989e+04 },
    { 5.4000000000000004e-09, 5.1844562566999997e-01, 6.0342140200847998e+03 },
    { 5.3899999999999998e-09, 8.2065219960000002e-01, 3.4901756238344001e+03 },
    { 5.3700000000000003e-09, 9.6991813199999999e-03, 3.8738265101433999e+03 },
    { 5.3300000000000004e-09, 3.7707919380999999e+00, 1.8454601664915001e+04 },
    { 5.2700000000000002e-09, 6.2089129765899997e+00, 1.3757737998458001e+03 },
    { 5.1499999999999998e-09, 5.1856286963800002e+00, 6.6757019290921999e+03 },
    { 5.1300000000000003e-09, 4.9401531230300000e+00, 3.2329489027611999e+03 },
    { 5.1099999999999999e-09, 3.0754336458300000e+00, 1.3517870106233400e+04 },
    { 5.0600000000000003e-09, 1.3772844947900000e+00, 9.4850327680040009e+03 },
    { 5.0000000000000001e-09, 2.9393345221399998e+00, 2.2434479570190001e+02 },
    { 4.9399999999999999e-09, 3.4853925282499998e+00, 1.3207029307364999e+04 },
    { 4.9200000000000004e-09, 4.3251366492800001e+00, 3.4482759506384000e+03 },
    { 4.8900000000000003e-09, 4.1765893970899999e+00, 3.7743241645766002e+03 },
    { 4.8799999999999997e-09, 2.9822380193100000e+00, 2.2218566345969998e+03 },
    { 4.7799999999999996e-09, 4.1615869166800001e+00, 1.3119721102825190e+04 },
    { 4.7699999999999999e-09, 5.0732405089899997e+00, 7.8959598727202001e+03 },
    { 4.7500000000000003e-09, 2.2322393489699999e+00, 2.1781377222920000e+03 },
    { 4.7399999999999998e-09, 1.7479687060800000e+00, 1.3892140671893800e+04 },
    { 4.7200000000000002e-09, 3.6718438661200001e+00, 1.2295954229609200e+04 },
    { 4.7200000000000002e-09, 3.3922281030000001e-01, 4.9367988689843996e+03 },
    { 4.6299999999999999e-09, 1.3884913781499999e+00, 3.2294257844121998e+03 },
    { 4.5299999999999999e-09, 2.0991409046000000e+00, 3.1131362939107998e+03 },
    { 4.4400000000000004e-09, 3.9587728951200001e+00, 6.7051032911474003e+03 },
    { 4.4299999999999998e-09, 1.5936577252599999e+00, 3.4517990689874000e+03 },
    { 4.2700000000000004e-09, 2.8619570721700001e+00, 1.1614433293732200e+04 },
    { 4.2199999999999999e-09, 2.1933642055000000e+00, 1.9513983595104201e+04 },
    { 4.2100000000000001e-09, 4.4984855447400003e+00, 4.3796390374902003e+03 },
    { 4.1299999999999996e-09, 4.3447995736799996e+00, 8.7999887137780006e+03 },
    { 4.1199999999999998e-09, 8.8247746033999996e-01, 3.5680885594888000e+03 },
    { 4.0700000000000002e-09, 1.2588467991600001e+00, 3.2893320888373901e+03 },
    { 4.0700000000000002e-09, 1.7233274658900000e+00, 3.3918927645622098e+03 },
    { 4.0199999999999998e-09, 1.7199238021700001e+00, 6.5710185321802001e+03 },
    { 3.9899999999999997e-09, 2.0132940332199998e+00, 1.3916019109641600e+04 },
    { 3.9199999999999997e-09, 4.3933746385600001e+00, 6.8436914895317996e+03 },
    { 3.9000000000000002e-09, 2.1075532049599999e+00, 6.7172527200770000e+03 },
    { 3.8899999999999996e-09, 6.0888276588299997e+00, 9.5881255422226004e+03 },
    { 3.8799999999999998e-09, 3.0301031930000000e-02, 6.0484411140864004e+03 },
    { 3.8400000000000000e-09, 5.8057793975400003e+00, 6.6851061887575997e+03 },
    { 3.7499999999999997e-09, 4.4759173260800003e+00, 1.1610910175383200e+04 },
    { 3.7399999999999999e-09, 4.0242591931699998e+00, 3.4153940252671000e+03 },
    { 3.7300000000000001e-09, 6.0114962111299999e+00, 1.0020856959031200e+04 },
    { 3.7000000000000000e-09, 1.3609456947600000e+00, 3.2741250177853999e+03 },
    { 3.6899999999999999e-09, 5.0254557522100001e+00, 1.3506508003540000e+02 },
    { 3.6300000000000001e-09, 1.2460380422799999e+00, 2.0355319398799999e+01 },
    { 3.6300000000000001e-09, 2.7928135878300000e+00, 1.0001061884606999e+04 },
    { 3.6000000000000000e-09, 1.6213066387299999e+00, 3.4070998356141999e+03 },
    { 3.6000000000000000e-09, 5.7790692476000005e-01, 5.2440492392010001e+03 },
    { 3.5899999999999998e-09, 5.0008013020900002e+00, 9.3809596727171993e+03 },
    { 3.5600000000000001e-09, 3.2554077734000000e+00, 1.1624747044077999e+03 },
    { 3.4699999999999998e-09, 1.2947512204700000e+00, 5.7249356974290004e+03 },
    { 3.4600000000000000e-09, 5.6077262194499999e+00, 9.6023526362241992e+03 },
    { 3.4499999999999999e-09, 4.5913262527800001e+00, 1.5252471184999999e+01 },
    { 3.4400000000000001e-09, 5.7100371961300000e+00, 1.2721572099417001e+04 },
    { 3.4400000000000001e-09, 4.7873990239000003e-01, 1.9034368125011999e+03 },
    { 3.3999999999999998e-09, 3.6450818995900001e+00, 7.3211221397135996e+03 },
    { 3.3999999999999998e-09, 4.7854189902000002e+00, 2.2324905056709398e+04 },
    { 3.3700000000000001e-09, 5.5576004957800000e+00, 1.6246663613219999e+02 },
    { 3.3499999999999998e-09, 4.5569089953399997e+00, 2.9389403049426001e+03 },
    { 3.3000000000000002e-09, 2.0662038883400000e+00, 3.3187615973734000e+03 },
    { 3.3000000000000002e-09, 9.1597037671000003e-01, 3.3624632560261998e+03 },
    { 3.2900000000000000e-09, 4.9430363066899998e+00, 1.0042612675591799e+04 },
    { 3.2700000000000001e-09, 6.1673533321600003e+00, 4.4593682188025996e+03 },
    { 3.1500000000000001e-09, 4.7228005313599999e+00, 2.3843232707292000e+03 },
    { 3.1099999999999998e-09, 5.5434706061499996e+00, 3.2350541665740002e+02 },
    { 3.1000000000000000e-09, 4.7084876202099997e+00, 1.4421831636988400e+04 },
    { 3.0899999999999999e-09, 6.1785656879999999e+00, 4.5695745400220003e+03 },
    { 3.0800000000000001e-09, 4.7479257146800000e+00, 1.6460333529524989e+04 },
    { 3.0500000000000000e-09, 5.2182546986100000e+00, 3.2658308281324998e+03 },
    { 3.0399999999999998e-09, 3.9189840278400001e+00, 7.2038022714934004e+03 },
    { 3.0100000000000002e-09, 6.4905373192000004e-01, 3.3406797370026002e+03 },
    { 3.0100000000000002e-09, 1.0560844056000001e+00, 3.3405451163970001e+03 },
    { 2.9800000000000000e-09, 5.6249511314000000e-01, 3.4168784979754000e+03 },
    { 2.9400000000000002e-09, 4.5514015174800004e+00, 1.0596182078434200e+04 },
    { 2.9300000000000000e-09, 3.6894923297000002e+00, 2.6758563815697999e+03 },
    { 2.9100000000000001e-09, 1.5471322256300000e+00, 6.6451969867221997e+03 },
    { 2.8499999999999999e-09, 2.0235421649099998e+00, 1.4314168113049800e+04 },
    { 2.8400000000000001e-09, 3.9542110726800002e+00, 1.4584298273120599e+04 },
    { 2.8400000000000001e-09, 2.1376251728800000e+00, 1.0156902360134800e+04 },
    { 2.8200000000000002e-09, 1.6619489597999999e+00, 8.6030992875280003e+02 },
    { 2.7999999999999998e-09, 1.6261510758100000e+00, 3.3975108016354002e+03 },
    { 2.7999999999999998e-09, 1.3560231892400001e+00, 3.2837140517642001e+03 },
    { 2.7999999999999998e-09, 3.4089656767399998e+00, 6.6882710900975999e+03 },
    { 2.7799999999999999e-09, 6.2452887437399998e+00, 7.3221024607817999e+03 },
    { 2.7799999999999999e-09, 1.0733659339899999e+00, 7.4781598567299994e+01 },
    { 2.7499999999999998e-09, 3.0810292100500001e+00, 5.7320492444297997e+03 },
    { 2.6599999999999999e-09, 5.0529256914899996e+00, 2.2777073781616000e+03 },
    { 2.5899999999999999e-09, 3.1310399492899998e+00, 5.0327780962021998e+03 },
    { 2.5300000000000002e-09, 1.6880183822399999e+00, 1.7101211136907201e+04 },
    { 2.5300000000000002e-09, 3.5231964422200002e+00, 1.6703044879846799e+04 },
    { 2.5300000000000002e-09, 1.0813016210699999e+00, 6.9478318951213996e+03 },
    { 2.5300000000000002e-09, 5.5539684847200004e+00, 1.6703079387151189e+04 },
    { 2.5300000000000002e-09, 6.2074036950200000e+00, 1.6062184526116800e+04 },
    { 2.5099999999999998e-09, 3.2344836541999999e+00, 1.1766263264514600e+04 },
    { 2.4899999999999999e-09, 1.6387577068800001e+00, 6.9927114822759995e+02 },
    { 2.4600000000000002e-09, 2.7302963600900001e+00, 3.3040070613956000e+03 },
    { 2.4600000000000002e-09, 2.5187790496000001e-01, 3.3772177920039999e+03 },
    { 2.4300000000000001e-09, 1.1785331404299999e+00, 3.0065511840298201e+04 },
    { 2.4199999999999999e-09, 3.8286910268300001e+00, 6.6218509914859997e+03 },
    { 2.4000000000000000e-09, 1.3532746026500000e+00, 2.0750449480734001e+03 },
    { 2.3899999999999998e-09, 3.5721915656599998e+00, 1.6276463942623001e+04 },
    { 2.3899999999999998e-09, 3.6650971957300000e+00, 1.1780490358516199e+04 },
    { 2.3899999999999998e-09, 4.7891810573600004e+00, 1.3936794505133999e+04 },
    { 2.3899999999999998e-09, 5.6549194814000003e-01, 6.1068800550648002e+03 },
    { 2.3800000000000001e-09, 4.9843787556799999e+00, 7.2555696517344004e+03 },
    { 2.3400000000000002e-09, 7.6568079766999997e-01, 3.1852593375684000e+03 },
    { 2.2499999999999999e-09, 3.6455184361900002e+00, 1.0018246851447600e+04 },
    { 2.2400000000000001e-09, 4.0774656719199998e+00, 1.0025427708751200e+04 },
    { 2.2300000000000000e-09, 4.3509066934899998e+00, 2.3878437747800000e+01 },
    { 2.2100000000000000e-09, 5.5766361638899999e+00, 4.0053684718298000e+03 },
    { 2.1999999999999998e-09, 2.2471443858100000e+00, 3.4959655158311998e+03 },
    { 2.1900000000000001e-09, 2.9387010601500001e+00, 5.3054510535538002e+03 },
    { 2.1799999999999999e-09, 3.3494825566999999e+00, 3.0297716278314001e+03 },
    { 2.1600000000000000e-09, 5.7175957396500001e+00, 1.0037089751284400e+04 },
    { 2.1499999999999998e-09, 4.1895907685399996e+00, 3.2125935833623998e+03 },
    { 2.1499999999999998e-09, 6.2454122818500002e+00, 1.5453539829788001e+03 },
    { 2.1299999999999999e-09, 5.2783596191099997e+00, 1.0448435470975401e+04 },
    { 2.1299999999999999e-09, 6.7649924264000005e-01, 7.7477203305896001e+03 },
    { 2.1200000000000001e-09, 5.3059116370100003e+00, 3.2304061054804001e+03 },
    { 2.1200000000000001e-09, 4.8026883979399999e+00, 1.3363430027867400e+04 },
    { 2.1100000000000000e-09, 3.9022292974699999e+00, 6.9443087767724001e+03 },
    { 2.1000000000000002e-09, 1.4769444377900001e+00, 4.5553474460203997e+03 },
    { 2.0900000000000000e-09, 3.3688749483699998e+00, 1.6858482532933202e+04 },
    { 2.0700000000000001e-09, 6.1128699251400000e+00, 1.4712317116458000e+04 },
    { 2.0500000000000002e-09, 1.5663810293800000e+00, 2.9459865416406001e+03 },
    { 2.0400000000000000e-09, 5.0756539680700001e+00, 3.4686312700372000e+03 },
    { 2.0200000000000001e-09, 1.4909977968799999e+00, 1.2012582297140400e+04 },
    { 2.0099999999999999e-09, 1.2833879047500001e+00, 1.8353246176068001e+03 },
    { 2.0000000000000001e-09, 4.0795007603200002e+00, 6.6773435180416000e+03 },
    { 2.0000000000000001e-09, 2.9904938403500001e+00, 1.0001481960700610e+04 },
    { 1.9900000000000000e-09, 2.9233269142800000e+00, 1.7924910699820401e+04 },
    { 1.9800000000000002e-09, 2.2514658210999999e-01, 6.5781320791810003e+03 },
    { 1.9800000000000002e-09, 1.1659753101500001e+00, 1.5508615123274400e+04 },
    { 1.9700000000000000e-09, 1.2623202942100000e+00, 1.5664035522708589e+04 },
    { 1.9500000000000001e-09, 4.2819289585999998e-01, 6.5461597733642002e+03 },
    { 1.9300000000000002e-09, 5.9146231563800002e+00, 3.6514532255682002e+03 },
    { 1.9200000000000000e-09, 5.2244418161299997e+00, 2.4803024979470001e+03 },
    { 1.9200000000000000e-09, 4.9215463843500000e+00, 1.4054607308025999e+04 },
    { 1.9200000000000000e-09, 3.2797442628600000e+00, 2.7515475996916002e+03 },
    { 1.9000000000000001e-09, 8.8866953545000005e-01, 1.2323423096008801e+04 },
    { 1.8899999999999999e-09, 2.8835038064000001e-01, 1.9402796952816600e+04 },
    { 1.8800000000000001e-09, 2.8036599780000002e+00, 1.3320548875408001e+03 },
    { 1.8500000000000000e-09, 4.3352797459700003e+00, 2.4092493398480001e+03 },
    { 1.8400000000000001e-09, 2.0645453485199998e+00, 4.8459002357928002e+03 },
    { 1.8400000000000001e-09, 3.9381069159000002e+00, 7.2180293654950001e+03 },
    { 1.8300000000000001e-09, 5.5657038189600003e+00, 8.5429707060349992e+03 },
    { 1.8300000000000001e-09, 5.8825866823800004e+00, 5.3321408344359997e+02 },
    { 1.8199999999999999e-09, 1.0975410800200001e+00, 6.8874104018367998e+03 },
    { 1.7900000000000000e-09, 6.2581174111299998e+00, 2.0775395492400001e+01 },
    { 1.7800000000000001e-09, 4.2143876990000002e-02, 6.4568800576977001e+03 },
    { 1.7800000000000001e-09, 1.8777295389599999e+00, 2.2847536148596000e+03 },
    { 1.7700000000000001e-09, 5.3911355304199997e+00, 7.3804960016271998e+03 },
    { 1.7700000000000001e-09, 5.2109630182500002e+00, 6.6867477777069998e+03 },
    { 1.7700000000000001e-09, 1.6745643304200000e+00, 8.9589322315612008e+03 },
    { 1.7500000000000000e-09, 5.2130528553000000e-01, 8.9828106693089994e+03 },
    { 1.7500000000000000e-09, 4.8622932351600001e+00, 8.9659784682591999e+03 },
    { 1.7400000000000000e-09, 3.2422496328900001e+00, 6.4181409300267997e+03 },
    { 1.7300000000000000e-09, 3.3695647576000001e+00, 1.4227094001599999e+01 },
    { 1.7300000000000000e-09, 1.6711307797699999e+00, 1.9004647949408401e+04 },
    { 1.7100000000000001e-09, 4.4374388820900004e+00, 6.7560064519669004e+03 },
    { 1.7100000000000001e-09, 4.4956777745799998e+00, 1.0629050485381999e+03 },
    { 1.7100000000000001e-09, 5.6475191629999999e-01, 2.3914368177299998e+03 },
    { 1.6900000000000000e-09, 3.7714459018000002e-01, 2.6555683724738001e+03 },
    { 1.6900000000000000e-09, 1.7278479526299999e+00, 1.9645525556790599e+04 },
    { 1.6800000000000000e-09, 1.4700647171300001e+00, 1.3575748802237200e+04 },
    { 1.6800000000000000e-09, 1.2496398565200000e+00, 5.6183198048614004e+03 },
    { 1.6800000000000000e-09, 1.3731230528800000e+00, 7.3140085927128002e+03 },
    { 1.6600000000000001e-09, 3.8285862422600001e+00, 1.5650793750310600e+04 },
    { 1.6200000000000000e-09, 5.9930555380400001e+00, 5.5187501489918004e+03 },
    { 1.6000000000000001e-09, 4.4548796028700002e+00, 2.4945295919485998e+03 },
    { 1.5900000000000001e-09, 2.6355627486400000e+00, 2.0085575391590000e+03 },
    { 1.5799999999999999e-09, 4.6098682197700001e+00, 1.4556896717023799e+04 },
    { 1.5799999999999999e-09, 5.9869723084500004e+00, 3.9296772537080001e+03 },
    { 1.5799999999999999e-09, 3.4251355245099999e+00, 9.8032106819999998e-01 },
    { 1.5700000000000000e-09, 3.3900422646699999e+00, 3.4508187479192002e+03 },
    { 1.5500000000000000e-09, 2.3678051049100000e+00, 6.1550570066540004e+03 },
    { 1.5400000000000001e-09, 1.9835891740700000e+00, 3.4995559444830001e+03 },
    { 1.5400000000000001e-09, 5.2762486677100000e+00, 1.2168002696574600e+04 },
    { 1.5400000000000001e-09, 2.0379293527700000e+00, 3.1339116894031999e+03 },
    { 1.5300000000000001e-09, 4.8580400874500000e+00, 3.3472960905739001e+03 },
    { 1.5300000000000001e-09, 4.4073194847800004e+00, 3.3339287628256998e+03 },
    { 1.5300000000000001e-09, 1.3489929844499999e+00, 1.2722552420485201e+04 },
    { 1.5199999999999999e-09, 6.6722406762999997e-01, 9.3890535407862008e+03 },
    { 1.5199999999999999e-09, 1.2346188942200000e+00, 4.6726673142405998e+03 },
    { 1.5100000000000000e-09, 2.7802659065399999e+00, 1.0309277421860000e+02 },
    { 1.5100000000000000e-09, 5.4556458042999996e+00, 1.0713994881326200e+04 },
    { 1.4900000000000000e-09, 1.0373927025500000e+00, 7.3587651353180001e+02 },
    { 1.4900000000000000e-09, 4.5238878906800002e+00, 3.0775285033270002e+03 },
    { 1.4800000000000001e-09, 3.0026390085900001e+00, 3.0536500655792001e+03 },
    { 1.4700000000000001e-09, 2.6468633285299998e+00, 6.5105519827804001e+03 },
    { 1.4599999999999999e-09, 2.0935946447000000e+00, 3.3974156888696002e+03 },
    { 1.4599999999999999e-09, 8.8857962035000004e-01, 3.2838091645300001e+03 },
    { 1.4599999999999999e-09, 4.3400413042099997e+00, 1.4788665740644001e+03 },
    { 1.4500000000000000e-09, 9.9358639574999996e-01, 2.0608178540718000e+03 },
    { 1.4300000000000000e-09, 2.1716643437899998e+00, 1.5052878090929901e+03 },
    { 1.3999999999999999e-09, 9.0450305555999999e-01, 6.6918615187493997e+03 },
    { 1.3999999999999999e-09, 6.0985780660800000e+00, 6.6705881880498000e+03 },
    { 1.3900000000000000e-09, 9.4728626820999995e-01, 3.5473131639963999e+03 },
    { 1.3800000000000000e-09, 1.4741849433000001e+00, 2.0047197678547800e+04 },
    { 1.3800000000000000e-09, 6.1584376470200004e+00, 2.2334736210149999e+03 },
    { 1.3700000000000000e-09, 5.8150965985300003e+00, 6.6741786167015998e+03 },
    { 1.3500000000000001e-09, 4.1021292346499996e+00, 2.6047359131680000e+03 },
    { 1.3500000000000001e-09, 5.1632303375799999e+00, 4.0764889402315998e+03 },
    { 1.3500000000000001e-09, 4.7486018292700001e+00, 3.6036963500726001e+03 },
    { 1.3399999999999999e-09, 2.2333481102300001e+00, 3.1910492295652002e+03 },
    { 1.3300000000000000e-09, 1.1215558477000001e+00, 1.2839872288705399e+04 },
    { 1.3300000000000000e-09, 6.2634845252900000e+00, 3.6275747878203902e+03 },
    { 1.3200000000000000e-09, 2.8182000335100001e+00, 8.4608283475120004e+02 },
    { 1.3200000000000000e-09, 5.6434848515100002e+00, 7.3662689076256001e+03 },
    { 1.3200000000000000e-09, 5.7461813127000001e+00, 3.0501269472302001e+03 },
    { 1.3100000000000000e-09, 5.3786654009000001e-01, 6.9016374958384004e+03 },
    { 1.3000000000000001e-09, 1.4201274040000000e+00, 1.0213285546211000e+04 },
    { 1.2799999999999999e-09, 9.2352133267000003e-01, 9.3748264467846002e+03 },
    { 1.2600000000000000e-09, 4.7841340065000004e+00, 6.1480107699560003e+03 },
    { 1.2600000000000000e-09, 4.4010423463699997e+00, 1.1670284037296800e+04 },
    { 1.2600000000000000e-09, 5.2353949938300000e+00, 1.3553897972910800e+04 },
    { 1.2600000000000000e-09, 4.8860786683299997e+00, 4.2719755135515998e+03 },
    { 1.2500000000000000e-09, 1.4469145529600000e+00, 3.6027866677399999e+01 },
    { 1.2500000000000000e-09, 3.0672873634500002e+00, 1.8606498946000200e+04 },
    { 1.2400000000000001e-09, 1.4043144899200000e+00, 6.5939166488598003e+03 },
    { 1.2400000000000001e-09, 4.7508114312599998e+00, 2.6316226244231998e+03 },
    { 1.2300000000000001e-09, 1.9746591875299999e+00, 1.4859801210651999e+03 },
    { 1.2300000000000001e-09, 2.9242813089399999e+00, 1.5906764126682599e+04 },
    { 1.2199999999999999e-09, 4.4187286847499996e+00, 3.3464696289993999e+03 },
    { 1.2199999999999999e-09, 4.8466308874799999e+00, 3.3347552244001999e+03 },
    { 1.2000000000000000e-09, 2.9801796797700000e+00, 3.3701042450032000e+03 },
    { 1.2000000000000000e-09, 1.9945852799999999e-03, 3.3111206083963998e+03 },
    { 1.1900000000000000e-09, 2.6756821799999999e+00, 1.5265886519300400e+04 },
    { 1.1900000000000000e-09, 4.6273681956000001e-01, 1.6511613867387401e+04 },
    { 1.1900000000000000e-09, 3.3704680837900001e+00, 4.4477512323846004e+03 },
    { 1.1900000000000000e-09, 3.5187893651000000e+00, 3.6310979061694002e+03 },
    { 1.1800000000000001e-09, 5.7742579490799999e+00, 9.8867722000640006e+03 },
    { 1.1800000000000001e-09, 7.8587501579999997e-02, 3.3178439300904001e+03 },
    { 1.1800000000000001e-09, 2.9035867634699999e+00, 3.3633809233092002e+03 },
    { 1.1800000000000001e-09, 2.0081377557400000e+00, 4.0256564809258002e+03 },
    { 1.1800000000000001e-09, 2.4017238445600002e+00, 6.6064432548323002e+03 },
    { 1.1700000000000001e-09, 2.4867582328500002e+00, 3.3371065620029999e+03 },
    { 1.1700000000000001e-09, 3.6370086857900001e+00, 3.3441182913966099e+03 },
    { 1.1700000000000001e-09, 5.6677803172900001e+00, 3.3441527987009999e+03 },
    { 1.1700000000000001e-09, 4.5598660135000002e-01, 3.3370720546985999e+03 },
    { 1.1700000000000001e-09, 2.6325916237000002e+00, 3.1842117061974000e+03 },
    { 1.1700000000000001e-09, 3.4958264135000000e-01, 3.4970131472021999e+03 },
    { 1.1700000000000001e-09, 6.2233408315999998e+00, 1.9054647649404001e+03 },
    { 1.1500000000000000e-09, 3.4292836320300002e+00, 6.7249437657046001e+03 },
    { 1.1500000000000000e-09, 4.8003949072200003e+00, 6.9976167230561996e+03 },
    { 1.1500000000000000e-09, 2.5024436340600000e+00, 3.6605365304199999e+01 },
    { 1.1500000000000000e-09, 3.5205151460100002e+00, 9.8156517316621994e+03 },
    { 1.1400000000000000e-09, 1.7848525581800001e+00, 5.8209149246467996e+03 },
    { 1.1400000000000000e-09, 1.3687809420499999e+00, 9.0655481241287998e+03 },
    { 1.1400000000000000e-09, 3.0976723871799998e+00, 3.9876232600146000e+03 },
    { 1.1300000000000000e-09, 2.1883124871000001e-01, 4.9225717749828000e+03 },
    { 1.1200000000000001e-09, 2.0838014168200001e+00, 2.2345260376108199e+04 },
    { 1.1200000000000001e-09, 3.9708345132999998e+00, 2.5748193622534000e+03 },
    { 1.1100000000000001e-09, 1.8143399796099999e+00, 3.1639186965660002e+02 },
    { 1.1100000000000001e-09, 3.7262402380599999e+00, 6.3577194367421998e+03 },
    { 1.1100000000000001e-09, 1.2891330291100001e+00, 1.0042192599498199e+04 },
    { 1.0999999999999999e-09, 1.5039610408999999e+00, 8.7308204539810006e+01 },
    { 1.0999999999999999e-09, 5.5410643869099996e+00, 4.1064054911461999e+03 },
    { 1.0900000000000000e-09, 3.8169160679999998e+00, 1.1250799393421599e+04 },
    { 1.0900000000000000e-09, 4.0664830406699997e+00, 2.1795214091614791e+04 },
    { 1.0900000000000000e-09, 5.8178870603600004e+00, 7.6579306444639997e+02 },
    { 1.0800000000000000e-09, 4.3097971099999999e+00, 2.9733880977374001e+03 },
    { 1.0800000000000000e-09, 5.0536431147199998e+00, 1.9800945956224801e+04 },
    { 1.0800000000000000e-09, 1.4123283373200000e+00, 3.7352383117590002e+03 },
    { 1.0700000000000000e-09, 1.7928382086100001e+00, 6.2865989683403996e+03 },
    { 1.0600000000000001e-09, 3.7703403444800001e+00, 1.3745346239022399e+04 },
    { 1.0600000000000001e-09, 5.7740055756300004e+00, 2.6413412784722000e+03 },
    { 1.0600000000000001e-09, 3.4962914700400001e+00, 2.1947111372700001e+04 },
    { 1.0600000000000001e-09, 3.9435918201999998e-01, 7.2144389368432003e+03 },
    { 1.0399999999999999e-09, 1.9231974426300000e+00, 4.2969015826703999e+03 },
    { 1.0399999999999999e-09, 2.3576708124399999e+00, 1.1925274092600601e+04 },
    { 1.0300000000000000e-09, 4.8858416225100001e+00, 6.2795527316424004e+03 },
    { 1.0200000000000000e-09, 4.4891209482400001e+00, 1.0721108428326999e+04 },
    { 1.0200000000000000e-09, 4.8578929667599997e+00, 3.2827337306959998e+03 },
    { 1.0200000000000000e-09, 4.4074666054699998e+00, 3.3984911227036000e+03 },
    { 1.0100000000000000e-09, 3.7681060431400000e+00, 1.0124930054318000e+04 },
    { 1.0000000000000001e-09, 2.5831536167500002e+00, 1.7917797152819589e+04 },
    { 1.0000000000000001e-09, 1.6620344579899999e+00, 2.6887366049730601e+04 },
    { 1.0000000000000001e-09, 2.8045958059500000e+00, 6.6538232973027998e+03 },
    { 9.7999999999999992e-10, 2.2754350908499998e+00, 6.7685330579394004e+03 },
    { 9.6999999999999996e-10, 3.7015064970500000e+00, 2.9221081038928000e+03 },
    { 9.6999999999999996e-10, 1.7823680099899999e+00, 6.7843176276182003e+03 },
    { 9.5999999999999999e-10, 5.7568406593499999e+00, 1.1187557921027999e+03 },
    { 9.5999999999999999e-10, 4.8402497594899998e+00, 1.0006584808914400e+04 },
    { 9.5000000000000003e-10, 1.0422807800000000e-01, 5.9373861913600003e+01 },
    { 9.5000000000000003e-10, 4.8018077797799998e+00, 1.0706881334325401e+04 },
    { 9.5000000000000003e-10, 3.8671030233699999e+00, 1.6555603041981999e+03 },
    { 9.4000000000000006e-10, 1.3612591614800000e+00, 1.7468855197945399e+04 },
    { 9.4000000000000006e-10, 8.5293916684000004e-01, 2.6793794999187999e+03 },
    { 9.4000000000000006e-10, 4.8217999025899996e+00, 1.2726810256271999e+03 },
    { 9.4000000000000006e-10, 6.0501032272800002e+00, 1.2942965062924000e+04 },
    { 9.2999999999999999e-10, 3.7190455487600000e+00, 2.7717905526723998e+03 },
    { 9.2999999999999999e-10, 2.8365885637699999e+00, 2.5382485042536000e+03 },
    { 9.2999999999999999e-10, 3.3702903366200001e+00, 1.0050286467567201e+04 },
    { 9.2999999999999999e-10, 2.2193348550000001e+00, 8.5846616659007996e+03 },
    { 9.2000000000000003e-10, 4.9700984407700002e+00, 3.7078367556622002e+03 },
    { 9.2000000000000003e-10, 6.0418254303600003e+00, 2.5407913015344002e+03 },
    { 9.2000000000000003e-10, 3.5510836429999998e-02, 2.1710241752912002e+03 },
    { 9.2000000000000003e-10, 2.4926807849900001e+00, 3.4925097077850000e+03 },
    { 9.2000000000000003e-10, 6.1404093043300003e+00, 3.6132853840513999e+03 },
    { 9.0999999999999996e-10, 4.4138286878399997e+00, 4.0496022289764001e+03 },
    { 8.9999999999999999e-10, 1.8097460409299999e+00, 6.8504405422599996e+02 },
    { 8.9999999999999999e-10, 5.1554748678599998e+00, 6.7405987153132000e+03 },
    { 8.7999999999999996e-10, 1.6111216904400001e+00, 7.3733824546264004e+03 },
    { 8.7999999999999996e-10, 6.0541851237400000e+00, 1.1883583132734801e+04 },
    { 8.7999999999999996e-10, 3.0994817911100001e+00, 3.0679394693482000e+03 },
    { 8.6999999999999999e-10, 7.4054571271000003e-01, 3.1816689089165998e+03 },
    { 8.6999999999999999e-10, 5.0528959553200004e+00, 9.8593706439672005e+03 },
    { 8.6000000000000003e-10, 8.6878584025000005e-01, 1.7654780539749601e+04 },
    { 8.4999999999999996e-10, 1.4083217667600001e+00, 1.6489763038060999e+04 },
    { 8.4999999999999996e-10, 4.5685416692900001e+00, 4.2942504339149389e+04 },
    { 8.4999999999999996e-10, 4.6157225258999999e+00, 3.6498907493555998e+03 },
    { 8.4999999999999996e-10, 4.6496370463299996e+00, 3.0313341040440000e+03 },
    { 8.4999999999999996e-10, 3.1340383592999999e+00, 1.0730827082375990e+04 },
    { 8.3999999999999999e-10, 4.0663774606000003e-01, 3.1526872789372101e+03 },
    { 8.3999999999999999e-10, 3.0020394856800001e+00, 4.3340653342557598e+04 },
    { 8.3000000000000003e-10, 5.5638270092799997e+00, 3.7591167495067998e+03 },
    { 8.3000000000000003e-10, 2.8195259179700001e+00, 1.8451078546565990e+04 },
    { 8.3000000000000003e-10, 1.1984162269100000e+00, 4.0398189919265998e+04 },
    { 8.1999999999999996e-10, 5.2333903141600002e+00, 1.4422612087601999e+03 },
    { 8.0999999999999999e-10, 2.0773042325900000e+00, 3.3263853326981998e+03 },
    { 8.0999999999999999e-10, 9.0487003245999997e-01, 3.3548395207014000e+03 },
    { 8.0000000000000003e-10, 6.1720006936300003e+00, 4.1429763491459998e+03 },
    { 8.0000000000000003e-10, 4.8962364268799998e+00, 2.1548962369291799e+04 },
    { 8.0000000000000003e-10, 4.3202839075000004e+00, 8.7562698014729995e+03 },
    { 8.0000000000000003e-10, 2.5722366135199999e+00, 2.7417057014825201e+04 },
    { 7.8999999999999996e-10, 4.5639698020899999e+00, 1.8208349942592002e+04 },
    { 7.8999999999999996e-10, 3.0798823761200000e+00, 4.7777880408984001e+03 },
    { 7.8999999999999996e-10, 4.2690644206800004e+00, 3.3787454623376002e+03 },
    { 7.8999999999999996e-10, 4.9962951515500000e+00, 3.3024793910620001e+03 },
    { 7.8999999999999996e-10, 2.7516767067200001e+00, 9.9470556815320997e+03 },
    { 7.8999999999999996e-10, 3.2489095149199998e+00, 5.5624690612967997e+03 },
    { 7.8999999999999996e-10, 5.7799144801000002e-01, 1.1695882514085999e+03 },
    { 7.8999999999999996e-10, 5.1088633362399998e+00, 1.2725453434775000e+04 },
    { 7.8999999999999996e-10, 1.3764363634400001e+00, 3.3295667264358999e+03 },
    { 7.8999999999999996e-10, 1.6057379016100000e+00, 3.3516581269636999e+03 },
    { 7.8999999999999996e-10, 6.1232080062000005e-01, 2.1258100970763999e+03 },
    { 7.7999999999999999e-10, 4.8757496097099997e+00, 5.2023582793351998e+03 },
    { 7.7000000000000003e-10, 2.0506918633700000e+00, 4.0018453534808000e+03 },
    { 7.7000000000000003e-10, 5.5429956449400004e+00, 3.9094343007272000e+03 },
    { 7.7000000000000003e-10, 9.7230677876000005e-01, 2.0199027649330190e+04 },
    { 7.7000000000000003e-10, 2.4335879950999999e+00, 6.5496828917131998e+03 },
    { 7.7000000000000003e-10, 1.0919268275100000e+00, 1.4128242771245599e+04 },
    { 7.7000000000000003e-10, 4.7271097965999997e-01, 5.4664898340789996e+03 },
    { 7.7000000000000003e-10, 3.1934292814099998e+00, 2.4606135553220000e+04 },
    { 7.7000000000000003e-10, 3.0764141102700000e+00, 4.7757600884592002e+03 },
    { 7.5999999999999996e-10, 3.6775555144199998e+00, 1.1071388056848000e+03 },
    { 7.5999999999999996e-10, 2.3048335624199998e+00, 4.2153969003049002e+04 },
    { 7.5999999999999996e-10, 2.7269240828700001e+00, 1.9403777273884800e+04 },
    { 7.5999999999999996e-10, 2.0551554831200001e+00, 1.6484467571974001e+03 },
    { 7.5999999999999996e-10, 3.1306044434000002e+00, 4.1748057328924799e+04 },
    { 7.5000000000000000e-10, 2.9399415516200000e+00, 6.7106543619355998e+03 },
    { 7.5000000000000000e-10, 5.6078335967999999e+00, 3.5285375744623998e+03 },
    { 7.4000000000000003e-10, 4.2208880328999996e+00, 2.5135826518314600e+04 },
    { 7.4000000000000003e-10, 5.0141719397299997e+00, 1.5636566656309000e+04 },
    { 7.4000000000000003e-10, 2.1482936249200000e+00, 3.2643463554241998e+03 },
    { 7.2999999999999996e-10, 5.3530273203799998e+00, 1.3369563253800001e+04 },
    { 7.2999999999999996e-10, 2.2890877985999998e+00, 6.3648329837430001e+03 },
    { 7.2999999999999996e-10, 5.0976824484299996e+00, 1.0177190369230801e+04 },
    { 7.2000000000000000e-10, 1.9150204398499999e+00, 1.6858415222630389e+04 },
    { 7.2000000000000000e-10, 7.1597283003000001e-01, 2.0213254743331789e+04 },
    { 7.2000000000000000e-10, 2.6194974363900001e+00, 3.3391279539914999e+03 },
    { 7.2000000000000000e-10, 3.6267682866000001e-01, 3.3420968994080999e+03 },
    { 7.1000000000000003e-10, 1.5384573980400000e+00, 2.0679314010725998e+03 },
    { 6.9999999999999996e-10, 1.7154515776199999e+00, 2.0809467624645200e+04 },
    { 6.9999999999999996e-10, 3.8984578636000000e+00, 3.5707710082907397e+04 },
    { 6.9000000000000000e-10, 3.4521018067400000e+00, 4.5088669755624600e+04 },
    { 6.9000000000000000e-10, 6.0399722189800000e+00, 1.2797945726280000e+03 },
    { 6.9000000000000000e-10, 5.0498527193899996e+00, 1.1353892167640201e+04 },
    { 6.9000000000000000e-10, 1.5951654703300000e+00, 4.3738802345965800e+04 },
    { 6.8000000000000003e-10, 3.9335137631900001e+00, 2.5287723799399799e+04 },
    { 6.8000000000000003e-10, 4.7722730112700003e+00, 4.4959547675990201e+04 },
    { 6.8000000000000003e-10, 4.8554625849199997e+00, 1.2808880303956799e+04 },
    { 6.8000000000000003e-10, 1.9998637348999999e+00, 6.6147374444852003e+03 },
    { 6.8000000000000003e-10, 4.1057665290300003e+00, 1.4867737515892190e+04 },
    { 6.6999999999999996e-10, 4.8456241786199996e+00, 5.4014302807716003e+03 },
    { 6.6999999999999996e-10, 3.9614409329000000e+00, 1.1236572299420000e+04 },
    { 6.6999999999999996e-10, 3.7948872175499999e+00, 1.7256631536341400e+04 },
    { 6.6999999999999996e-10, 4.6623680500300004e+00, 9.3296796788726006e+03 },
    { 6.6000000000000000e-10, 5.6404680207100002e+00, 2.0111003364398000e+03 },
    { 6.6000000000000000e-10, 1.0824980383600000e+00, 3.4142133826149998e+03 },
    { 6.6000000000000000e-10, 1.8996762266900000e+00, 3.2670114707846001e+03 },
    { 6.6000000000000000e-10, 7.3968143000000004e-03, 9.9979588423515997e+03 },
    { 6.6000000000000000e-10, 5.3432657709100004e+00, 2.4889574795991601e+04 },
    { 6.5000000000000003e-10, 5.2375671056600002e+00, 8.8593625756916008e+03 },
    { 6.5000000000000003e-10, 7.6344895488999998e-01, 6.7086264094963999e+03 },
    { 6.5000000000000003e-10, 1.0650340635000000e-01, 4.1404335518651997e+03 },
    { 6.5000000000000003e-10, 9.3949119244000001e-01, 6.6808047773059998e+03 },
    { 6.5000000000000003e-10, 6.0070238467300001e+00, 4.0796338922674193e+04 },
    { 6.3999999999999996e-10, 4.3534425489300004e+00, 9.8014246376606006e+03 },
    { 6.3999999999999996e-10, 1.9119916893800000e+00, 4.3562754445840001e+03 },
    { 6.3999999999999996e-10, 2.0376768679400001e+00, 1.6958018313299999e+02 },
    { 6.3999999999999996e-10, 7.0379327004000003e-01, 3.1938007401762002e+03 },
    { 6.3999999999999996e-10, 2.2783809950100000e+00, 3.4874241132234001e+03 },
    { 6.3999999999999996e-10, 8.7689051214000002e-01, 2.2743409379516401e+04 },
    { 6.3999999999999996e-10, 3.0471802141400000e+00, 1.3295120902599999e+03 },
    { 6.3000000000000000e-10, 4.5819942896799999e+00, 2.5841293202242188e+04 },
    { 6.3000000000000000e-10, 1.5722648660300000e+00, 2.0742945708426400e+04 },
    { 6.3000000000000000e-10, 2.2424416780000000e+00, 9.1615273513466000e+03 },
    { 6.2000000000000003e-10, 3.2479002589800001e+00, 1.8461715211915791e+04 },
    { 6.2000000000000003e-10, 2.1900742201200001e+00, 2.0040151441849790e+04 },
    { 6.2000000000000003e-10, 4.4081379895200001e+00, 9.3136308685179995e+02 },
    { 6.2000000000000003e-10, 1.2836695199799999e+00, 1.2988111830836000e+04 },
    { 6.2000000000000003e-10, 2.9648777421500001e+00, 3.3574446277495899e+03 },
    { 6.2000000000000003e-10, 1.7296522900000000e-02, 3.3237802256499999e+03 },
    { 6.0999999999999996e-10, 4.3776158845499999e+00, 2.6555010621709998e+03 },
    { 6.0999999999999996e-10, 3.9449459627499999e+00, 1.9617076369322789e+04 },
    { 6.0000000000000000e-10, 1.3366551574000001e+00, 3.1022753170856198e+04 },
    { 6.0000000000000000e-10, 5.5950202700600000e+00, 1.0016314355791999e+04 },
    { 6.0000000000000000e-10, 5.4685913810700004e+00, 1.0184303916231600e+04 },
    { 5.9000000000000003e-10, 3.3866931292100002e+00, 1.5121102785216000e+04 },
    { 5.9000000000000003e-10, 1.0284357967300000e+00, 4.0398835749273999e+03 },
    { 5.9000000000000003e-10, 1.9922626896600000e+00, 5.9453483398678000e+03 },
    { 5.9000000000000003e-10, 8.4746741291000005e-01, 5.2236939198022001e+03 },
    { 5.9000000000000003e-10, 5.8288838752499998e+00, 1.2516366872048000e+04 },
    { 5.9000000000000003e-10, 2.1352733249600000e+00, 4.2897880356695996e+03 },
    { 5.9000000000000003e-10, 5.1209619418900001e+00, 2.5239591004846002e+03 },
    { 5.7999999999999996e-10, 1.5519898435999999e-01, 1.5423267560871000e+04 },
    { 5.7999999999999996e-10, 8.4276173940000004e-02, 4.5884967762441003e+04 },
    { 5.7999999999999996e-10, 1.7952211612300000e+00, 3.4126681600545999e+03 },
    { 5.7999999999999996e-10, 1.1869531038200001e+00, 3.2685566933450000e+03 },
    { 5.7999999999999996e-10, 5.4418649491300002e+00, 6.8508050365325998e+03 },
    { 5.7000000000000000e-10, 1.4845243910000000e-01, 1.2928737968922400e+04 },
    { 5.7000000000000000e-10, 6.1418959687400001e+00, 3.1162676309979001e+03 },
    { 5.6000000000000003e-10, 3.1829706134000002e+00, 2.9784912899689998e+03 },
    { 5.6000000000000003e-10, 6.0823889588300002e+00, 3.7027335634306000e+03 },
    { 5.6000000000000003e-10, 3.5845363892900002e+00, 8.7420427074714007e+03 },
    { 5.6000000000000003e-10, 3.0867595651799999e+00, 3.8252024502790802e+04 },
    { 5.6000000000000003e-10, 3.1298371784799999e+00, 2.1265523126520198e+04 },
    { 5.4999999999999996e-10, 1.6321552352999999e-01, 2.1150813365883601e+04 },
    { 5.4999999999999996e-10, 5.5588015872699996e+00, 1.2715438873484400e+04 },
    { 5.4999999999999996e-10, 4.2874162928799997e+00, 2.3539707386332801e+04 },
    { 5.4999999999999996e-10, 4.0464167666700002e+00, 4.4035174752379999e+03 },
    { 5.4999999999999996e-10, 5.2296440760799996e+00, 4.1236038717067197e+04 },
    { 5.4999999999999996e-10, 4.0633397135099996e+00, 1.1527125089192399e+04 },
    { 5.4000000000000000e-10, 1.1476472170900001e+00, 5.6076158292088003e+03 },
    { 5.4000000000000000e-10, 1.0954628695599999e+00, 1.8849227549974199e+04 },
    { 5.4000000000000000e-10, 3.6335733481500001e+00, 2.7357626859950001e+03 },
    { 5.4000000000000000e-10, 1.3220578549599999e+00, 2.0426571092422000e+04 },
    { 5.4000000000000000e-10, 1.0431172747599999e+00, 1.0558588118401999e+03 },
    { 5.4000000000000000e-10, 4.7833677288200001e+00, 4.6283116765849190e+04 },
    { 5.3000000000000003e-10, 4.6677337161399999e+00, 6.7888883773382004e+03 },
    { 5.3000000000000003e-10, 3.7695709610699999e+00, 1.4481205498902000e+04 },
    { 5.3000000000000003e-10, 5.6190542057000004e-01, 8.3367851575978002e+03 },
    { 5.1999999999999996e-10, 4.7208985352399999e+00, 6.8127668150860000e+03 },
    { 5.1999999999999996e-10, 1.2114523657400000e+00, 4.8859664096786000e+03 },
    { 5.1999999999999996e-10, 4.5515480682999998e-01, 2.4491425792583399e+04 },
    { 5.1999999999999996e-10, 2.8783495349999999e-01, 1.9719521738548001e+03 },
    { 5.1999999999999996e-10, 2.1763439156000000e+00, 2.9632388187840002e+03 },
    { 5.1999999999999996e-10, 8.0583034944999998e-01, 3.7179860346156001e+03 },
    { 5.1999999999999996e-10, 5.9685299997800003e+00, 6.6669977593980002e+03 },
    { 5.1999999999999996e-10, 4.8620079768500002e+00, 3.3821632176846001e+03 },
    { 5.1999999999999996e-10, 4.4033515953800002e+00, 3.2990616357150002e+03 },
    { 5.1999999999999996e-10, 4.8920873500699997e+00, 1.8546323056346000e+03 },
    { 5.1999999999999996e-10, 2.3553374795600002e+00, 1.4610540519464000e+03 },
    { 5.1000000000000000e-10, 1.6760583781200000e+00, 4.5486818759032787e+04 },
    { 5.1000000000000000e-10, 6.7722361596000002e-01, 1.6328724257535790e+04 },
    { 5.1000000000000000e-10, 6.1666149757099999e+00, 1.2410731300548599e+04 },
    { 5.1000000000000000e-10, 2.8143295913899999e+00, 8.2265788363783995e+03 },
    { 5.1000000000000000e-10, 4.7067103476499996e+00, 5.7498617665477996e+03 },
    { 5.1000000000000000e-10, 4.3912398450700003e+00, 5.4085438277723997e+03 },
    { 5.0000000000000003e-10, 4.5701648532899997e+00, 3.7895426290367403e+04 },
    { 5.0000000000000003e-10, 1.6665502636900000e+00, 1.7276610936682000e+03 },
    { 5.0000000000000003e-10, 1.7458517639000001e-01, 3.9048322509607198e+04 },
    { 5.0000000000000003e-10, 3.2762904004400002e+00, 3.5649572224017002e+03 },
    { 5.0000000000000003e-10, 3.7650795450699999e+00, 3.7422845484569998e+03 },
    { 5.0000000000000003e-10, 3.7800509960699999e+00, 5.2616784674559995e+02 },
    { 4.8999999999999996e-10, 2.4840939439900001e+00, 2.0597243963041201e+04 },
    { 4.8999999999999996e-10, 4.6336425023699999e+00, 2.4341528317456599e+04 },
    { 4.8999999999999996e-10, 3.0398956222600000e+00, 5.5516418709848200e+04 },
    { 4.8999999999999996e-10, 4.8269858604799998e+00, 4.1194487926082402e+04 },
    { 4.8999999999999996e-10, 3.9769494059300001e+00, 1.7660303464057000e+04 },
    { 4.8999999999999996e-10, 4.8892085835600003e+00, 4.0257237912286000e+03 },
    { 4.8999999999999996e-10, 1.4516327052600000e+00, 1.9406678288174600e+04 },
    { 4.8999999999999996e-10, 8.2675139375999995e-01, 1.0787630344545800e+04 },
    { 4.8000000000000000e-10, 1.5543477061900000e+00, 1.8830814931023999e+03 },
    { 4.8000000000000000e-10, 1.5245835968000001e+00, 1.9190478178446800e+04 },
    { 4.8000000000000000e-10, 1.9635672788700000e+00, 3.2765246659615801e+04 },
    { 4.8000000000000000e-10, 4.8689792818799997e+00, 1.0191417463232399e+04 },
    { 4.8000000000000000e-10, 3.5407535409399999e+00, 8.1865126624925997e+03 },
    { 4.8000000000000000e-10, 3.3955507055000000e-01, 1.4955045720431999e+04 },
    { 4.8000000000000000e-10, 1.2068135424099999e+00, 7.7999806455024000e+03 },
    { 4.8000000000000000e-10, 9.3062224201999999e-01, 1.8592030553546001e+03 },
    { 4.8000000000000000e-10, 1.9544550821300000e+00, 1.2825645194703800e+04 },
    { 4.8000000000000000e-10, 5.9389035162599999e+00, 1.7810200939183800e+04 },
    { 4.8000000000000000e-10, 4.5177627350199998e+00, 3.3370219980480001e+03 },
    { 4.7000000000000003e-10, 2.9324305822799999e+00, 3.3163395663023999e+04 },
    { 4.7000000000000003e-10, 7.7055080600000003e-01, 1.1876469585733999e+04 },
    { 4.7000000000000003e-10, 5.8055280068100004e+00, 2.2573520587628000e+03 },
    { 4.7000000000000003e-10, 4.5636881037999999e-01, 1.7412051935775598e+04 },
    { 4.6000000000000001e-10, 4.5464043499000004e+00, 1.6865528769631201e+04 },
    { 4.6000000000000001e-10, 1.5279721673100000e+00, 4.4768099409878603e+04 },
    { 4.6000000000000001e-10, 2.7726411537000001e-01, 1.7952584437210000e+03 },
    { 4.6000000000000001e-10, 1.1011885467000000e-01, 5.8814037282341997e+03 },
    { 4.6000000000000001e-10, 1.9369139187600000e+00, 4.8827431185732603e+04 },
    { 4.6000000000000001e-10, 1.3524185254800001e+00, 9.1757544453482005e+03 },
    { 4.6000000000000001e-10, 1.9977733354700000e+00, 2.3249494088155998e+03 },
    { 4.6000000000000001e-10, 3.5808016175000001e+00, 1.9247376553382401e+04 },
    { 4.6000000000000001e-10, 4.6198476821699996e+00, 3.3716965065866403e+04 },
    { 4.6000000000000001e-10, 1.8369837651000001e-01, 1.2995225377836799e+04 },
    { 4.5000000000000000e-10, 3.1530464524099999e+00, 3.9733961660129999e+03 },
    { 4.5000000000000000e-10, 4.6701049119900002e+00, 1.2655674786264001e+03 },
    { 4.5000000000000000e-10, 2.7440148084499998e+00, 1.6063164847185000e+04 },
    { 4.5000000000000000e-10, 1.2149018896399999e+00, 1.5849865751747000e+04 },
    { 4.5000000000000000e-10, 1.1522393821900001e+00, 3.9662826190122000e+03 },
    { 4.5000000000000000e-10, 1.8299348828599999e+00, 2.7149422343873998e+03 },
    { 4.5000000000000000e-10, 4.1451021572600002e+00, 4.1572657529149901e+03 },
    { 4.3999999999999998e-10, 5.8092834423999999e+00, 3.4365916539176001e+03 },
    { 4.3999999999999998e-10, 3.4560761298300000e+00, 3.2446331994819998e+03 },
    { 4.3999999999999998e-10, 6.0674155108900001e+00, 1.7499360140315399e+04 },
    { 4.3999999999999998e-10, 9.3617830234999999e-01, 2.9698287511335799e+04 },
    { 4.3000000000000001e-10, 5.8956736688199998e+00, 9.7670387712897180e+04 },
    { 4.3000000000000001e-10, 5.3142417939500000e+00, 1.2199975002391400e+04 },
    { 4.3000000000000001e-10, 1.9086123130499999e+00, 8.1234860621598000e+03 },
    { 4.3000000000000001e-10, 2.2627419016000000e-01, 3.8265844794894001e+03 },
    { 4.3000000000000001e-10, 2.7559000748900000e+00, 2.8546403739102002e+03 },
    { 4.3000000000000001e-10, 3.4191232720300002e+00, 4.6681265769257399e+04 },
    { 4.3000000000000001e-10, 8.2862094100000006e-01, 2.7832038219283200e+04 },
    { 4.3000000000000001e-10, 5.4367527273000000e+00, 1.3524916342931399e+04 },
    { 4.2000000000000000e-10, 5.9123604545599999e+00, 1.2729665967486000e+04 },
    { 4.2000000000000000e-10, 3.5211606137200002e+00, 1.5106875691214400e+04 },
    { 4.2000000000000000e-10, 8.1055569531000005e-01, 2.6519779438219998e+03 },
    { 4.2000000000000000e-10, 2.7595832265600002e+00, 2.0235122826310399e+04 },
    { 4.2000000000000000e-10, 3.7972841283299998e+00, 1.2979553174576000e+04 },
    { 4.0999999999999998e-10, 5.8074928130999997e+00, 2.8230187222691391e+04 },
    { 4.0999999999999998e-10, 7.1986367039999999e-01, 3.3852662599400001e+03 },
    { 4.0999999999999998e-10, 2.2623105946500002e+00, 3.2959585934595998e+03 },
    { 4.0999999999999998e-10, 3.5124667951099999e+00, 4.6701245169597996e+03 },
    { 4.0999999999999998e-10, 5.4308870857800002e+00, 1.4469588512484001e+04 },
    { 4.0999999999999998e-10, 9.3833632649999998e-01, 2.9580054632350191e+04 },
    { 4.0999999999999998e-10, 4.7500774841600002e+00, 7.7883636590843998e+03 },
    { 4.0999999999999998e-10, 3.9916057478200000e+00, 1.3532029889932201e+04 },
    { 4.0000000000000001e-10, 1.9389134237200001e+00, 2.0728718614424790e+04 },
    { 4.0000000000000001e-10, 3.4615539895000003e-01, 2.2487371692841600e+04 },
    { 4.0000000000000001e-10, 5.6309443372499999e+00, 3.9454621674046002e+03 },
    { 4.0000000000000001e-10, 8.1846047046000003e-01, 2.5827984119541401e+04 },
    { 4.0000000000000001e-10, 4.6440992798899998e+00, 3.5244820136017902e+03 },
    { 4.0000000000000001e-10, 4.6212602923399997e+00, 3.1567428397978001e+03 },
    { 3.9000000000000000e-10, 3.6700465461600000e+00, 7.4299006053259995e+02 },
    { 3.9000000000000000e-10, 4.7349256215100004e+00, 3.6105859086315599e+04 },
    { 3.9000000000000000e-10, 3.3997444808799999e+00, 2.7149872905026000e+03 },
    { 3.9000000000000000e-10, 1.8344009155000001e+00, 5.5116366019910001e+03 },
    { 3.9000000000000000e-10, 4.4087168772699998e+00, 3.3371566186536002e+03 },
    { 3.9000000000000000e-10, 1.7150500413700001e+00, 3.3440682347460001e+03 },
    { 3.9000000000000000e-10, 3.5548237988500002e+00, 1.1567191263078201e+04 },
    { 3.9000000000000000e-10, 2.1249721717700001e+00, 2.7490692478044799e+04 },
    { 3.9000000000000000e-10, 3.7839660841000000e-01, 4.9225580189140797e+04 },
    { 3.7999999999999998e-10, 9.5007899496000003e-01, 3.0171070100423999e+03 },
    { 3.7999999999999998e-10, 9.2745494031999998e-01, 1.9683563778237200e+04 },
    { 3.7999999999999998e-10, 3.0310328529400001e+00, 1.4849324896942001e+03 },
    { 3.7999999999999998e-10, 8.7453118322000001e-01, 3.0704822666290002e+03 },
    { 3.7999999999999998e-10, 2.1076430818300000e+00, 3.6107425867706002e+03 },
    { 3.7999999999999998e-10, 2.6096842933400000e+00, 3.8407444902224990e+04 },
    { 3.7999999999999998e-10, 4.7354624641300003e+00, 4.4690520752216391e+04 },
    { 3.7000000000000001e-10, 2.0104942389099998e+00, 2.4093276789175201e+04 },
    { 3.7000000000000001e-10, 3.2594211740199999e+00, 3.4115114069274598e+04 },
    { 3.7000000000000001e-10, 5.3745667814100004e+00, 9.3848410080752001e+03 },
    { 3.7000000000000001e-10, 2.0797770082699998e+00, 3.9601891912449602e+04 },
    { 3.7000000000000001e-10, 1.4344432310299999e+00, 2.0752664362475400e+04 },
    { 3.6000000000000000e-10, 4.0104455922000000e+00, 2.0897822303990001e+03 },
    { 3.6000000000000000e-10, 2.2332418835499999e+00, 3.7626398678557998e+03 },
    { 3.6000000000000000e-10, 7.4893238149999997e-01, 2.9185849855438000e+03 },
    { 3.6000000000000000e-10, 2.7885555385199998e+00, 4.5554147563232000e+03 },
    { 3.6000000000000000e-10, 3.7193668356999998e+00, 6.6446194880953999e+03 },
    { 3.6000000000000000e-10, 2.6405868192600002e+00, 1.2082655134171200e+04 },
    { 3.6000000000000000e-10, 2.7135004042199999e+00, 2.5685872802808000e+04 },
    { 3.6000000000000000e-10, 4.8841247461400004e+00, 1.3673290505667601e+04 },
    { 3.4999999999999998e-10, 5.1474793759499997e+00, 3.8086874556479001e+04 },
    { 3.4999999999999998e-10, 2.9897256097499998e+00, 1.6850521225016000e+03 },
    { 3.4999999999999998e-10, 2.8028637947000001e-01, 4.7163862265456000e+03 },
    { 3.4999999999999998e-10, 4.6704149414899998e+00, 1.5010896463996600e+04 },
    { 3.4999999999999998e-10, 2.1917294269099998e+00, 2.4076444588125400e+04 },
    { 3.4999999999999998e-10, 1.6213542525900000e+00, 3.6261279485749801e+04 },
    { 3.4999999999999998e-10, 3.9535447915999999e-01, 1.6335837804536601e+04 },
    { 3.4999999999999998e-10, 4.5683937069200002e+00, 2.1393541969857601e+04 },
    { 3.4999999999999998e-10, 5.8854552190499998e+00, 8.0132797409404002e+03 },
    { 3.4999999999999998e-10, 5.8365926213400003e+00, 5.2175806283148399e+04 },
    { 3.4999999999999998e-10, 6.0981136446199997e+00, 5.3517127631395997e+03 },
    { 3.4999999999999998e-10, 3.6142980223699999e+00, 3.3036168360888000e+03 },
    { 3.4999999999999998e-10, 5.6510615498599996e+00, 3.3776080173107998e+03 },
    { 3.4000000000000001e-10, 9.8572990462999999e-01, 7.4835887758458002e+03 },
    { 3.4000000000000001e-10, 2.5485592641100001e+00, 1.7013902932367400e+04 },
    { 3.4000000000000001e-10, 2.2187436877100000e+00, 3.5066832475525203e+04 },
    { 3.4000000000000001e-10, 3.4149366820200000e+00, 1.3355336159798400e+04 },
    { 3.4000000000000001e-10, 6.1075411726300004e+00, 3.3704944703100000e+03 },
    { 3.4000000000000001e-10, 3.1578183996000000e+00, 3.3107303830895999e+03 },
    { 3.4000000000000001e-10, 3.3743210229299998e+00, 3.5863130482341599e+04 },
    { 3.4000000000000001e-10, 2.2347943503800001e+00, 2.7433889215874991e+04 },
    { 3.4000000000000001e-10, 1.6012516219499999e+00, 1.6702994823196201e+04 },
    { 3.4000000000000001e-10, 5.0118155388099996e+00, 8.3296716105970008e+03 },
    { 3.4000000000000001e-10, 4.3975124843500000e+00, 4.1349908325516590e+04 },
    { 3.4000000000000001e-10, 2.1700085050800002e+00, 1.7932024246821202e+04 },
    { 3.4000000000000001e-10, 4.0258850769599999e+00, 3.9203742909041393e+04 },
    { 3.3000000000000000e-10, 2.9939500470000002e-02, 1.2502139778046399e+04 },
    { 3.3000000000000000e-10, 4.3145819876799996e+00, 1.4164813629245400e+04 },
    { 3.3000000000000000e-10, 1.2576970589200001e+00, 3.7234917052708001e+03 },
    { 3.3000000000000000e-10, 1.7244772061300000e+00, 2.9577331481288002e+03 },
    { 3.3000000000000000e-10, 1.5058901814700001e+00, 7.4470179178460003e+03 },
    { 3.3000000000000000e-10, 4.7206262116000000e+00, 4.8300160102690003e+04 },
    { 3.3000000000000000e-10, 8.2205966845000000e-01, 3.1405649703079402e+04 },
    { 3.1999999999999998e-10, 3.4571941627600000e+00, 4.4238727946367999e+03 },
    { 3.1999999999999998e-10, 4.5666348919099997e+00, 1.1979701285736001e+03 },
    { 3.1999999999999998e-10, 2.1744044733900001e+00, 4.0292469095776000e+03 },
    { 3.1999999999999998e-10, 3.6848231469499999e+00, 3.7455726495974399e+04 },
    { 3.1999999999999998e-10, 4.9546769999100002e+00, 4.5030871311075998e+03 },
    { 3.1999999999999998e-10, 4.7501547373699999e+00, 3.3561544666432201e+04 },
    { 3.1999999999999998e-10, 1.1319772278000000e+00, 6.8307880505342000e+03 },
    { 3.1999999999999998e-10, 2.8043081427000000e-01, 3.6659428489158003e+04 },
    { 3.1000000000000002e-10, 3.5907536502199999e+00, 3.1014069410748002e+03 },
    { 3.1000000000000002e-10, 5.6746059220099996e+00, 3.5798179123248001e+03 },
    { 3.1000000000000002e-10, 2.0709743552100002e+00, 1.3605178310773201e+04 },
    { 3.1000000000000002e-10, 4.2903112448599998e+00, 1.6872642316631998e+04 },
    { 3.1000000000000002e-10, 5.0704184187800001e+00, 2.0995392966449399e+04 },
    { 3.1000000000000002e-10, 3.2864946539700002e+00, 9.0726616711295992e+03 },
    { 3.1000000000000002e-10, 2.7902109471699998e+00, 6.8365779425310002e+03 },
    { 3.1000000000000002e-10, 2.8026582718999999e+00, 3.2522518055641802e+04 },
    { 3.1000000000000002e-10, 4.5001645108999999e-01, 2.3937856389740999e+04 },
    { 3.0000000000000000e-10, 3.4445911210900002e+00, 3.2769127994973802e+04 },
    { 3.0000000000000000e-10, 1.2554648686000000e-01, 9.9187445058808007e+03 },
    { 3.0000000000000000e-10, 2.4388817011900001e+00, 1.4577184726119800e+04 },
    { 3.0000000000000000e-10, 5.8842337264299998e+00, 3.9662375628969999e+03 },
    { 3.0000000000000000e-10, 5.0261428195100004e+00, 4.8031133178916200e+04 },
    { 3.0000000000000000e-10, 2.2105445126199998e+00, 1.7232753098593599e+04 },
    { 3.0000000000000000e-10, 4.2710903103600000e+00, 3.0376352639166598e+04 },
    { 2.8999999999999998e-10, 5.3077643009199997e+00, 5.1371745605615994e+04 },
    { 2.8999999999999998e-10, 4.0834026645800003e+00, 1.3002338924837601e+04 },
    { 2.8999999999999998e-10, 3.7522806630400001e+00, 2.7035740212466801e+04 },
    { 2.8999999999999998e-10, 4.3166804894100004e+00, 1.3517802795930600e+04 },
    { 2.8999999999999998e-10, 4.7385391939900003e+00, 2.0206141196330998e+04 },
    { 2.8999999999999998e-10, 2.0471914459799998e+00, 4.8220217980449997e+03 },
    { 2.8999999999999998e-10, 1.9051383573800000e+00, 4.1990785932898798e+04 },
    { 2.8000000000000002e-10, 1.1426724612900001e+00, 4.9535637597313998e+03 },
    { 2.8000000000000002e-10, 4.5994062235399999e+00, 2.7682140744156401e+04 },
    { 2.8000000000000002e-10, 1.8834758740299999e+00, 5.8351420186484002e+03 },
    { 2.8000000000000002e-10, 4.7317799256899997e+00, 2.1000915890756802e+04 },
    { 2.8000000000000002e-10, 1.7966268347500001e+00, 3.8813356576349201e+04 },
    { 2.8000000000000002e-10, 5.4634398757599998e+00, 3.3483750974158002e+03 },
    { 2.8000000000000002e-10, 3.8019196964700002e+00, 3.3328497559838002e+03 },
    { 2.8000000000000002e-10, 5.4723497224599997e+00, 7.3459808985296004e+03 },
    { 2.8000000000000002e-10, 2.0215685374699999e+00, 3.5173061568336002e+03 },
    { 2.8000000000000002e-10, 9.6060572757999996e-01, 3.1639186965660001e+03 },
    { 2.7000000000000000e-10, 2.5269744049499998e+00, 7.5344212351515998e+03 },
};

static const VSOP87Term VSOP87A_MARS_X2[387] = {
    { 5.6323938999999995e-04, 0.0000000000000000e+00, 0.0000000000000000e+00 },
    { 2.2122527999999999e-04, 3.5437211327200000e+00, 6.6812248533995999e+03 },
    { 6.0914089999999999e-05, 3.9327264964899999e+00, 1.0021837280099400e+04 },
    { 1.4519980000000000e-05, 3.6465566645999998e+00, 3.3406124266997999e+03 },
    { 1.1306130000000000e-05, 4.2882702322200004e+00, 1.3362449706799200e+04 },
    { 1.8261000000000000e-06, 4.6352266012500003e+00, 1.6703062133498999e+04 },
    { 1.6890400000000001e-06, 4.6879782549400000e+00, 3.3441355450487999e+03 },
    { 1.6838399999999999e-06, 4.5797432664200004e+00, 3.3370893083507999e+03 },
    { 9.4102999999999995e-07, 9.4931392640000001e-01, 3.1851920272655998e+03 },
    { 8.9578000000000004e-07, 2.0378274509000001e+00, 3.4960328261340001e+03 },
    { 4.0689000000000001e-07, 5.0333169699200004e+00, 6.6847479717486003e+03 },
    { 3.6711000000000001e-07, 1.7736288916700000e+00, 1.0593819301891999e+03 },
    { 2.7548999999999999e-07, 4.9785966332899996e+00, 2.0043674560198800e+04 },
    { 2.4699000000000002e-07, 3.6356053418299998e+00, 3.1491641605882000e+03 },
    { 2.4321000000000002e-07, 5.7724765601400003e+00, 2.5443144198834002e+03 },
    { 2.3325000000000000e-07, 5.6300915178100004e+00, 3.5320606928113998e+03 },
    { 2.0394000000000001e-07, 3.4879450681400002e+00, 4.1369104335162001e+03 },
    { 1.8400000000000001e-07, 3.4557597944600000e+00, 3.0978838227257902e+03 },
    { 1.7394999999999999e-07, 4.3413172376000002e+00, 2.9424634232916001e+03 },
    { 1.6953999999999999e-07, 5.8256029394800004e+00, 3.5833410306738001e+03 },
    { 1.6878000000000001e-07, 2.3554256771400000e+00, 2.7870430238573999e+03 },
    { 1.6238000000000000e-07, 1.6760593814500000e+00, 1.1944470102246000e+03 },
    { 1.5902000000000000e-07, 1.2178432380699999e+00, 5.6218429232103999e+03 },
    { 1.5421999999999999e-07, 4.9400857215800000e+00, 3.7387614301079998e+03 },
    { 1.5232000000000000e-07, 2.7850117251000001e-01, 1.5925960136327999e+03 },
    { 1.4259999999999999e-07, 6.6247586522000002e-01, 3.8941818295421999e+03 },
    { 1.3670000000000000e-07, 2.5687475500399999e+00, 1.5542039943419999e+02 },
    { 1.3281999999999999e-07, 5.1608236170400001e+00, 1.9907450170410000e+03 },
    { 1.2674999999999999e-07, 3.7599021812700002e+00, 2.3888940204492001e+03 },
    { 1.2328000000000000e-07, 2.1497436156199998e+00, 3.3415927477680002e+03 },
    { 1.2311000000000000e-07, 8.3471101076999998e-01, 3.3396321056316001e+03 },
    { 1.1826000000000000e-07, 3.0803800817000000e+00, 7.9629800681639995e+02 },
    { 1.1403999999999999e-07, 9.8319348386000005e-01, 2.1461654164751999e+03 },
    { 1.1303000000000000e-07, 1.3019818710600000e+00, 6.5258044539654002e+03 },
    { 1.0589000000000000e-07, 2.1703381940900002e+00, 4.3999943568890003e+03 },
    { 9.4609999999999998e-08, 5.5671653345200003e+00, 4.2923308329503998e+03 },
    { 8.8749999999999997e-08, 2.0845658075400002e+00, 4.5350594369244000e+03 },
    { 8.6490000000000002e-08, 4.1653177936299999e+00, 4.6904798363585996e+03 },
    { 8.5689999999999995e-08, 2.7377441365499999e+00, 5.0886288397668004e+03 },
    { 8.3389999999999994e-08, 1.2422111137100000e+00, 2.9577158944766002e+03 },
    { 8.1289999999999995e-08, 5.3544691864600003e+00, 1.0025360398448400e+04 },
    { 7.9910000000000006e-08, 1.3125563718400000e+00, 5.4867778431750003e+03 },
    { 7.5810000000000002e-08, 2.3046524912000002e+00, 1.5890728952838001e+03 },
    { 7.5440000000000006e-08, 1.7618046397300000e+00, 3.7235089589230001e+03 },
    { 6.8740000000000005e-08, 2.6751996409099998e+00, 1.7480164130670000e+03 },
    { 6.1350000000000003e-08, 5.3776351249300003e+00, 7.0793738568077997e+03 },
    { 6.0489999999999994e-08, 1.1221394738599999e+00, 5.5356940284240000e+02 },
    { 5.8799999999999997e-08, 5.0233731243899999e+00, 7.1135470007999997e+00 },
    { 5.5850000000000000e-08, 4.3586744272600004e+00, 1.3498674096587999e+03 },
    { 5.4450000000000001e-08, 5.4345892699100000e+00, 2.4272860397400001e+02 },
    { 5.3400000000000002e-08, 5.9222768348099999e+00, 9.5171840625059997e+02 },
    { 5.3069999999999998e-08, 4.0678548633200000e+00, 6.4897765872880000e+03 },
    { 5.2880000000000001e-08, 6.1884926083399998e+00, 5.8849268465832001e+03 },
    { 5.0510000000000000e-08, 2.1801614952200001e+00, 2.2741169495098002e+03 },
    { 4.7699999999999997e-08, 3.6305129740999997e-01, 3.3334988796990001e+03 },
    { 4.6210000000000002e-08, 3.8685474077499999e+00, 7.4775228602159996e+03 },
    { 4.4700000000000003e-08, 4.6312200259500003e+00, 2.2883440435113998e+03 },
    { 4.3329999999999998e-08, 6.7828219297000003e-01, 4.9332084403325998e+03 },
    { 4.1479999999999998e-08, 2.6772108307100000e+00, 3.3477259737006002e+03 },
    { 3.9869999999999999e-08, 5.3178640796699996e+00, 2.3384286986898602e+04 },
    { 3.8059999999999998e-08, 7.0901266883000003e-01, 5.0921519581158000e+03 },
    { 3.7959999999999997e-08, 4.0830253812399997e+00, 3.8703033917943999e+03 },
    { 3.2780000000000000e-08, 2.3485392730500001e+00, 4.2659819087599999e+02 },
    { 3.2000000000000002e-08, 3.8501021701000000e+00, 6.4384962494255997e+03 },
    { 2.8839999999999999e-08, 2.2178230708000002e+00, 3.8289653222319998e+02 },
    { 2.5209999999999999e-08, 5.5489828535700001e+00, 5.3313574437407997e+03 },
    { 2.4129999999999998e-08, 2.1240900089400001e+00, 6.8366452528338004e+03 },
    { 2.4129999999999998e-08, 1.0087167678300000e+00, 6.3989728631399998e+02 },
    { 2.2209999999999998e-08, 4.9197664029499997e+00, 6.6777017350506003e+03 },
    { 2.2069999999999998e-08, 5.8977531808699997e+00, 9.4921463150048003e+03 },
    { 2.1870000000000000e-08, 2.7187108269700002e+00, 6.1276554505572003e+03 },
    { 2.1410000000000001e-08, 2.5432809471300000e+00, 7.7406067835887998e+03 },
    { 2.0079999999999998e-08, 2.5429625249500001e+00, 6.6822051744678001e+03 },
    { 2.0009999999999999e-08, 5.0337892319500002e+00, 5.2969096509459996e+02 },
    { 1.9790000000000001e-08, 9.5853558189999999e-01, 6.6812075997474003e+03 },
    { 1.9790000000000001e-08, 2.9885827291300000e+00, 6.6812421070518003e+03 },
    { 1.9620000000000000e-08, 6.1721172695900002e+00, 6.9239534573736000e+03 },
    { 1.9530000000000000e-08, 6.1203727766200000e+00, 1.9144826611159999e+02 },
    { 1.9020000000000001e-08, 2.4623431527999999e+00, 4.9296853219836003e+03 },
    { 1.8810000000000001e-08, 4.1316587715899997e+00, 5.7295064471490005e+03 },
    { 1.7280000000000000e-08, 4.7568764510800001e+00, 3.5539115221378001e+03 },
    { 1.7269999999999998e-08, 4.4264158482599996e+00, 6.6741113063987996e+03 },
    { 1.6759999999999999e-08, 5.7237532991700002e+00, 2.8109214616052000e+03 },
    { 1.6309999999999998e-08, 2.4574114810199998e+00, 7.8756718636242003e+03 },
    { 1.5729999999999999e-08, 4.5050827502199997e+00, 3.1273133312618002e+03 },
    { 1.5720000000000001e-08, 1.3356096255600001e+00, 8.9695688969109997e+03 },
    { 1.5589999999999999e-08, 5.1817201363800001e+00, 8.9553418029093991e+03 },
    { 1.5430000000000000e-08, 2.8316985959599998e+00, 2.1187638603783998e+03 },
    { 1.4839999999999999e-08, 5.7668552710199998e+00, 1.0419986283507600e+04 },
    { 1.4829999999999999e-08, 1.6505148629299999e+00, 9.8664168806651996e+03 },
    { 1.4640000000000001e-08, 5.6783303603300004e+00, 1.3365972825148199e+04 },
    { 1.4020000000000001e-08, 4.9749532103899998e+00, 2.6997348193175999e+03 },
    { 1.2750000000000000e-08, 4.0121637522800002e+00, 6.4087760738220004e+02 },
    { 1.2150000000000000e-08, 2.9390372284900002e+00, 3.7672106175757999e+03 },
    { 1.1590000000000000e-08, 1.6035667596500001e+00, 6.2983283211764001e+03 },
    { 1.1570000000000000e-08, 6.6416438122999999e-01, 6.2546266625236003e+03 },
    { 1.1380000000000000e-08, 3.8631420590000000e-02, 1.2832758741704600e+04 },
    { 1.1050000000000000e-08, 5.0244548839099998e+00, 7.2109158184941998e+03 },
    { 1.0770000000000000e-08, 2.1498620249100000e+00, 1.2303067776610000e+04 },
    { 1.0360000000000000e-08, 4.4128738043100002e+00, 3.9814900340820000e+03 },
    { 9.8999999999999993e-09, 4.4683446387699997e+00, 9.8303890139878004e+03 },
    { 9.5000000000000007e-09, 2.5861778748999997e-01, 4.5624609930212000e+03 },
    { 8.8200000000000006e-09, 2.0793504221900001e+00, 1.7515395314160000e+03 },
    { 8.7700000000000001e-09, 4.2248830155499997e+00, 1.0818135286915800e+04 },
    { 8.5099999999999998e-09, 1.3569465257300000e+00, 2.2812304965106000e+03 },
    { 8.4200000000000003e-09, 3.0407135485599999e+00, 6.6883384004004001e+03 },
    { 8.2100000000000004e-09, 3.3818189376299999e+00, 1.0021854533751601e+04 },
    { 8.2100000000000004e-09, 1.3511549659800000e+00, 1.0021820026447200e+04 },
    { 8.1400000000000004e-09, 4.0825113466999996e+00, 6.1515338883049999e+03 },
    { 8.1199999999999993e-09, 1.9670178366100000e+00, 6.0413275670856001e+03 },
    { 7.5699999999999993e-09, 5.3347365041300003e+00, 9.6236882766912004e+03 },
    { 7.3399999999999999e-09, 7.5626361103999995e-01, 1.0018314161750401e+04 },
    { 7.2300000000000001e-09, 6.1986752222400003e+00, 8.5319638175199998e+02 },
    { 7.1699999999999998e-09, 3.6460438747900001e+00, 3.1781457905676002e+03 },
    { 6.8500000000000001e-09, 1.0314965100000000e+00, 8.2738208670324002e+03 },
    { 6.7299999999999997e-09, 5.6189608941899998e+00, 3.5030790628320001e+03 },
    { 6.6199999999999999e-09, 5.2987932641100004e+00, 6.8945239488376001e+03 },
    { 6.5100000000000001e-09, 2.2034541460399999e+00, 3.9814900340819997e+02 },
    { 6.4599999999999996e-09, 5.3181866209999997e+00, 6.6811575430967996e+03 },
    { 6.4100000000000000e-09, 4.9002950999600001e+00, 6.6812921637024001e+03 },
    { 6.3300000000000003e-09, 5.3863226380100002e+00, 6.0403472460173998e+03 },
    { 6.1700000000000000e-09, 2.5676253860400000e+00, 3.2533042221599999e+03 },
    { 5.9600000000000001e-09, 4.1458038468000002e-01, 3.4279206312396000e+03 },
    { 5.6999999999999998e-09, 2.5870383814900002e+00, 1.5643680203309799e+04 },
    { 5.6100000000000003e-09, 5.6615036941200003e+00, 2.6724899413598399e+04 },
    { 5.4500000000000000e-09, 4.9861758832499996e+00, 2.7662676283649998e+03 },
    { 5.4500000000000000e-09, 1.0391297476300001e+00, 6.6802445323313996e+03 },
    { 4.9499999999999997e-09, 6.0644858665500001e+00, 2.9140142358237999e+03 },
    { 4.9300000000000001e-09, 2.5887800004699999e+00, 1.0390266107903999e+03 },
    { 4.8699999999999999e-09, 4.3803129630299997e+00, 3.9149572250346000e+03 },
    { 4.7699999999999999e-09, 4.1395660804199998e+00, 7.2347942562420003e+03 },
    { 4.4200000000000000e-09, 2.6882256979000001e+00, 7.6329432596502002e+03 },
    { 4.2500000000000000e-09, 3.2732043284099999e+00, 2.6484548254729998e+03 },
    { 4.1100000000000001e-09, 5.7436601052700000e+00, 1.2964300703391000e+04 },
    { 4.1000000000000003e-09, 1.5825152701300000e+00, 3.9805097130138001e+03 },
    { 3.9499999999999998e-09, 6.1265281923600003e+00, 2.7007151403858002e+03 },
    { 3.8000000000000001e-09, 2.8927512267300002e+00, 1.1081219210288600e+04 },
    { 3.7200000000000000e-09, 5.8727293049400000e+00, 4.0327700279266001e+03 },
    { 3.7099999999999998e-09, 5.2855427023899999e+00, 7.0641213856227996e+03 },
    { 3.6899999999999999e-09, 2.7459279915399999e+00, 8.2702977486834006e+03 },
    { 3.6699999999999999e-09, 6.2535956369000001e-01, 8.9624553499102003e+03 },
    { 3.5800000000000000e-09, 5.9103982081500002e+00, 8.6719698704406001e+03 },
    { 3.4100000000000000e-09, 1.5519971701499999e+00, 8.0310922630584000e+03 },
    { 3.3799999999999999e-09, 5.2201525287999999e+00, 8.4327643848156004e+03 },
    { 3.3299999999999999e-09, 2.8199543332999999e-01, 2.3015858159093900e+03 },
    { 3.2599999999999999e-09, 2.9178977750100001e+00, 1.0022817601167601e+04 },
    { 3.1800000000000002e-09, 2.3995933798800002e+00, 2.1329909543799999e+02 },
    { 3.1500000000000001e-09, 3.8937613087000000e-01, 1.6173371168404399e+04 },
    { 3.0600000000000002e-09, 4.3653416761399999e+00, 6.8726731195111997e+03 },
    { 3.0500000000000000e-09, 4.7720926565299999e+00, 5.7434479833479998e+02 },
    { 3.0000000000000000e-09, 2.8538578347299999e+00, 5.4593762870782002e+03 },
    { 2.8900000000000002e-09, 4.3873724937100000e+00, 9.7791086761253991e+03 },
    { 2.8499999999999999e-09, 1.7307142811900000e+00, 1.2310181323610799e+04 },
    { 2.8299999999999999e-09, 6.1378845170499998e+00, 1.3760598710207400e+04 },
    { 2.7999999999999998e-09, 2.8323090642500000e+00, 1.1216284290324000e+04 },
    { 2.7799999999999999e-09, 3.1617350479400002e+00, 5.8280284716475999e+03 },
    { 2.7200000000000001e-09, 5.8529322042800001e+00, 1.0551528245194000e+04 },
    { 2.6799999999999998e-09, 5.3001214792900004e+00, 1.0021904590402200e+04 },
    { 2.6799999999999998e-09, 8.2005853571999998e-01, 2.4874160449477999e+03 },
    { 2.6700000000000001e-09, 5.7085373759699998e+00, 1.0021769969796600e+04 },
    { 2.6599999999999999e-09, 2.9635839962800001e+00, 1.8984292630009601e+04 },
    { 2.6299999999999998e-09, 3.0487507445499999e+00, 9.4682678772569998e+03 },
    { 2.6000000000000001e-09, 5.9900554651000004e-01, 5.6421982426092000e+03 },
    { 2.5200000000000000e-09, 3.2881724300699999e+00, 6.2830758499914000e+03 },
    { 2.5099999999999998e-09, 4.8208384547599996e+00, 1.0664954771900000e+03 },
    { 2.4600000000000002e-09, 2.3564125999999999e-01, 1.0264565884073400e+04 },
    { 2.4500000000000000e-09, 6.5336059003000002e-01, 8.4292412664666008e+03 },
    { 2.4199999999999999e-09, 6.6542784607000005e-01, 1.3358926588450200e+04 },
    { 2.3899999999999998e-09, 6.0044188699300003e+00, 1.6706585251847999e+04 },
    { 2.1200000000000001e-09, 3.4154037310800001e+00, 7.1078230442756003e+03 },
    { 2.0900000000000000e-09, 1.2658957872300001e+00, 5.6147293762095996e+03 },
    { 2.0500000000000002e-09, 2.2262370213099998e+00, 1.2218485663214001e+03 },
    { 1.9900000000000000e-09, 3.7336204396100001e+00, 1.3362466960451400e+04 },
    { 1.9900000000000000e-09, 1.7028992538600001e+00, 1.3362432453146999e+04 },
    { 1.9599999999999998e-09, 1.9972349467799999e+00, 1.3207029307364999e+04 },
    { 1.9300000000000002e-09, 4.4387280234200004e+00, 9.0701188738487999e+03 },
    { 1.7700000000000001e-09, 1.9492771076900000e+00, 9.6389407478762005e+03 },
    { 1.7500000000000000e-09, 2.5924716694800001e+00, 9.3819399937853996e+03 },
    { 1.7200000000000001e-09, 4.8476942528100002e+00, 1.3171001440687600e+04 },
    { 1.6999999999999999e-09, 2.9951435337299999e+00, 4.3796390374902003e+03 },
    { 1.6700000000000000e-09, 1.9911173238700000e+00, 1.0522683831884001e+03 },
    { 1.5700000000000000e-09, 1.6247582139000000e-01, 8.8273902698748007e+03 },
    { 1.5500000000000000e-09, 5.4863893316199999e+00, 1.2295954229609200e+04 },
    { 1.4800000000000001e-09, 5.7536055949999998e+00, 1.0235136375537400e+04 },
    { 1.4100000000000001e-09, 3.4233919589299999e+00, 1.0028950827100200e+04 },
    { 1.3999999999999999e-09, 3.8784833174300002e+00, 1.0575406682941801e+04 },
    { 1.3900000000000000e-09, 1.2006533320099999e+00, 1.4371756141986000e+03 },
    { 1.3399999999999999e-09, 1.9376192972199999e+00, 1.9034368125011999e+03 },
    { 1.3100000000000000e-09, 1.8224987590199999e+00, 3.0297716278314001e+03 },
    { 1.3100000000000000e-09, 1.1969957337599999e+00, 6.6608695340007998e+03 },
    { 1.2900000000000001e-09, 3.9077914988600000e+00, 6.6964773245846000e+03 },
    { 1.2100000000000000e-09, 2.8474029853600000e+00, 1.0973555686350001e+04 },
    { 1.1900000000000000e-09, 3.7878891783599999e+00, 9.1686408983473993e+03 },
    { 1.1800000000000001e-09, 5.2429614996699998e+00, 1.4956319713459999e+02 },
    { 1.1700000000000001e-09, 5.8437907767499997e+00, 6.7015801727983999e+03 },
    { 1.1599999999999999e-09, 1.1595587055400001e+00, 3.6514532255682002e+03 },
    { 1.1400000000000000e-09, 4.7573783022100002e+00, 7.3221024607817999e+03 },
    { 1.1400000000000000e-09, 3.7201985401100002e+00, 6.5187582172674001e+03 },
    { 1.0800000000000000e-09, 2.1033352377300001e+00, 5.2440492392010001e+03 },
    { 1.0700000000000000e-09, 5.1440413179800002e+00, 4.4071079038897997e+03 },
    { 1.0500000000000001e-09, 1.9036122121700001e+00, 1.1371704689758200e+04 },
    { 1.0399999999999999e-09, 3.1348773094800002e+00, 6.6659723822145997e+03 },
    { 1.0100000000000000e-09, 4.4646300786499999e+00, 1.4158747713615599e+04 },
    { 1.0100000000000000e-09, 6.0312608203900000e+00, 1.1769853693166400e+04 },
    { 1.0100000000000000e-09, 2.8446635234600000e+00, 1.1773376811515400e+04 },
    { 9.5999999999999999e-10, 6.0958955217000002e+00, 1.6304913130090799e+04 },
    { 9.5000000000000003e-10, 3.0620246397000001e-01, 6.8162899334350004e+03 },
    { 9.4000000000000006e-10, 5.3024456857500004e+00, 9.8085381846614000e+03 },
    { 9.2999999999999999e-10, 6.0303321038600002e+00, 2.0752664362475400e+04 },
    { 9.2000000000000003e-10, 4.7350247698199999e+00, 1.0014723733098601e+04 },
    { 9.0999999999999996e-10, 1.8787602058199999e+00, 2.5287723799399799e+04 },
    { 8.9999999999999999e-10, 5.5142001861500001e+00, 1.7256631536341400e+04 },
    { 8.7999999999999996e-10, 6.0629231466100002e+00, 1.8353246176068001e+03 },
    { 8.4999999999999996e-10, 4.5449951854999998e-01, 4.1938088084518004e+03 },
    { 8.3000000000000003e-10, 3.7618554566100002e+00, 1.5052878090929901e+03 },
    { 8.1999999999999996e-10, 3.2427570840200000e+00, 6.4679257579615996e+03 },
    { 8.0000000000000003e-10, 2.3538170025100000e+00, 2.1795214091614791e+04 },
    { 7.7999999999999999e-10, 5.6081990379000004e+00, 1.0404733812322600e+04 },
    { 7.5999999999999996e-10, 1.7812622823999999e-01, 9.3809596727171993e+03 },
    { 7.2999999999999996e-10, 8.8325190136999998e-01, 6.6604494579071998e+03 },
    { 7.2999999999999996e-10, 1.2235624624600001e+00, 1.4712317116458000e+04 },
    { 7.2000000000000000e-10, 1.8056450929300001e+00, 2.7035740212466801e+04 },
    { 7.1000000000000003e-10, 2.1026020432300001e+00, 1.5113989238215199e+04 },
    { 7.1000000000000003e-10, 5.5488410364699998e+00, 1.4128242771245599e+04 },
    { 6.9000000000000000e-10, 6.0036590853499998e+00, 3.0065511840298201e+04 },
    { 6.8000000000000003e-10, 3.3003204367599999e+00, 3.3406296803519999e+03 },
    { 6.8000000000000003e-10, 1.2699309889899999e+00, 3.3405951730475999e+03 },
    { 6.6999999999999996e-10, 2.6956437254300001e+00, 6.9215760122680001e+02 },
    { 6.6999999999999996e-10, 4.6509816344899999e+00, 2.2334736210149999e+03 },
    { 6.6000000000000000e-10, 2.1255122567900000e+00, 3.0376352639166598e+04 },
    { 6.5000000000000003e-10, 5.4986504458200001e+00, 1.7412051935775598e+04 },
    { 6.5000000000000003e-10, 3.2445470003699999e+00, 2.6316226244231998e+03 },
    { 6.5000000000000003e-10, 1.0413563831999999e+00, 2.0199094959632999e+04 },
    { 6.5000000000000003e-10, 5.3414957996199997e+00, 6.1068800550648002e+03 },
    { 6.3999999999999996e-10, 7.6110443806000005e-01, 6.7685330579394004e+03 },
    { 6.3999999999999996e-10, 5.7136030409899998e+00, 1.3119721102825190e+04 },
    { 6.3000000000000000e-10, 4.7749574070899996e+00, 1.3517870106233400e+04 },
    { 6.3000000000000000e-10, 5.6569950840700001e+00, 1.3362517017102000e+04 },
    { 6.3000000000000000e-10, 4.4103912098700002e+00, 2.2743409379516401e+04 },
    { 6.2000000000000003e-10, 3.2328749097700000e+00, 5.9890672521728002e+03 },
    { 6.0999999999999996e-10, 1.6214155966099999e+00, 1.2935851515923199e+04 },
    { 5.7000000000000000e-10, 7.5475068766999998e-01, 1.6699539015149989e+04 },
    { 5.6000000000000003e-10, 3.2332496267400002e+00, 1.1128976085784199e+04 },
    { 5.6000000000000003e-10, 5.6608917698700001e+00, 9.4992598620055996e+03 },
    { 5.6000000000000003e-10, 6.0651307877000002e+00, 1.3362382396496399e+04 },
    { 5.6000000000000003e-10, 3.0452955348000001e+00, 6.7020002488919999e+03 },
    { 5.3000000000000003e-10, 9.6281646589000003e-01, 8.4256508378147992e+03 },
    { 5.3000000000000003e-10, 5.3311903449800004e+00, 7.9030734197210004e+03 },
    { 5.1999999999999996e-10, 4.1082808955800001e+00, 1.1243685846420800e+04 },
    { 5.1999999999999996e-10, 1.4139316963499999e+00, 1.6062184526116800e+04 },
    { 5.1000000000000000e-10, 1.0897729974100001e+00, 1.2721572099417001e+04 },
    { 5.1000000000000000e-10, 7.6284557905000006e-01, 1.9513983595104201e+04 },
    { 5.0000000000000003e-10, 2.8802837296999999e+00, 6.5939166488598003e+03 },
    { 5.0000000000000003e-10, 6.0203144857400002e+00, 4.0496022289764001e+03 },
    { 5.0000000000000003e-10, 1.6634326696999999e-01, 1.3892140671893800e+04 },
    { 4.8999999999999996e-10, 3.2399907641199999e+00, 1.4421831636988400e+04 },
    { 4.8999999999999996e-10, 1.4718844672700000e+00, 1.4584298273120599e+04 },
    { 4.8999999999999996e-10, 2.8712436881199999e+00, 4.8459002357928002e+03 },
    { 4.8999999999999996e-10, 4.4859423447499998e+00, 6.6573464156518003e+03 },
    { 4.8999999999999996e-10, 3.0597912521700001e+00, 1.1610910175383200e+04 },
    { 4.8000000000000000e-10, 3.2238662454100000e+00, 1.0177257679533601e+04 },
    { 4.7000000000000003e-10, 2.0322603942400002e+00, 7.3211221397135996e+03 },
    { 4.7000000000000003e-10, 1.3070285510999999e+00, 2.1265523126520198e+04 },
    { 4.7000000000000003e-10, 2.8748757200699999e+00, 2.3141558382924599e+04 },
    { 4.6000000000000001e-10, 5.8934360449299996e+00, 1.3149150611361199e+04 },
    { 4.6000000000000001e-10, 3.3027867180000003e-02, 3.3335661900017999e+03 },
    { 4.6000000000000001e-10, 6.0907390514599999e+00, 3.3476586633978000e+03 },
    { 4.6000000000000001e-10, 6.2085264441100003e+00, 2.5685872802808000e+04 },
    { 4.5000000000000000e-10, 5.2189477326100002e+00, 6.8436914895317996e+03 },
    { 4.5000000000000000e-10, 3.4116709495599999e+00, 2.8230187222691391e+04 },
    { 4.3000000000000001e-10, 3.0889032826300000e+00, 8.7999887137780006e+03 },
    { 4.3000000000000001e-10, 2.1889781459600002e+00, 6.6848152820513997e+03 },
    { 4.3000000000000001e-10, 2.5662203337400000e+00, 6.7051032911474003e+03 },
    { 4.3000000000000001e-10, 1.5260553624099999e+00, 6.7096740408674004e+03 },
    { 4.3000000000000001e-10, 1.7578547221800001e+00, 6.6776344247478000e+03 },
    { 4.3000000000000001e-10, 7.8042917902999998e-01, 7.2555696517344004e+03 },
    { 4.3000000000000001e-10, 2.3397573178100002e+00, 1.8454601664915001e+04 },
    { 4.2000000000000000e-10, 6.2471811381400002e+00, 9.4850327680040009e+03 },
    { 4.2000000000000000e-10, 3.2796849137600002e+00, 1.3363430027867400e+04 },
    { 4.2000000000000000e-10, 4.4176659061099999e+00, 4.4477512323846004e+03 },
    { 4.0999999999999998e-10, 9.6907528519999994e-02, 1.6460333529524989e+04 },
    { 4.0999999999999998e-10, 3.5073747085600000e+00, 3.1570799649391200e+04 },
    { 4.0000000000000001e-10, 6.1605943900000004e-01, 1.4054607308025999e+04 },
    { 4.0000000000000001e-10, 2.1057657875300002e+00, 8.9828106693089994e+03 },
    { 3.9000000000000000e-10, 3.6854327791300001e+00, 1.3553897972910800e+04 },
    { 3.7999999999999998e-10, 4.0171577874700004e+00, 6.5316616562649997e+03 },
    { 3.7999999999999998e-10, 1.0915307382799999e+00, 1.1614433293732200e+04 },
    { 3.7999999999999998e-10, 3.3306520310600001e+00, 2.2324905056709398e+04 },
    { 3.7000000000000001e-10, 3.1162502724299999e+00, 2.7433889215874991e+04 },
    { 3.6000000000000000e-10, 2.5253474027300000e+00, 1.2323423096008801e+04 },
    { 3.6000000000000000e-10, 4.6922007396599996e+00, 6.1586474353058002e+03 },
    { 3.6000000000000000e-10, 1.4029164297300001e+00, 2.4606135553220000e+04 },
    { 3.4999999999999998e-10, 3.4898251841199999e+00, 1.7654780539749601e+04 },
    { 3.4999999999999998e-10, 1.2810745886699999e+00, 2.3695127785766999e+04 },
    { 3.4000000000000001e-10, 4.1922175975700000e+00, 1.3916019109641600e+04 },
    { 3.4000000000000001e-10, 4.7297216819000001e+00, 2.6084021806216198e+04 },
    { 3.4000000000000001e-10, 3.2040932590100000e+00, 1.9004647949408401e+04 },
    { 3.3000000000000000e-10, 1.5844780995900001e+00, 4.7777880408984001e+03 },
    { 3.3000000000000000e-10, 4.5869377696000004e+00, 1.8606498946000200e+04 },
    { 3.3000000000000000e-10, 5.0902091105800000e+00, 3.1172650645983002e+04 },
    { 3.3000000000000000e-10, 1.6723679794899999e+00, 2.8628336226099600e+04 },
    { 3.1999999999999998e-10, 4.7359068269999999e-01, 3.0774501642574800e+04 },
    { 3.1999999999999998e-10, 2.1334513108099999e+00, 5.0992655051166003e+03 },
    { 3.1999999999999998e-10, 2.8743208625599999e+00, 1.5664035522708589e+04 },
    { 3.1000000000000002e-10, 1.1639166550000000e-02, 8.1865126624925997e+03 },
    { 3.1000000000000002e-10, 5.3878733875999996e-01, 4.1064054911461999e+03 },
    { 3.1000000000000002e-10, 2.4433869262900001e+00, 2.5748193622534000e+03 },
    { 3.1000000000000002e-10, 5.5056259300699999e+00, 9.2255392732830005e+03 },
    { 3.1000000000000002e-10, 5.6487040060300000e+00, 2.8180350086059998e+03 },
    { 3.0000000000000000e-10, 2.0759018072400002e+00, 1.6276463942623001e+04 },
    { 3.0000000000000000e-10, 6.6979072956999997e-01, 2.4076444588125400e+04 },
    { 3.0000000000000000e-10, 3.9944358250000001e+00, 1.7444259844152000e+03 },
    { 3.0000000000000000e-10, 3.4128900240700002e+00, 2.9978203635758400e+04 },
    { 2.8999999999999998e-10, 6.2115824800799997e+00, 2.8038079146044001e+03 },
    { 2.8999999999999998e-10, 2.8036663014000002e-01, 2.9026485229507791e+04 },
    { 2.8999999999999998e-10, 3.7034438250299999e+00, 3.3318816062458201e+04 },
    { 2.8999999999999998e-10, 4.0399021485900004e+00, 1.7586530784168001e+03 },
    { 2.8999999999999998e-10, 2.7662728737200002e+00, 2.4093276789175201e+04 },
    { 2.8000000000000002e-10, 1.6766301951200000e+00, 1.2509253325047201e+04 },
    { 2.8000000000000002e-10, 2.1672556893400001e+00, 1.5650793750310600e+04 },
    { 2.8000000000000002e-10, 5.0208994887399996e+00, 1.3745346239022399e+04 },
    { 2.7000000000000000e-10, 4.4042897896700000e+00, 1.5265886519300400e+04 },
    { 2.7000000000000000e-10, 1.0774552279999999e-01, 1.7101211136907201e+04 },
    { 2.7000000000000000e-10, 4.5485812542999998e-01, 1.7810200939183800e+04 },
    { 2.7000000000000000e-10, 3.0958856712000000e+00, 2.6637591209058592e+04 },
    { 2.7000000000000000e-10, 4.8304237289299996e+00, 2.2747290714874402e+04 },
    { 2.7000000000000000e-10, 3.8395164981400001e+00, 8.5846616659007996e+03 },
    { 2.5999999999999998e-10, 2.5887939770000001e-01, 2.0735832161425591e+04 },
    { 2.5999999999999998e-10, 9.4849562976000001e-01, 1.6066065861474801e+04 },
    { 2.5999999999999998e-10, 2.2331997528700001e+00, 5.6289564702112002e+03 },
    { 2.5999999999999998e-10, 3.9919946065400000e+00, 1.1925274092600601e+04 },
    { 2.5000000000000002e-10, 8.9364920474999998e-01, 2.0354515359067202e+04 },
    { 2.5000000000000002e-10, 2.0244605773999999e-01, 2.1548962369291799e+04 },
    { 2.5000000000000002e-10, 4.9316240716999999e+00, 1.4071439509075790e+04 },
    { 2.4000000000000000e-10, 3.9513667350900001e+00, 3.6659428489158003e+04 },
    { 2.4000000000000000e-10, 2.0000970730200001e+00, 1.9402796952816600e+04 },
    { 2.4000000000000000e-10, 8.3608961695999995e-01, 3.8774169387952002e+03 },
    { 2.4000000000000000e-10, 3.3554415228000001e-01, 1.0596182078434200e+04 },
    { 2.3000000000000001e-10, 5.7049871326700003e+00, 4.3928808098882000e+03 },
    { 2.3000000000000001e-10, 8.9003486537999998e-01, 3.7455726495974399e+04 },
    { 2.3000000000000001e-10, 5.2196425116900000e+00, 3.2920667059049993e+04 },
    { 2.1999999999999999e-10, 7.4960129550000004e-02, 8.4398779318163997e+03 },
    { 2.1999999999999999e-10, 1.4103942882200000e+00, 1.2722552420485201e+04 },
    { 2.1999999999999999e-10, 5.5050986891399996e+00, 3.6261279485749801e+04 },
    { 2.1999999999999999e-10, 5.3260573811000000e+00, 2.0597243963041201e+04 },
    { 2.1999999999999999e-10, 5.1367940181199998e+00, 2.1947111372700001e+04 },
    { 2.1000000000000000e-10, 2.0310149559799999e+00, 5.0850384111149997e+03 },
    { 2.1000000000000000e-10, 2.7155163684399999e+00, 2.3296978782358790e+04 },
    { 2.1000000000000000e-10, 5.6280520379099999e+00, 3.3405451163970001e+03 },
    { 2.1000000000000000e-10, 4.8929949859099997e+00, 2.9580054632350191e+04 },
    { 2.1000000000000000e-10, 4.8221374471000000e-01, 1.7013902932367400e+04 },
    { 2.1000000000000000e-10, 2.6249733563500000e+00, 2.4341528317456599e+04 },
    { 2.1000000000000000e-10, 2.3747764159400000e+00, 2.1000915890756802e+04 },
    { 2.1000000000000000e-10, 7.3975045388000005e-01, 3.5863130482341599e+04 },
    { 2.0000000000000001e-10, 5.2049378146700001e+00, 3.3406797370026002e+03 },
    { 2.0000000000000001e-10, 5.6847714775200000e+00, 3.5231183490000002e+00 },
    { 2.0000000000000001e-10, 3.2850984644699999e+00, 1.0629050485381999e+03 },
    { 2.0000000000000001e-10, 3.9433346858100000e+00, 3.4911412076091001e+04 },
    { 2.0000000000000001e-10, 5.2713739510500002e+00, 1.0787630344545800e+04 },
    { 2.0000000000000001e-10, 2.6324611129400002e+00, 7.7883636590843998e+03 },
    { 2.0000000000000001e-10, 4.9367208181400004e+00, 1.0213285546211000e+04 },
    { 1.8999999999999999e-10, 4.1212127419300000e+00, 4.0000040915857790e+04 },
    { 1.8000000000000000e-10, 6.1998827544499999e+00, 1.8208349942592002e+04 },
    { 1.8000000000000000e-10, 1.3932920785000000e+00, 3.8631898447936001e+03 },
    { 1.8000000000000000e-10, 2.0310544117900000e+00, 2.4491425792583399e+04 },
    { 1.8000000000000000e-10, 3.8348468133900000e+00, 3.4925097077850000e+03 },
    { 1.8000000000000000e-10, 2.2889201052499999e+00, 3.1887151456145998e+03 },
    { 1.8000000000000000e-10, 2.2305895210000002e+00, 1.8451078546565990e+04 },
    { 1.8000000000000000e-10, 7.2384948956999995e-01, 2.4889574795991601e+04 },
    { 1.8000000000000000e-10, 5.6221710214299998e+00, 1.4867737515892190e+04 },
    { 1.7000000000000001e-10, 2.3172467161200001e+00, 2.3937856389740999e+04 },
    { 1.7000000000000001e-10, 2.8472138660800002e+00, 5.6183198048614004e+03 },
    { 1.7000000000000001e-10, 9.4803978582000004e-01, 2.3539707386332801e+04 },
    { 1.7000000000000001e-10, 2.6497777952899999e+00, 2.7682140744156401e+04 },
    { 1.7000000000000001e-10, 5.8570173580300002e+00, 7.3733824546264004e+03 },
    { 1.5999999999999999e-10, 2.8086511945899999e+00, 3.1022753170856198e+04 },
    { 1.5999999999999999e-10, 3.8438392468400000e+00, 2.2345260376108199e+04 },
    { 1.5999999999999999e-10, 3.7790021333400001e+00, 1.0448435470975401e+04 },
    { 1.5999999999999999e-10, 4.2132719394000001e-01, 3.2522518055641802e+04 },
    { 1.5999999999999999e-10, 1.0234742966599999e+00, 3.9203742909041393e+04 },
    { 1.5999999999999999e-10, 3.4112161091900002e+00, 1.2808880303956799e+04 },
    { 1.5000000000000000e-10, 5.9715298619699997e+00, 2.3546753623030800e+04 },
    { 1.5000000000000000e-10, 2.5682420442999998e+00, 1.7277406931833801e+04 },
    { 1.5000000000000000e-10, 5.8934366993799996e+00, 2.9389403049426001e+03 },
    { 1.5000000000000000e-10, 2.3033021926000000e-01, 3.7422845484569998e+03 },
    { 1.5000000000000000e-10, 1.3791703343699999e+00, 2.0809467624645200e+04 },
    { 1.5000000000000000e-10, 2.2161107727200000e+00, 3.5464981478933398e+04 },
};

static const VSOP87Term VSOP87A_MARS_X3[135] = {
    { 8.4999899999999992e-06, 0.0000000000000000e+00, 0.0000000000000000e+00 },
    { 5.6013299999999996e-06, 1.9400755205800000e+00, 6.6812248533995999e+03 },
    { 3.1806199999999998e-06, 2.3939169578900001e+00, 1.0021837280099400e+04 },
    { 1.1345800000000000e-06, 2.7568010410900001e+00, 3.3406124266997999e+03 },
    { 8.8769000000000003e-07, 2.7766874212700001e+00, 1.3362449706799200e+04 },
    { 1.9119000000000001e-07, 3.1383532300199999e+00, 1.6703062133498999e+04 },
    { 1.7277000000000000e-07, 3.5431932729600000e+00, 3.4960328261340001e+03 },
    { 1.7102000000000001e-07, 5.7232899001600002e+00, 3.1851920272655998e+03 },
    { 7.9800000000000000e-08, 3.0768233032499999e+00, 3.3441355450487999e+03 },
    { 7.9380000000000000e-08, 6.1921078975199997e+00, 3.3370893083507999e+03 },
    { 3.7380000000000001e-08, 4.3179854532500004e+00, 3.5833410306738001e+03 },
    { 3.7009999999999999e-08, 4.9478642660199998e+00, 3.0978838227257902e+03 },
    { 3.5999999999999998e-08, 3.4905725639699998e+00, 2.0043674560198800e+04 },
    { 2.8710000000000000e-08, 3.5106304166100002e+00, 6.6847479717486003e+03 },
    { 2.3960000000000001e-08, 6.0812342591800004e+00, 6.5258044539654002e+03 },
    { 2.3809999999999999e-08, 2.1401175326600002e+00, 3.8941818295421999e+03 },
    { 2.3660000000000001e-08, 8.4278889817000002e-01, 2.7870430238573999e+03 },
    { 2.2169999999999999e-08, 4.1010521700500000e+00, 1.5542039943419999e+02 },
    { 1.2940000000000000e-08, 7.4157779671000001e-01, 4.2923308329503998e+03 },
    { 1.2779999999999999e-08, 2.2430293686299998e+00, 2.3888940204492001e+03 },
    { 1.1609999999999999e-08, 9.2511464091999995e-01, 3.5320606928113998e+03 },
    { 1.1580000000000000e-08, 1.9610479393400000e+00, 4.1369104335162001e+03 },
    { 1.1560000000000001e-08, 2.0632971539299998e+00, 3.1491641605882000e+03 },
    { 1.0260000000000001e-08, 1.0233257975200001e+00, 2.5443144198834002e+03 },
    { 9.9900000000000005e-09, 2.4081778139300001e+00, 3.3396321056316001e+03 },
    { 9.9599999999999995e-09, 5.7020320129000002e-01, 3.3415927477680002e+03 },
    { 9.0799999999999993e-09, 5.6411365739699999e+00, 4.6904798363585996e+03 },
    { 8.8699999999999994e-09, 3.6454862205100000e+00, 1.9907450170410000e+03 },
    { 7.9799999999999993e-09, 3.2633511690899999e+00, 3.7235089589230001e+03 },
    { 7.9400000000000003e-09, 6.0101776233599997e+00, 2.9577158944766002e+03 },
    { 7.7799999999999992e-09, 3.8689863271200000e+00, 1.0025360398448400e+04 },
    { 7.3000000000000000e-09, 5.7139245110000003e-01, 4.5350594369244000e+03 },
    { 6.3199999999999997e-09, 4.2432804223599998e+00, 5.0886288397668004e+03 },
    { 6.2499999999999997e-09, 3.8396724350500002e+00, 2.3384286986898602e+04 },
    { 6.2099999999999999e-09, 5.0594395343800000e+00, 1.5925960136327999e+03 },
    { 6.0000000000000000e-09, 2.3899240962800001e+00, 2.1461654164751999e+03 },
    { 4.8799999999999997e-09, 4.6709286300299997e+00, 6.9239534573736000e+03 },
    { 4.4999999999999998e-09, 3.3989701155400001e+00, 3.7387614301079998e+03 },
    { 4.4900000000000000e-09, 1.9392811529800000e+00, 3.3334988796990001e+03 },
    { 4.1899999999999998e-09, 1.0867644294200001e+00, 3.3477259737006002e+03 },
    { 4.1100000000000001e-09, 5.8978821030099997e+00, 2.9424634232916001e+03 },
    { 3.9899999999999997e-09, 6.3956537358999999e-01, 4.3999943568890003e+03 },
    { 3.9700000000000001e-09, 2.5008731490999998e+00, 6.4897765872880000e+03 },
    { 3.7300000000000001e-09, 3.3855730046399999e+00, 6.8366452528338004e+03 },
    { 3.5699999999999999e-09, 1.5120327085999999e-01, 9.8664168806651996e+03 },
    { 3.5600000000000001e-09, 2.4201359347000001e-01, 1.0593819301891999e+03 },
    { 3.5600000000000001e-09, 1.2012768312700000e+00, 6.1276554505572003e+03 },
    { 3.4600000000000000e-09, 2.3531015655099998e+00, 7.4775228602159996e+03 },
    { 2.9499999999999999e-09, 2.8102862130399999e+00, 5.6218429232103999e+03 },
    { 2.9499999999999999e-09, 3.8225189930400001e+00, 7.0793738568077997e+03 },
    { 2.3699999999999999e-09, 5.1931853386400002e+00, 6.4384962494255997e+03 },
    { 2.1499999999999998e-09, 2.6037540979700000e+00, 5.7295064471490005e+03 },
    { 2.1400000000000001e-09, 9.8789182008999998e-01, 6.6822051744678001e+03 },
    { 1.7800000000000001e-09, 4.0044183124900004e+00, 5.3313574437407997e+03 },
    { 1.7300000000000000e-09, 4.2136404547400002e+00, 1.3365972825148199e+04 },
    { 1.6300000000000000e-09, 5.4376308878600002e+00, 4.9332084403325998e+03 },
    { 1.5600000000000000e-09, 1.0305614931900000e+00, 7.7406067835887998e+03 },
    { 1.5400000000000001e-09, 9.3377729775999996e-01, 7.8756718636242003e+03 },
    { 1.5400000000000001e-09, 6.0973996290099999e+00, 6.6777017350506003e+03 },
    { 1.4900000000000000e-09, 9.3873863269999996e-02, 6.2983283211764001e+03 },
    { 1.4300000000000000e-09, 2.3555370399100002e+00, 7.2347942562420003e+03 },
    { 1.0800000000000000e-09, 4.2392124569099998e+00, 1.0419986283507600e+04 },
    { 1.0000000000000001e-09, 2.9132759218099999e+00, 9.8303890139878004e+03 },
    { 9.9000000000000010e-10, 4.1863077042099999e+00, 2.6724899413598399e+04 },
    { 8.3999999999999999e-10, 2.7198420834100001e+00, 1.0818135286915800e+04 },
    { 8.1999999999999996e-10, 1.4750245134000000e+00, 6.6883384004004001e+03 },
    { 7.4000000000000003e-10, 1.0267247892700000e+00, 4.9296853219836003e+03 },
    { 7.4000000000000003e-10, 9.3262918683999996e-01, 7.6329432596502002e+03 },
    { 6.5000000000000003e-10, 5.0206033550600004e+00, 1.0264565884073400e+04 },
    { 6.0999999999999996e-10, 2.8829408809400001e+00, 7.9629800681639995e+02 },
    { 5.7000000000000000e-10, 1.5524658172100001e+00, 9.4682678772569998e+03 },
    { 5.7000000000000000e-10, 4.7965556770099997e+00, 1.2832758741704600e+04 },
    { 5.7000000000000000e-10, 3.2132879140699999e+00, 7.2109158184941998e+03 },
    { 5.4000000000000000e-10, 4.1655683013299996e+00, 9.4921463150048003e+03 },
    { 5.1000000000000000e-10, 5.8272674584999997e+00, 8.2738208670324002e+03 },
    { 5.0000000000000003e-10, 5.0123821286000003e-01, 1.3207029307364999e+04 },
    { 4.8999999999999996e-10, 5.6721396697599999e+00, 6.6812075997474003e+03 },
    { 4.8999999999999996e-10, 1.4197242561200001e+00, 6.6812421070518003e+03 },
    { 4.6000000000000001e-10, 4.1006037277900003e+00, 3.9814900340819997e+02 },
    { 4.5000000000000000e-10, 5.7875456491500001e+00, 8.0310922630584000e+03 },
    { 4.3999999999999998e-10, 2.8212497269000001e+00, 3.8703033917943999e+03 },
    { 4.3000000000000001e-10, 1.3774522365999999e+00, 1.0022817601167601e+04 },
    { 4.2000000000000000e-10, 1.8392846469399999e+00, 1.0021854533751601e+04 },
    { 4.2000000000000000e-10, 6.0918423315599997e+00, 1.0021820026447200e+04 },
    { 3.7000000000000001e-10, 1.9254777844000001e-01, 8.8273902698748007e+03 },
    { 3.7000000000000001e-10, 5.8418647072600001e+00, 1.0018314161750401e+04 },
    { 3.6000000000000000e-10, 1.3967524964600000e+00, 1.1081219210288600e+04 },
    { 3.6000000000000000e-10, 3.3785527046300001e+00, 7.0641213856227996e+03 },
    { 3.6000000000000000e-10, 2.1597015084999999e-01, 1.2303067776610000e+04 },
    { 3.4999999999999998e-10, 2.9686245373500002e+00, 9.0701188738487999e+03 },
    { 3.4000000000000001e-10, 3.8002992241200002e+00, 2.8109214616052000e+03 },
    { 3.3000000000000000e-10, 1.0112024362100001e+00, 1.5643680203309799e+04 },
    { 3.3000000000000000e-10, 4.3899151804600001e+00, 8.6719698704406001e+03 },
    { 3.1999999999999998e-10, 6.0054346091299999e+00, 2.2812304965106000e+03 },
    { 3.1000000000000002e-10, 1.5654333614500000e+00, 1.1944470102246000e+03 },
    { 3.1000000000000002e-10, 2.5116910845599998e+00, 6.6802445323313996e+03 },
    { 2.7000000000000000e-10, 4.5397773687100003e+00, 1.6706585251847999e+04 },
    { 2.5999999999999998e-10, 1.2983505608799999e+00, 1.1216284290324000e+04 },
    { 2.5999999999999998e-10, 3.7187607123599999e+00, 9.6236882766912004e+03 },
    { 2.5000000000000002e-10, 2.9024523349300000e+00, 1.0575406682941801e+04 },
    { 2.5000000000000002e-10, 4.6198952076899999e+00, 1.3760598710207400e+04 },
    { 2.5000000000000002e-10, 3.6983702056099999e+00, 6.8945239488376001e+03 },
    { 2.4000000000000000e-10, 5.3026964257799998e+00, 8.4327643848156004e+03 },
    { 2.4000000000000000e-10, 1.3203660667799999e+00, 8.2702977486834006e+03 },
    { 2.3000000000000001e-10, 3.8277747946399998e+00, 1.3498674096587999e+03 },
    { 2.3000000000000001e-10, 2.7829716223599998e+00, 5.4867778431750003e+03 },
    { 2.1000000000000000e-10, 5.9053840618500004e+00, 8.9624553499102003e+03 },
    { 2.1000000000000000e-10, 2.5883685286100002e+00, 1.7480164130670000e+03 },
    { 2.1000000000000000e-10, 1.5516674251600000e+00, 1.0973555686350001e+04 },
    { 2.0000000000000001e-10, 4.5032113933000001e-01, 9.6389407478762005e+03 },
    { 2.0000000000000001e-10, 1.2380965108699999e+00, 3.7672106175757999e+03 },
    { 1.8999999999999999e-10, 5.8209013916099996e+00, 9.5171840625059997e+02 },
    { 1.8000000000000000e-10, 2.1194212708400002e+00, 6.6741113063987996e+03 },
    { 1.8000000000000000e-10, 2.9845388364600001e+00, 1.0177257679533601e+04 },
    { 1.8000000000000000e-10, 3.6994077486100001e+00, 5.6147293762095996e+03 },
    { 1.8000000000000000e-10, 9.0284270729000005e-01, 5.5356940284240000e+02 },
    { 1.7000000000000001e-10, 3.2820205613600000e+00, 1.3171001440687600e+04 },
    { 1.5999999999999999e-10, 3.7089389866200002e+00, 6.6811575430967996e+03 },
    { 1.5999999999999999e-10, 4.9948648675899996e+00, 2.9140142358237999e+03 },
    { 1.5999999999999999e-10, 4.0290215340600000e+00, 1.0551528245194000e+04 },
    { 1.5999999999999999e-10, 3.3004425997700002e+00, 6.6812921637024001e+03 },
    { 1.5000000000000000e-10, 1.4702366667700000e+00, 5.4593762870782002e+03 },
    { 1.4000000000000001e-10, 1.8559182629400000e+00, 1.0028950827100200e+04 },
    { 1.4000000000000001e-10, 2.4763128547999999e-01, 1.2310181323610799e+04 },
    { 1.2999999999999999e-10, 3.7523695539000002e-01, 1.9144826611159999e+02 },
    { 1.0000000000000000e-10, 5.9314703357700003e+00, 5.6289564702112002e+03 },
    { 1.0000000000000000e-10, 5.7024932296099999e+00, 2.4272860397400001e+02 },
    { 1.0000000000000000e-10, 3.2342668885000001e+00, 1.0014723733098601e+04 },
    { 1.0000000000000000e-10, 5.1914408539999997e+00, 1.6173371168404399e+04 },
    { 8.9999999999999999e-11, 4.7283243429799997e+00, 8.4292412664666008e+03 },
    { 8.9999999999999999e-11, 4.1049505150299996e+00, 1.2295954229609200e+04 },
    { 8.9999999999999999e-11, 1.5648903451600000e+00, 1.7515395314160000e+03 },
    { 8.9999999999999999e-11, 1.9023391397600000e+00, 3.3406296803519999e+03 },
    { 8.9999999999999999e-11, 6.1559508273899999e+00, 3.3405951730475999e+03 },
    { 7.0000000000000004e-11, 3.0836419920900000e+00, 1.4158747713615599e+04 },
};

static const VSOP87Term VSOP87A_MARS_X4[41] = {
    { 2.5348999999999998e-07, 3.1415926535900001e+00, 0.0000000000000000e+00 },
    { 1.2260000000000000e-07, 8.3414775886000003e-01, 1.0021837280099400e+04 },
    { 1.0035000000000000e-07, 4.0425836056999997e-01, 6.6812248533995999e+03 },
    { 5.1980000000000000e-08, 1.2429950997899999e+00, 1.3362449706799200e+04 },
    { 3.4310000000000001e-08, 8.5852911695000000e-01, 3.3406124266997999e+03 },
    { 2.4270000000000002e-08, 5.0544454998399999e+00, 3.4960328261340001e+03 },
    { 2.4179999999999999e-08, 4.2142053716700003e+00, 3.1851920272655998e+03 },
    { 1.4970000000000001e-08, 1.6226903698799999e+00, 1.6703062133498999e+04 },
    { 5.9500000000000003e-09, 2.8107612079600002e+00, 3.5833410306738001e+03 },
    { 5.9200000000000002e-09, 1.7219890479000000e-01, 3.0978838227257902e+03 },
    { 3.8199999999999996e-09, 4.5753494081300001e+00, 6.5258044539654002e+03 },
    { 3.5199999999999998e-09, 1.9863056824400001e+00, 2.0043674560198800e+04 },
    { 2.8600000000000000e-09, 5.6218555489300002e+00, 2.7870430238573999e+03 },
    { 2.8600000000000000e-09, 3.6453081606000000e+00, 3.8941818295421999e+03 },
    { 2.5899999999999999e-09, 1.4629578163800001e+00, 3.3441355450487999e+03 },
    { 2.5599999999999998e-09, 1.5399767185400000e+00, 3.3370893083507999e+03 },
    { 1.5100000000000000e-09, 1.9454277862400000e+00, 6.6847479717486003e+03 },
    { 1.2799999999999999e-09, 5.2579045710000001e+00, 6.8366452528338004e+03 },
    { 8.7999999999999996e-10, 3.1673963542800001e+00, 6.9239534573736000e+03 },
    { 7.1000000000000003e-10, 2.3436424684200001e+00, 2.3384286986898602e+04 },
    { 6.3000000000000000e-10, 4.9302412010200003e+00, 9.8664168806651996e+03 },
    { 5.6000000000000003e-10, 2.3505534314399998e+00, 1.0025360398448400e+04 },
    { 4.8999999999999996e-10, 5.9800141646900000e+00, 6.1276554505572003e+03 },
    { 3.4000000000000001e-10, 3.9772363793999999e-01, 6.4384962494255997e+03 },
    { 1.8000000000000000e-10, 9.6034807462000005e-01, 6.4897765872880000e+03 },
    { 1.5000000000000000e-10, 8.2846675812000004e-01, 7.4775228602159996e+03 },
    { 1.4000000000000001e-10, 3.8277111026499999e+00, 7.2347942562420003e+03 },
    { 1.2999999999999999e-10, 5.7212639217600003e+00, 1.0177257679533601e+04 },
    { 1.2999999999999999e-10, 2.7238420755199999e+00, 1.3365972825148199e+04 },
    { 1.2000000000000000e-10, 5.7557566816900003e+00, 6.6822051744678001e+03 },
    { 1.2000000000000000e-10, 5.3719198432999997e+00, 4.3999943568890003e+03 },
    { 1.2000000000000000e-10, 1.1189835911099999e+00, 5.7295064471490005e+03 },
    { 1.0999999999999999e-10, 3.5193319840599999e+00, 1.0264565884073400e+04 },
    { 1.0999999999999999e-10, 2.2650345530100000e+00, 7.0793738568077997e+03 },
    { 1.0999999999999999e-10, 2.6753158262499999e+00, 2.6724899413598399e+04 },
    { 1.0000000000000000e-10, 2.5104910238699998e+00, 5.3313574437407997e+03 },
    { 7.9999999999999995e-11, 4.8887460912999998e+00, 6.2983283211764001e+03 },
    { 7.0000000000000004e-11, 5.7263749836300004e+00, 7.8756718636242003e+03 },
    { 7.0000000000000004e-11, 3.9058849760399998e+00, 4.9332084403325998e+03 },
    { 3.9999999999999998e-11, 5.2832579160100002e+00, 1.3207029307364999e+04 },
    { 3.9999999999999998e-11, 5.4398967229999998e-02, 9.4682678772569998e+03 },
};

static const VSOP87Term VSOP87A_MARS_X5[21] = {
    { 3.6000000000000000e-09, 5.5472157816600003e+00, 1.0021837280099400e+04 },
    { 2.7400000000000001e-09, 2.7046704209999999e+00, 3.1851920272655998e+03 },
    { 2.7400000000000001e-09, 2.8001490932000000e-01, 3.4960328261340001e+03 },
    { 2.4000000000000000e-09, 5.9796775504999999e+00, 1.3362449706799200e+04 },
    { 1.6000000000000001e-09, 5.8212789585299998e+00, 3.3406124266997999e+03 },
    { 9.2000000000000003e-10, 8.5339101880000004e-02, 1.6703062133498999e+04 },
    { 8.6999999999999999e-10, 3.1415926535900001e+00, 0.0000000000000000e+00 },
    { 7.7000000000000003e-10, 1.6829464835400001e+00, 3.0978838227257902e+03 },
    { 7.7000000000000003e-10, 1.2992277815100000e+00, 3.5833410306738001e+03 },
    { 6.9000000000000000e-10, 5.2110516576099997e+00, 6.6812248533995999e+03 },
    { 4.8999999999999996e-10, 3.0646473155599998e+00, 6.5258044539654002e+03 },
    { 2.7000000000000000e-10, 4.7343416858999998e-01, 2.0043674560198800e+04 },
    { 2.5999999999999998e-10, 5.1737766967200001e+00, 3.8941818295421999e+03 },
    { 2.5999999999999998e-10, 4.0915828755100003e+00, 2.7870430238573999e+03 },
    { 1.2999999999999999e-10, 4.4755146347000002e-01, 6.8366452528338004e+03 },
    { 1.2999999999999999e-10, 1.6608059461000000e+00, 6.9239534573736000e+03 },
    { 7.0000000000000004e-11, 3.4214366932700000e+00, 9.8664168806651996e+03 },
    { 6.0000000000000000e-11, 3.8635391847000000e-01, 6.6847479717486003e+03 },
    { 5.0000000000000002e-11, 4.4671229463400000e+00, 6.1276554505572003e+03 },
    { 5.0000000000000002e-11, 8.4565489791000004e-01, 2.3384286986898602e+04 },
    { 3.9999999999999998e-11, 1.8861046464200000e+00, 6.4384962494255997e+03 },
};

static const VSOP87Term VSOP87A_MARS_Y0[1612] = {
    { 1.5155897627699999e+00, 4.6321220658800000e+00, 3.3406124266997999e+03 },
    { 8.6554811019999997e-02, 0.0000000000000000e+00, 0.0000000000000000e+00 },
    { 7.0645502390000003e-02, 4.9705189290199998e+00, 6.6812248533995999e+03 },
    { 4.9387284799999997e-03, 5.3087780669400004e+00, 1.0021837280099400e+04 },
    { 4.0917421999999998e-04, 5.6469826370299998e+00, 1.3362449706799200e+04 },
    { 2.1036783999999999e-04, 2.3240270955000000e-01, 3.3370893083507999e+03 },
    { 2.1012921000000001e-04, 5.8902277365300000e+00, 3.3441355450487999e+03 },
    { 1.3324176999999999e-04, 5.3425938972399996e+00, 5.2969096509459996e+02 },
    { 1.1370034000000000e-04, 3.2613140880099998e+00, 1.0593819301891999e+03 },
    { 8.3462199999999995e-05, 3.4246470400200000e+00, 2.2812304965106000e+03 },
    { 6.7646999999999998e-05, 2.0459406691200002e+00, 3.3405951730475999e+03 },
    { 6.7646530000000006e-05, 4.0767123006199997e+00, 3.3406296803519999e+03 },
    { 5.4000420000000000e-05, 5.8150779319400003e+00, 2.9424634232916001e+03 },
    { 5.2632680000000000e-05, 7.5811089992000003e-01, 6.1515338883049999e+03 },
    { 4.8495229999999998e-05, 2.8519098755000001e+00, 5.6218429232103999e+03 },
    { 4.8095279999999999e-05, 2.7875310552999999e-01, 3.7387614301079998e+03 },
    { 3.8050730000000000e-05, 5.9423429639899998e+00, 3.9814900340819997e+02 },
    { 3.7242929999999997e-05, 5.9851601332200000e+00, 1.6703062133498999e+04 },
    { 3.6095269999999998e-05, 4.5324448829400001e+00, 7.9629800681639995e+02 },
    { 2.3944899999999998e-05, 5.3467881619100002e+00, 3.1491641605882000e+03 },
    { 2.2510270000000002e-05, 7.6938193891999995e-01, 3.5320606928113998e+03 },
    { 2.2341210000000000e-05, 3.2490911376499998e+00, 8.9624553499102003e+03 },
    { 2.2260890000000002e-05, 1.2398424247000001e-01, 3.3405451163970001e+03 },
    { 2.2260300000000001e-05, 5.9986731628800003e+00, 3.3406797370026002e+03 },
    { 2.1775910000000000e-05, 1.2334436516000000e-01, 6.2830758499914000e+03 },
    { 1.9757690000000002e-05, 5.6949816579000001e-01, 6.6777017350506003e+03 },
    { 1.9564109999999999e-05, 6.2309584355399998e+00, 6.6847479717486003e+03 },
    { 1.6972139999999999e-05, 8.1869636263000001e-01, 2.5443144198834002e+03 },
    { 1.6904390000000000e-05, 1.5833116398500000e+00, 5.8849268465832001e+03 },
    { 1.6283949999999998e-05, 4.9525090688800004e+00, 4.1369104335162001e+03 },
    { 1.5758540000000001e-05, 2.7826683524300000e+00, 5.0921519581158000e+03 },
    { 1.5718799999999999e-05, 3.2767949865000001e+00, 1.5890728952838001e+03 },
    { 1.4697830000000001e-05, 3.0415060688000001e-01, 3.8703033917943999e+03 },
    { 1.4396190000000000e-05, 5.1950595843800000e+00, 2.8109214616052000e+03 },
    { 1.3859460000000001e-05, 3.7343719115799998e+00, 1.7515395314160000e+03 },
    { 1.2818900000000000e-05, 3.1303527568199998e+00, 1.1944470102246000e+03 },
    { 9.5500699999999999e-06, 3.8004405291299999e+00, 4.2659819087599999e+02 },
    { 8.1914900000000004e-06, 3.5878644054000000e+00, 4.3999943568890003e+03 },
    { 7.1920400000000002e-06, 1.0835473505000000e+00, 9.4921463150048003e+03 },
    { 7.0990699999999999e-06, 4.5055612715200004e+00, 1.9144826611159999e+02 },
    { 6.9940700000000001e-06, 5.7173749790999997e+00, 2.1329909543799999e+02 },
    { 6.3062599999999998e-06, 2.3843421727399998e+00, 6.6812075997474003e+03 },
    { 6.3062599999999998e-06, 4.4151138042299998e+00, 6.6812421070518003e+03 },
    { 5.9256300000000000e-06, 2.8221411236799998e+00, 5.4867778431750003e+03 },
    { 5.8140800000000002e-06, 2.3808797611400001e+00, 3.1851920272655998e+03 },
    { 5.5136099999999999e-06, 3.7372081391299998e+00, 3.4960328261340001e+03 },
    { 5.3270199999999997e-06, 1.7262961868200000e+00, 1.5925960136327999e+03 },
    { 4.9173600000000003e-06, 6.0462673906999997e-01, 7.0793738568077997e+03 },
    { 4.1718700000000003e-06, 2.3928885516400000e+00, 3.3396321056316001e+03 },
    { 4.1708300000000004e-06, 3.7297529179400000e+00, 3.3415927477680002e+03 },
    { 4.0317599999999998e-06, 3.4953201486899999e+00, 8.4327643848156004e+03 },
    { 3.9273100000000004e-06, 1.3911077183600000e+00, 2.1461654164751999e+03 },
    { 3.5989399999999998e-06, 4.0107402780000002e-02, 2.0043674560198800e+04 },
    { 3.5224800000000000e-06, 2.3202946595900000e+00, 6.2546266625236003e+03 },
    { 3.5155900000000000e-06, 3.5986069265500000e+00, 1.2303067776610000e+04 },
    { 3.2106400000000001e-06, 3.7146017149999999e-01, 3.5539115221378001e+03 },
    { 3.1350099999999999e-06, 3.5346828687400000e+00, 4.5350594369244000e+03 },
    { 3.1310799999999998e-06, 5.6176620277899998e+00, 3.1273133312618002e+03 },
    { 2.8185499999999999e-06, 4.2150291554000002e-01, 7.1135470007999997e+00 },
    { 2.7115199999999999e-06, 3.7550883198000001e-01, 6.4679257579615996e+03 },
    { 2.6675500000000000e-06, 2.8800620999400000e+00, 4.5624609930212000e+03 },
    { 2.6580700000000000e-06, 4.1365720573400004e+00, 5.0886288397668004e+03 },
    { 2.5925300000000001e-06, 3.2102915510000002e-01, 1.9907450170410000e+03 },
    { 2.4457499999999998e-06, 2.8055607194099998e+00, 2.9140142358237999e+03 },
    { 2.3373999999999998e-06, 4.6548637344999999e-01, 9.6236882766912004e+03 },
    { 2.2302900000000002e-06, 1.9813432860200000e+00, 3.3198370312073998e+03 },
    { 2.2130799999999999e-06, 4.1409471125800001e+00, 3.3613878221922000e+03 },
    { 2.1854200000000000e-06, 5.6874423316499998e+00, 6.4897765872880000e+03 },
    { 2.1262200000000002e-06, 2.2257859456300002e+00, 1.7480164130670000e+03 },
    { 2.0771699999999998e-06, 9.0733951999999995e-01, 1.0018314161750401e+04 },
    { 2.0752600000000000e-06, 4.6238962685000001e-01, 6.6811575430967996e+03 },
    { 2.0752600000000000e-06, 5.3893240000000002e-02, 6.6812921637024001e+03 },
    { 2.0507900000000001e-06, 2.8680735926000001e-01, 1.0025360398448400e+04 },
    { 1.9313999999999998e-06, 3.3608306735800002e+00, 2.1187638603783998e+03 },
    { 1.7836899999999999e-06, 4.1054389820499999e+00, 1.2218485663214001e+03 },
    { 1.7687699999999999e-06, 2.6985464633900000e+00, 2.9577158944766002e+03 },
    { 1.6654500000000001e-06, 1.5986599629500000e+00, 1.0213285546211000e+04 },
    { 1.6458300000000000e-06, 5.2878184859699999e+00, 7.4775228602159996e+03 },
    { 1.5687600000000001e-06, 5.1978660930100000e+00, 2.3888940204492001e+03 },
    { 1.5382400000000000e-06, 2.8750072037900001e+00, 6.3989728631399998e+02 },
    { 1.4848800000000000e-06, 3.3424434131499998e+00, 3.7235089589230001e+03 },
    { 1.4765899999999999e-06, 9.0344856519000005e-01, 7.2109158184941998e+03 },
    { 1.4666600000000000e-06, 2.8439785609900001e+00, 3.3370219980480001e+03 },
    { 1.4651299999999999e-06, 5.5486817952700003e+00, 4.6904798363585996e+03 },
    { 1.4605400000000001e-06, 3.2751524264000000e+00, 3.3442028553516002e+03 },
    { 1.4381000000000000e-06, 3.7899857116000000e+00, 2.7870430238573999e+03 },
    { 1.3200700000000001e-06, 3.4289015575400001e+00, 1.3498674096587999e+03 },
    { 1.2368000000000000e-06, 2.5281122546499999e+00, 7.9030734197210004e+03 },
    { 1.2015600000000000e-06, 1.8381318465700001e+00, 9.2255392732830005e+03 },
    { 1.2011600000000000e-06, 3.0914209186699999e+00, 1.1773376811515400e+04 },
    { 1.1809100000000001e-06, 3.6147149965900001e+00, 4.9296853219836003e+03 },
    { 1.1548900000000001e-06, 1.9556152309099999e+00, 3.3334988796990001e+03 },
    { 1.1469100000000000e-06, 2.2605359489999999e+00, 3.8941818295421999e+03 },
    { 1.1193499999999999e-06, 4.7814849548299998e+00, 3.2055473466643998e+03 },
    { 1.1047100000000000e-06, 1.2194257730300000e+00, 1.5542039943419999e+02 },
    { 1.0659300000000000e-06, 1.3376847666100000e+00, 3.4756775067352000e+03 },
    { 1.0415500000000000e-06, 7.5138682236999998e-01, 4.2923308329503998e+03 },
    { 1.0334100000000000e-06, 8.2609816626999999e-01, 3.8289653222319998e+02 },
    { 9.5749999999999994e-07, 4.2418121975799998e+00, 3.3477259737006002e+03 },
    { 9.1808999999999998e-07, 2.6886626303800001e+00, 9.5952390892233998e+03 },
    { 8.8744000000000001e-07, 1.4159265508200001e+00, 1.2832758741704600e+04 },
    { 8.6744999999999995e-07, 4.7516080177899997e+00, 9.5171840625059997e+02 },
    { 8.5293000000000000e-07, 2.1136277117000000e+00, 4.9332084403325998e+03 },
    { 8.4990000000000001e-07, 5.1193779931399996e+00, 3.0978838227257902e+03 },
    { 8.4318999999999997e-07, 5.6904777331599998e+00, 6.6741113063987996e+03 },
    { 8.0587000000000000e-07, 9.7053595715999996e-01, 3.5833410306738001e+03 },
    { 7.4027000000000005e-07, 4.3185788380299996e+00, 3.7672106175757999e+03 },
    { 7.3605999999999999e-07, 3.9152758213599999e+00, 7.7406067835887998e+03 },
    { 7.2943000000000004e-07, 3.4597423710699999e+00, 1.0522683831884001e+03 },
    { 6.8070999999999996e-07, 6.1177464983399998e+00, 5.5356940284240000e+02 },
    { 6.6771000000000004e-07, 3.8287550598300002e+00, 2.7007151403858002e+03 },
    { 6.6161000000000001e-07, 4.7533228825100000e+00, 1.0021854533751601e+04 },
    { 6.6161000000000001e-07, 2.7225512510200001e+00, 1.0021820026447200e+04 },
    { 6.6066000000000004e-07, 3.2366821620000000e-02, 1.0664954771900000e+03 },
    { 6.4893999999999999e-07, 3.0258474389100001e+00, 6.0413275670856001e+03 },
    { 5.4122999999999998e-07, 2.7211901435599999e+00, 6.5258044539654002e+03 },
    { 5.3310999999999999e-07, 9.3765731928999996e-01, 1.0419986283507600e+04 },
    { 4.9846000000000003e-07, 3.5226287249800001e+00, 3.3121632392319998e+03 },
    { 4.9388999999999998e-07, 3.7266734022999998e-01, 2.2741169495098002e+03 },
    { 4.9142999999999995e-07, 2.6008275239900001e+00, 3.3690616141676001e+03 },
    { 4.7250999999999999e-07, 3.9414727734900001e+00, 1.5643680203309799e+04 },
    { 4.6824999999999997e-07, 2.4832778159100002e+00, 8.4292412664666008e+03 },
    { 4.5414000000000002e-07, 3.2970356892799999e+00, 2.2883440435113998e+03 },
    { 4.4798999999999998e-07, 2.7429927839400001e+00, 3.0775285033270002e+03 },
    { 4.3247000000000001e-07, 3.7556254004000000e-01, 1.4956319713459999e+02 },
    { 4.3006999999999999e-07, 1.1394641180600000e+00, 3.3253599555147998e+03 },
    { 4.2804000000000001e-07, 4.9819351279299999e+00, 3.3558648978848000e+03 },
    { 4.0833000000000000e-07, 2.0372418433499999e+00, 5.3680451209540001e+02 },
    { 4.0036999999999998e-07, 3.3842730473199998e+00, 3.6036963500726001e+03 },
    { 3.9750000000000001e-07, 2.8682297544500002e+00, 4.0327700279266001e+03 },
    { 3.9042999999999998e-07, 2.7321391138600002e+00, 6.6802445323313996e+03 },
    { 3.8910000000000001e-07, 4.0681043732299997e+00, 6.6822051744678001e+03 },
    { 3.8629000000000000e-07, 3.8215663546399998e+00, 8.0310922630584000e+03 },
    { 3.8472999999999998e-07, 7.7890834630000005e-01, 6.8945239488376001e+03 },
    { 3.8416000000000002e-07, 5.5071848202400000e+00, 5.2257741809380002e+02 },
    { 3.6267000000000000e-07, 3.7822711569000000e-01, 2.3384286986898602e+04 },
    { 3.6125999999999999e-07, 7.0225541330999997e-01, 9.8085381846614000e+03 },
    { 3.2887000000000001e-07, 5.2877756254999997e-01, 2.4272860397400001e+02 },
    { 3.2361000000000002e-07, 3.8690208552300001e+00, 7.8756718636242003e+03 },
    { 3.1627000000000001e-07, 3.5415575710399998e+00, 2.6484548254729998e+03 },
    { 3.1454000000000001e-07, 2.6471376687200001e+00, 5.6289564702112002e+03 },
    { 3.1440000000000001e-07, 6.9172824613999995e-01, 5.3313574437407997e+03 },
    { 3.0949000000000000e-07, 5.8314085890999996e+00, 3.5231183490000002e+00 },
    { 2.9995999999999999e-07, 3.5948805795699998e+00, 1.0309277421860000e+02 },
    { 2.9779000000000002e-07, 1.1312684861100000e+00, 7.0641213856227996e+03 },
    { 2.9756000000000003e-07, 3.8477634081400001e+00, 3.1781457905676002e+03 },
    { 2.8657000000000000e-07, 2.3263170516999998e+00, 3.5030790628320001e+03 },
    { 2.7767000000000002e-07, 2.5928481657000000e+00, 7.3733824546264004e+03 },
    { 2.7655999999999998e-07, 8.0781446139000002e-01, 1.2964300703391000e+04 },
    { 2.7614999999999999e-07, 6.0384247127100004e+00, 5.6147293762095996e+03 },
    { 2.7209000000000000e-07, 3.0994591549999999e+00, 1.1243685846420800e+04 },
    { 2.5142000000000001e-07, 4.0416560679600000e+00, 1.6484467571974001e+03 },
    { 2.5137000000000001e-07, 5.0624282500900000e+00, 3.3350895023923999e+03 },
    { 2.5085000000000000e-07, 1.0597839251600001e+00, 3.3461353510071999e+03 },
    { 2.4696999999999999e-07, 4.5117606768199998e+00, 6.9215760122680001e+02 },
    { 2.4414000000000001e-07, 4.9116275309999999e-02, 7.6266071275599998e+01 },
    { 2.4344000000000001e-07, 5.1534865624900004e+00, 7.6329432596502002e+03 },
    { 2.3678000000000001e-07, 2.2204328474100001e+00, 3.3202571073009999e+03 },
    { 2.3510000000000001e-07, 3.9007712617200001e+00, 3.3609677460985899e+03 },
    { 2.3405999999999999e-07, 4.3819320683500003e+00, 1.7586530784168001e+03 },
    { 2.3118000000000001e-07, 2.6740693947800001e+00, 1.6921656695024001e+03 },
    { 2.2994999999999999e-07, 1.2453217450900000e+00, 1.3358926588450200e+04 },
    { 2.2802000000000000e-07, 6.0280341258099996e+00, 9.8303890139878004e+03 },
    { 2.2672000000000001e-07, 6.2541647411000001e-01, 1.3365972825148199e+04 },
    { 2.2359999999999999e-07, 3.2913231715600002e+00, 1.5113989238215199e+04 },
    { 2.1808000000000000e-07, 3.9209935673000001e-01, 1.0021904590402200e+04 },
    { 2.1808000000000000e-07, 8.0059574356999996e-01, 1.0021769969796600e+04 },
    { 2.1387000000000000e-07, 2.3330437944100000e+00, 6.6604494579071998e+03 },
    { 2.0715000000000000e-07, 2.5221642476100001e+00, 1.5819593482830001e+03 },
    { 2.0655999999999999e-07, 4.4772382767699996e+00, 6.7020002488919999e+03 },
    { 2.0613999999999999e-07, 1.5811272146999999e+00, 1.0713994881326200e+04 },
    { 2.0391999999999999e-07, 1.6949722034000001e+00, 1.0404733812322600e+04 },
    { 2.0199000000000000e-07, 3.6371619140600000e+00, 9.3819399937853996e+03 },
    { 1.9931999999999999e-07, 5.8989330809400000e+00, 2.8180350086059998e+03 },
    { 1.9593999999999999e-07, 1.9448862286699999e+00, 8.5319638175199998e+02 },
    { 1.9537999999999999e-07, 5.3642873788800003e+00, 2.7078286873866000e+03 },
    { 1.9245999999999999e-07, 5.6807332580200001e+00, 3.3299757613500001e+03 },
    { 1.9170999999999999e-07, 4.3949269084999998e-01, 3.3512490920496002e+03 },
    { 1.8895999999999999e-07, 1.2478161870000000e-01, 3.8631898447936001e+03 },
    { 1.8713000000000001e-07, 5.4537259275099998e+00, 6.4384962494255997e+03 },
    { 1.8341000000000000e-07, 4.1776107304899996e+00, 1.1371704689758200e+04 },
    { 1.7718000000000000e-07, 5.6252678975299997e+00, 1.0818135286915800e+04 },
    { 1.7629000000000000e-07, 5.6214845253599997e+00, 6.8726731195111997e+03 },
    { 1.7462000000000001e-07, 5.4762681422300004e+00, 3.0740053849780002e+03 },
    { 1.7426000000000000e-07, 3.8910436603999998e+00, 7.4781598567299994e+01 },
    { 1.7151999999999999e-07, 3.6904551226000000e+00, 6.8366452528338004e+03 },
    { 1.6730000000000000e-07, 8.7141658781999998e-01, 3.9733961660129999e+03 },
    { 1.6513000000000000e-07, 5.7398291903999998e-01, 6.1586474353058002e+03 },
    { 1.6273999999999999e-07, 1.3341875424500000e+00, 1.0551528245194000e+04 },
    { 1.6226000000000000e-07, 3.0359029733499998e+00, 6.2983283211764001e+03 },
    { 1.6185999999999999e-07, 6.4022063666999995e-01, 3.6072194684216001e+03 },
    { 1.6142000000000000e-07, 5.5538628839999999e+00, 5.7295064471490005e+03 },
    { 1.6080000000000001e-07, 4.0622232733699999e+00, 6.1444203413041996e+03 },
    { 1.5678999999999999e-07, 2.8920932107800001e+00, 3.3335661900017999e+03 },
    { 1.5187000000000000e-07, 3.7015724021600001e+00, 5.4593762870782002e+03 },
    { 1.5103000000000000e-07, 5.6717403730599996e+00, 1.0973555686350001e+04 },
    { 1.4987000000000000e-07, 3.2037530522800002e+00, 5.0992655051166003e+03 },
    { 1.4941000000000001e-07, 5.7476789533600003e+00, 6.5316616562649997e+03 },
    { 1.4418000000000001e-07, 2.1259774343999999e-01, 7.2347942562420003e+03 },
    { 1.3923000000000001e-07, 3.0376149109999999e+00, 1.2935851515923199e+04 },
    { 1.3864000000000000e-07, 5.1448515129499999e+00, 6.5461597733642002e+03 },
    { 1.3731000000000000e-07, 3.6802949466900001e+00, 4.3371173787679999e+02 },
    { 1.3717000000000000e-07, 4.1370510984399997e+00, 6.1276554505572003e+03 },
    { 1.3696999999999999e-07, 3.1768801329000000e+00, 6.6776344247478000e+03 },
    { 1.3658000000000000e-07, 3.0767464052600002e+00, 3.3476586633978000e+03 },
    { 1.3624999999999999e-07, 7.8468384068999997e-01, 4.1948464387519999e+02 },
    { 1.3602000000000000e-07, 4.0163683473600003e+00, 3.4153940252671000e+03 },
    { 1.3560000000000000e-07, 6.0419391457099998e+00, 5.0850384111149997e+03 },
    { 1.3479000000000000e-07, 3.6119248416700001e+00, 6.6848152820513997e+03 },
    { 1.3476000000000000e-07, 1.5152260589999999e-02, 1.5961864422845999e+03 },
    { 1.3108000000000001e-07, 4.6269227053000000e-01, 3.4437052009183999e+03 },
    { 1.3096999999999999e-07, 4.6413227443700000e+00, 6.6883384004004001e+03 },
    { 1.3089000000000000e-07, 2.3087557907000000e-01, 8.9553418029093991e+03 },
    { 1.2942000000000000e-07, 3.0942851406099998e+00, 8.9695688969109997e+03 },
    { 1.2795000000000000e-07, 7.2051055393999996e-01, 3.3367310913418000e+03 },
    { 1.2778000000000000e-07, 5.4097243772399999e+00, 3.3444937620577998e+03 },
    { 1.2672000000000001e-07, 2.1067378611600001e+00, 3.2658308281324998e+03 },
    { 1.2454999999999999e-07, 4.8528070368199998e+00, 2.4874160449477999e+03 },
    { 1.2440000000000000e-07, 3.4721824249000002e+00, 4.9890591838971995e+03 },
    { 1.2244999999999999e-07, 1.0757249986299999e+00, 1.3506508003540000e+02 },
    { 1.2179999999999999e-07, 3.0734718301299999e+00, 3.2812385647862002e+03 },
    { 1.2157000000000000e-07, 6.0195990552399996e+00, 1.0014723733098601e+04 },
    { 1.2109000000000001e-07, 3.3008426672100000e+00, 3.9805097130138001e+03 },
    { 1.1983999999999999e-07, 7.1488214373000003e-01, 1.7444259844152000e+03 },
    { 1.1920000000000000e-07, 3.0568472581099999e+00, 3.3999862886134001e+03 },
    { 1.1461000000000001e-07, 1.8288951613200000e+00, 3.0242205570432002e+03 },
    { 1.1398000000000001e-07, 5.5826407315399997e+00, 3.2375196524812000e+03 },
    { 1.1119000000000001e-07, 3.9489679144699998e+00, 8.2702977486834006e+03 },
    { 1.1077000000000001e-07, 1.6796313682999999e+00, 6.8162899334350004e+03 },
    { 1.1044000000000000e-07, 1.2428421883000000e+00, 1.1020632121940000e+02 },
    { 1.0886999999999999e-07, 5.6086077450399996e+00, 3.3167339889519999e+03 },
    { 1.0779000000000000e-07, 5.2098721533000003e-01, 3.3644908644476000e+03 },
    { 1.0723000000000000e-07, 1.7481413843200000e+00, 1.6173371168404399e+04 },
    { 1.0586000000000000e-07, 3.7048548451099999e+00, 5.8280284716475999e+03 },
    { 1.0572000000000000e-07, 4.9111944928000001e-01, 2.9211277828246002e+03 },
    { 1.0555000000000000e-07, 7.7207843625999995e-01, 1.2168002696574600e+04 },
    { 1.0404000000000000e-07, 4.2947307148400000e+00, 3.6570042963564001e+03 },
    { 9.8669999999999996e-08, 2.3873121824200001e+00, 1.4584298273120599e+04 },
    { 9.4559999999999994e-08, 1.7821570375800000e+00, 1.1769853693166400e+04 },
    { 9.3460000000000004e-08, 4.4762533512899996e+00, 3.3045845600224002e+03 },
    { 9.3050000000000003e-08, 7.8791668509000001e-01, 1.0575406682941801e+04 },
    { 9.2140000000000003e-08, 1.6543612853700000e+00, 3.3766402933772001e+03 },
    { 9.0270000000000000e-08, 3.3168129990700002e+00, 6.7843176276182003e+03 },
    { 8.9579999999999999e-08, 4.7938419982000002e-01, 3.3401923506061898e+03 },
    { 8.9579999999999999e-08, 5.6508299574700001e+00, 3.3410325027934000e+03 },
    { 8.9179999999999995e-08, 2.4502041945199999e+00, 8.2738208670324002e+03 },
    { 8.3920000000000000e-08, 1.7865442596200001e+00, 1.2566151699982800e+04 },
    { 8.2840000000000006e-08, 6.0742426519099997e+00, 6.6049587821240002e+03 },
    { 8.2189999999999997e-08, 3.1513356698599999e+00, 8.8273902698748007e+03 },
    { 7.7519999999999995e-08, 4.9419507056600001e+00, 1.6246663613219999e+02 },
    { 7.6930000000000001e-08, 5.8446969424799997e+00, 1.8617458526354001e+03 },
    { 7.6389999999999998e-08, 4.2428269461400001e+00, 1.1081219210288600e+04 },
    { 7.6090000000000002e-08, 3.5129031904299999e+00, 3.3040070613956000e+03 },
    { 7.5810000000000002e-08, 2.5732941522599999e+00, 3.1344268782626000e+03 },
    { 7.5580000000000006e-08, 1.8661226341099999e+00, 3.2090704650133998e+03 },
    { 7.5190000000000000e-08, 2.6141261910200000e+00, 3.3772177920039999e+03 },
    { 7.4649999999999997e-08, 1.3026014576700000e+00, 6.9239534573736000e+03 },
    { 7.3599999999999997e-08, 5.0939694018599999e+00, 1.3362466960451400e+04 },
    { 7.3399999999999996e-08, 3.0601173036199998e+00, 1.3362432453146999e+04 },
    { 7.3300000000000001e-08, 5.4497207737500002e+00, 3.7600970705750001e+03 },
    { 7.3020000000000001e-08, 4.8871689850299997e+00, 2.1781377222920000e+03 },
    { 7.2960000000000000e-08, 5.6705014754500001e+00, 6.3278373931320004e+02 },
    { 7.2810000000000001e-08, 3.4968785992900000e+00, 3.5467979751369999e+03 },
    { 6.9399999999999999e-08, 4.2539390722899997e+00, 3.4721543883862000e+03 },
    { 6.9339999999999998e-08, 3.8716388325300000e+00, 8.3296716105970008e+03 },
    { 6.9230000000000005e-08, 2.2326198258600001e+00, 3.1699395560806001e+03 },
    { 6.8550000000000002e-08, 2.2567806033000001e-01, 2.6308392337279997e+02 },
    { 6.8350000000000000e-08, 2.3045693025000000e+00, 6.6064432548323002e+03 },
    { 6.6489999999999995e-08, 6.1291298017300004e+00, 1.3757737998458001e+03 },
    { 6.5489999999999999e-08, 4.3394575993300002e+00, 1.1187557921027999e+03 },
    { 6.5200000000000001e-08, 3.2996910343699999e+00, 3.2968935143948001e+03 },
    { 6.5120000000000003e-08, 3.3762129467499999e+00, 4.4593682188025996e+03 },
    { 6.4930000000000000e-08, 3.8033980986100002e+00, 3.8133035637799999e+01 },
    { 6.4739999999999996e-08, 4.7302616529900003e+00, 7.1078230442756003e+03 },
    { 6.4529999999999996e-08, 3.8923467104500000e+00, 3.5112852973190002e+03 },
    { 6.4490000000000004e-08, 7.6654877575000002e-01, 1.4788665740644001e+03 },
    { 6.4459999999999996e-08, 4.2130455172600003e+00, 3.3410423098265001e+03 },
    { 6.4459999999999996e-08, 1.9107214013800000e+00, 3.3401825435730998e+03 },
    { 6.3749999999999998e-08, 4.1553924502899999e+00, 8.4398779318163997e+03 },
    { 6.3619999999999997e-08, 2.8277248142400002e+00, 3.3843313390048002e+03 },
    { 6.2989999999999997e-08, 6.1405451136700000e+00, 1.4314168113049800e+04 },
    { 6.2919999999999997e-08, 2.8336015469600002e+00, 2.6087903141574199e+04 },
    { 6.2800000000000006e-08, 2.7071514371899998e+00, 2.2218566345969998e+03 },
    { 6.2610000000000003e-08, 1.2700234047600001e+00, 9.5628915597059995e+02 },
    { 6.2590000000000007e-08, 5.9256971500000000e-01, 1.9648386268540000e+03 },
    { 6.2499999999999997e-08, 4.3801653677200001e+00, 1.1624747044077999e+03 },
    { 6.2099999999999994e-08, 1.3495548086700000e+00, 1.3916019109641600e+04 },
    { 6.1099999999999998e-08, 5.4412842024500003e+00, 2.2041264243879999e+02 },
    { 6.0539999999999998e-08, 4.2857398187699998e+00, 1.8984292630009601e+04 },
    { 6.0030000000000002e-08, 1.2747354742899999e+00, 1.3760598710207400e+04 },
    { 5.8740000000000002e-08, 5.4116598132300000e+00, 6.4087760738220004e+02 },
    { 5.7749999999999998e-08, 1.7326609424399999e+00, 1.0184303916231600e+04 },
    { 5.6920000000000003e-08, 3.5921323965499998e+00, 4.5695745400220003e+03 },
    { 5.6160000000000001e-08, 3.0543212613600002e+00, 9.8664168806651996e+03 },
    { 5.5770000000000002e-08, 2.9707198483399999e+00, 6.8436914895317996e+03 },
    { 5.5750000000000000e-08, 2.9308205892100001e+00, 9.4917560896980001e+02 },
    { 5.4130000000000002e-08, 3.0685404443700000e+00, 2.0618554843720000e+02 },
    { 5.3930000000000001e-08, 4.7550469372300004e+00, 1.2289621133221999e+03 },
    { 5.3620000000000000e-08, 1.7026677933000001e-01, 2.7662676283649998e+03 },
    { 5.2590000000000003e-08, 2.8614611348499999e+00, 2.2434479570190001e+02 },
    { 5.1779999999999998e-08, 1.8120267240300001e+00, 3.2741250177853999e+03 },
    { 5.0390000000000003e-08, 2.4409300234100000e+00, 6.2475131155228000e+03 },
    { 5.0060000000000000e-08, 4.3112725904499998e+00, 3.4070998356141999e+03 },
    { 4.8720000000000002e-08, 3.8741517994999999e+00, 4.3928808098882000e+03 },
    { 4.7890000000000001e-08, 3.6397268503000002e-01, 4.4071079038897997e+03 },
    { 4.7220000000000002e-08, 5.4723533323300000e+00, 6.2617402095243997e+03 },
    { 4.7020000000000000e-08, 2.4524893270799999e+00, 2.1116503133776000e+03 },
    { 4.5860000000000002e-08, 2.9363801239299998e+00, 6.7096740408674004e+03 },
    { 4.5260000000000003e-08, 6.0942752559300004e+00, 3.1910492295652002e+03 },
    { 4.4950000000000002e-08, 2.4890248189199999e+00, 1.0177257679533601e+04 },
    { 4.4500000000000001e-08, 8.9168729076999997e-01, 1.3524916342931399e+04 },
    { 4.3900000000000003e-08, 1.0323151127800001e+00, 1.3149150611361199e+04 },
    { 4.3830000000000003e-08, 1.4898075734400000e+00, 6.6659723822145997e+03 },
    { 4.3600000000000000e-08, 1.1523073127300001e+00, 1.0235136375537400e+04 },
    { 4.3140000000000001e-08, 3.2686864956499999e+00, 3.8774169387952002e+03 },
    { 4.3119999999999998e-08, 6.1914447107699999e+00, 2.8073983432562000e+03 },
    { 4.2200000000000001e-08, 5.3528302500000001e-03, 2.6997348193175999e+03 },
    { 4.1070000000000003e-08, 3.0705821121199999e+00, 1.0020856959031200e+04 },
    { 4.0949999999999999e-08, 4.4065936374300003e+00, 1.0022817601167601e+04 },
    { 4.0460000000000000e-08, 2.2058347731999999e+00, 1.4054607308025999e+04 },
    { 4.0439999999999997e-08, 2.8116321496699999e+00, 1.3517870106233400e+04 },
    { 4.0189999999999998e-08, 5.3209215972699999e+00, 6.6964773245846000e+03 },
    { 4.0079999999999999e-08, 3.3586337874500001e+00, 5.7434479833479998e+02 },
    { 4.0070000000000001e-08, 1.9405096418000001e-01, 2.6660704172179999e+02 },
    { 3.9860000000000001e-08, 1.0375744687000001e-01, 6.0484411140864004e+03 },
    { 3.9839999999999998e-08, 5.8400831193700000e+00, 3.9814900340820000e+03 },
    { 3.9610000000000002e-08, 5.3104033515199998e+00, 2.3843232707292000e+03 },
    { 3.9540000000000002e-08, 3.8335112858799998e+00, 6.6527756659318002e+03 },
    { 3.9029999999999999e-08, 5.7451822440700004e+00, 3.9149572250346000e+03 },
    { 3.8220000000000001e-08, 6.4521458655999997e-01, 3.4901756238344001e+03 },
    { 3.7669999999999999e-08, 7.1635417611999996e-01, 2.6724899413598399e+04 },
    { 3.7030000000000001e-08, 4.8025153653099997e+00, 6.5710185321802001e+03 },
    { 3.6799999999999999e-08, 2.9169016767699998e+00, 2.9069006888230001e+03 },
    { 3.5920000000000000e-08, 6.2675274457199999e+00, 3.8738265101433999e+03 },
    { 3.4819999999999997e-08, 4.2010739568900002e+00, 1.1216284290324000e+04 },
    { 3.4440000000000003e-08, 4.7927514936999999e-01, 8.4256508378147992e+03 },
    { 3.4100000000000001e-08, 4.7562462727500003e+00, 1.3936794505133999e+04 },
    { 3.3869999999999998e-08, 3.5814970695600001e+00, 1.8454601664915001e+04 },
    { 3.3589999999999998e-08, 2.2789348068800002e+00, 2.8038079146044001e+03 },
    { 3.3390000000000003e-08, 5.1236085755299996e+00, 1.7277406931833801e+04 },
    { 3.2969999999999997e-08, 1.0276922798799999e+00, 8.6719698704406001e+03 },
    { 3.2730000000000003e-08, 1.6507675883799999e+00, 5.5187501489918004e+03 },
    { 3.2640000000000000e-08, 1.1458686870000001e+00, 1.6304913130090799e+04 },
    { 3.2269999999999997e-08, 2.7607356228200000e+00, 6.4701083331480004e+02 },
    { 3.1090000000000002e-08, 6.1303841736200004e+00, 4.5553474460203997e+03 },
    { 3.0930000000000000e-08, 6.2807718726299999e+00, 5.3054510535538002e+03 },
    { 3.0190000000000001e-08, 5.3535144024700001e+00, 5.2023582793351998e+03 },
    { 3.0099999999999998e-08, 1.8062247208500000e+00, 2.0085575391590000e+03 },
    { 2.9499999999999999e-08, 3.6823692950900000e+00, 6.9443087767724001e+03 },
    { 2.9480000000000000e-08, 5.8660502034900004e+00, 6.4146178116778001e+03 },
    { 2.9049999999999999e-08, 3.2584950532799999e+00, 1.6858482532933202e+04 },
    { 2.9029999999999999e-08, 4.6926040040999997e+00, 6.6123292678099995e+02 },
    { 2.8769999999999999e-08, 4.9007491031099999e+00, 1.3320548875408001e+03 },
    { 2.8609999999999999e-08, 4.1825730840999997e-01, 3.3420968994080999e+03 },
    { 2.8270000000000001e-08, 5.7054397737700002e+00, 3.3391279539914999e+03 },
    { 2.7710000000000001e-08, 6.0909168536499996e+00, 9.8722740829647992e+03 },
    { 2.7470000000000000e-08, 4.8816369394999999e+00, 5.7249356974290004e+03 },
    { 2.7170000000000001e-08, 4.1924142610899997e+00, 6.5187582172674001e+03 },
    { 2.6480000000000000e-08, 2.5194526175199998e+00, 1.5110466119866200e+04 },
    { 2.6449999999999999e-08, 3.8969113606099999e+00, 5.9890672521728002e+03 },
    { 2.6230000000000001e-08, 1.5835507213800000e+00, 1.6699539015149989e+04 },
    { 2.5819999999999999e-08, 9.6409333016999998e-01, 1.6706585251847999e+04 },
    { 2.5460000000000001e-08, 3.5402757886299998e+00, 1.2722552420485201e+04 },
    { 2.5320000000000001e-08, 3.2926078755899999e+00, 6.4750393049623999e+03 },
    { 2.5279999999999999e-08, 8.2719172909999999e-02, 1.3171001440687600e+04 },
    { 2.5230000000000002e-08, 2.0146312483700002e+00, 8.4608283475120004e+02 },
    { 2.5110000000000001e-08, 5.8822208299299996e+00, 5.3321408344359997e+02 },
    { 2.4830000000000001e-08, 2.3017953167699998e+00, 3.3132108706029999e+03 },
    { 2.4560000000000000e-08, 3.8220475472400000e+00, 3.3680139827966000e+03 },
    { 2.4480000000000001e-08, 3.0202827920200002e+00, 6.4181409300267997e+03 },
    { 2.4350000000000000e-08, 4.4131265808000002e+00, 9.4850327680040009e+03 },
    { 2.4319999999999999e-08, 2.4053315076300001e+00, 3.1710322435667999e+03 },
    { 2.4280000000000000e-08, 1.7899151537600000e+00, 1.7256631536341400e+04 },
    { 2.4270000000000002e-08, 3.8892764632999999e-01, 6.0403472460173998e+03 },
    { 2.4229999999999999e-08, 3.1799495824699999e+00, 5.7320492444297997e+03 },
    { 2.4159999999999999e-08, 7.3041207551999998e-01, 1.3362517017102000e+04 },
    { 2.4159999999999999e-08, 1.1389084623600001e+00, 1.3362382396496399e+04 },
    { 2.4050000000000000e-08, 4.8460504678699996e+00, 9.1686408983473993e+03 },
    { 2.4050000000000000e-08, 2.0257003589000000e-01, 3.5610250691386000e+03 },
    { 2.4000000000000000e-08, 4.5775872670600002e+00, 4.6726673142405998e+03 },
    { 2.3980000000000000e-08, 2.5664952221599999e+00, 1.7924910699820401e+04 },
    { 2.3919999999999998e-08, 4.3437440679600003e+00, 1.0596182078434200e+04 },
    { 2.3770000000000000e-08, 3.7570187725199999e+00, 3.5101926098327999e+03 },
    { 2.3660000000000001e-08, 1.0504556484100001e+00, 1.5508615123274400e+04 },
    { 2.3600000000000000e-08, 2.9449499707900002e+00, 1.4712317116458000e+04 },
    { 2.3569999999999999e-08, 6.5442854372000003e-01, 6.4608122109608003e+03 },
    { 2.3520000000000001e-08, 5.4013887722099998e+00, 6.6757019290921999e+03 },
    { 2.2810000000000000e-08, 2.2871228960000001e+00, 6.6430918177617996e+03 },
    { 2.2740000000000000e-08, 1.3954991825900001e+00, 6.6867477777069998e+03 },
    { 2.2729999999999999e-08, 2.6771625453799999e+00, 1.0001061884606999e+04 },
    { 2.2469999999999999e-08, 6.5129974134000002e-01, 3.3433639373107999e+03 },
    { 2.2469999999999999e-08, 5.4724671773000004e+00, 3.3378609160888000e+03 },
    { 2.2220000000000000e-08, 1.2629484376800000e+00, 7.3140085927128002e+03 },
    { 2.2209999999999998e-08, 2.5601424086500000e+00, 6.6608695340007998e+03 },
    { 2.2160000000000001e-08, 9.1630812455999999e-01, 9.4992598620055996e+03 },
    { 2.1990000000000000e-08, 5.3005441951299996e+00, 2.8449187467800002e+01 },
    { 2.1940000000000000e-08, 3.2507896717500002e+00, 2.0775395492400001e+01 },
    { 2.1850000000000000e-08, 4.8167656102200000e+00, 1.0042612675591799e+04 },
    { 2.1670000000000001e-08, 1.1767854689000000e-01, 2.1258774073792001e+03 },
    { 2.1600000000000002e-08, 4.9225426340500000e+00, 9.7791086761253991e+03 },
    { 2.1550000000000001e-08, 3.6877419798300002e+00, 1.7952584437210000e+03 },
    { 2.1390000000000002e-08, 4.0671704810199998e+00, 7.7999806455024000e+03 },
    { 2.1270000000000001e-08, 5.8167571046899997e+00, 3.1201997842609999e+03 },
    { 2.1089999999999999e-08, 5.4415878968999998e-01, 7.2038022714934004e+03 },
    { 2.1010000000000001e-08, 2.6512458922999999e-01, 1.4227094001599999e+01 },
    { 2.0999999999999999e-08, 3.3570755161000001e+00, 3.9296772537080001e+03 },
    { 2.0960000000000000e-08, 4.3326426974199999e+00, 1.3119721102825190e+04 },
    { 2.0720000000000000e-08, 4.6932984159600002e+00, 1.6460333529524989e+04 },
    { 2.0650000000000000e-08, 4.9388965272000002e-01, 1.2295954229609200e+04 },
    { 2.0420000000000000e-08, 3.1110095529700001e+00, 1.5453539829788001e+03 },
    { 2.0400000000000001e-08, 1.0794302363399999e+00, 1.2147350193206000e+03 },
    { 2.0160000000000000e-08, 4.2325373443299998e+00, 6.7015801727983999e+03 },
    { 1.9960000000000001e-08, 1.3333528750000000e-02, 7.9277488846740005e+02 },
    { 1.9799999999999999e-08, 5.9570963182599996e+00, 1.4158747713615599e+04 },
    { 1.9750000000000001e-08, 5.7074924699700000e+00, 8.5429707060349992e+03 },
    { 1.9659999999999999e-08, 3.4596854681700000e+00, 1.2310181323610799e+04 },
    { 1.9329999999999998e-08, 2.1146733520000001e+00, 3.2304061054804001e+03 },
    { 1.8930000000000001e-08, 3.7880807205200000e+00, 7.2555696517344004e+03 },
    { 1.8920000000000000e-08, 5.7985639935000002e+00, 7.9982112516539996e+02 },
    { 1.8780000000000000e-08, 2.8299651224000000e+00, 2.7515475996916002e+03 },
    { 1.8690000000000001e-08, 5.2970255758100002e+00, 3.1131362939107998e+03 },
    { 1.8670000000000001e-08, 1.7565023610999999e+00, 1.3892140671893800e+04 },
    { 1.8620000000000001e-08, 6.0292508736099997e+00, 6.6705881880498000e+03 },
    { 1.8609999999999999e-08, 1.2657364548000001e+00, 2.9459865416406001e+03 },
    { 1.8550000000000001e-08, 4.0013387410000001e-01, 3.4168784979754000e+03 },
    { 1.8550000000000001e-08, 5.1104670489000004e+00, 1.3553897972910800e+04 },
    { 1.8439999999999998e-08, 4.0108905797299998e+00, 1.0390266107903999e+03 },
    { 1.8380000000000000e-08, 3.3844393790899998e+00, 1.6276463942623001e+04 },
    { 1.8050000000000000e-08, 7.7929673675000000e-01, 6.6918615187493997e+03 },
    { 1.7700000000000000e-08, 5.3966097018000001e+00, 3.2350541665740002e+02 },
    { 1.7479999999999998e-08, 3.0001946000099999e+00, 5.0327780962021998e+03 },
    { 1.7210000000000000e-08, 3.2193537174800002e+00, 6.4568800576977001e+03 },
    { 1.7150000000000001e-08, 3.8110298345800002e+00, 4.0398835749273999e+03 },
    { 1.7020000000000000e-08, 5.3249868858000005e-01, 2.6758563815697999e+03 },
    { 1.7020000000000000e-08, 4.2639470325000001e-01, 8.6460634802536006e+03 },
    { 1.7000000000000000e-08, 8.2628421143999997e-01, 3.5680885594888000e+03 },
    { 1.6820000000000001e-08, 6.2284924163499999e+00, 4.7757600884592002e+03 },
    { 1.6720000000000000e-08, 9.8005371506000005e-01, 6.9478318951213996e+03 },
    { 1.6689999999999999e-08, 9.2596345980999994e-01, 1.4351476617594001e+03 },
    { 1.6600000000000000e-08, 3.3748379070899999e+00, 9.6389407478762005e+03 },
    { 1.6510000000000000e-08, 4.3396858889200001e+00, 6.7560064519669004e+03 },
    { 1.6390000000000000e-08, 4.2339228344300004e+00, 2.6793794999187999e+03 },
    { 1.6370000000000000e-08, 5.8926819600300000e+00, 9.0701188738487999e+03 },
    { 1.6000000000000001e-08, 6.1192530384800001e+00, 1.9054647649404001e+03 },
    { 1.5990000000000000e-08, 3.4536815264900000e+00, 3.2125935833623998e+03 },
    { 1.5889999999999999e-08, 2.0018336515000001e+00, 2.4945295919485998e+03 },
    { 1.5819999999999999e-08, 3.5966761967599998e+00, 3.4508187479192002e+03 },
    { 1.5690000000000000e-08, 4.2718737379999999e-01, 1.4844727083000000e+00 },
    { 1.5630000000000002e-08, 3.1695360210399999e+00, 6.6741786167015998e+03 },
    { 1.5559999999999998e-08, 5.0334370593899997e+00, 6.9927114822759995e+02 },
    { 1.5550000000000000e-08, 4.9862802511100002e+00, 1.0028950827100200e+04 },
    { 1.5530000000000001e-08, 2.3286769751700001e+00, 4.1938088084518004e+03 },
    { 1.5519999999999999e-08, 5.4676607635899996e+00, 9.8867722000640006e+03 },
    { 1.5480000000000000e-08, 6.2467856340500001e+00, 6.5781320791810003e+03 },
    { 1.5469999999999999e-08, 3.7359583366800000e+00, 7.4299006053259995e+02 },
    { 1.5469999999999999e-08, 7.0212881340000005e-02, 1.3355336159798400e+04 },
    { 1.5320000000000001e-08, 5.5007524069000002e+00, 4.0053684718298000e+03 },
    { 1.5220000000000000e-08, 2.6717173992099998e+00, 3.4686312700372000e+03 },
    { 1.5160000000000001e-08, 2.8378814937899999e+00, 4.9225717749828000e+03 },
    { 1.5020000000000001e-08, 4.9779696843399996e+00, 3.3787454623376002e+03 },
    { 1.4920000000000001e-08, 2.0233584701799998e+00, 6.1550570066540004e+03 },
    { 1.4839999999999999e-08, 4.2331532991399996e+00, 2.6047359131680000e+03 },
    { 1.4769999999999999e-08, 5.0592323303900004e+00, 6.6475604512999996e+02 },
    { 1.4629999999999999e-08, 4.9843078002399999e+00, 3.3187615973734000e+03 },
    { 1.4629999999999999e-08, 1.1394591184000000e+00, 3.3624632560261998e+03 },
    { 1.4580000000000001e-08, 4.1734566588500002e+00, 1.0156630178842000e+03 },
    { 1.4550000000000000e-08, 3.9572469450800001e+00, 2.6936015933849999e+03 },
    { 1.4540000000000000e-08, 4.0500377951999997e+00, 8.7999887137780006e+03 },
    { 1.4440000000000001e-08, 3.5197899254000000e+00, 1.0018246851447600e+04 },
    { 1.4429999999999999e-08, 5.1225089785400000e+00, 1.9800945956224801e+04 },
    { 1.4400000000000000e-08, 2.3307849514800001e+00, 2.6413412784722000e+03 },
    { 1.4340000000000000e-08, 4.8713909595800002e+00, 2.0750449480734001e+03 },
    { 1.4340000000000000e-08, 4.1661759448400000e+00, 3.2837140517642001e+03 },
    { 1.4300000000000001e-08, 3.9548118755399999e+00, 1.0025427708751200e+04 },
    { 1.4289999999999999e-08, 1.0321897001500000e+00, 1.2995225377836799e+04 },
    { 1.4279999999999999e-08, 4.4798781880899998e+00, 9.4682678772569998e+03 },
    { 1.4219999999999999e-08, 1.7384933914000000e+00, 1.7395219734725801e+04 },
    { 1.4190000000000000e-08, 5.9823323159799999e+00, 3.9321532631000000e+00 },
    { 1.4149999999999999e-08, 2.8155058809400000e+00, 9.5881255422226004e+03 },
    { 1.4110000000000000e-08, 5.7866643829099997e+00, 7.8959598727202001e+03 },
    { 1.4030000000000001e-08, 1.9571514884800001e+00, 3.3975108016354002e+03 },
    { 1.3949999999999999e-08, 4.7305345540200001e+00, 5.8906482700820004e+02 },
    { 1.3710000000000000e-08, 1.3077559896999999e-01, 9.9455712088237997e+03 },
    { 1.3610000000000001e-08, 3.1305455649999998e+00, 6.0342140200847998e+03 },
    { 1.3469999999999999e-08, 2.0622365432400001e+00, 1.9513983595104201e+04 },
    { 1.3300000000000000e-08, 4.1064950440099999e+00, 5.6253660415594004e+03 },
    { 1.3290000000000000e-08, 1.6260081282600001e+00, 3.3472960905739001e+03 },
    { 1.3290000000000000e-08, 4.4977587903799998e+00, 3.3339287628256998e+03 },
    { 1.3250000000000000e-08, 5.1033250441600000e+00, 5.9373861913600003e+01 },
    { 1.3119999999999999e-08, 3.6005587890099999e+00, 3.3111829181637900e+03 },
    { 1.3119999999999999e-08, 2.5232081296300000e+00, 3.3700419352357999e+03 },
    { 1.3110000000000000e-08, 8.2653092367000003e-01, 3.3024793910620001e+03 },
    { 1.3070000000000001e-08, 4.8629498468100003e+00, 1.1909238918756000e+03 },
    { 1.2930000000000001e-08, 3.6706950434499999e+00, 1.0124930054318000e+04 },
    { 1.2860000000000001e-08, 3.0379131124999997e-01, 3.1526872789372101e+03 },
    { 1.2730000000000001e-08, 5.0503187895200004e+00, 6.6487408914400007e+01 },
    { 1.2720000000000001e-08, 4.6863506995300002e+00, 6.9976167230561996e+03 },
    { 1.2690000000000000e-08, 3.6284769918599999e+00, 3.1887151456145998e+03 },
    { 1.2680000000000000e-08, 3.6825683329399999e+00, 6.7405987153132000e+03 },
    { 1.2660000000000001e-08, 4.0447578267699997e+00, 6.8874104018367998e+03 },
    { 1.2600000000000000e-08, 1.6951668129499999e+00, 3.7422845484569998e+03 },
    { 1.2290000000000000e-08, 9.6680072190999999e-01, 3.1456410422392000e+03 },
    { 1.2220000000000000e-08, 2.9711871603300000e+00, 3.1842117061974000e+03 },
    { 1.2180000000000001e-08, 2.0257475145299999e+00, 1.0156902360134800e+04 },
    { 1.2050000000000000e-08, 3.3270760150399998e+00, 6.6882710900975999e+03 },
    { 1.2010000000000000e-08, 6.0984517867700001e+00, 1.6062184526116800e+04 },
    { 1.2000000000000000e-08, 3.9284712911500002e+00, 2.9709126107594002e+03 },
    { 1.1980000000000000e-08, 5.1693260091199997e+00, 9.0655481241287998e+03 },
    { 1.1970000000000001e-08, 4.6658916751500001e+00, 2.2847536148596000e+03 },
    { 1.1930000000000000e-08, 5.7446119387200003e+00, 6.6851061887575997e+03 },
    { 1.1930000000000000e-08, 1.0554035661900001e+00, 6.6773435180416000e+03 },
    { 1.1870000000000000e-08, 4.3441497730299998e+00, 5.8351420186484002e+03 },
    { 1.1850000000000000e-08, 5.7339655550000002e+00, 9.6023526362241992e+03 },
    { 1.1830000000000001e-08, 4.1854431020899998e+00, 1.1670284037296800e+04 },
    { 1.1749999999999999e-08, 1.8979110972300000e+00, 4.0764889402315998e+03 },
    { 1.1710000000000000e-08, 3.6361958735700002e+00, 2.7717905526723998e+03 },
    { 1.1570000000000000e-08, 2.5957150450299999e+00, 4.8859664096786000e+03 },
    { 1.1460000000000000e-08, 3.1530995596300002e+00, 3.4970131472021999e+03 },
    { 1.1409999999999999e-08, 9.7184844334999998e-01, 1.3154196168639999e+02 },
    { 1.1360000000000000e-08, 2.7209358117999999e-01, 1.9402796952816600e+04 },
    { 1.1280000000000001e-08, 3.4092280402499999e+00, 6.6218509914859997e+03 },
    { 1.1269999999999999e-08, 4.0309466631299999e+00, 1.7654780539749601e+04 },
    { 1.1160000000000000e-08, 4.2911012217600000e+00, 1.1610910175383200e+04 },
    { 1.1120000000000000e-08, 4.5077816760199996e+00, 1.5252471184999999e+01 },
    { 1.1090000000000000e-08, 1.5302915236700001e+00, 3.9462588505920002e+02 },
    { 1.1059999999999999e-08, 2.7706807307800001e+00, 2.5976223661672002e+03 },
    { 1.1059999999999999e-08, 6.0947650659999998e-02, 3.2643463554241998e+03 },
    { 1.1040000000000000e-08, 1.7193175836700001e+00, 6.1480107699560003e+03 },
    { 1.1020000000000000e-08, 2.4136929860600000e+00, 6.3648329837430001e+03 },
    { 1.1020000000000000e-08, 3.6933151482500000e+00, 7.3211221397135996e+03 },
    { 1.1010000000000001e-08, 4.2187844484400001e+00, 3.7743241645766002e+03 },
    { 1.0970000000000000e-08, 7.4070558229000005e-01, 1.3745346239022399e+04 },
    { 1.0929999999999999e-08, 4.5387138848799999e+00, 5.6183198048614004e+03 },
    { 1.0859999999999999e-08, 2.7498496766499998e+00, 2.3249494088155998e+03 },
    { 1.0789999999999999e-08, 6.1021915360000001e-01, 9.3890535407862008e+03 },
    { 1.0710000000000000e-08, 4.5461927936800004e+00, 5.1952447323344004e+03 },
    { 1.0680000000000000e-08, 5.6737306167700003e+00, 1.2721572099417001e+04 },
    { 1.0519999999999999e-08, 4.3613352154599996e+00, 1.1979701285736001e+03 },
    { 1.0430000000000000e-08, 2.4968171894200002e+00, 3.4925097077850000e+03 },
    { 1.0390000000000001e-08, 3.5365785784999998e-01, 4.9367988689843996e+03 },
    { 1.0300000000000000e-08, 2.1949688917099999e+00, 3.7103122426402001e+03 },
    { 1.0239999999999999e-08, 6.6014784719999997e-02, 1.1766263264514600e+04 },
    { 1.0130000000000000e-08, 5.4827877893899997e+00, 2.2777073781616000e+03 },
    { 1.0130000000000000e-08, 4.3895423755899996e+00, 8.8593625756916008e+03 },
    { 1.0090000000000000e-08, 4.1546969160099998e+00, 4.1429763491459998e+03 },
    { 1.0090000000000000e-08, 5.9434442955100000e+00, 6.6573464156518003e+03 },
    { 9.9499999999999998e-09, 2.6138866358000001e-01, 1.6335837804536601e+04 },
    { 9.9200000000000005e-09, 6.2083860491099996e+00, 4.1404335518651997e+03 },
    { 9.8999999999999993e-09, 4.3832693568599996e+00, 3.1816689089165998e+03 },
    { 9.8899999999999996e-09, 3.6700532992200001e+00, 2.0199094959632999e+04 },
    { 9.8600000000000003e-09, 1.6920619536299999e+00, 2.0809467624645200e+04 },
    { 9.7700000000000008e-09, 5.7668815962700002e+00, 3.3619480671667998e+03 },
    { 9.7700000000000008e-09, 3.5688532237999998e-01, 3.3192767862328001e+03 },
    { 9.7700000000000008e-09, 2.1231139736300002e+00, 2.5382485042536000e+03 },
    { 9.6899999999999994e-09, 4.5033863233300000e+00, 1.2726810256271999e+03 },
    { 9.6899999999999994e-09, 5.1663314800100002e+00, 8.9589322315612008e+03 },
    { 9.6799999999999997e-09, 4.4322743016299997e+00, 9.3136308685179995e+02 },
    { 9.6699999999999999e-09, 1.9276420253000000e+00, 9.6546129511370000e+03 },
    { 9.6299999999999992e-09, 2.7873443923700001e+00, 1.1614433293732200e+04 },
    { 9.5999999999999999e-09, 2.4850364855699998e+00, 3.9094343007272000e+03 },
    { 9.5700000000000007e-09, 4.8212614506899998e+00, 6.6451969867221997e+03 },
    { 9.4300000000000007e-09, 5.3353755796700000e+00, 3.3441527987009999e+03 },
    { 9.4300000000000007e-09, 3.9299839925600000e+00, 3.3370720546985999e+03 },
    { 9.4300000000000007e-09, 5.9607556240499999e+00, 3.3371065620029999e+03 },
    { 9.4300000000000007e-09, 3.3046039481800000e+00, 3.3441182913966099e+03 },
    { 9.2900000000000008e-09, 2.0767739218900001e+00, 4.0018453534808000e+03 },
    { 9.2500000000000001e-09, 1.8270745447500001e+00, 1.4859801210651999e+03 },
    { 9.1899999999999999e-09, 2.0470122490999998e+00, 2.0679314010725998e+03 },
    { 9.0900000000000007e-09, 2.0683334208300002e+00, 3.3263853326981998e+03 },
    { 9.0200000000000007e-09, 5.1567242652600003e+00, 1.6958018313299999e+02 },
    { 8.9999999999999995e-09, 4.5092452505300002e+00, 8.9659784682591999e+03 },
    { 8.9999999999999995e-09, 1.0021176945700001e+00, 4.0167212175719999e+02 },
    { 8.9700000000000003e-09, 3.7352789134900002e+00, 1.1780490358516199e+04 },
    { 8.9000000000000003e-09, 4.7571215038999997e+00, 6.7477122623140003e+03 },
    { 8.8900000000000005e-09, 3.5597826080799999e+00, 7.9101869667217998e+03 },
    { 8.8800000000000008e-09, 1.4311725215000000e+00, 1.6865528769631201e+04 },
    { 8.8400000000000001e-09, 5.1695220600200003e+00, 3.6605365304199999e+01 },
    { 8.8400000000000001e-09, 1.8731528873700001e+00, 3.2329489027611999e+03 },
    { 8.7899999999999996e-09, 4.0936189869500001e+00, 3.3548395207014000e+03 },
    { 8.7600000000000004e-09, 5.3586613041100000e+00, 2.3914368177299998e+03 },
    { 8.6800000000000006e-09, 2.6564953495600001e+00, 9.9470556815320997e+03 },
    { 8.6399999999999999e-09, 1.9892476698599999e+00, 6.7172527200770000e+03 },
    { 8.5600000000000002e-09, 8.0926398516999998e-01, 6.7051032911474003e+03 },
    { 8.4399999999999998e-09, 3.2123907598699999e+00, 4.3562754445840001e+03 },
    { 8.4100000000000005e-09, 4.2375940579900000e+00, 3.4482759506384000e+03 },
    { 8.3799999999999996e-09, 3.6694787467399999e+00, 3.1162676309979001e+03 },
    { 8.3600000000000001e-09, 1.4161905309500000e+00, 1.7067287061920001e+02 },
    { 8.3400000000000006e-09, 8.1428487363000002e-01, 6.6808047773059998e+03 },
    { 8.3400000000000006e-09, 5.9857306312800000e+00, 6.6816449294931999e+03 },
    { 8.3400000000000006e-09, 2.1190181669000001e+00, 2.4150080051345001e+04 },
    { 8.2499999999999994e-09, 4.2702663934600000e+00, 3.2533042221599999e+03 },
    { 8.2100000000000004e-09, 6.1677153375599998e+00, 3.3584249488178002e+03 },
    { 8.2100000000000004e-09, 6.2392368882599998e+00, 3.3227999045818001e+03 },
    { 8.1899999999999992e-09, 4.9070025392299996e+00, 8.0236392244620004e+02 },
    { 8.1799999999999995e-09, 5.3827992590699996e+00, 9.3809596727171993e+03 },
    { 8.1799999999999995e-09, 4.7866345628099998e+00, 8.0132797409404002e+03 },
    { 8.1400000000000004e-09, 3.3989912340199999e+00, 1.6703044879846799e+04 },
    { 8.1400000000000004e-09, 5.4297628655199999e+00, 1.6703079387151189e+04 },
    { 8.1199999999999993e-09, 1.1367706503299999e+00, 1.1045700263900001e+01 },
    { 8.0499999999999993e-09, 3.6947174873300002e+00, 1.0451548361876000e+03 },
    { 7.9500000000000001e-09, 1.8535445869400000e+00, 3.4279206312396000e+03 },
    { 7.9200000000000008e-09, 5.6745096858600004e+00, 7.1007094972748000e+03 },
    { 7.8999999999999996e-09, 1.6432347912700001e+00, 1.0264565884073400e+04 },
    { 7.8800000000000001e-09, 4.5798360582999997e+00, 1.4421831636988400e+04 },
    { 7.8700000000000003e-09, 3.7797675058600002e+00, 7.3587651353180001e+02 },
    { 7.8700000000000003e-09, 3.7296688312700002e+00, 3.3086401208829998e+03 },
    { 7.8700000000000003e-09, 2.3940980873700002e+00, 3.3725847325166001e+03 },
    { 7.7900000000000006e-09, 1.7172925790700000e+00, 3.4995559444830001e+03 },
    { 7.7900000000000006e-09, 4.6335616761800003e+00, 2.2324905056709398e+04 },
    { 7.7599999999999997e-09, 1.3747299215700000e+00, 6.0199919266185998e+03 },
    { 7.4199999999999996e-09, 3.6875326514400002e+00, 2.5478375382324002e+03 },
    { 7.3499999999999996e-09, 1.1134722835199999e+00, 2.4092493398480001e+03 },
    { 7.3000000000000000e-09, 1.0072774866700001e+00, 6.0164688082696002e+03 },
    { 7.2900000000000003e-09, 6.8449736491000002e-01, 2.2743409379516401e+04 },
    { 7.2500000000000004e-09, 4.6353226491699999e+00, 3.2294257844121998e+03 },
    { 7.2300000000000001e-09, 3.9247575246899999e+00, 3.7352383117590002e+03 },
    { 7.2200000000000003e-09, 2.2334195463400000e+00, 1.5906764126682599e+04 },
    { 7.2000000000000000e-09, 5.5570511850299997e+00, 2.0597243963041201e+04 },
    { 7.2000000000000000e-09, 3.7922741939000000e+00, 1.0629050485381999e+03 },
    { 7.1600000000000001e-09, 1.7298604596600000e+00, 1.9004647949408401e+04 },
    { 7.1399999999999997e-09, 6.2445300373099997e+00, 4.5030871311075998e+03 },
    { 7.1099999999999996e-09, 1.6333263036900001e+00, 5.6421982426092000e+03 },
    { 7.0600000000000000e-09, 2.9487437092200000e+00, 6.7178302187037998e+03 },
    { 7.0399999999999997e-09, 1.0429702098000000e+00, 3.5473131639963999e+03 },
    { 6.9999999999999998e-09, 3.7774630504900002e+00, 3.6132853840513999e+03 },
    { 6.9399999999999996e-09, 1.4884158518399999e+00, 3.4517990689874000e+03 },
    { 6.9399999999999996e-09, 8.7755088752999999e-01, 1.2655674786264001e+03 },
    { 6.9399999999999996e-09, 4.5407250819399998e+00, 4.3718912305000003e+01 },
    { 6.9200000000000001e-09, 2.3688255127600000e+00, 3.0679394693482000e+03 },
    { 6.9100000000000003e-09, 5.8762532115899999e+00, 6.6797403806912998e+03 },
    { 6.8800000000000002e-09, 2.4147749559700000e+00, 2.0426571092422000e+04 },
    { 6.8400000000000004e-09, 2.6026669057300000e+00, 1.4371756141986000e+03 },
    { 6.8299999999999998e-09, 4.4925383820900002e+00, 3.3366802734367002e+03 },
    { 6.8299999999999998e-09, 1.6312285365500001e+00, 3.3445445799629001e+03 },
    { 6.7200000000000000e-09, 5.8340460600900004e+00, 6.6669977593980002e+03 },
    { 6.6599999999999997e-09, 1.2748110893400000e+00, 1.5664035522708589e+04 },
    { 6.6299999999999996e-09, 7.1782437194000004e-01, 2.3015858159093900e+03 },
    { 6.6100000000000001e-09, 3.5166546088300001e+00, 7.2702896804078000e+03 },
    { 6.5199999999999998e-09, 1.5585237155800000e+00, 3.2126608936652001e+03 },
    { 6.5199999999999998e-09, 4.4579946417400000e+00, 4.8597205278960001e+02 },
    { 6.5100000000000001e-09, 7.8545063650000002e-01, 6.3577194367421998e+03 },
    { 6.4899999999999997e-09, 4.6001566751100000e+00, 1.4807872442630000e+02 },
    { 6.4899999999999997e-09, 9.1092905161000004e-01, 1.6413332101966000e+03 },
    { 6.4800000000000000e-09, 1.0781840711499999e+00, 4.1333873151671996e+03 },
    { 6.4400000000000001e-09, 3.7433972537999999e-01, 4.2897880356695996e+03 },
    { 6.3899999999999996e-09, 1.9344938176599999e+00, 5.4085438277723997e+03 },
    { 6.3799999999999999e-09, 3.4887517886300001e+00, 3.8265844794894001e+03 },
    { 6.3700000000000001e-09, 5.5755269639599998e+00, 1.5104766984290001e+02 },
    { 6.3400000000000000e-09, 1.6029819359799999e+00, 1.7101211136907201e+04 },
    { 6.3400000000000000e-09, 2.5692386383599999e+00, 6.5105519827804001e+03 },
    { 6.3300000000000003e-09, 5.0509069854700002e+00, 2.9632388187840002e+03 },
    { 6.3099999999999999e-09, 3.7735154757900000e+00, 3.3295667264358999e+03 },
    { 6.3099999999999999e-09, 2.3502514428499999e+00, 3.3516581269636999e+03 },
    { 6.3099999999999999e-09, 4.8627415689599998e+00, 3.3431552239806001e+03 },
    { 6.2499999999999997e-09, 2.3681432519899999e+00, 5.9382347928669997e+03 },
    { 6.2499999999999997e-09, 5.8523342364800000e+00, 5.2175806283148399e+04 },
    { 6.2099999999999999e-09, 1.4229275692000001e+00, 1.7468855197945399e+04 },
    { 6.2099999999999999e-09, 5.1173741721999999e+00, 1.0448435470975401e+04 },
    { 6.1900000000000003e-09, 4.5632234637200000e+00, 3.4685639597344002e+03 },
    { 6.1900000000000003e-09, 3.3972496727500001e+00, 1.3207029307364999e+04 },
    { 6.1499999999999996e-09, 5.1970504765000003e-01, 1.1695882514085999e+03 },
    { 6.1300000000000001e-09, 2.8166647863800001e+00, 1.8451078546565990e+04 },
    { 6.1300000000000001e-09, 2.3262246913100002e+00, 6.5496828917131998e+03 },
    { 6.1000000000000000e-09, 5.2300768800500004e+00, 9.9116309588799995e+03 },
    { 6.0399999999999998e-09, 5.0441166135100000e+00, 2.3736415492642000e+03 },
    { 6.0399999999999998e-09, 2.7695168056199999e+00, 3.3701042450032000e+03 },
    { 6.0399999999999998e-09, 3.3542501130200000e+00, 3.3111206083963998e+03 },
    { 6.0200000000000003e-09, 4.5511698104000002e+00, 6.6816547365263004e+03 },
    { 6.0200000000000003e-09, 2.6380557815699999e+00, 2.8546403739102002e+03 },
    { 6.0200000000000003e-09, 2.2488456945099999e+00, 6.6807949702729002e+03 },
    { 6.0000000000000000e-09, 1.0784289625000001e+00, 3.3380696294190002e+03 },
    { 6.0000000000000000e-09, 3.1650430743600002e+00, 6.7249437657046001e+03 },
    { 5.9699999999999999e-09, 5.8719714742800004e+00, 2.5407913015344002e+03 },
    { 5.8900000000000001e-09, 3.6595073092999999e+00, 7.3804960016271998e+03 },
    { 5.8900000000000001e-09, 1.3656085084500000e+00, 2.0355319398799999e+01 },
    { 5.8600000000000000e-09, 5.8600343107499997e+00, 3.5285375744623998e+03 },
    { 5.8600000000000000e-09, 2.1920060526400000e+00, 2.2345260376108199e+04 },
    { 5.8399999999999997e-09, 3.1894691337999997e-01, 2.1710241752912002e+03 },
    { 5.8399999999999997e-09, 2.9692426277499999e+00, 8.2265788363783995e+03 },
    { 5.8399999999999997e-09, 4.9946027291000004e+00, 3.1339116894031999e+03 },
    { 5.6100000000000003e-09, 4.7900311097500001e+00, 4.2719755135515998e+03 },
    { 5.5899999999999999e-09, 2.6295548022199999e+00, 6.3140005244371996e+03 },
    { 5.5800000000000002e-09, 6.2065219276899999e+00, 4.0256564809258002e+03 },
    { 5.5700000000000004e-09, 4.0901446029499997e+00, 3.1639186965660002e+02 },
    { 5.5500000000000001e-09, 1.0713603449700000e+00, 3.7179860346156001e+03 },
    { 5.5299999999999997e-09, 2.1219106659200002e+00, 3.4874241132234001e+03 },
    { 5.4599999999999998e-09, 3.9615639490699999e+00, 3.1938007401762002e+03 },
    { 5.4199999999999999e-09, 2.9521103316700001e+00, 1.5961191319817999e+03 },
    { 5.3899999999999998e-09, 5.9509723540000004e-01, 3.1852593375684000e+03 },
    { 5.3599999999999997e-09, 4.9417755183300001e+00, 3.1984337996636000e+03 },
    { 5.3499999999999999e-09, 3.3973319455200000e+00, 9.0726616711295992e+03 },
    { 5.3499999999999999e-09, 1.8232023221500000e+00, 8.6030992875280003e+02 },
    { 5.3300000000000004e-09, 7.2901836769999995e-01, 1.2323423096008801e+04 },
    { 5.3300000000000004e-09, 5.1528562308900003e+00, 3.5355838111603998e+03 },
    { 5.3199999999999998e-09, 9.4811229139999995e-02, 1.0554497769261000e+03 },
    { 5.3100000000000001e-09, 3.2267825356299999e+00, 6.6375059410945996e+03 },
    { 5.3000000000000003e-09, 5.7788638764200000e+00, 1.6992792165031999e+03 },
    { 5.2700000000000002e-09, 3.8288416649300001e+00, 1.1140593072202200e+04 },
    { 5.2700000000000002e-09, 4.4388169495899996e+00, 3.2893320888373901e+03 },
    { 5.2700000000000002e-09, 1.6849499690500001e+00, 3.3918927645622098e+03 },
    { 5.2199999999999998e-09, 2.2697791497000002e+00, 3.3417930693519002e+03 },
    { 5.2199999999999998e-09, 3.8539877689400002e+00, 3.3394317840477001e+03 },
    { 5.2000000000000002e-09, 4.8177826634100001e+00, 3.8287927857099999e+02 },
    { 5.1300000000000003e-09, 5.3587678375800003e+00, 2.9912639426919998e+02 },
    { 5.1099999999999999e-09, 1.2741164688300000e+00, 1.6489763038060999e+04 },
    { 5.0899999999999996e-09, 4.4072665022399997e+00, 6.8127668150860000e+03 },
    { 5.0899999999999996e-09, 4.4731874763499997e+00, 1.7657666254175999e+03 },
    { 5.0600000000000003e-09, 1.1817861779400001e+00, 3.4827910537359999e+03 },
    { 5.0499999999999997e-09, 3.2691007356799999e+00, 1.0633140834523001e+03 },
    { 5.0499999999999997e-09, 5.4167081090300000e+00, 3.4959655158311998e+03 },
    { 5.0199999999999996e-09, 3.0530743101100000e+00, 5.5624690612967997e+03 },
    { 5.0000000000000001e-09, 9.6736492113999994e-01, 4.8194790007641996e+03 },
    { 4.9200000000000004e-09, 3.8674489808899999e+00, 1.1250799393421599e+04 },
    { 4.9099999999999998e-09, 6.1268654384900003e+00, 3.2883521117870000e+03 },
    { 4.9099999999999998e-09, 6.2800867873300001e+00, 3.3928727416125998e+03 },
    { 4.8900000000000003e-09, 1.4962289047099999e+00, 1.3575748802237200e+04 },
    { 4.8699999999999999e-09, 3.8927617531199998e+00, 1.6555603041981999e+03 },
    { 4.8600000000000002e-09, 3.2762048421899999e+00, 1.0050286467567201e+04 },
    { 4.8300000000000001e-09, 3.7726092768899999e+00, 3.3466783423296001e+03 },
    { 4.8300000000000001e-09, 2.3511576417500000e+00, 3.3345465110700002e+03 },
    { 4.8099999999999997e-09, 5.7433539526199997e+00, 6.6851570066627000e+03 },
    { 4.7799999999999996e-09, 4.2026304919899999e+00, 1.1439910691340000e+02 },
    { 4.7799999999999996e-09, 4.0378409778200002e+00, 5.0956750764648004e+03 },
    { 4.7699999999999999e-09, 4.4261119243099998e+00, 4.3796390374902003e+03 },
    { 4.7699999999999999e-09, 2.1505407598800002e+00, 6.6147374444852003e+03 },
    { 4.7600000000000001e-09, 3.1592870545700000e+00, 2.2954575905122001e+03 },
    { 4.7500000000000003e-09, 4.4245966895200004e+00, 2.3878437747800000e+01 },
    { 4.7500000000000003e-09, 4.3059594526199998e+00, 6.8508050365325998e+03 },
    { 4.7500000000000003e-09, 2.9774359074699999e+00, 1.3295120902599999e+03 },
    { 4.7500000000000003e-09, 3.8910729365500001e+00, 2.1795214091614791e+04 },
    { 4.7399999999999998e-09, 1.4710048229000000e-01, 4.2460691233646003e+03 },
    { 4.7300000000000000e-09, 1.6554987582000000e+00, 4.2969015826703999e+03 },
    { 4.6800000000000004e-09, 3.7534237553700001e+00, 4.1866952614510001e+03 },
    { 4.5999999999999998e-09, 2.8729425314100001e+00, 2.2812477501628000e+03 },
    { 4.5999999999999998e-09, 8.4217089992000005e-01, 2.2812132428584000e+03 },
    { 4.5600000000000000e-09, 1.8268405501800000e+00, 1.0006584808914400e+04 },
    { 4.5600000000000000e-09, 3.6364633966100000e+00, 9.3748264467846002e+03 },
    { 4.5500000000000002e-09, 2.7134777363699998e+00, 8.9624726035624008e+03 },
    { 4.5500000000000002e-09, 6.8270610488000005e-01, 8.9624380962579999e+03 },
    { 4.5299999999999999e-09, 2.0213534477200001e+00, 2.9577331481288002e+03 },
    { 4.5200000000000001e-09, 5.0803184513999999e-01, 6.1068800550648002e+03 },
    { 4.5200000000000001e-09, 1.0731302082600001e+00, 4.3075833041353999e+03 },
    { 4.4999999999999998e-09, 9.2683652576999997e-01, 8.1163725151589997e+03 },
    { 4.4800000000000002e-09, 3.4086748951399999e+00, 1.3361469385730999e+04 },
    { 4.4800000000000002e-09, 3.4016816436299999e+00, 2.2772983432474998e+03 },
    { 4.4699999999999997e-09, 4.7447013010100001e+00, 1.3363430027867400e+04 },
    { 4.4400000000000004e-09, 2.8150228175000003e-01, 8.9828106693089994e+03 },
    { 4.4299999999999998e-09, 5.2410717300599998e+00, 1.8546323056346000e+03 },
    { 4.4200000000000000e-09, 4.5808507260999998e-01, 2.2747613278899999e+02 },
    { 4.3999999999999997e-09, 3.1227495982100000e+00, 3.5649572224017002e+03 },
    { 4.3699999999999996e-09, 3.3111846337000000e-01, 2.2851626497737002e+03 },
    { 4.3599999999999998e-09, 1.0259915549900001e+00, 2.3539707386332801e+04 },
    { 4.3400000000000003e-09, 3.2563509976999999e-01, 7.2144389368432003e+03 },
    { 4.3299999999999997e-09, 2.4973342608500002e+00, 3.3761751611683999e+03 },
    { 4.3299999999999997e-09, 3.6264326577900001e+00, 3.3050496922312000e+03 },
    { 4.3200000000000000e-09, 1.2002531362600000e+00, 9.0545669666480001e+02 },
    { 4.2999999999999996e-09, 1.6981325181000001e-01, 4.7163862265456000e+03 },
    { 4.2999999999999996e-09, 2.8354344501000001e-01, 2.6555683724738001e+03 },
    { 4.2999999999999996e-09, 4.0177403397100004e+00, 7.6968878712838005e+03 },
    { 4.2700000000000004e-09, 2.3622313679900002e+00, 1.5748458012822000e+03 },
    { 4.2500000000000000e-09, 6.1654531412100004e+00, 5.8884499649321997e+03 },
    { 4.2500000000000000e-09, 2.8922986632000001e+00, 2.1265523126520198e+04 },
    { 4.2500000000000000e-09, 4.0005854670399996e+00, 7.2180293654950001e+03 },
    { 4.2400000000000002e-09, 3.1309571718000001e-01, 5.8814037282341997e+03 },
    { 4.2299999999999997e-09, 5.6590692084300001e+00, 1.0037089751284400e+04 },
    { 4.2199999999999999e-09, 1.9034701123300000e+00, 5.7533848848968000e+03 },
    { 4.2199999999999999e-09, 4.3891896666600001e+00, 5.2236939198022001e+03 },
    { 4.1499999999999999e-09, 3.5847285913100002e+00, 2.1947111372700001e+04 },
    { 4.1400000000000002e-09, 6.9798224448000001e-01, 7.7477203305896001e+03 },
    { 4.1299999999999996e-09, 4.6218101225200003e+00, 7.4835887758458002e+03 },
    { 4.1000000000000003e-09, 5.0246167001600002e+00, 2.4803024979470001e+03 },
    { 4.0800000000000000e-09, 6.1044982143100004e+00, 1.1883583132734801e+04 },
    { 4.0700000000000002e-09, 1.2258355324700001e+00, 1.8849227549974199e+04 },
    { 4.0400000000000001e-09, 3.6556298526400002e+00, 6.6446194880953999e+03 },
    { 4.0300000000000004e-09, 3.0371517087700002e+00, 1.8606498946000200e+04 },
    { 4.0199999999999998e-09, 2.2199197504000001e-01, 6.1515511419572003e+03 },
    { 4.0199999999999998e-09, 4.4744056507199996e+00, 6.1515166346528003e+03 },
    { 4.0199999999999998e-09, 2.7413474063800001e+00, 1.6850521225016000e+03 },
    { 3.9700000000000001e-09, 5.5562776025599998e+00, 2.3141558382924599e+04 },
    { 3.9499999999999998e-09, 5.7152567330400004e+00, 5.1546387109299997e+02 },
    { 3.9499999999999998e-09, 4.1677657015799996e+00, 9.9933880926315996e+03 },
    { 3.9499999999999998e-09, 1.7535868115000000e+00, 3.3371566186536002e+03 },
    { 3.9400000000000000e-09, 1.7224799893000001e-01, 4.9819456368964002e+03 },
    { 3.9400000000000000e-09, 3.5401444971600000e+00, 9.7974924843975004e+03 },
    { 3.9199999999999997e-09, 4.1333000597299998e+00, 3.7234917052708001e+03 },
    { 3.9199999999999997e-09, 1.4812745345300000e+00, 1.9645525556790599e+04 },
    { 3.8899999999999996e-09, 1.0544761268800000e+00, 3.0065511840298201e+04 },
    { 3.8899999999999996e-09, 4.5293758317500004e+00, 1.8052929543157799e+04 },
    { 3.8799999999999998e-09, 4.5356306404600000e+00, 8.5643063465020005e+03 },
    { 3.8799999999999998e-09, 2.2083485547100001e+00, 2.0618019358533598e+04 },
    { 3.8400000000000000e-09, 4.3016640220199998e+00, 7.7334932365880004e+03 },
    { 3.8300000000000002e-09, 3.2864023088000001e+00, 2.9216880277992000e+03 },
    { 3.8199999999999996e-09, 8.3080012584999996e-01, 3.0171070100423999e+03 },
    { 3.8199999999999996e-09, 1.1799112295200000e+00, 3.2673153008407999e+03 },
    { 3.8199999999999996e-09, 4.9438556891200003e+00, 3.4139095525588000e+03 },
    { 3.7900000000000004e-09, 5.7782231695700004e+00, 7.3662689076256001e+03 },
    { 3.7900000000000004e-09, 2.2796586685600002e+00, 1.4610540519464000e+03 },
    { 3.7799999999999998e-09, 1.0765085821000000e+00, 9.8014246376606006e+03 },
    { 3.7600000000000003e-09, 1.9792582798500000e+00, 2.0735832161425591e+04 },
    { 3.6600000000000002e-09, 3.3368635395699999e+00, 9.1257024366559995e+02 },
    { 3.6399999999999998e-09, 1.9370376341000001e-01, 1.5806146839442001e+04 },
    { 3.6399999999999998e-09, 7.1572042054999996e-01, 6.6827093261078999e+03 },
    { 3.6300000000000001e-09, 2.5325287877700000e+00, 2.5685872802808000e+04 },
    { 3.6100000000000001e-09, 5.0090696418400000e+00, 2.7682140744156401e+04 },
    { 3.6100000000000001e-09, 1.5011656473199999e+00, 5.7498617665477996e+03 },
    { 3.6100000000000001e-09, 1.4525341061799999e+00, 6.8504405422599996e+02 },
    { 3.6000000000000000e-09, 3.4299021192499999e+00, 1.6063164847185000e+04 },
    { 3.5499999999999999e-09, 2.4266715655100000e+00, 5.6360700172119996e+03 },
    { 3.5499999999999999e-09, 6.1724218893199998e+00, 7.3221024607817999e+03 },
    { 3.5400000000000002e-09, 6.4235154390000004e-01, 4.6061799053262002e+03 },
    { 3.5199999999999998e-09, 2.6151223846299998e+00, 1.2508301963008000e+03 },
    { 3.5100000000000001e-09, 4.1357883262400001e+00, 6.8518977240187996e+03 },
    { 3.5100000000000001e-09, 2.8046306691499998e+00, 5.4522627400773999e+03 },
    { 3.5100000000000001e-09, 1.2561529818000000e-01, 1.3212886509664600e+04 },
    { 3.4400000000000001e-09, 5.3714477853100000e+00, 1.6894510399610601e+04 },
    { 3.4400000000000001e-09, 1.6236119440999999e-01, 5.6882187402739999e+02 },
    { 3.4200000000000002e-09, 1.0539651983100000e+00, 1.4128242771245599e+04 },
    { 3.3999999999999998e-09, 3.8385613938700001e+00, 1.2509253325047201e+04 },
    { 3.3900000000000001e-09, 4.5306501074399996e+00, 1.8830814931023999e+03 },
    { 3.3900000000000001e-09, 4.5112919920900003e+00, 1.4556896717023799e+04 },
    { 3.3900000000000001e-09, 2.5931753018999998e-01, 3.8560762977927998e+03 },
    { 3.3900000000000001e-09, 5.8644493884499997e+00, 2.8251485556068001e+03 },
    { 3.3600000000000000e-09, 2.9323771482200001e+00, 9.8032106819999998e-01 },
    { 3.3499999999999998e-09, 8.0197784093000002e-01, 1.5636566656309000e+04 },
    { 3.3499999999999998e-09, 2.7536965897200001e+00, 3.0608259223474001e+03 },
    { 3.3299999999999999e-09, 1.8307345408400000e+00, 5.4391805909619995e+02 },
    { 3.3000000000000002e-09, 6.1849202623700004e+00, 1.0736090241908000e+03 },
    { 3.2799999999999998e-09, 6.1505725327700000e+00, 2.4351557300350000e+03 },
    { 3.2799999999999998e-09, 2.0391080019599999e+00, 3.3974156888696002e+03 },
    { 3.2799999999999998e-09, 4.0846589166799996e+00, 3.2838091645300001e+03 },
    { 3.2700000000000001e-09, 4.0488043676499998e+00, 3.3538541990977901e+03 },
    { 3.2700000000000001e-09, 2.0749625509900000e+00, 3.3273706543018002e+03 },
    { 3.2700000000000001e-09, 4.4019740643700001e+00, 1.4681168652360000e+02 },
    { 3.2599999999999999e-09, 4.2051783497099997e+00, 3.7078367556622002e+03 },
    { 3.2500000000000002e-09, 5.3615247176700001e+00, 5.3362311835770004e+02 },
    { 3.2200000000000000e-09, 4.8370641905899996e+00, 1.7085958665722199e+04 },
    { 3.2099999999999999e-09, 3.9004570181999998e-01, 2.2670034025089999e+03 },
    { 3.2099999999999999e-09, 2.8288560533700000e+00, 3.7595368256003999e+03 },
    { 3.2000000000000001e-09, 4.0448504401400003e+00, 2.5287723799399799e+04 },
    { 3.2000000000000001e-09, 5.0439157027699997e+00, 1.1904918773201800e+04 },
    { 3.1899999999999999e-09, 3.5912243669800001e+00, 9.8156517316621994e+03 },
    { 3.1700000000000000e-09, 2.4042359862299998e+00, 2.9733880977374001e+03 },
    { 3.1599999999999998e-09, 5.5914942438899997e+00, 3.4401820825693999e+03 },
    { 3.1599999999999998e-09, 5.3227267474999995e-01, 3.2410427708302000e+03 },
    { 3.1599999999999998e-09, 2.3282899549199998e+00, 5.6218601768626004e+03 },
    { 3.1599999999999998e-09, 2.9751832342000001e-01, 5.6218256695582004e+03 },
    { 3.1599999999999998e-09, 1.0116087791399999e+00, 5.4903009615239998e+03 },
    { 3.1599999999999998e-09, 2.1847219516199998e+00, 5.2575881183150000e+02 },
    { 3.1399999999999999e-09, 2.5990377796300002e+00, 1.5265886519300400e+04 },
    { 3.1399999999999999e-09, 2.6477470005599999e+00, 9.9837042444616000e+03 },
    { 3.1300000000000002e-09, 1.3039120825899999e+00, 6.2865989683403996e+03 },
    { 3.1200000000000000e-09, 4.2232801454000004e+00, 4.4035174752379999e+03 },
    { 3.1099999999999998e-09, 5.9959801683900000e+00, 3.3914448860056000e+03 },
    { 3.1099999999999998e-09, 1.2778675025000000e-01, 3.2897799673939999e+03 },
    { 3.1000000000000000e-09, 5.6429719206600000e+00, 6.4537487206105998e+03 },
    { 3.1000000000000000e-09, 6.1972171326000003e-01, 2.0608178540718000e+03 },
    { 3.0699999999999999e-09, 6.2229580210500002e+00, 9.7552302383776005e+03 },
    { 3.0699999999999999e-09, 2.0001319282100001e+00, 8.7491562544722001e+03 },
    { 3.0600000000000002e-09, 4.4769732758099998e+00, 8.7562698014729995e+03 },
    { 3.0600000000000002e-09, 1.3598690925300001e+00, 1.2012582297140400e+04 },
    { 3.0399999999999998e-09, 3.3721918505200001e+00, 3.6203989310522002e+03 },
    { 3.0199999999999999e-09, 4.9199641003299996e+00, 1.2825645194703800e+04 },
    { 3.0100000000000002e-09, 3.6384373925400002e+00, 9.5979227217800002e+01 },
    { 3.0100000000000002e-09, 2.3527717678500002e+00, 9.4956694333537998e+03 },
    { 3.0000000000000000e-09, 2.2172528779300000e+00, 1.9872218986920000e+03 },
    { 3.0000000000000000e-09, 3.9041959437800000e+00, 5.8209149246467996e+03 },
    { 2.9899999999999998e-09, 4.0593043636599999e+00, 1.0284921203472200e+04 },
    { 2.9800000000000000e-09, 8.4851226310000000e-02, 1.1236572299420000e+04 },
    { 2.9600000000000001e-09, 3.0613639074900001e+00, 1.8792514776260001e+02 },
    { 2.9400000000000002e-09, 1.9217343951800001e+00, 2.0040151441849790e+04 },
    { 2.9400000000000002e-09, 1.4412955860000001e+00, 5.4832547248259998e+03 },
    { 2.9199999999999998e-09, 5.6538985124799996e+00, 3.0501269472302001e+03 },
    { 2.9100000000000001e-09, 9.0595258506999998e-01, 1.1986675906953400e+04 },
    { 2.9100000000000001e-09, 4.0326659183400002e+00, 3.6722432896240002e+02 },
    { 2.8999999999999999e-09, 1.3020104409600000e+00, 2.0047197678547800e+04 },
    { 2.8900000000000002e-09, 6.5812694050999998e-01, 6.9016374958384004e+03 },
    { 2.8699999999999998e-09, 2.3882415280600000e+00, 1.9497138446060001e+02 },
    { 2.8699999999999998e-09, 1.0595522458800000e+00, 1.2465534412742199e+04 },
    { 2.8600000000000000e-09, 4.9037644102800000e+00, 1.8983512179396000e+03 },
    { 2.8499999999999999e-09, 4.1090402367400003e+00, 2.7978650435240002e+02 },
    { 2.8499999999999999e-09, 5.6726195027000004e-01, 2.0670073729660001e+02 },
    { 2.8299999999999999e-09, 3.0109950348700001e+00, 5.2440492392010001e+03 },
    { 2.8299999999999999e-09, 1.2124742570900000e+00, 1.2839872288705399e+04 },
    { 2.8299999999999999e-09, 4.5319727353600001e+00, 9.8593706439672005e+03 },
    { 2.8200000000000002e-09, 3.5294083336400002e+00, 5.1063790521173996e+03 },
    { 2.8100000000000000e-09, 3.8833632029100000e+00, 5.9617837400899998e+02 },
    { 2.7999999999999998e-09, 4.2006444847000002e-01, 1.6511613867387401e+04 },
    { 2.7999999999999998e-09, 5.3712619737800003e+00, 5.4014302807716003e+03 },
    { 2.7600000000000000e-09, 4.9656919249699998e+00, 6.6701791531357003e+03 },
    { 2.7400000000000001e-09, 4.7933656497600001e+00, 1.2306590894958999e+04 },
    { 2.7200000000000001e-09, 4.7250644687600003e+00, 1.0706881334325401e+04 },
    { 2.7200000000000001e-09, 3.5170844435299999e+00, 2.4280421830342002e+03 },
    { 2.7099999999999999e-09, 2.2764063317000000e-01, 1.3686602528450001e+03 },
    { 2.7099999999999999e-09, 3.8470823226999999e+00, 3.3360416769797998e+03 },
    { 2.7099999999999999e-09, 2.2766845959399999e+00, 3.3451831764198000e+03 },
    { 2.6900000000000000e-09, 2.8894421446299998e+00, 2.7444340526907999e+03 },
    { 2.6900000000000000e-09, 1.1146846301100000e+00, 3.6027866677399999e+01 },
    { 2.6900000000000000e-09, 1.8386795899299999e+00, 3.8667802734453999e+03 },
    { 2.6799999999999998e-09, 5.1956076638699997e+00, 2.1496885348241999e+03 },
    { 2.6799999999999998e-09, 1.4770323230500000e+00, 1.6702994823196201e+04 },
    { 2.6799999999999998e-09, 1.0685359362000000e+00, 1.6703129443801801e+04 },
    { 2.6700000000000001e-09, 9.6036007447000005e-01, 6.8307880505342000e+03 },
    { 2.6700000000000001e-09, 5.7480985002300002e+00, 5.4156573747732000e+03 },
    { 2.6500000000000002e-09, 2.9061812858699998e+00, 3.0536500655792001e+03 },
    { 2.6299999999999998e-09, 5.3233017234200002e+00, 4.5315363185754004e+03 },
    { 2.6200000000000001e-09, 4.6948508418000001e-01, 3.6310979061694002e+03 },
    { 2.6099999999999999e-09, 1.2343530136900001e+00, 2.0206141196330998e+04 },
    { 2.5800000000000002e-09, 2.6884630015000002e+00, 9.4886231966558007e+03 },
    { 2.5700000000000000e-09, 3.5665793980000000e+00, 5.4303946570988001e+03 },
    { 2.5599999999999998e-09, 6.0544671779200003e+00, 1.5189728108520001e+02 },
    { 2.5500000000000001e-09, 3.1721503128499999e+00, 1.2928737968922400e+04 },
    { 2.5500000000000001e-09, 4.8735400686599997e+00, 2.1548962369291799e+04 },
    { 2.5500000000000001e-09, 2.8239307252399999e+00, 2.6482170809624400e+04 },
    { 2.5500000000000001e-09, 1.5671174334200000e+00, 1.7276610936682000e+03 },
    { 2.5500000000000001e-09, 1.5535083098300000e+00, 1.9942681353900000e+03 },
    { 2.5500000000000001e-09, 5.2890845290000001e+00, 2.9424806769438001e+03 },
    { 2.5500000000000001e-09, 3.2583128975100002e+00, 2.9424461696394001e+03 },
    { 2.5200000000000000e-09, 3.0399710002400000e+00, 3.3821632176846001e+03 },
    { 2.5200000000000000e-09, 3.0837959183999999e+00, 3.2990616357150002e+03 },
    { 2.5099999999999998e-09, 2.1551566530800002e+00, 3.3375093844444000e+03 },
    { 2.5099999999999998e-09, 3.9686102655600002e+00, 3.3437154689551999e+03 },
    { 2.5099999999999998e-09, 3.3713707429599999e+00, 9.7587533567266000e+03 },
    { 2.5099999999999998e-09, 1.5432370406200000e+00, 1.9034368125011999e+03 },
    { 2.5099999999999998e-09, 1.2300198308000000e-01, 1.2797945726280000e+03 },
    { 2.5099999999999998e-09, 5.0202133658800001e+00, 2.7905661422063999e+03 },
    { 2.5000000000000001e-09, 1.3361485613999999e-01, 2.1426422981261999e+03 },
    { 2.5000000000000001e-09, 9.5280868139999997e-01, 9.6983318634419993e+03 },
    { 2.5000000000000001e-09, 5.4333316399799996e+00, 2.4889574795991601e+04 },
    { 2.4899999999999999e-09, 2.5832846261800002e+00, 3.0846420503278000e+03 },
    { 2.4699999999999999e-09, 5.7395080802000003e+00, 1.0016314355791999e+04 },
    { 2.4600000000000002e-09, 3.8983011241800001e+00, 1.5650793750310600e+04 },
    { 2.4399999999999998e-09, 2.2153090792399999e+00, 5.3491699658588004e+03 },
    { 2.4399999999999998e-09, 5.8536202707199996e+00, 2.8144445799542000e+03 },
    { 2.4300000000000001e-09, 1.9151721882299999e+00, 1.4422612087601999e+03 },
    { 2.4300000000000001e-09, 4.2364379460399997e+00, 9.3296796788726006e+03 },
    { 2.4199999999999999e-09, 2.1316158556299998e+00, 1.1925274092600601e+04 },
    { 2.4000000000000000e-09, 1.7339769363499999e+00, 1.0027360204406799e+04 },
    { 2.4000000000000000e-09, 2.7849135877400002e+00, 5.0851057214177999e+03 },
    { 2.3899999999999998e-09, 1.6388953326600000e+00, 3.3440682347460001e+03 },
    { 2.3699999999999999e-09, 3.2175907591400001e+00, 3.6275747878203902e+03 },
    { 2.3499999999999999e-09, 2.1033064932099999e+00, 5.9453483398678000e+03 },
    { 2.3400000000000002e-09, 6.1162133739999999e-02, 1.7499360140315399e+04 },
    { 2.3300000000000000e-09, 3.3995018339300000e+00, 4.9961727308979998e+03 },
    { 2.3300000000000000e-09, 5.9067693285600003e+00, 1.9146759266141798e+04 },
    { 2.3300000000000000e-09, 3.1750779021900000e+00, 5.3517127631395997e+03 },
    { 2.3199999999999998e-09, 3.4644520787899999e+00, 3.1639186965660001e+03 },
    { 2.3199999999999998e-09, 2.6593148398499999e+00, 3.5173061568336002e+03 },
    { 2.3100000000000001e-09, 2.6398409763699999e+00, 6.6538232973027998e+03 },
    { 2.3100000000000001e-09, 2.5206604270200001e+00, 9.0939973115966004e+03 },
    { 2.3100000000000001e-09, 5.4046848565000003e+00, 3.1022753170856198e+04 },
    { 2.2999999999999999e-09, 1.6259097452000000e+00, 1.0654621019412600e+04 },
    { 2.2800000000000000e-09, 2.1391973733800000e+00, 3.6969981594040001e+02 },
    { 2.2699999999999998e-09, 5.7165970201700000e+00, 1.8688593996361999e+03 },
    { 2.2699999999999998e-09, 2.5960689698000001e+00, 9.2788472195668000e+03 },
    { 2.2699999999999998e-09, 6.0358043399300003e+00, 5.6257750764735001e+03 },
    { 2.2600000000000001e-09, 3.5405085473700000e+00, 3.5965828030717998e+03 },
    { 2.2499999999999999e-09, 2.6134404462399998e+00, 2.9389403049426001e+03 },
    { 2.2499999999999999e-09, 4.6702409181600002e+00, 2.4341528317456599e+04 },
    { 2.2499999999999999e-09, 1.9084938760800001e+00, 2.3958631785233400e+04 },
    { 2.2400000000000001e-09, 4.8220691557100004e+00, 5.5229243074000003e+00 },
    { 2.2400000000000001e-09, 6.2225434388099998e+00, 3.5428658218739001e+03 },
    { 2.2400000000000001e-09, 6.1844087870099997e+00, 3.1383590315257002e+03 },
    { 2.2400000000000001e-09, 2.1154064278400000e+00, 7.7750543983900002e+01 },
    { 2.2400000000000001e-09, 1.5242734782800000e+00, 6.5331461289732997e+03 },
    { 2.2300000000000000e-09, 3.1351694867400002e+00, 3.3362503903100001e+03 },
    { 2.2300000000000000e-09, 2.9885974319000002e+00, 3.3449744630895998e+03 },
    { 2.2300000000000000e-09, 3.4944239276000000e-01, 3.5904286517999999e+00 },
    { 2.2300000000000000e-09, 3.0129026641399999e+00, 1.3341674311306801e+04 },
    { 2.2200000000000002e-09, 1.2046055506200000e+00, 1.8353246176068001e+03 },
    { 2.2200000000000002e-09, 6.0072143354399996e+00, 3.0704149563261999e+03 },
    { 2.2100000000000000e-09, 9.4687593506000001e-01, 3.6498907493555998e+03 },
    { 2.1999999999999998e-09, 2.7451154181100001e+00, 6.5116446702665999e+03 },
    { 2.1999999999999998e-09, 3.1971131422500001e+00, 3.9876232600146000e+03 },
    { 2.1999999999999998e-09, 5.4828486823000000e-01, 6.5301771835566997e+03 },
    { 2.1900000000000001e-09, 1.5646093297999999e+00, 8.5846616659007996e+03 },
    { 2.1900000000000001e-09, 4.3056731143400002e+00, 3.0704822666290002e+03 },
    { 2.1799999999999999e-09, 5.8878197440899998e+00, 6.2830931036436004e+03 },
    { 2.1799999999999999e-09, 3.8570481125999998e+00, 6.2830585963392004e+03 },
    { 2.1499999999999998e-09, 5.1553289311199997e+00, 1.3383225102291601e+04 },
    { 2.1499999999999998e-09, 5.1631354406399996e+00, 5.5116366019910001e+03 },
    { 2.1499999999999998e-09, 2.8675671473900000e+00, 5.6179107699472997e+03 },
    { 2.1400000000000001e-09, 5.2474500390500003e+00, 6.0484974070479996e+02 },
    { 2.1400000000000001e-09, 2.3769099224099999e+00, 1.3858952763361999e+03 },
    { 2.1200000000000001e-09, 1.4538044968000000e-01, 2.3924171387982001e+03 },
    { 2.1200000000000001e-09, 4.8387951396600002e+00, 4.5385825552734004e+03 },
    { 2.1100000000000000e-09, 7.4742043485999998e-01, 1.9676450231236398e+04 },
    { 2.1100000000000000e-09, 5.0414428527400004e+00, 1.2199975002391400e+04 },
    { 2.1100000000000000e-09, 1.1514886939700000e+00, 3.8906587111931899e+03 },
    { 2.1000000000000002e-09, 5.8105914704300003e+00, 6.6784733427886003e+03 },
    { 2.1000000000000002e-09, 6.2515379302699996e+00, 1.4849324896942001e+03 },
    { 2.1000000000000002e-09, 9.8942403448000005e-01, 6.6839763640106003e+03 },
    { 2.0799999999999998e-09, 2.6658944340800002e+00, 4.2531826703653996e+03 },
    { 2.0799999999999998e-09, 1.2337487348300000e+00, 1.0721108428326999e+04 },
    { 2.0700000000000001e-09, 2.8982503606200001e+00, 1.0001481960700610e+04 },
    { 2.0599999999999999e-09, 1.8011705855200000e+00, 7.0828969751568002e+03 },
    { 2.0500000000000002e-09, 6.7759896422999999e-01, 1.0593646765369999e+03 },
    { 2.0500000000000002e-09, 2.7083705957199999e+00, 1.0593991838413999e+03 },
    { 2.0400000000000000e-09, 7.6733899171999997e-01, 6.3820984591304004e+03 },
    { 2.0299999999999998e-09, 3.1667662317000000e+00, 4.4664817658033999e+03 },
    { 2.0099999999999999e-09, 5.3553317605900004e+00, 9.3606043533184002e+03 },
    { 2.0099999999999999e-09, 3.9916401617299999e+00, 3.7387441764557998e+03 },
    { 2.0099999999999999e-09, 6.0224117932199999e+00, 3.7387786837601998e+03 },
    { 2.0000000000000001e-09, 6.2628908264999996e+00, 7.5866815500643997e+03 },
    { 2.0000000000000001e-09, 1.1649489032000000e-01, 3.6108098970734000e+03 },
    { 1.9900000000000000e-09, 3.2613701961600001e+00, 3.0297716278314001e+03 },
    { 1.9900000000000000e-09, 3.9472164462400001e+00, 2.0897822303990001e+03 },
    { 1.9800000000000002e-09, 3.9213115295000001e+00, 3.0943607043767902e+03 },
    { 1.9800000000000002e-09, 4.4388161080400002e+00, 1.8208349942592002e+04 },
    { 1.9700000000000000e-09, 1.1540923220999999e+00, 3.4365916539176001e+03 },
    { 1.9700000000000000e-09, 4.9696745965400000e+00, 3.2446331994819998e+03 },
    { 1.9700000000000000e-09, 2.0677207241000001e-01, 1.5106875691214400e+04 },
    { 1.9700000000000000e-09, 5.7502659435799997e+00, 8.1661573430937997e+03 },
    { 1.9599999999999998e-09, 1.8216404114700000e+00, 3.6107425867706002e+03 },
    { 1.9300000000000002e-09, 2.4322365832799999e+00, 7.4171013669313998e+03 },
    { 1.9300000000000002e-09, 4.9338226399499998e+00, 3.0313341040440000e+03 },
    { 1.9200000000000000e-09, 3.4840779983700001e+00, 1.5121102785216000e+04 },
    { 1.9200000000000000e-09, 2.0079917095000002e+00, 3.3157536678838001e+03 },
    { 1.9200000000000000e-09, 4.1157752091399997e+00, 3.3654711855157998e+03 },
    { 1.9099999999999998e-09, 5.5052293913300003e+00, 3.7529835235741998e+03 },
    { 1.9099999999999998e-09, 6.1853752731000000e-01, 2.9282413298254000e+03 },
    { 1.9099999999999998e-09, 1.0785653242100000e+00, 1.1731986822019999e+02 },
    { 1.9099999999999998e-09, 8.8273497404000001e-01, 1.0544414698193201e+04 },
    { 1.9099999999999998e-09, 3.6133201567600000e+00, 1.1567191263078201e+04 },
    { 1.9099999999999998e-09, 6.2441184948999995e-01, 3.2020242283153998e+03 },
    { 1.9099999999999998e-09, 5.4993550691499999e+00, 3.4792006250842001e+03 },
    { 1.8899999999999999e-09, 5.1351023446899999e+00, 1.1353892167640201e+04 },
    { 1.8899999999999999e-09, 5.7729131670599996e+00, 2.7835199055083999e+03 },
    { 1.8899999999999999e-09, 4.5706656768300000e+00, 1.0042192599498199e+04 },
    { 1.8899999999999999e-09, 8.2918253552999999e-01, 2.7832038219283200e+04 },
    { 1.8800000000000001e-09, 1.8939361921000000e-01, 3.3633809233092002e+03 },
    { 1.8800000000000001e-09, 5.9343732994299998e+00, 3.3178439300904001e+03 },
    { 1.8800000000000001e-09, 4.0235506847000002e-01, 2.7357626859950001e+03 },
    { 1.8800000000000001e-09, 3.9295858694299999e+00, 2.5748193622534000e+03 },
    { 1.8800000000000001e-09, 8.7790891192999998e-01, 2.3853709021002001e+03 },
    { 1.8800000000000001e-09, 1.1031100158700000e+00, 1.7444932947180000e+03 },
    { 1.8700000000000000e-09, 1.0474683879900000e+00, 2.6194731954936001e+03 },
    { 1.8500000000000000e-09, 5.0506006776200003e+00, 4.0617516579060002e+03 },
    { 1.8500000000000000e-09, 3.0711455178299998e+00, 3.9367908007087999e+03 },
    { 1.8500000000000000e-09, 3.8071563992000002e-01, 4.4522546718018002e+03 },
    { 1.8400000000000001e-09, 6.1155365962000001e+00, 1.2942965062924000e+04 },
    { 1.8400000000000001e-09, 3.9151811779500001e+00, 5.2201708014531996e+03 },
    { 1.8300000000000001e-09, 1.2527465376400000e+00, 4.8788528626777997e+03 },
    { 1.8300000000000001e-09, 5.0891524598700002e+00, 6.7914311746189996e+03 },
    { 1.8199999999999999e-09, 1.3312797427000000e+00, 5.9600856221934000e+03 },
    { 1.8199999999999999e-09, 4.8132711024399999e+00, 8.5358571590341999e+03 },
    { 1.8000000000000000e-09, 5.0309394882899996e+00, 1.7550626497650001e+03 },
    { 1.8000000000000000e-09, 5.3000811524499998e+00, 3.3325185586307998e+03 },
    { 1.8000000000000000e-09, 8.2368576619000000e-01, 3.3487062947688000e+03 },
    { 1.7900000000000000e-09, 2.2023536169200000e+00, 3.5868641490228001e+03 },
    { 1.7900000000000000e-09, 2.1436765949200001e+00, 1.7232753098593599e+04 },
    { 1.7900000000000000e-09, 5.0921452971500001e+00, 2.7149422343873998e+03 },
    { 1.7900000000000000e-09, 1.0316216214900000e+00, 3.9662826190122000e+03 },
    { 1.7700000000000001e-09, 2.8622676044799999e+00, 3.6514532255682002e+03 },
    { 1.7700000000000001e-09, 5.3166614297499999e+00, 2.5727636073511999e+03 },
    { 1.7700000000000001e-09, 3.0024238711200000e+00, 2.9784912899689998e+03 },
    { 1.7599999999999999e-09, 3.4801079014499998e+00, 6.3429821544165998e+03 },
    { 1.7599999999999999e-09, 3.9035346199099998e+00, 1.3465542481017799e+04 },
    { 1.7500000000000000e-09, 5.6963558748700001e+00, 1.2299544658261000e+04 },
    { 1.7500000000000000e-09, 4.1328163870400001e+00, 6.7086264094963999e+03 },
    { 1.7500000000000000e-09, 5.9162546746800002e+00, 2.7576135153214000e+03 },
    { 1.7400000000000000e-09, 4.5581441310299997e+00, 1.0096618878666701e+04 },
    { 1.7400000000000000e-09, 4.8366516567100000e+00, 3.1557625187295998e+03 },
    { 1.7400000000000000e-09, 5.5658276896000003e-01, 1.0787630344545800e+04 },
    { 1.7300000000000000e-09, 5.8978947134900004e+00, 2.8230187222691391e+04 },
    { 1.7200000000000001e-09, 3.8251563777699999e+00, 1.9617076369322789e+04 },
    { 1.7200000000000001e-09, 1.3673341075600001e+00, 1.0867834862860000e+03 },
    { 1.7100000000000001e-09, 7.6571313540000005e-02, 1.0011200614749599e+04 },
    { 1.7100000000000001e-09, 4.5833900755499997e+00, 2.1393541969857601e+04 },
    { 1.6999999999999999e-09, 4.4547726378599997e+00, 2.8628336226099600e+04 },
    { 1.6999999999999999e-09, 5.9780428968199999e+00, 4.8255449163940002e+03 },
    { 1.6999999999999999e-09, 4.1712937499100002e+00, 6.6776844813983998e+03 },
    { 1.6999999999999999e-09, 6.2274796436799997e+00, 5.6076158292088003e+03 },
    { 1.6999999999999999e-09, 6.2020653813999997e+00, 6.6777189887027898e+03 },
    { 1.6900000000000000e-09, 3.9068144018100002e+00, 4.6940029547076001e+03 },
    { 1.6900000000000000e-09, 3.5473361492000000e+00, 6.6847307180964099e+03 },
    { 1.6900000000000000e-09, 5.5781077806999999e+00, 6.6847652254007999e+03 },
    { 1.6900000000000000e-09, 2.6354444683000000e-01, 1.1456984941858800e+04 },
    { 1.6900000000000000e-09, 4.6223270073900000e+00, 7.0641041319706001e+03 },
    { 1.6800000000000000e-09, 1.2483220248700000e+00, 1.1116422451020001e+03 },
    { 1.6800000000000000e-09, 4.3144821098000002e-01, 6.1657609823066005e+03 },
    { 1.6800000000000000e-09, 1.1174168253600001e+00, 1.0032473945449199e+04 },
    { 1.6800000000000000e-09, 5.3823482074299998e+00, 2.7490692478044799e+04 },
    { 1.6700000000000000e-09, 5.1394488626200001e+00, 1.5894351778320001e+02 },
    { 1.6700000000000000e-09, 4.2456512338000002e-01, 2.4491425792583399e+04 },
    { 1.6700000000000000e-09, 1.3210186418600001e+00, 1.0288444321821200e+04 },
    { 1.6700000000000000e-09, 5.4982070370500002e+00, 5.8132911893219998e+03 },
    { 1.6700000000000000e-09, 3.7112695842700001e+00, 8.3367851575978002e+03 },
    { 1.6600000000000001e-09, 5.4860411147899999e+00, 6.7193578890374001e+03 },
    { 1.6600000000000001e-09, 4.6106393944000001e-01, 5.4664898340789996e+03 },
    { 1.6600000000000001e-09, 1.1646429443499999e+00, 6.9087009861885999e+03 },
    { 1.6600000000000001e-09, 3.8582324278200000e+00, 1.0766352393859999e+02 },
    { 1.6600000000000001e-09, 3.5655637274699998e+00, 6.5532060100622002e+03 },
    { 1.6500000000000001e-09, 1.8945890775500001e+00, 6.5668257464861999e+03 },
    { 1.6500000000000001e-09, 4.2827361677999998e-01, 1.0636665349799999e+01 },
    { 1.6500000000000001e-09, 5.8517995045899998e+00, 7.3459808985296004e+03 },
    { 1.6500000000000001e-09, 3.9998751701900002e+00, 8.0061661939395999e+03 },
    { 1.6399999999999999e-09, 4.6068710166600004e+00, 1.1553611574070001e+03 },
    { 1.6300000000000000e-09, 1.4934835314000000e-01, 3.3458116178656001e+03 },
    { 1.6300000000000000e-09, 5.9744185654999997e+00, 3.3354132355339998e+03 },
    { 1.6300000000000000e-09, 4.7065867852999999e-01, 1.3286183635523599e+04 },
    { 1.6300000000000000e-09, 6.1329032379399999e+00, 2.9648466951296000e+03 },
    { 1.6200000000000000e-09, 4.0163918126999998e-01, 1.6695948586498191e+04 },
    { 1.6200000000000000e-09, 5.1400510647399997e+00, 2.7267295735160002e+02 },
    { 1.6200000000000000e-09, 5.9665923905299998e+00, 8.8522490286907996e+03 },
    { 1.6100000000000000e-09, 4.1487404102999997e-01, 3.8977049478911999e+03 },
    { 1.6100000000000000e-09, 3.7573197330900001e+00, 4.5766880870227997e+03 },
    { 1.6000000000000001e-09, 3.8984795006400002e+00, 3.2873044804159999e+03 },
    { 1.6000000000000001e-09, 2.2252874180000002e+00, 3.3939203729835999e+03 },
    { 1.6000000000000001e-09, 3.3596683761800001e+00, 1.0081211142013000e+04 },
    { 1.5900000000000001e-09, 3.6710189409599998e+00, 1.4481205498902000e+04 },
    { 1.5900000000000001e-09, 1.3811751479000001e+00, 3.3464696289993999e+03 },
    { 1.5900000000000001e-09, 4.7425917707399998e+00, 3.3347552244001999e+03 },
    { 1.5900000000000001e-09, 3.2347103512800000e+00, 8.9585231966471001e+03 },
    { 1.5900000000000001e-09, 1.6407854844000000e-01, 8.9663875031733005e+03 },
    { 1.5799999999999999e-09, 3.1718264022899998e+00, 6.0921600263913997e+03 },
    { 1.5600000000000000e-09, 5.0178445268500003e+00, 8.4362875031645999e+03 },
    { 1.5500000000000000e-09, 4.3738906333900003e+00, 4.6869567180095901e+03 },
    { 1.5500000000000000e-09, 4.1130098606400001e+00, 2.5135826518314600e+04 },
    { 1.5400000000000001e-09, 3.9839534355700001e+00, 8.7710070837986004e+03 },
    { 1.5400000000000001e-09, 2.0901613669799999e+00, 1.3463442913097999e+03 },
    { 1.5300000000000001e-09, 2.2099888780999999e-01, 7.4470179178460003e+03 },
    { 1.5300000000000001e-09, 5.7486685860300000e+00, 9.1539036160218002e+03 },
    { 1.5300000000000001e-09, 3.1213274978199999e+00, 3.7027335634306000e+03 },
    { 1.5300000000000001e-09, 3.5072433555000000e+00, 4.9225580189140797e+04 },
    { 1.5199999999999999e-09, 2.3497109086600001e+00, 2.0111003364398000e+03 },
    { 1.5199999999999999e-09, 3.5218587640200001e+00, 3.3406469340041999e+03 },
    { 1.5199999999999999e-09, 5.7435008082100003e+00, 3.3405779193953999e+03 },
    { 1.5100000000000000e-09, 4.4599086960800003e+00, 1.5010896463996600e+04 },
    { 1.5100000000000000e-09, 4.3310606861999998e+00, 1.0228022828536599e+04 },
    { 1.5100000000000000e-09, 1.2587671405000000e+00, 3.5254623346700000e+03 },
    { 1.5100000000000000e-09, 9.1764249974000001e-01, 1.8592030553546001e+03 },
    { 1.5100000000000000e-09, 4.6349641877399996e+00, 8.9625226602130006e+03 },
    { 1.5100000000000000e-09, 4.7949004616800002e+00, 2.2812978068133998e+03 },
    { 1.5100000000000000e-09, 5.2033968485299997e+00, 2.2811631862078002e+03 },
    { 1.5100000000000000e-09, 5.0434605745800001e+00, 8.9623880396074001e+03 },
    { 1.5000000000000000e-09, 3.6319515906299999e+00, 2.9221081038928000e+03 },
    { 1.4900000000000000e-09, 3.2471999674000002e-01, 3.6641178433571999e+03 },
    { 1.4900000000000000e-09, 5.6434045188999997e+00, 1.2406160550828599e+04 },
    { 1.4800000000000001e-09, 2.4061236499300001e+00, 7.0509246693400000e+03 },
    { 1.4800000000000001e-09, 2.4476990140599999e+00, 1.5052878090929901e+03 },
    { 1.4700000000000001e-09, 6.1292233753000003e-01, 1.8983339642874000e+03 },
    { 1.4700000000000001e-09, 3.0529245684000000e-01, 9.9187445058808007e+03 },
    { 1.4599999999999999e-09, 5.3336477823399999e+00, 1.3369563253800001e+04 },
    { 1.4599999999999999e-09, 3.4180400613000000e-01, 2.7149872905026000e+03 },
    { 1.4599999999999999e-09, 4.5894612196200004e+00, 2.9424634232916000e+04 },
    { 1.4599999999999999e-09, 1.7263761242400000e+00, 3.2827337306959998e+03 },
    { 1.4599999999999999e-09, 4.3973907943999997e+00, 3.3984911227036000e+03 },
    { 1.4500000000000000e-09, 5.8271519767799997e+00, 1.4577184726119800e+04 },
    { 1.4500000000000000e-09, 3.3353171626800000e+00, 8.3733905229020002e+03 },
    { 1.4400000000000000e-09, 1.4891951690700000e+00, 8.3225580635961996e+03 },
    { 1.4400000000000000e-09, 5.6740758054400002e+00, 3.9454621674046002e+03 },
    { 1.4400000000000000e-09, 5.5635472730700002e+00, 1.8023719907218001e+03 },
    { 1.4400000000000000e-09, 4.2861851517799998e+00, 8.4469914788171991e+03 },
    { 1.4300000000000000e-09, 5.8140498178500000e+00, 1.3227384626763800e+04 },
    { 1.4300000000000000e-09, 4.6681910217500002e+00, 5.2983375065529999e+03 },
    { 1.4300000000000000e-09, 3.2904601382599998e+00, 1.0014791043401399e+04 },
    { 1.4300000000000000e-09, 5.8118874143000001e+00, 2.2289701815978001e+03 },
    { 1.4200000000000001e-09, 2.2869132186400001e+00, 1.8795583747533999e+03 },
    { 1.4100000000000001e-09, 2.2669390862499998e+00, 6.2795527316424004e+03 },
    { 1.4100000000000001e-09, 1.2259157056100001e+00, 3.0414860324306001e+03 },
    { 1.4100000000000001e-09, 2.6133351681499999e+00, 5.0882198048526998e+03 },
    { 1.4100000000000001e-09, 6.2764956990999998e+00, 3.7163781582699999e+03 },
    { 1.4100000000000001e-09, 5.9800800273799997e+00, 5.0960841113789002e+03 },
    { 1.3999999999999999e-09, 3.2923396093499999e+00, 1.8461715211915791e+04 },
    { 1.3999999999999999e-09, 6.0792171061299998e+00, 2.2334736210149999e+03 },
    { 1.3999999999999999e-09, 4.0010661191799999e+00, 1.4867737515892190e+04 },
    { 1.3900000000000000e-09, 1.2899525670600001e+00, 4.7828736354599996e+03 },
    { 1.3800000000000000e-09, 4.0240665796800004e+00, 7.1671969061891996e+03 },
    { 1.3800000000000000e-09, 3.2400773523200002e+00, 3.4803105662225998e+03 },
    { 1.3800000000000000e-09, 2.8836895663200002e+00, 3.2009142871770000e+03 },
    { 1.3700000000000000e-09, 1.1083187810599999e+00, 1.5382404359780001e+03 },
    { 1.3700000000000000e-09, 1.4775834115299999e+00, 6.7030756827260002e+03 },
    { 1.3700000000000000e-09, 7.3335617282999999e-01, 1.2729665967486000e+04 },
    { 1.3700000000000000e-09, 1.9841170938199999e+00, 3.4554813863667601e+04 },
    { 1.3700000000000000e-09, 4.2953511095600003e+00, 6.1373067943034002e+03 },
    { 1.3700000000000000e-09, 3.8901960173000001e+00, 1.4580775154771600e+04 },
    { 1.3700000000000000e-09, 3.2202209665599999e+00, 1.4587821391469601e+04 },
    { 1.3700000000000000e-09, 1.5905605769300000e+00, 8.2194652893776001e+03 },
    { 1.3600000000000001e-09, 4.9490412232400001e+00, 4.6655537672398004e+03 },
    { 1.3600000000000001e-09, 5.6892316606100000e+00, 4.2888077146014002e+03 },
    { 1.3600000000000001e-09, 4.0221396174999997e+00, 8.6958483081884006e+03 },
    { 1.3600000000000001e-09, 5.8699964921900003e+00, 7.6579306444639997e+02 },
    { 1.3399999999999999e-09, 4.6113282531499999e+00, 2.9541927761276002e+03 },
    { 1.3399999999999999e-09, 5.3066227699899997e+00, 6.6593740240732004e+03 },
    { 1.3300000000000000e-09, 5.7206966315600001e+00, 4.2958539512994003e+03 },
    { 1.3300000000000000e-09, 4.5045265047900003e+00, 6.6243264784639996e+03 },
    { 1.3300000000000000e-09, 6.2309935659600004e+00, 2.1150813365883601e+04 },
    { 1.3300000000000000e-09, 2.0063412038000000e-01, 3.9236113380781999e+03 },
    { 1.3300000000000000e-09, 7.5283219590999995e-01, 6.1476017350418997e+03 },
    { 1.3300000000000000e-09, 2.2954890001199999e+00, 6.7381232283352001e+03 },
    { 1.3200000000000000e-09, 4.6970783014200004e+00, 1.2171525814923600e+04 },
    { 1.3200000000000000e-09, 4.3802594273800004e+00, 1.0824201202545601e+04 },
    { 1.3200000000000000e-09, 5.8698504193899996e+00, 2.4726787626221999e+03 },
    { 1.3000000000000001e-09, 2.7842391594300002e+00, 5.2967371144239996e+02 },
    { 1.3000000000000001e-09, 3.0879513860199999e+00, 7.5937950970652000e+03 },
    { 1.3000000000000001e-09, 4.8150107909199997e+00, 5.2970821874679996e+02 },
    { 1.3000000000000001e-09, 8.0714408433999996e-01, 4.1084612460484004e+03 },
    { 1.3000000000000001e-09, 5.5382127661800000e+00, 1.4267906403464000e+04 },
    { 1.3000000000000001e-09, 3.9443024509900000e+00, 6.1554660415681001e+03 },
    { 1.3000000000000001e-09, 1.1036879022399999e+00, 2.6084021806216198e+04 },
    { 1.2900000000000001e-09, 3.3754415915200000e+00, 1.1527125089192399e+04 },
    { 1.2900000000000001e-09, 3.0957657423399998e+00, 3.1014069410748002e+03 },
    { 1.2900000000000001e-09, 3.0280011763000001e+00, 3.5798179123248001e+03 },
    { 1.2900000000000001e-09, 2.1943766372300000e+00, 3.7199858405739901e+03 },
    { 1.2900000000000001e-09, 3.9293902814099999e+00, 2.9612390128256002e+03 },
    { 1.2900000000000001e-09, 2.9391055067799998e+00, 4.5914426230006002e+03 },
    { 1.2799999999999999e-09, 5.3781394766800004e+00, 3.6001732317236001e+03 },
    { 1.2799999999999999e-09, 7.4562744195999997e-01, 3.0810516216760002e+03 },
    { 1.2700000000000000e-09, 3.6747024047500001e+00, 1.0294510237451001e+04 },
    { 1.2700000000000000e-09, 2.4918265040300001e+00, 3.7591167495067998e+03 },
    { 1.2700000000000000e-09, 4.3874754908400000e+00, 2.6555010621709998e+03 },
    { 1.2600000000000000e-09, 2.8235988406000001e-01, 1.8447488117914188e+04 },
    { 1.2600000000000000e-09, 4.1515026751899997e+00, 4.0836024872324001e+03 },
    { 1.2500000000000000e-09, 1.8427682273299999e+00, 3.0227978476430399e+04 },
    { 1.2400000000000001e-09, 4.8172949884099996e+00, 1.0177190369230801e+04 },
    { 1.2400000000000001e-09, 1.9641324213900000e+00, 6.6879085172737005e+03 },
    { 1.2400000000000001e-09, 2.2797009956299998e+00, 9.1249219860424000e+03 },
    { 1.2400000000000001e-09, 4.3401574547999999e-01, 3.5320779464635998e+03 },
    { 1.2400000000000001e-09, 4.6864294211599997e+00, 3.5320434391591998e+03 },
    { 1.2300000000000001e-09, 2.8613324227699999e+00, 6.7106543619355998e+03 },
    { 1.2300000000000001e-09, 2.9798479285899999e+00, 1.4591411820121401e+04 },
    { 1.2300000000000001e-09, 4.3660051624499996e+00, 2.1852512692927999e+03 },
    { 1.2199999999999999e-09, 5.7943168002299998e+00, 2.2573520587628000e+03 },
    { 1.2199999999999999e-09, 4.8541934551999999e+00, 1.2360756603230000e+03 },
    { 1.2199999999999999e-09, 6.0285103249300001e+00, 6.8365779425310002e+03 },
    { 1.2199999999999999e-09, 4.6618053391899998e+00, 4.9535637597313998e+03 },
    { 1.2100000000000000e-09, 4.7230094010499997e+00, 3.0907079659576002e+03 },
    { 1.2100000000000000e-09, 1.4007575175900000e+00, 3.5905168874420001e+03 },
    { 1.2100000000000000e-09, 2.8828671494200000e+00, 4.1795817144501998e+03 },
    { 1.2100000000000000e-09, 1.1211279286300000e+00, 2.7676617819849002e+04 },
    { 1.2100000000000000e-09, 2.6387176207800001e+00, 3.0774501642574800e+04 },
    { 1.2000000000000000e-09, 4.1401258891000001e+00, 4.8459002357928002e+03 },
    { 1.2000000000000000e-09, 6.1055623313199998e+00, 3.2020949298290998e+03 },
    { 1.2000000000000000e-09, 1.8204587320000000e-02, 3.4791299235705001e+03 },
    { 1.1900000000000000e-09, 8.8310894307999999e-01, 1.4717530270636000e+03 },
    { 1.1800000000000001e-09, 2.6743915721599998e+00, 7.5344212351515998e+03 },
    { 1.1800000000000001e-09, 5.3917456356000004e+00, 3.3435813721164000e+03 },
    { 1.1800000000000001e-09, 7.3202128304000003e-01, 3.3376434812831999e+03 },
    { 1.1800000000000001e-09, 5.0452469057900000e+00, 2.5239591004846002e+03 },
    { 1.1800000000000001e-09, 4.0768477877000002e+00, 3.3969956127760001e+03 },
    { 1.1800000000000001e-09, 2.0469191309400001e+00, 3.2842292406236002e+03 },
    { 1.1599999999999999e-09, 5.0226237179300002e+00, 1.0338229149756000e+04 },
    { 1.1599999999999999e-09, 1.5122608933099999e+00, 3.7270320772720002e+03 },
    { 1.1599999999999999e-09, 2.9835664556400001e+00, 8.9766824439118009e+03 },
    { 1.1599999999999999e-09, 5.4906868113899998e+00, 1.4694504594340000e+04 },
    { 1.1500000000000000e-09, 2.2387316496900000e+00, 2.7433889215874991e+04 },
    { 1.1500000000000000e-09, 3.1390222876900000e+00, 2.5016431389494001e+03 },
    { 1.1500000000000000e-09, 4.6704065118200004e+00, 2.6316226244231998e+03 },
    { 1.1500000000000000e-09, 4.2520331432000003e+00, 2.1000915890756802e+04 },
    { 1.1500000000000000e-09, 3.7171451781300000e+00, 1.9719521738548001e+03 },
    { 1.1400000000000000e-09, 5.7819882967499998e+00, 3.9662375628969999e+03 },
    { 1.1400000000000000e-09, 4.2379837703199996e+00, 1.3517802795930600e+04 },
    { 1.1400000000000000e-09, 1.2865925805400000e+00, 3.0635774329584001e+03 },
    { 1.1400000000000000e-09, 4.8371743380999996e+00, 3.6176474204412002e+03 },
    { 1.1300000000000000e-09, 5.6798486404100004e+00, 4.7828908891122001e+03 },
    { 1.1300000000000000e-09, 3.4396439612599998e+00, 9.4819528790159995e+02 },
    { 1.1200000000000001e-09, 1.6458310604299999e+00, 4.1064054911461999e+03 },
    { 1.1200000000000001e-09, 1.2681966608699999e+00, 2.7401556096800000e+01 },
    { 1.1200000000000001e-09, 1.3374324904599999e+00, 2.0995392966449399e+04 },
    { 1.1200000000000001e-09, 3.1230497305200000e+00, 2.4606135553220000e+04 },
    { 1.1200000000000001e-09, 1.6739135104300000e+00, 1.4900690142777201e+04 },
    { 1.1200000000000001e-09, 3.7486722005100002e+00, 5.2953295770634004e+03 },
    { 1.1100000000000001e-09, 3.4909295575499999e+00, 6.8376255739019998e+03 },
    { 1.1100000000000001e-09, 2.4627772978900002e+00, 2.7417057014825201e+04 },
    { 1.0999999999999999e-09, 4.2442948732100003e+00, 2.9805016447382000e+03 },
    { 1.0900000000000000e-09, 3.9121417223299999e+00, 6.9538978107512003e+03 },
    { 1.0800000000000000e-09, 4.1236077765300001e+00, 2.4990446074220000e+02 },
    { 1.0800000000000000e-09, 5.3848313374799996e+00, 5.2168043612432397e+04 },
    { 1.0700000000000000e-09, 2.1777192421299998e+00, 7.0758507384588002e+03 },
    { 1.0700000000000000e-09, 2.4322367220299999e+00, 3.3695940566792001e+03 },
    { 1.0700000000000000e-09, 3.6915301966100000e+00, 3.3116307967204002e+03 },
    { 1.0600000000000001e-09, 1.5103316184999999e-01, 3.2367097656207599e+04 },
    { 1.0600000000000001e-09, 3.1966498575000002e-01, 4.4535100352782203e+04 },
    { 1.0600000000000001e-09, 2.4592728255799998e+00, 1.0191417463232399e+04 },
    { 1.0600000000000001e-09, 4.5513755614900004e+00, 8.6923251898393992e+03 },
    { 1.0500000000000001e-09, 3.8413923508500001e+00, 1.7821817925601790e+04 },
    { 1.0500000000000001e-09, 3.9228835071299999e+00, 6.9610113577519996e+03 },
    { 1.0500000000000001e-09, 3.9383232943900000e+00, 4.2388934936307000e+04 },
    { 1.0399999999999999e-09, 4.6880798398600003e+00, 3.1449827042751800e+04 },
    { 1.0399999999999999e-09, 4.5494263259000001e-01, 2.5827984119541401e+04 },
    { 1.0399999999999999e-09, 4.7911331323299997e+00, 3.1491814142404000e+03 },
    { 1.0399999999999999e-09, 3.5688781724199998e+00, 4.4082528487759998e+02 },
    { 1.0399999999999999e-09, 2.7603615008400002e+00, 3.1491469069360001e+03 },
    { 1.0300000000000000e-09, 5.0365875039399999e+00, 1.0593146198863999e+03 },
    { 1.0300000000000000e-09, 4.8443787407999999e+00, 5.5004628449339998e+02 },
    { 1.0300000000000000e-09, 6.0193315594900003e+00, 1.8617068301047198e+04 },
    { 1.0300000000000000e-09, 5.8967981388000004e+00, 2.1258100970763999e+03 },
    { 1.0300000000000000e-09, 2.6420963409999998e+00, 2.0235122826310399e+04 },
    { 1.0300000000000000e-09, 4.6280911170900003e+00, 1.0594492404919999e+03 },
    { 1.0300000000000000e-09, 3.4120817097899998e+00, 6.7772040806174000e+03 },
    { 1.0200000000000000e-09, 2.9918853220999999e+00, 3.4115114069274598e+04 },
    { 1.0200000000000000e-09, 3.1338918437899999e+00, 2.2147430875962000e+03 },
    { 1.0200000000000000e-09, 3.4921151096699998e+00, 2.7946747979919801e+04 },
    { 1.0200000000000000e-09, 1.6186393778999999e+00, 3.5244820136017902e+03 },
    { 1.0200000000000000e-09, 4.5051275407400002e+00, 3.1567428397978001e+03 },
    { 1.0100000000000000e-09, 3.1913723022100000e+00, 2.1791690973265791e+04 },
    { 1.0100000000000000e-09, 7.6366487467999999e-01, 9.8774627292800005e+02 },
    { 1.0000000000000001e-09, 7.9624016231000005e-01, 2.6519779438219998e+03 },
    { 1.0000000000000001e-09, 5.4438174084400002e+00, 3.3406969906548002e+03 },
    { 1.0000000000000001e-09, 3.8215421637899998e+00, 3.3405278627448101e+03 },
    { 1.0000000000000001e-09, 5.8523137952799997e+00, 3.3405623700492001e+03 },
    { 1.0000000000000001e-09, 3.4130457769500002e+00, 3.3406624833504002e+03 },
    { 9.9000000000000010e-10, 6.1530932027400000e+00, 4.5554147563232000e+03 },
    { 9.9000000000000010e-10, 5.7830078545700001e+00, 5.3125646005545996e+03 },
    { 9.9000000000000010e-10, 6.6802090434000005e-01, 1.7554716846791000e+03 },
    { 9.9000000000000010e-10, 4.0685410496900003e+00, 3.2227996238077999e+03 },
    { 9.9000000000000010e-10, 2.0552258689500000e+00, 3.4584252295918000e+03 },
    { 9.9000000000000010e-10, 5.8352322326900001e+00, 3.0927832265580003e+02 },
    { 9.9000000000000010e-10, 1.4176145341699999e+00, 1.9683563778237200e+04 },
    { 9.9000000000000010e-10, 2.0986297844999999e+00, 3.3050046361159998e+03 },
    { 9.9000000000000010e-10, 4.0251371341400004e+00, 3.3762202172836000e+03 },
    { 9.9000000000000010e-10, 2.6690620302300001e+00, 1.4814093713452000e+03 },
    { 9.7999999999999992e-10, 9.7971049081999995e-01, 1.7840442797444000e+03 },
    { 9.7999999999999992e-10, 5.4008431240499997e+00, 2.4768602189352201e+04 },
    { 9.7999999999999992e-10, 7.8906971914000001e-01, 2.4532976583568201e+04 },
    { 9.7999999999999992e-10, 3.0917735493300000e+00, 3.9302096962196001e+03 },
    { 9.6999999999999996e-10, 1.5795912019500000e+00, 3.1214201436967800e+04 },
    { 9.6999999999999996e-10, 5.9885782606999998e+00, 4.5494581429748789e+04 },
    { 9.6999999999999996e-10, 1.7363217071299999e+00, 4.0257237912286000e+03 },
    { 9.6999999999999996e-10, 1.6727441932300000e+00, 1.3002338924837601e+04 },
    { 9.5999999999999999e-10, 1.5575660034700001e+00, 4.0496022289764001e+03 },
    { 9.5999999999999999e-10, 6.2409801047200002e+00, 3.3574446277495899e+03 },
    { 9.5999999999999999e-10, 6.1659721211000003e+00, 3.3237802256499999e+03 },
    { 9.5000000000000003e-10, 3.1247428209199999e+00, 3.3704944703100000e+03 },
    { 9.5000000000000003e-10, 2.9990240977200000e+00, 3.3107303830895999e+03 },
    { 9.5000000000000003e-10, 3.7138977551500001e+00, 1.6975735090850601e+04 },
    { 9.5000000000000003e-10, 2.1486058970999999e-01, 1.1506769769793600e+04 },
    { 9.5000000000000003e-10, 8.4451353118000005e-01, 5.2962365479180005e+02 },
    { 9.5000000000000003e-10, 1.3975261618499999e+00, 1.1560077716077400e+04 },
    { 9.5000000000000003e-10, 2.8625503225400002e+00, 8.1865126624925997e+03 },
    { 9.5000000000000003e-10, 4.3601714434000000e-01, 5.2975827539739998e+02 },
    { 9.5000000000000003e-10, 2.7332852103999999e-01, 4.1592636929490604e+04 },
    { 9.5000000000000003e-10, 2.3130763218500001e+00, 2.1045367663768002e+03 },
    { 9.4000000000000006e-10, 3.6606677684200002e+00, 1.0610902107107600e+04 },
    { 9.4000000000000006e-10, 2.3438228687099998e+00, 3.3467456526323999e+03 },
    { 9.4000000000000006e-10, 3.7799440499300001e+00, 3.3344792007671999e+03 },
    { 9.4000000000000006e-10, 1.9543607635000000e+00, 7.9628075316419995e+02 },
    { 9.4000000000000006e-10, 3.9851323949999999e+00, 7.9631526046859994e+02 },
    { 9.4000000000000006e-10, 4.0405649156700001e+00, 9.1757544453482005e+03 },
    { 9.2999999999999999e-10, 5.3539024655200000e+00, 6.4745241161029999e+03 },
    { 9.2999999999999999e-10, 5.8436051086700003e+00, 1.7810200939183800e+04 },
    { 9.2000000000000003e-10, 1.7122760717700001e+00, 1.3828873468465999e+03 },
    { 9.2000000000000003e-10, 1.9180704551500001e+00, 1.5535308913140000e+02 },
    { 9.2000000000000003e-10, 1.4672842634900001e+00, 3.1172650645983002e+04 },
    { 9.2000000000000003e-10, 1.7240615648399999e+00, 1.2892132603618200e+04 },
    { 9.2000000000000003e-10, 4.7911837819200001e+00, 1.9577250798532000e+03 },
    { 9.2000000000000003e-10, 1.9061509301999999e-01, 3.2509208972941000e+04 },
    { 9.2000000000000003e-10, 6.0010869173400003e+00, 5.1371745605615994e+04 },
    { 9.2000000000000003e-10, 5.1828346636399996e+00, 1.8263211299832999e+03 },
    { 9.0999999999999996e-10, 5.1260415448699996e+00, 1.4469588512484001e+04 },
    { 9.0999999999999996e-10, 2.7315511632499998e+00, 7.8263709424722590e+04 },
    { 9.0999999999999996e-10, 1.1444165519300000e+00, 5.0818176202773597e+04 },
    { 9.0999999999999996e-10, 1.7773275635300001e+00, 2.4093276789175201e+04 },
    { 9.0999999999999996e-10, 4.9943353638100003e+00, 1.1876469585733999e+04 },
    { 8.9999999999999999e-10, 5.6777957572400002e+00, 1.5746772977528401e+04 },
    { 8.9999999999999999e-10, 1.5991411589400000e+00, 2.3546753623030800e+04 },
    { 8.9999999999999999e-10, 3.3820254096200002e+00, 4.6701245169597996e+03 },
    { 8.9999999999999999e-10, 4.0689888074300002e+00, 1.8028003474038989e+04 },
    { 8.9999999999999999e-10, 5.5263443016100000e+00, 6.8241872144623194e+04 },
    { 8.9999999999999999e-10, 9.7223021620000002e-02, 2.8857030601725000e+03 },
    { 8.9999999999999999e-10, 6.0265438970199998e+00, 3.7955217932270998e+03 },
    { 8.9000000000000003e-10, 6.4518470974999997e-01, 3.3680812930993902e+03 },
    { 8.9000000000000003e-10, 5.4785822088899998e+00, 3.3131435603002001e+03 },
    { 8.9000000000000003e-10, 1.7443466441200000e+00, 2.2531090605146590e+04 },
    { 8.9000000000000003e-10, 1.6458048107800001e+00, 6.5344131668760001e+03 },
    { 8.9000000000000003e-10, 6.1734136610399997e+00, 3.3329386347243999e+03 },
    { 8.9000000000000003e-10, 6.2335385647799999e+00, 3.3482862186752000e+03 },
    { 8.9000000000000003e-10, 4.1504697152899999e+00, 6.3115250374591997e+03 },
    { 8.9000000000000003e-10, 3.1833565226400000e+00, 1.7476073781529001e+03 },
    { 8.9000000000000003e-10, 3.3285675773799999e+00, 3.0639856638632999e+04 },
    { 8.9000000000000003e-10, 3.3417781448100001e+00, 1.0817087655544799e+04 },
    { 8.7999999999999996e-10, 3.9271052522600001e+00, 7.2774032274086003e+03 },
    { 8.7999999999999996e-10, 2.7514934792000001e-01, 2.7510151571800002e+03 },
    { 8.6999999999999999e-10, 6.0893179174999998e+00, 1.2410731300548599e+04 },
    { 8.6999999999999999e-10, 7.3442435079999993e-02, 1.3105431699056200e+04 },
    { 8.6999999999999999e-10, 4.2606248660599997e+00, 1.5117512356564201e+04 },
    { 8.6999999999999999e-10, 4.0074144000000001e-04, 2.4336005393149200e+04 },
    { 8.6999999999999999e-10, 3.0453272817300001e+00, 1.9403777273884800e+04 },
    { 8.6000000000000003e-10, 5.3769272513499997e+00, 1.1375227808107200e+04 },
    { 8.6000000000000003e-10, 5.3513150813100001e+00, 3.7626398678557998e+03 },
    { 8.6000000000000003e-10, 7.7245183733000000e-01, 2.9185849855438000e+03 },
    { 8.6000000000000003e-10, 3.4148808372300001e+00, 2.3009949110935391e+04 },
    { 8.6000000000000003e-10, 4.7657180929900003e+00, 2.5442971662312002e+03 },
    { 8.6000000000000003e-10, 3.6122960562899999e+00, 3.3852662599400001e+03 },
    { 8.6000000000000003e-10, 2.5114708623499999e+00, 3.2959585934595998e+03 },
    { 8.6000000000000003e-10, 5.1330441729999998e-01, 2.5443316735356002e+03 },
    { 8.6000000000000003e-10, 3.1017625256399999e+00, 5.3552358814886002e+03 },
    { 8.4999999999999996e-10, 5.9598605118899997e+00, 8.0947746944699998e+02 },
    { 8.4999999999999996e-10, 4.8683345223199996e+00, 1.0558588118401999e+03 },
    { 8.4999999999999996e-10, 2.6807337470800001e+00, 5.4789936689978000e+04 },
    { 8.4999999999999996e-10, 4.8366177110699997e+00, 1.4715840234806999e+04 },
    { 8.4999999999999996e-10, 4.5854145953299996e+00, 2.8109387152574000e+03 },
    { 8.4999999999999996e-10, 1.0744731977999999e+00, 3.1938179938284002e+03 },
    { 8.4999999999999996e-10, 5.0492937208399997e+00, 3.4874068595712001e+03 },
    { 8.4999999999999996e-10, 2.5546429638400001e+00, 2.8109042079530000e+03 },
    { 8.4999999999999996e-10, 1.0785640076700000e+00, 4.1572657529149901e+03 },
    { 8.3999999999999999e-10, 3.0038727818100002e+00, 4.7777880408984001e+03 },
    { 8.3999999999999999e-10, 9.2785811250000005e-01, 2.9425307335943999e+03 },
    { 8.3999999999999999e-10, 2.9153882233799999e+00, 3.0835944189567999e+03 },
    { 8.3999999999999999e-10, 3.2083786952600000e+00, 3.5976304344427999e+03 },
    { 8.3999999999999999e-10, 1.3363544993400001e+00, 2.9423961129887998e+03 },
    { 8.3999999999999999e-10, 2.7327322601100001e+00, 4.8016664786461997e+03 },
    { 8.3999999999999999e-10, 5.7507523221000003e+00, 4.5331398359598599e+04 },
    { 8.3999999999999999e-10, 5.2010191401799997e+00, 1.6858415222630389e+04 },
    { 8.3000000000000003e-10, 3.7909395387500000e+00, 1.1071388056848000e+03 },
    { 8.3000000000000003e-10, 2.2322472309900001e+00, 2.0103048422112399e+04 },
    { 8.3000000000000003e-10, 5.8401366288599998e+00, 4.4477512323846004e+03 },
    { 8.3000000000000003e-10, 2.1714755771799998e+00, 3.2522518055641802e+04 },
    { 8.3000000000000003e-10, 4.2977431269700004e+00, 1.1776899929864399e+04 },
    { 8.3000000000000003e-10, 4.4139928271000001e-01, 1.5276455874347401e+04 },
    { 8.1999999999999996e-10, 5.4771009209999999e+00, 3.3300430716527999e+03 },
    { 8.1999999999999996e-10, 6.4666599763999999e-01, 3.3511817817468000e+03 },
    { 8.1999999999999996e-10, 5.8327444061000000e-01, 2.2317791509708590e+04 },
    { 8.1999999999999996e-10, 4.3569716553999998e+00, 1.2988111830836000e+04 },
    { 8.1999999999999996e-10, 2.2491534425899999e+00, 6.5735613294610002e+03 },
    { 8.1999999999999996e-10, 5.8215543579900002e+00, 2.6369792420824001e+03 },
    { 8.0999999999999999e-10, 2.9379903524200000e+00, 1.5524675299795999e+03 },
    { 8.0999999999999999e-10, 5.4496868860500003e+00, 3.1907149515729998e+03 },
    { 8.0999999999999999e-10, 6.7408003259000004e-01, 3.4905099018266001e+03 },
    { 8.0999999999999999e-10, 4.3414615636800002e+00, 3.3477432273528002e+03 },
    { 8.0999999999999999e-10, 1.7823053549600001e+00, 3.3334816260468001e+03 },
    { 8.0999999999999999e-10, 1.1061786865400001e+00, 1.3533905280078000e+03 },
    { 8.0999999999999999e-10, 5.1705465088000002e+00, 3.4043483250032000e+03 },
    { 8.0999999999999999e-10, 9.5322040984000000e-01, 3.2768765283963999e+03 },
    { 8.0999999999999999e-10, 9.6354720612000000e-01, 2.9175373541728000e+03 },
    { 8.0999999999999999e-10, 1.6590542827999999e+00, 2.9104911174747999e+03 },
    { 8.0999999999999999e-10, 1.3815411587699999e+00, 1.9573357457017799e+04 },
    { 8.0999999999999999e-10, 2.3420855448500002e+00, 9.4329775286197386e+04 },
    { 8.0999999999999999e-10, 4.0187155429399999e+00, 5.1358708704208002e+03 },
    { 8.0999999999999999e-10, 3.2878424876499999e+00, 1.7498312508944389e+04 },
    { 8.0000000000000003e-10, 3.3816398819500000e+00, 1.9547171503636000e+03 },
    { 8.0000000000000003e-10, 2.1385582331299999e+00, 6.1516011986078001e+03 },
    { 8.0000000000000003e-10, 2.5470546199799999e+00, 6.1514665780021996e+03 },
    { 8.0000000000000003e-10, 3.3984838688000001e+00, 9.7054454104428005e+03 },
    { 8.0000000000000003e-10, 2.7496915744199999e+00, 5.8513445648900997e+04 },
    { 8.0000000000000003e-10, 1.6640598510900000e+00, 3.3772609896293002e+03 },
    { 8.0000000000000003e-10, 4.4597070675500001e+00, 3.3039638637703001e+03 },
    { 7.8999999999999996e-10, 5.5348657334000002e+00, 3.1979518007846400e+04 },
    { 7.8999999999999996e-10, 3.2952937845000002e-01, 4.4238727946367999e+03 },
    { 7.8999999999999996e-10, 1.1883352580200000e+00, 5.0973596602207792e+04 },
    { 7.8999999999999996e-10, 3.7380126200500001e+00, 1.5930050485469001e+03 },
    { 7.8999999999999996e-10, 4.4434090512599997e+00, 3.3716965065866403e+04 },
    { 7.7999999999999999e-10, 1.4021001577000001e+00, 2.6887366049730601e+04 },
    { 7.7999999999999999e-10, 4.4394016771200002e+00, 8.8309133882238002e+03 },
    { 7.7999999999999999e-10, 5.9621100214000000e-01, 1.3249413405400001e+03 },
    { 7.7999999999999999e-10, 2.9101498068999998e+00, 1.2715438873484400e+04 },
    { 7.7999999999999999e-10, 7.8353401756999996e-01, 7.7680083396855998e+03 },
    { 7.7999999999999999e-10, 1.2108363028700000e+00, 3.6105859086315599e+04 },
    { 7.7999999999999999e-10, 3.7557072528400002e+00, 4.2266603761290003e+02 },
    { 7.7999999999999999e-10, 3.7144904607700000e+00, 1.2979553174576000e+04 },
    { 7.7999999999999999e-10, 4.7807686933299998e+00, 1.7078827865069201e+04 },
    { 7.7000000000000003e-10, 7.0689947434000000e-01, 4.3053034413910001e+02 },
    { 7.7000000000000003e-10, 2.2206990976999999e-01, 4.5088669755624600e+04 },
    { 7.7000000000000003e-10, 3.4142097321599998e+00, 1.0507809332889001e+04 },
    { 7.7000000000000003e-10, 2.3944459337100001e+00, 2.4076444588125400e+04 },
    { 7.7000000000000003e-10, 8.3449424179999995e-02, 1.1368181571409201e+04 },
    { 7.5999999999999996e-10, 5.3880565357499997e+00, 4.0292469095776000e+03 },
    { 7.5999999999999996e-10, 5.1811277084000000e+00, 9.0904741932476009e+03 },
    { 7.5999999999999996e-10, 4.0383155577199998e+00, 3.0376352639166598e+04 },
    { 7.5999999999999996e-10, 8.2488746308000005e-01, 2.6357675084636001e+04 },
    { 7.5999999999999996e-10, 4.4329194104100003e+00, 1.1258693391035999e+03 },
    { 7.5999999999999996e-10, 9.2136957038000000e-01, 1.0085494708834000e+03 },
    { 7.5999999999999996e-10, 9.4087301710000001e-02, 8.1604321851422384e+04 },
    { 7.5999999999999996e-10, 1.7902088345400000e+00, 3.4126681600545999e+03 },
    { 7.5999999999999996e-10, 4.3335580840999999e+00, 3.2685566933450000e+03 },
    { 7.5999999999999996e-10, 1.0105349724599999e+00, 1.3532029889932201e+04 },
    { 7.5999999999999996e-10, 4.5317004104700001e+00, 6.6772927001364997e+03 },
    { 7.5000000000000000e-10, 4.5110684631999998e-01, 1.4797597368558600e+04 },
    { 7.5000000000000000e-10, 8.9569804119999999e-02, 1.5851407420206999e+03 },
    { 7.4000000000000003e-10, 4.0081892899899998e+00, 6.6034743094157002e+03 },
    { 7.4000000000000003e-10, 3.3712966780600002e+00, 1.4061720855026801e+04 },
    { 7.4000000000000003e-10, 5.2433583166300002e+00, 5.2094718263360000e+03 },
    { 7.4000000000000003e-10, 3.4255590093400001e+00, 1.2148023296234001e+03 },
    { 7.4000000000000003e-10, 2.3408785777599999e+00, 3.4363365597556003e+04 },
    { 7.4000000000000003e-10, 3.4917123887999999e+00, 2.3937856389740999e+04 },
    { 7.2999999999999996e-10, 4.2165805996400003e+00, 7.4764752288449999e+03 },
    { 7.2999999999999996e-10, 3.0089897642199999e+00, 6.8092436967370004e+03 },
    { 7.2999999999999996e-10, 3.7453135499600001e+00, 1.3635122664150800e+04 },
    { 7.2999999999999996e-10, 6.4702279583000000e-01, 6.4932997056370104e+03 },
    { 7.2999999999999996e-10, 6.2038146544600004e+00, 7.2500467274270004e+03 },
    { 7.2999999999999996e-10, 4.4577304259600004e+00, 6.6954519474012004e+03 },
    { 7.2999999999999996e-10, 2.0329837978200000e+00, 3.1925337022734998e+03 },
    { 7.2999999999999996e-10, 1.0302038403500000e+00, 3.1895647568569002e+03 },
    { 7.2000000000000000e-10, 3.9932273060700001e+00, 9.3848410080752001e+03 },
    { 7.2000000000000000e-10, 4.7759850318500003e+00, 4.8220217980449997e+03 },
    { 7.2000000000000000e-10, 4.2463586727099996e+00, 9.2783996850280005e+02 },
    { 7.2000000000000000e-10, 3.8565839635999999e+00, 3.5403720012000002e+00 },
    { 7.2000000000000000e-10, 1.8258123320999999e+00, 3.5058646967999998e+00 },
    { 7.2000000000000000e-10, 4.7956418345299996e+00, 7.8995503013719999e+03 },
    { 7.2000000000000000e-10, 5.3306813481000004e-01, 3.4142478899194002e+03 },
    { 7.2000000000000000e-10, 5.5906987838299997e+00, 3.2669769634802001e+03 },
    { 7.2000000000000000e-10, 6.2296111133599998e+00, 3.3435134409896000e+03 },
    { 7.2000000000000000e-10, 6.1773411124599997e+00, 3.3377114124099999e+03 },
    { 7.1000000000000003e-10, 4.2190755098099997e+00, 5.6219102335132002e+03 },
    { 7.1000000000000003e-10, 4.3076910315900001e+00, 2.3539640076029998e+04 },
    { 7.1000000000000003e-10, 4.3862296441400002e+00, 1.2140601140477800e+04 },
    { 7.1000000000000003e-10, 2.4644595465999999e-01, 5.8857031136547987e+04 },
    { 7.1000000000000003e-10, 5.8121523893000004e-01, 3.3716044114483998e+03 },
    { 7.1000000000000003e-10, 5.5425516797099998e+00, 3.3096204419512001e+03 },
    { 7.1000000000000003e-10, 4.6275718966600001e+00, 5.6217756129075997e+03 },
    { 7.1000000000000003e-10, 1.3005884682500000e+00, 6.4862534689390004e+03 },
    { 6.9999999999999996e-10, 3.5421977656900001e+00, 5.7843095593425996e+03 },
    { 6.9999999999999996e-10, 5.8980354133499997e+00, 2.3320629558164001e+03 },
    { 6.9999999999999996e-10, 9.2757117435000003e-01, 7.8436995578074002e+03 },
    { 6.9999999999999996e-10, 4.2090928367500000e+00, 1.9669336684235590e+04 },
    { 6.9999999999999996e-10, 3.0887549553700002e+00, 1.2192861455390601e+04 },
    { 6.9999999999999996e-10, 1.1381884435500000e+00, 1.8977179083008799e+04 },
    { 6.9999999999999996e-10, 2.4917281740799999e+00, 1.1787603905517000e+04 },
    { 6.9999999999999996e-10, 6.2351530635500003e+00, 4.1850432280699999e+02 },
    { 6.9000000000000000e-10, 1.7833800470000000e-01, 5.1769894609024203e+04 },
    { 6.9000000000000000e-10, 2.6537564425900002e+00, 2.1272636673520989e+04 },
    { 6.9000000000000000e-10, 3.0085498395100001e+00, 9.4206206196899996e+02 },
    { 6.9000000000000000e-10, 1.0311996202500000e+00, 2.3553867170031601e+04 },
    { 6.9000000000000000e-10, 1.6208433764999999e-01, 3.1866871931753999e+03 },
    { 6.9000000000000000e-10, 5.9616825809899998e+00, 3.4945376602242000e+03 },
    { 6.9000000000000000e-10, 2.8940561241200000e+00, 1.4422784624123999e+03 },
    { 6.9000000000000000e-10, 1.0258334425100000e+00, 5.8849441002353897e+03 },
    { 6.9000000000000000e-10, 7.4102855275000001e-01, 6.7574909246752004e+03 },
    { 6.9000000000000000e-10, 2.2087211745499999e+00, 1.6129513330315999e+03 },
    { 6.9000000000000000e-10, 5.0591561288999998e+00, 4.5589378746722005e+03 },
    { 6.9000000000000000e-10, 5.2782471182000004e+00, 5.8849095929309997e+03 },
    { 6.8000000000000003e-10, 5.5384189446400001e+00, 8.5500842530358004e+03 },
    { 6.8000000000000003e-10, 7.3828510395000002e-01, 8.9482282559085997e+03 },
    { 6.8000000000000003e-10, 5.5071975324300002e+00, 2.1957680727747000e+04 },
    { 6.8000000000000003e-10, 5.1402199072799997e+00, 4.8971805736551996e+03 },
    { 6.8000000000000003e-10, 4.3916617760000003e+00, 1.6066065861474801e+04 },
    { 6.6999999999999996e-10, 3.4345890491200000e+00, 3.9813174975599998e+02 },
    { 6.6999999999999996e-10, 4.0051854235900004e+00, 6.1124029793722002e+03 },
    { 6.6999999999999996e-10, 5.4653606806099999e+00, 3.9816625706040003e+02 },
    { 6.6999999999999996e-10, 6.2539624581300002e+00, 2.6239442205650390e+04 },
    { 6.6999999999999996e-10, 2.2327036093000001e+00, 3.8813356576349201e+04 },
    { 6.6999999999999996e-10, 3.8582264073200001e+00, 1.3358859278147400e+04 },
    { 6.6999999999999996e-10, 5.4424795873800003e+00, 4.3964712385399998e+03 },
    { 6.6999999999999996e-10, 6.1812702105100001e+00, 4.1748057328924799e+04 },
    { 6.6999999999999996e-10, 4.2931709789700001e+00, 1.3366040135451000e+04 },
    { 6.6000000000000000e-10, 4.3053609268999997e+00, 1.1247208964769799e+04 },
    { 6.6000000000000000e-10, 4.1765758350200004e+00, 2.9048547946960002e+02 },
    { 6.6000000000000000e-10, 1.7919362192099999e+00, 2.2582926841020000e+02 },
    { 6.6000000000000000e-10, 2.3072676743999998e+00, 1.6056051300184190e+04 },
    { 6.6000000000000000e-10, 2.0696812402300000e+00, 3.7386941198052000e+03 },
    { 6.6000000000000000e-10, 5.6471941886100003e+00, 4.1555249061209997e+02 },
    { 6.6000000000000000e-10, 3.6457046397899999e+00, 1.7669373013379999e+02 },
    { 6.6000000000000000e-10, 2.2494281767199999e+00, 1.9247376553382401e+04 },
    { 6.6000000000000000e-10, 3.6052016048900000e+00, 1.2725453434775000e+04 },
    { 6.6000000000000000e-10, 4.1749058902499998e+00, 3.3487805632822001e+03 },
    { 6.6000000000000000e-10, 1.9488610283900001e+00, 3.3324442901174002e+03 },
    { 6.6000000000000000e-10, 2.8253643130200001e+00, 3.3776080173107998e+03 },
    { 6.6000000000000000e-10, 3.2984026056200002e+00, 3.3036168360888000e+03 },
    { 6.6000000000000000e-10, 1.6611848533900000e+00, 3.7388287404108000e+03 },
    { 6.6000000000000000e-10, 4.0741503104900003e+00, 1.0832603679370000e+03 },
    { 6.6000000000000000e-10, 3.1514097999000001e+00, 2.8997871418221998e+03 },
    { 6.5000000000000003e-10, 3.8649250638099999e+00, 3.3573773174468001e+03 },
    { 6.5000000000000003e-10, 2.2588418548300000e+00, 3.3238475359528002e+03 },
    { 6.5000000000000003e-10, 3.7350800002000000e+00, 3.2268829871314001e+03 },
    { 6.5000000000000003e-10, 2.3886869184399999e+00, 3.4543418662682002e+03 },
    { 6.5000000000000003e-10, 2.2871724465100001e+00, 9.5524152459959998e+02 },
    { 6.5000000000000003e-10, 1.6626798859800000e+00, 5.5258636959925998e+03 },
    { 6.5000000000000003e-10, 5.6301944214299997e+00, 4.1236038717067197e+04 },
    { 6.3999999999999996e-10, 1.1861891299700000e+00, 2.7579523650957399e+04 },
    { 6.3999999999999996e-10, 3.9268573972400000e+00, 1.6328724257535790e+04 },
    { 6.3999999999999996e-10, 6.3002697211000003e-01, 1.7412051935775598e+04 },
    { 6.3999999999999996e-10, 6.1362841180499998e+00, 1.2082655134171200e+04 },
    { 6.3999999999999996e-10, 5.0446545370000004e+00, 6.5639049851793998e+03 },
    { 6.3999999999999996e-10, 5.9407517905000002e+00, 2.1258409579519390e+04 },
    { 6.3999999999999996e-10, 1.2829656695600000e+00, 4.8273861782890199e+04 },
    { 6.3000000000000000e-10, 2.6513590011399999e+00, 1.5003782916995800e+04 },
    { 6.3000000000000000e-10, 5.8625987172200000e+00, 3.7937341387420003e+02 },
    { 6.3000000000000000e-10, 1.0652423477099999e+00, 9.7227108858301999e+03 },
    { 6.3000000000000000e-10, 4.4038954755299997e+00, 4.1369276871683996e+03 },
    { 6.2000000000000003e-10, 5.6827333895400001e+00, 4.2307507252699997e+02 },
    { 6.2000000000000003e-10, 5.7990867963500001e+00, 7.0569905849697998e+03 },
    { 6.2000000000000003e-10, 2.2040599437199999e+00, 6.2885158605009997e+02 },
    { 6.2000000000000003e-10, 5.0986024342900000e+00, 4.3764389113990001e+02 },
    { 6.2000000000000003e-10, 4.7674041936600000e+00, 3.7057577492566197e+04 },
    { 6.2000000000000003e-10, 3.7607136729700001e+00, 2.2286032299359999e+02 },
    { 6.2000000000000003e-10, 8.4993046180999998e-01, 1.1030386750982800e+04 },
    { 6.0999999999999996e-10, 4.2519954275999998e+00, 3.8702861381421999e+03 },
    { 6.0999999999999996e-10, 6.2827670590900002e+00, 3.8703206454465999e+03 },
    { 6.0999999999999996e-10, 4.7097144418500001e+00, 1.1118664228759999e+02 },
    { 6.0999999999999996e-10, 4.5002897631899996e+00, 2.7117608406497002e+03 },
    { 6.0999999999999996e-10, 2.2172491347599999e+00, 1.7013902932367400e+04 },
    { 6.0999999999999996e-10, 5.2338007175300003e+00, 9.1615273513466000e+03 },
    { 6.0999999999999996e-10, 8.8840285577000000e-01, 7.6304004623694000e+03 },
    { 6.0000000000000000e-10, 5.8854319903899999e+00, 1.1663170490295999e+04 },
    { 6.0000000000000000e-10, 3.7069546661300001e+00, 3.3481911059094000e+03 },
    { 6.0000000000000000e-10, 2.4168122525100002e+00, 3.3330337474901899e+03 },
    { 5.9000000000000003e-10, 5.2762647464399999e+00, 1.1964825077627000e+04 },
    { 5.9000000000000003e-10, 3.9402726008800002e+00, 9.6617264981377994e+03 },
    { 5.9000000000000003e-10, 2.4434823163499999e+00, 2.2854596021804002e+04 },
    { 5.9000000000000003e-10, 3.5686293696100000e+00, 1.6342951351537400e+04 },
    { 5.7999999999999996e-10, 4.3743602775700001e+00, 4.5659841113701996e+03 },
    { 5.7999999999999996e-10, 3.5881262257600000e+00, 1.4319691037357199e+04 },
    { 5.7999999999999996e-10, 3.6717469099500000e+00, 2.7035740212466801e+04 },
    { 5.7999999999999996e-10, 5.4437572232799996e+00, 1.6070278394185800e+04 },
    { 5.7999999999999996e-10, 2.6310447937700001e+00, 3.7703978024255797e+04 },
    { 5.7999999999999996e-10, 2.7365647239999999e-02, 4.7120241901558002e+03 },
    { 5.7999999999999996e-10, 6.0964012714000004e+00, 1.9692006632437999e+03 },
    { 5.7999999999999996e-10, 4.5561643288100004e+00, 2.6637591209058592e+04 },
    { 5.7000000000000000e-10, 2.3808610470199998e+00, 8.2941761864311993e+03 },
    { 5.7000000000000000e-10, 4.0051252165199998e+00, 1.0227765648850000e+03 },
    { 5.7000000000000000e-10, 1.3200525852800000e+00, 1.5842752204746201e+04 },
    { 5.7000000000000000e-10, 5.2007546839199996e+00, 4.1427486983178802e+04 },
    { 5.7000000000000000e-10, 5.7749023189999997e+00, 3.3163395663023999e+04 },
    { 5.6000000000000003e-10, 4.3855890098100003e+00, 1.6199975697295999e+03 },
    { 5.6000000000000003e-10, 4.8625084035199997e+00, 1.2795153303460000e+02 },
    { 5.6000000000000003e-10, 4.5487570136900004e+00, 6.6938910461739999e+02 },
    { 5.4999999999999996e-10, 1.7744544876799999e+00, 3.3258751443741999e+03 },
    { 5.4999999999999996e-10, 4.3493124309700004e+00, 3.3553497090254000e+03 },
    { 5.4999999999999996e-10, 5.6751918998499997e+00, 1.3738215438369400e+04 },
    { 5.4999999999999996e-10, 5.0857479835400001e+00, 2.5665517483409199e+04 },
    { 5.4999999999999996e-10, 1.2304144427899999e+00, 8.3896928775039999e+02 },
    { 5.4000000000000000e-10, 3.9170992305099999e+00, 8.5287436120334005e+03 },
};

static const VSOP87Term VSOP87A_MARS_Y1[969] = {
    { 1.4273242100000000e-02, 3.1415926535900001e+00, 0.0000000000000000e+00 },
    { 5.5106375300000001e-03, 3.5212832040199999e+00, 6.6812248533995999e+03 },
    { 7.7091913000000003e-04, 3.8608268575300002e+00, 1.0021837280099400e+04 },
    { 3.7310491000000002e-04, 1.1601695844500000e+00, 3.3406124266997999e+03 },
    { 9.5821220000000005e-05, 4.1994235447900001e+00, 1.3362449706799200e+04 },
    { 2.6176950000000000e-05, 1.4728455551999999e+00, 3.3370893083507999e+03 },
    { 2.6115719999999999e-05, 4.6503077249800002e+00, 3.3441355450487999e+03 },
    { 1.1629549999999999e-05, 4.5377850357599998e+00, 1.6703062133498999e+04 },
    { 9.0067800000000002e-06, 1.7425626070900000e+00, 1.0593819301891999e+03 },
    { 4.3198999999999999e-06, 1.2012241978300000e+00, 2.9424634232916001e+03 },
    { 3.9435499999999997e-06, 4.9111939779600000e+00, 6.6847479717486003e+03 },
    { 3.8998200000000000e-06, 4.3877971356099996e+00, 5.6218429232103999e+03 },
    { 3.8012199999999998e-06, 4.9307372944400001e+00, 3.7387614301079998e+03 },
    { 3.3666099999999999e-06, 3.6399087961899999e+00, 3.1491641605882000e+03 },
    { 3.3360400000000002e-06, 8.8647104051000003e-01, 3.1851920272655998e+03 },
    { 3.1832400000000001e-06, 2.4812234547699998e+00, 3.5320606928113998e+03 },
    { 3.1674299999999999e-06, 5.2331652426900002e+00, 3.4960328261340001e+03 },
    { 3.1146799999999999e-06, 2.6614947420399999e+00, 2.5443144198834002e+03 },
    { 2.8300600000000000e-06, 3.0127055539400001e+00, 7.9629800681639995e+02 },
    { 2.5971500000000002e-06, 3.4528500754000002e+00, 4.1369104335162001e+03 },
    { 2.0427700000000002e-06, 1.6219695620500001e+00, 1.1944470102246000e+03 },
    { 1.5909399999999999e-06, 2.0047496316100002e+00, 1.5890728952838001e+03 },
    { 1.4050600000000001e-06, 4.8761106036999999e+00, 2.0043674560198800e+04 },
    { 1.3305400000000001e-06, 3.0649774949899999e+00, 5.8849268465832001e+03 },
    { 1.3222100000000001e-06, 2.1290373340900000e+00, 4.3999943568890003e+03 },
    { 1.2852600000000000e-06, 2.2328095819999999e-01, 1.5925960136327999e+03 },
    { 1.2674800000000000e-06, 1.2503129990599999e+00, 6.6777017350506003e+03 },
    { 1.0302300000000001e-06, 4.4865311758799997e+00, 5.4867778431750003e+03 },
    { 1.0135200000000000e-06, 3.9706120873800002e+00, 3.3396321056316001e+03 },
    { 1.0125300000000000e-06, 2.1520329505600002e+00, 3.3415927477680002e+03 },
    { 9.0029000000000002e-07, 4.3063192565700001e+00, 3.8703033917943999e+03 },
    { 8.3977000000000001e-07, 5.1043829904000004e+00, 1.9907450170410000e+03 },
    { 8.1297000000000001e-07, 4.1075191103300002e+00, 5.0921519581158000e+03 },
    { 7.9143000000000004e-07, 2.3446637841000002e+00, 4.2659819087599999e+02 },
    { 7.8864999999999996e-07, 3.6932330229199999e+00, 2.1461654164751999e+03 },
    { 7.7131999999999995e-07, 5.3529398907200001e+00, 7.0793738568077997e+03 },
    { 7.5448000000000002e-07, 2.0386421581100000e+00, 4.5350594369244000e+03 },
    { 7.0493000000000000e-07, 2.2943463516699998e+00, 2.7870430238573999e+03 },
    { 6.9177000000000001e-07, 5.8163420783099999e+00, 5.0886288397668004e+03 },
    { 6.3835000000000000e-07, 3.7002830572400001e+00, 2.3888940204492001e+03 },
    { 5.7286999999999997e-07, 5.2146258059899999e+00, 1.0025360398448400e+04 },
    { 5.7046000000000004e-07, 3.7817257612900002e+00, 3.8941818295421999e+03 },
    { 5.6817000000000004e-07, 4.9713786115599996e+00, 7.1135470007999997e+00 },
    { 5.6118999999999996e-07, 3.3784373100999998e-01, 3.0978838227257902e+03 },
    { 5.5486999999999999e-07, 5.9285675561100000e+00, 9.4921463150048003e+03 },
    { 5.4005999999999998e-07, 1.1784821966700001e+00, 2.9577158944766002e+03 },
    { 5.3524999999999998e-07, 2.7277219363500000e+00, 1.5542039943419999e+02 },
    { 5.2915000000000005e-07, 5.7601212876899996e+00, 3.5833410306738001e+03 },
    { 5.0121000000000003e-07, 8.9975887576000002e-01, 4.6904798363585996e+03 },
    { 4.9195000000000001e-07, 9.3502149184000005e-01, 6.6812075997474003e+03 },
    { 4.9195000000000001e-07, 2.9657932555099999e+00, 6.6812421070518003e+03 },
    { 4.9131000000000002e-07, 6.2216519243399997e+00, 1.9144826611159999e+02 },
    { 4.7371000000000002e-07, 4.0705884411300000e+00, 6.4897765872880000e+03 },
    { 4.6746999999999999e-07, 4.9037228662699999e+00, 3.7235089589230001e+03 },
    { 4.3883999999999998e-07, 2.3232857635699999e+00, 4.2923308329503998e+03 },
    { 3.9091000000000000e-07, 3.8064095517499998e+00, 7.4775228602159996e+03 },
    { 3.7258999999999998e-07, 4.7756072907600000e+00, 3.5539115221378001e+03 },
    { 3.6147000000000002e-07, 1.3549151302200000e+00, 3.1273133312618002e+03 },
    { 3.5282000000000001e-07, 1.2325484345300000e+00, 6.5258044539654002e+03 },
    { 3.4808999999999999e-07, 4.1819236259499997e+00, 1.7480164130670000e+03 },
    { 3.3178999999999998e-07, 3.5170722267099999e+00, 3.3334988796990001e+03 },
    { 3.0407000000000000e-07, 5.0710451909799996e+00, 1.3498674096587999e+03 },
    { 2.9410000000000002e-07, 3.7710180219999998e+00, 6.2546266625236003e+03 },
    { 2.9191000000000002e-07, 1.0880657714799999e+00, 6.3989728631399998e+02 },
    { 2.8052000000000003e-07, 2.6698550998999999e+00, 3.3477259737006002e+03 },
    { 2.7517000000000000e-07, 2.1756512843200002e+00, 1.2303067776610000e+04 },
    { 2.7441999999999999e-07, 6.2028270679999997e-01, 4.9332084403325998e+03 },
    { 2.7323000000000002e-07, 2.0957818999999999e-02, 9.5171840625059997e+02 },
    { 2.7239000000000002e-07, 1.3565921941200001e+00, 5.5356940284240000e+02 },
    { 2.5853000000000003e-07, 1.7504753720899999e+00, 2.8109214616052000e+03 },
    { 2.4778000000000001e-07, 2.2932745221999999e+00, 2.1187638603783998e+03 },
    { 2.4443999999999998e-07, 4.6560465287500001e+00, 1.0664954771900000e+03 },
    { 2.3424000000000001e-07, 2.3496018782100001e+00, 3.8289653222319998e+02 },
    { 2.1261000000000000e-07, 2.2737036536000002e+00, 4.9296853219836003e+03 },
    { 1.8678999999999999e-07, 5.3087705645299996e+00, 2.4272860397400001e+02 },
    { 1.8299000000000000e-07, 5.3187303719600001e+00, 9.6236882766912004e+03 },
    { 1.7704000000000000e-07, 2.4667331344200001e+00, 7.7406067835887998e+03 },
    { 1.7050000000000000e-07, 7.3617668320999996e-01, 3.3202571073009999e+03 },
    { 1.7034999999999999e-07, 1.1307617058299999e+00, 6.6741113063987996e+03 },
    { 1.6988999999999999e-07, 5.2143535389900002e+00, 2.3384286986898602e+04 },
    { 1.6932000000000001e-07, 5.3859688550899998e+00, 3.3609677460985899e+03 },
    { 1.6726000000000000e-07, 5.3796301876100001e+00, 7.2109158184941998e+03 },
    { 1.6504000000000000e-07, 3.8122794462399998e+00, 4.5624609930212000e+03 },
    { 1.6201999999999999e-07, 5.2967578844599998e+00, 6.6811575430967996e+03 },
    { 1.6201999999999999e-07, 4.8882305799700001e+00, 6.6812921637024001e+03 },
    { 1.6030000000000000e-07, 2.6770519612700001e+00, 3.3253599555147998e+03 },
    { 1.5998000000000001e-07, 5.1624982328800000e+00, 1.0522683831884001e+03 },
    { 1.5941000000000000e-07, 3.4441140272099999e+00, 3.3558648978848000e+03 },
    { 1.4765999999999999e-07, 2.0634709533500000e+00, 2.2741169495098002e+03 },
    { 1.4564000000000000e-07, 3.5017458980999999e-01, 1.0018314161750401e+04 },
    { 1.3784000000000000e-07, 6.2614952115999998e+00, 1.2832758741704600e+04 },
    { 1.3353999999999999e-07, 1.6120222952200001e+00, 2.2883440435113998e+03 },
    { 1.3054000000000000e-07, 2.9194268934699998e+00, 3.7672106175757999e+03 },
    { 1.2926999999999999e-07, 4.1602292835999997e+00, 2.2812304965106000e+03 },
    { 1.2672000000000001e-07, 5.4855096000200003e+00, 5.3313574437407997e+03 },
    { 1.2499999999999999e-07, 5.7165713270599996e+00, 1.0419986283507600e+04 },
    { 1.2471000000000001e-07, 2.5225224645900002e+00, 6.6822051744678001e+03 },
    { 1.1971000000000000e-07, 3.5152568831999997e-01, 5.3680451209540001e+02 },
    { 1.1308000000000000e-07, 5.0295213840899997e+00, 6.0413275670856001e+03 },
    { 1.1249000000000000e-07, 9.0083527171000000e-01, 5.2257741809380002e+02 },
    { 1.0891000000000000e-07, 6.8185023632999997e-01, 6.4384962494255997e+03 },
    { 1.0802000000000000e-07, 6.2680034608800002e+00, 5.2969096509459996e+02 },
    { 1.0388000000000000e-07, 1.4308276235999999e+00, 5.6147293762095996e+03 },
    { 1.0337000000000001e-07, 3.3057090345900000e+00, 1.0021854533751601e+04 },
    { 1.0337000000000001e-07, 1.2749307010299999e+00, 1.0021820026447200e+04 },
    { 1.0293999999999999e-07, 2.3859020729100000e+00, 7.8756718636242003e+03 },
    { 9.9410000000000001e-08, 5.4622363533099998e+00, 3.9814900340819997e+02 },
    { 9.2270000000000005e-08, 5.1874999368000001e+00, 6.8366452528338004e+03 },
    { 8.8459999999999999e-08, 6.2566814858099997e+00, 3.2055473466643998e+03 },
    { 8.3560000000000002e-08, 6.1498032758000001e+00, 3.4756775067352000e+03 },
    { 8.0960000000000005e-08, 5.5336431806500004e+00, 1.3365972825148199e+04 },
    { 7.8320000000000003e-08, 4.0633097733200003e+00, 5.7295064471490005e+03 },
    { 7.7890000000000005e-08, 2.6475186857600002e+00, 6.1276554505572003e+03 },
    { 7.7410000000000003e-08, 5.0275632029799997e+00, 1.7515395314160000e+03 },
    { 7.7369999999999997e-08, 5.7464645129900003e+00, 2.7007151403858002e+03 },
    { 7.6160000000000002e-08, 3.5521683664900001e+00, 6.1515338883049999e+03 },
    { 7.4019999999999997e-08, 2.5080988738799999e+00, 1.5643680203309799e+04 },
    { 7.2880000000000001e-08, 5.3492272906000000e+00, 6.8945239488376001e+03 },
    { 7.0609999999999995e-08, 3.4054417965699999e+00, 8.9624553499102003e+03 },
    { 6.8880000000000005e-08, 9.3821376085999997e-01, 5.6289564702112002e+03 },
    { 6.7430000000000002e-08, 4.4565031499699996e+00, 9.8303890139878004e+03 },
    { 6.6530000000000001e-08, 3.0407980043400000e+00, 3.1781457905676002e+03 },
    { 6.5509999999999996e-08, 4.2523862052199997e+00, 6.6802445323313996e+03 },
    { 6.3580000000000004e-08, 4.0205443973000001e+00, 3.3167339889519999e+03 },
    { 6.3110000000000001e-08, 2.1053055207600000e+00, 3.3644908644476000e+03 },
    { 6.2569999999999997e-08, 3.0846735709200002e+00, 3.5030790628320001e+03 },
    { 6.2139999999999999e-08, 1.5297194990900000e+00, 6.2983283211764001e+03 },
    { 6.0049999999999998e-08, 7.4860541850000004e-02, 8.5319638175199998e+02 },
    { 5.7830000000000002e-08, 4.9386122300500004e+00, 2.1329909543799999e+02 },
    { 5.5689999999999998e-08, 4.1485667648000000e+00, 1.0818135286915800e+04 },
    { 5.5360000000000001e-08, 2.5516350880299998e+00, 1.7586530784168001e+03 },
    { 5.4940000000000001e-08, 6.1009025267999997e+00, 6.9239534573736000e+03 },
    { 5.4739999999999999e-08, 2.4241650562000000e-01, 7.6329432596502002e+03 },
    { 5.3050000000000002e-08, 2.6497526065299999e+00, 2.6484548254729998e+03 },
    { 5.1140000000000000e-08, 5.1354185801199996e+00, 8.0310922630584000e+03 },
    { 5.0780000000000002e-08, 4.6815102272200004e+00, 1.5961864422845999e+03 },
    { 5.0029999999999999e-08, 2.0341368975899998e+00, 4.3371173787679999e+02 },
    { 4.8419999999999999e-08, 5.1694370603899999e+00, 6.1586474353058002e+03 },
    { 4.7990000000000002e-08, 3.0659249729300000e+00, 6.6883384004004001e+03 },
    { 4.7790000000000000e-08, 5.7603661008199998e+00, 6.1444203413041996e+03 },
    { 4.6520000000000002e-08, 2.6333299701800001e+00, 2.9140142358237999e+03 },
    { 4.6430000000000000e-08, 1.6433255999200000e+00, 7.2347942562420003e+03 },
    { 4.5800000000000000e-08, 1.4160811948700001e+00, 5.0850384111149997e+03 },
    { 4.5419999999999999e-08, 2.4470286052499999e+00, 7.0641213856227996e+03 },
    { 4.4419999999999997e-08, 2.5070106886099999e+00, 1.1773376811515400e+04 },
    { 4.3569999999999999e-08, 4.1284923398100002e+00, 2.8180350086059998e+03 },
    { 4.3359999999999999e-08, 5.6565851522799999e+00, 1.2964300703391000e+04 },
    { 4.2139999999999999e-08, 3.4971778233899999e+00, 4.0327700279266001e+03 },
    { 4.1280000000000003e-08, 1.5731263969600000e+00, 9.8664168806651996e+03 },
    { 4.0709999999999999e-08, 1.8854533958199999e+00, 3.8631898447936001e+03 },
    { 3.9699999999999998e-08, 5.0543963687300000e+00, 3.3121632392319998e+03 },
    { 3.8940000000000003e-08, 1.0621284066200001e+00, 3.3690616141676001e+03 },
    { 3.8840000000000002e-08, 1.8999892746500000e+00, 8.9553418029093991e+03 },
    { 3.8320000000000002e-08, 1.4251846793800000e+00, 8.9695688969109997e+03 },
    { 3.8169999999999997e-08, 3.9099898245000002e+00, 6.4087760738220004e+02 },
    { 3.7210000000000000e-08, 3.0245419084799998e+00, 3.3461353510071999e+03 },
    { 3.7200000000000002e-08, 3.0980982448200001e+00, 3.3350895023923999e+03 },
    { 3.6290000000000003e-08, 5.9699997236100000e+00, 3.3045845600224002e+03 },
    { 3.6220000000000003e-08, 4.4268381380099999e+00, 1.5819593482830001e+03 },
    { 3.5770000000000002e-08, 1.5378568235000001e-01, 3.3766402933772001e+03 },
    { 3.5740000000000001e-08, 2.4419483168500000e+00, 1.7444259844152000e+03 },
    { 3.5560000000000002e-08, 9.6329089901999998e-01, 8.2738208670324002e+03 },
    { 3.5019999999999999e-08, 1.2835521646100001e+00, 5.0992655051166003e+03 },
    { 3.4100000000000001e-08, 5.2258477085599999e+00, 1.0021904590402200e+04 },
    { 3.4000000000000000e-08, 5.6328887747899996e+00, 1.0021769969796600e+04 },
    { 3.3530000000000003e-08, 2.0177499757500001e+00, 9.2255392732830005e+03 },
    { 3.2450000000000003e-08, 5.2174867632100002e+00, 1.4956319713459999e+02 },
    { 3.2080000000000000e-08, 1.5665845953099999e+00, 2.6997348193175999e+03 },
    { 3.1559999999999999e-08, 1.6178345794900000e+00, 3.9805097130138001e+03 },
    { 3.1300000000000002e-08, 3.8813131819600000e+00, 3.3444937620577998e+03 },
    { 3.1289999999999998e-08, 2.4944007595500000e+00, 5.4593762870782002e+03 },
    { 3.1280000000000000e-08, 3.7542482500199998e+00, 8.4327643848156004e+03 },
    { 3.1170000000000001e-08, 2.2473256352500002e+00, 3.3367310913418000e+03 },
    { 3.0839999999999997e-08, 2.1639027416000001e-01, 1.3358926588450200e+04 },
    { 3.0190000000000001e-08, 2.4797248396199998e+00, 4.1948464387519999e+02 },
    { 3.0099999999999998e-08, 5.9803251822499996e+00, 1.0551528245194000e+04 },
    { 2.9929999999999997e-08, 4.3464848929300004e+00, 3.9814900340820000e+03 },
    { 2.9340000000000000e-08, 5.1109476281999999e-01, 2.4874160449477999e+03 },
    { 2.8760000000000001e-08, 2.5845045722200002e+00, 8.2702977486834006e+03 },
    { 2.7510000000000000e-08, 2.1453643621500000e+00, 6.4679257579615996e+03 },
    { 2.7109999999999999e-08, 5.8379779380899999e+00, 5.8280284716475999e+03 },
    { 2.5279999999999999e-08, 4.2167629703299996e+00, 3.3335661900017999e+03 },
    { 2.5100000000000000e-08, 3.1296140818000001e-01, 1.6173371168404399e+04 },
    { 2.4859999999999999e-08, 2.4395631995000000e-01, 6.2830758499914000e+03 },
    { 2.4470000000000000e-08, 4.9276332321599998e+00, 2.7662676283649998e+03 },
    { 2.4280000000000000e-08, 2.8003744852799999e+00, 1.1081219210288600e+04 },
    { 2.3910000000000000e-08, 5.5144208403099997e+00, 9.8085381846614000e+03 },
    { 2.3330000000000001e-08, 3.5143293445100001e+00, 8.8273902698748007e+03 },
    { 2.3230000000000000e-08, 5.6044993566699999e+00, 9.3819399937853996e+03 },
    { 2.3149999999999999e-08, 3.6914418501599999e+00, 8.4292412664666008e+03 },
    { 2.1859999999999999e-08, 5.0304365537000004e+00, 4.4071079038897997e+03 },
    { 2.1540000000000000e-08, 1.5347526650000001e-01, 6.8726731195111997e+03 },
    { 2.1530000000000002e-08, 6.0435380552200000e+00, 1.2218485663214001e+03 },
    { 2.1139999999999999e-08, 1.6388816849700001e+00, 3.3476586633978000e+03 },
    { 2.1109999999999998e-08, 2.0636503523799998e+00, 1.5113989238215199e+04 },
    { 2.0590000000000001e-08, 5.5527404246899996e+00, 2.6724899413598399e+04 },
    { 1.8870000000000000e-08, 4.8825597950699997e+00, 7.6266071275599998e+01 },
    { 1.8620000000000001e-08, 6.0648181711299998e+00, 1.3760598710207400e+04 },
    { 1.8559999999999999e-08, 1.0662971818800000e+00, 3.9149572250346000e+03 },
    { 1.8489999999999999e-08, 2.1162148727500001e+00, 1.0575406682941801e+04 },
    { 1.8419999999999999e-08, 1.2182382297500001e+00, 1.6921656695024001e+03 },
    { 1.8310000000000000e-08, 4.0896087799099998e+00, 6.2475131155228000e+03 },
    { 1.8139999999999999e-08, 3.7943388158500002e+00, 2.2041264243879999e+02 },
    { 1.8069999999999999e-08, 3.7997961014300001e+00, 6.6964773245846000e+03 },
    { 1.7809999999999999e-08, 8.7193778743000006e-01, 6.6604494579071998e+03 },
    { 1.7710000000000001e-08, 5.9857555923000005e-01, 1.0973555686350001e+04 },
    { 1.7669999999999999e-08, 1.0791144896500000e+00, 6.6608695340007998e+03 },
    { 1.7439999999999999e-08, 2.2355971934999999e-01, 6.8162899334350004e+03 },
    { 1.7380000000000001e-08, 2.0782081439000000e+00, 3.2812385647862002e+03 },
    { 1.7170000000000001e-08, 3.6449648045899998e+00, 1.3362466960451400e+04 },
    { 1.7170000000000001e-08, 1.6141931792299999e+00, 1.3362432453146999e+04 },
    { 1.6980000000000001e-08, 4.0455992979800000e+00, 3.3999862886134001e+03 },
    { 1.6849999999999999e-08, 3.3257948666899999e+00, 7.1078230442756003e+03 },
    { 1.6680000000000001e-08, 6.0430699218999995e-01, 3.3406296803519999e+03 },
    { 1.6680000000000001e-08, 4.8567365982300004e+00, 3.3405951730475999e+03 },
    { 1.6379999999999998e-08, 1.9638480014699999e+00, 6.0403472460173998e+03 },
    { 1.6280000000000001e-08, 2.8807383800199999e+00, 1.0022817601167601e+04 },
    { 1.6120000000000001e-08, 5.6420292902100000e+00, 3.5231183490000002e+00 },
    { 1.5920000000000000e-08, 4.8287099631999997e+00, 2.9211277828246002e+03 },
    { 1.5840000000000002e-08, 5.8270473919899999e+00, 8.6719698704406001e+03 },
    { 1.5399999999999999e-08, 2.3157623258300002e+00, 8.4398779318163997e+03 },
    { 1.5329999999999999e-08, 1.4614712554900000e+00, 3.8774169387952002e+03 },
    { 1.5290000000000000e-08, 5.2821983165199997e+00, 3.5467979751369999e+03 },
    { 1.5030000000000000e-08, 3.0336387831399998e+00, 6.7020002488919999e+03 },
    { 1.4999999999999999e-08, 4.8854546188099999e+00, 5.7434479833479998e+02 },
    { 1.4990000000000001e-08, 4.5700881788299998e+00, 2.0618554843720000e+02 },
    { 1.4979999999999999e-08, 1.4714788458300001e+00, 1.0014723733098601e+04 },
    { 1.4940000000000000e-08, 8.0790809310000000e-01, 3.1344268782626000e+03 },
    { 1.4149999999999999e-08, 1.0497056856900000e+00, 6.4701083331480004e+02 },
    { 1.4110000000000000e-08, 2.8410614824099998e+00, 1.8984292630009601e+04 },
    { 1.3780000000000000e-08, 2.7260615011699998e+00, 1.1216284290324000e+04 },
    { 1.3730000000000000e-08, 2.5316871671099999e+00, 1.0390266107903999e+03 },
    { 1.3599999999999999e-08, 7.0553844479000005e-01, 3.0740053849780002e+03 },
    { 1.3049999999999999e-08, 5.6879369567499998e+00, 6.7015801727983999e+03 },
    { 1.3020000000000000e-08, 2.9869402134900001e+00, 6.6659723822145997e+03 },
    { 1.2450000000000001e-08, 4.0758961653499997e+00, 1.0177257679533601e+04 },
    { 1.2310000000000001e-08, 2.9245838042900001e+00, 1.2289621133221999e+03 },
    { 1.2190000000000000e-08, 5.4183747408300000e+00, 3.6072194684216001e+03 },
    { 1.1949999999999999e-08, 5.8093985080700001e+00, 3.3613878221922000e+03 },
    { 1.1609999999999999e-08, 3.0017004685999998e-01, 3.3198370312073998e+03 },
    { 1.1590000000000000e-08, 5.0083494225800003e+00, 1.1371704689758200e+04 },
    { 1.1570000000000000e-08, 5.7885249242499999e+00, 1.0235136375537400e+04 },
    { 1.1220000000000001e-08, 5.9596905289800004e+00, 1.1020632121940000e+02 },
    { 1.1210000000000001e-08, 5.8588247069000001e+00, 1.6706585251847999e+04 },
    { 1.1080000000000001e-08, 1.7115326667499999e+00, 1.2935851515923199e+04 },
    { 1.1040000000000000e-08, 1.7745657603600000e+00, 4.5695745400220003e+03 },
    { 1.0929999999999999e-08, 9.2444199357000001e-01, 6.5316616562649997e+03 },
    { 1.0920000000000000e-08, 4.0319532353199996e+00, 2.8038079146044001e+03 },
    { 1.0890000000000000e-08, 7.1433907377999994e-01, 3.1699395560806001e+03 },
    { 1.0680000000000000e-08, 2.1672175348499998e+00, 6.6848152820513997e+03 },
    { 1.0630000000000000e-08, 1.7298287471000000e+00, 6.6776344247478000e+03 },
    { 1.0560000000000000e-08, 4.5658591594400004e+00, 2.9069006888230001e+03 },
    { 1.0370000000000000e-08, 1.4645296564700001e+00, 4.5553474460203997e+03 },
    { 1.0290000000000000e-08, 3.7591035341799999e+00, 6.2617402095243997e+03 },
    { 1.0200000000000000e-08, 5.4092938526700003e+00, 3.5112852973190002e+03 },
    { 1.0099999999999999e-08, 5.7599905268200002e+00, 3.2533042221599999e+03 },
    { 1.0020000000000000e-08, 4.3098035564999998e-01, 9.7791086761253991e+03 },
    { 9.9800000000000007e-09, 3.9012609381600001e+00, 1.1769853693166400e+04 },
    { 9.9300000000000002e-09, 2.2009398315699999e+00, 8.4256508378147992e+03 },
    { 9.7399999999999999e-09, 3.6385416352000000e-01, 3.4279206312396000e+03 },
    { 9.6699999999999999e-09, 2.9568238997699998e+00, 9.5952390892233998e+03 },
    { 9.3800000000000003e-09, 4.8191924694899999e+00, 4.9890591838971995e+03 },
    { 9.2799999999999994e-09, 4.4159037570099997e+00, 9.0701188738487999e+03 },
    { 9.2199999999999992e-09, 2.9926482833599999e+00, 9.4682678772569998e+03 },
    { 9.1800000000000001e-09, 4.8195163906799996e+00, 1.3171001440687600e+04 },
    { 8.8900000000000005e-09, 4.7787851512600001e+00, 3.5610250691386000e+03 },
    { 8.8699999999999994e-09, 7.8830636357999995e-01, 3.7600970705750001e+03 },
    { 8.7999999999999994e-09, 9.0436253322000004e-01, 3.3132108706029999e+03 },
    { 8.7399999999999992e-09, 1.2089365253000001e-01, 9.5628915597059995e+02 },
    { 8.7000000000000001e-09, 5.2194071384600003e+00, 3.3680139827966000e+03 },
    { 8.4499999999999996e-09, 5.5518221557800000e+00, 9.4992598620055996e+03 },
    { 8.4000000000000008e-09, 4.8382566480599998e+00, 2.1258774073792001e+03 },
    { 8.2499999999999994e-09, 4.1652632558300002e+00, 1.8617458526354001e+03 },
    { 8.1899999999999992e-09, 1.2813993909600001e+00, 3.1201997842609999e+03 },
    { 8.1199999999999993e-09, 3.0945449993000002e-01, 9.1686408983473993e+03 },
    { 8.0000000000000005e-09, 3.2207234355700001e+00, 6.5187582172674001e+03 },
    { 7.9400000000000003e-09, 5.9949654890100001e+00, 1.6304913130090799e+04 },
    { 7.8199999999999999e-09, 1.3088625475900000e+00, 6.3278373931320004e+02 },
    { 7.6899999999999997e-09, 1.8809188397000001e+00, 9.6389407478762005e+03 },
    { 7.5699999999999993e-09, 4.4995346967899996e+00, 1.4158747713615599e+04 },
    { 7.4999999999999993e-09, 4.5100768491499998e+00, 3.2968935143948001e+03 },
    { 7.4400000000000000e-09, 1.1448672766600001e+00, 3.4437052009183999e+03 },
    { 7.3700000000000000e-09, 1.6137053051400001e+00, 3.3843313390048002e+03 },
    { 7.3000000000000000e-09, 1.4502318402500001e+00, 6.7096740408674004e+03 },
    { 7.2699999999999999e-09, 2.5280059641400001e+00, 1.0404733812322600e+04 },
    { 7.2500000000000004e-09, 1.6056815133800000e+00, 6.4750393049623999e+03 },
    { 7.1600000000000001e-09, 1.8268188991100001e+00, 1.2310181323610799e+04 },
    { 7.0999999999999999e-09, 2.3242090341199999e+00, 6.4608122109608003e+03 },
    { 7.0100000000000004e-09, 5.0995327961500001e+00, 3.3111829181637900e+03 },
    { 7.0100000000000004e-09, 1.0242341224900000e+00, 3.3700419352357999e+03 },
    { 6.8800000000000002e-09, 3.4331922275200002e+00, 1.0028950827100200e+04 },
    { 6.8699999999999996e-09, 4.2366198975799998e+00, 1.3517870106233400e+04 },
    { 6.7400000000000003e-09, 1.3994072026400000e+00, 2.8073983432562000e+03 },
    { 6.6999999999999996e-09, 4.3749015482400004e+00, 6.6573464156518003e+03 },
    { 6.6899999999999999e-09, 5.2291177121999999e+00, 2.7078286873866000e+03 },
    { 6.4700000000000002e-09, 5.6511804496900000e+00, 4.3928808098882000e+03 },
    { 6.4599999999999996e-09, 4.9972935673599999e+00, 3.2375196524812000e+03 },
    { 6.4599999999999996e-09, 4.2044303744800002e+00, 2.1116503133776000e+03 },
    { 6.3300000000000003e-09, 4.3972121000999997e-01, 6.9215760122680001e+02 },
    { 6.3000000000000002e-09, 1.2355948808499999e+00, 6.6049587821240002e+03 },
    { 6.3000000000000002e-09, 2.1323113618399998e+00, 1.1243685846420800e+04 },
    { 6.2900000000000004e-09, 2.8762848508599999e+00, 1.2147350193206000e+03 },
    { 6.2700000000000001e-09, 5.8230474165399997e+00, 1.3149150611361199e+04 },
    { 6.2300000000000002e-09, 1.5818615271000000e-01, 1.0264565884073400e+04 },
    { 6.1300000000000001e-09, 2.8294809357699999e+00, 5.9890672521728002e+03 },
    { 6.0799999999999997e-09, 4.6074610926900004e+00, 9.4917560896980001e+02 },
    { 5.9600000000000001e-09, 1.1490737756800000e+00, 1.2566151699982800e+04 },
    { 5.8999999999999999e-09, 4.8699066892699996e+00, 6.0342140200847998e+03 },
    { 5.8399999999999997e-09, 5.5159339979999999e-01, 4.1938088084518004e+03 },
    { 5.7699999999999997e-09, 2.2672837005700002e+00, 1.8454601664915001e+04 },
    { 5.7600000000000000e-09, 2.6490813160100002e+00, 1.3916019109641600e+04 },
    { 5.6299999999999998e-09, 1.1089733390200001e+00, 1.4371756141986000e+03 },
    { 5.5700000000000004e-09, 5.5670149749700002e+00, 1.3362517017102000e+04 },
    { 5.5700000000000004e-09, 5.9755383450800004e+00, 1.3362382396496399e+04 },
    { 5.4800000000000001e-09, 2.5237691531399999e+00, 3.3406797370026002e+03 },
    { 5.4800000000000001e-09, 2.9317674351899998e+00, 3.3405451163970001e+03 },
    { 5.4400000000000002e-09, 4.0037320260999998e-01, 1.6699539015149989e+04 },
    { 5.4000000000000004e-09, 3.8747971851799998e+00, 7.9030734197210004e+03 },
    { 5.3899999999999998e-09, 3.7294272373399999e+00, 2.1781377222920000e+03 },
    { 5.3899999999999998e-09, 5.5327114575899996e+00, 3.4901756238344001e+03 },
    { 5.3800000000000000e-09, 4.7222972090599997e+00, 3.8738265101433999e+03 },
    { 5.3599999999999997e-09, 5.6929531488000000e+00, 2.6936015933849999e+03 },
    { 5.3499999999999999e-09, 3.2399073235600002e+00, 5.6421982426092000e+03 },
    { 5.3000000000000003e-09, 4.6340355355099998e+00, 1.3757737998458001e+03 },
    { 5.3000000000000003e-09, 5.1889217535799999e+00, 7.2555696517344004e+03 },
    { 5.1499999999999998e-09, 5.8668089451699998e+00, 1.3119721102825190e+04 },
    { 5.1499999999999998e-09, 3.6148323695900002e+00, 6.6757019290921999e+03 },
    { 5.1300000000000003e-09, 3.3693223915699999e+00, 3.2329489027611999e+03 },
    { 5.0600000000000003e-09, 2.3869898869999999e+00, 2.3015858159093900e+03 },
    { 5.0499999999999997e-09, 6.0930548268200004e+00, 9.4850327680040009e+03 },
    { 5.0099999999999999e-09, 1.3691839810799999e+00, 2.2434479570190001e+02 },
    { 4.9399999999999999e-09, 1.9146129803900001e+00, 1.3207029307364999e+04 },
    { 4.9200000000000004e-09, 2.7543763501599998e+00, 3.4482759506384000e+03 },
    { 4.9099999999999998e-09, 2.6037657515000001e+00, 3.7743241645766002e+03 },
    { 4.8799999999999997e-09, 1.4112038383100001e+00, 2.2218566345969998e+03 },
    { 4.8300000000000001e-09, 2.0818705898499998e+00, 1.2295954229609200e+04 },
    { 4.7699999999999999e-09, 2.9852910020099999e+00, 6.8436914895317996e+03 },
    { 4.7399999999999998e-09, 1.7715160287000001e-01, 1.3892140671893800e+04 },
    { 4.7200000000000002e-09, 5.0517945119299998e+00, 4.9367988689843996e+03 },
    { 4.6299999999999999e-09, 6.1008446616200001e+00, 3.2294257844121998e+03 },
    { 4.5299999999999999e-09, 5.2827133920000002e-01, 3.1131362939107998e+03 },
    { 4.4599999999999999e-09, 4.3099740967400004e+00, 6.0484411140864004e+03 },
    { 4.4400000000000004e-09, 2.3879765683200000e+00, 6.7051032911474003e+03 },
    { 4.4299999999999998e-09, 2.2898768550000000e-02, 3.4517990689874000e+03 },
    { 4.2500000000000000e-09, 1.3063494303900001e+00, 1.1614433293732200e+04 },
    { 4.2199999999999999e-09, 6.2250110261000002e-01, 1.9513983595104201e+04 },
    { 4.2100000000000001e-09, 2.9281123929000001e+00, 4.3796390374902003e+03 },
    { 4.1899999999999998e-09, 5.1900494769000005e-01, 1.3506508003540000e+02 },
    { 4.1299999999999996e-09, 2.7739345532500002e+00, 8.7999887137780006e+03 },
    { 4.1199999999999998e-09, 5.5949478064699996e+00, 3.5680885594888000e+03 },
    { 4.0700000000000002e-09, 5.9712357795499997e+00, 3.2893320888373901e+03 },
    { 4.0700000000000002e-09, 1.5253113908999999e-01, 3.3918927645622098e+03 },
    { 4.0199999999999998e-09, 1.4917407616000000e-01, 6.5710185321802001e+03 },
    { 4.0000000000000002e-09, 4.5355180083900004e+00, 9.5881255422226004e+03 },
    { 3.9799999999999999e-09, 1.1703249109800000e+00, 7.8959598727202001e+03 },
    { 3.9000000000000002e-09, 5.3675687815999995e-01, 6.7172527200770000e+03 },
    { 3.8700000000000001e-09, 3.1055583930899999e+00, 7.3733824546264004e+03 },
    { 3.8499999999999997e-09, 5.7766323703199998e+00, 1.0596182078434200e+04 },
    { 3.8400000000000000e-09, 4.2349830707400002e+00, 6.6851061887575997e+03 },
    { 3.7499999999999997e-09, 2.9050799537800001e+00, 1.1610910175383200e+04 },
    { 3.7399999999999999e-09, 2.4724376950400000e+00, 3.4153940252671000e+03 },
    { 3.7300000000000001e-09, 4.4406998843299998e+00, 1.0020856959031200e+04 },
    { 3.7000000000000000e-09, 6.0733140433699999e+00, 3.2741250177853999e+03 },
    { 3.6899999999999999e-09, 2.1355122962199999e+00, 1.4584298273120599e+04 },
    { 3.6600000000000002e-09, 6.2098979877999998e+00, 2.3843232707292000e+03 },
    { 3.6300000000000001e-09, 1.2220172610400000e+00, 1.0001061884606999e+04 },
    { 3.6100000000000001e-09, 8.9534079814000000e-01, 1.2721572099417001e+04 },
    { 3.6000000000000000e-09, 5.0531361410000003e-02, 3.4070998356141999e+03 },
    { 3.5699999999999999e-09, 1.6391104102200000e+00, 1.1624747044077999e+03 },
    { 3.5300000000000000e-09, 2.8715989279400000e+00, 2.0355319398799999e+01 },
    { 3.5199999999999998e-09, 5.1052280974999997e-01, 2.6308392337279997e+02 },
    { 3.4999999999999999e-09, 4.0559958828799996e+00, 9.6023526362241992e+03 },
    { 3.4600000000000000e-09, 3.6855991990799999e+00, 3.2658308281324998e+03 },
    { 3.3999999999999998e-09, 2.0742507953099998e+00, 7.3211221397135996e+03 },
    { 3.3999999999999998e-09, 3.2139500588600001e+00, 2.2324905056709398e+04 },
    { 3.3799999999999999e-09, 6.7656842154999997e-01, 9.3809596727171993e+03 },
    { 3.3799999999999999e-09, 4.7351332153000003e+00, 4.4593682188025996e+03 },
    { 3.3700000000000001e-09, 3.0060981433800000e+00, 1.5252471184999999e+01 },
    { 3.3499999999999998e-09, 2.9860421443399998e+00, 2.9389403049426001e+03 },
    { 3.3000000000000002e-09, 4.9540756154999999e-01, 3.3187615973734000e+03 },
    { 3.3000000000000002e-09, 5.6283593570899999e+00, 3.3624632560261998e+03 },
    { 3.2900000000000000e-09, 3.3722399798899998e+00, 1.0042612675591799e+04 },
    { 3.2799999999999998e-09, 6.2246193482900001e+00, 1.6460333529524989e+04 },
    { 3.2200000000000000e-09, 9.5155923545999999e-01, 1.6246663613219999e+02 },
    { 3.1800000000000002e-09, 6.0147056401999999e+00, 5.7249356974290004e+03 },
    { 3.1300000000000002e-09, 3.9899135386200002e+00, 3.2350541665740002e+02 },
    { 3.1000000000000000e-09, 3.1376754113700001e+00, 1.4421831636988400e+04 },
    { 3.0399999999999998e-09, 2.3481002538900002e+00, 7.2038022714934004e+03 },
    { 2.9800000000000000e-09, 5.2750656085500003e+00, 3.4168784979754000e+03 },
    { 2.9300000000000000e-09, 2.1184946599700001e+00, 2.6758563815697999e+03 },
    { 2.9100000000000001e-09, 6.2595212060199996e+00, 6.6451969867221997e+03 },
    { 2.8800000000000000e-09, 1.3831304090300001e+00, 1.6062184526116800e+04 },
    { 2.8400000000000001e-09, 5.6682884607999995e-01, 1.0156902360134800e+04 },
    { 2.8200000000000002e-09, 1.5183035478000000e+00, 5.7320492444297997e+03 },
    { 2.8200000000000002e-09, 9.1417686910000004e-02, 8.6030992875280003e+02 },
    { 2.7999999999999998e-09, 5.5354749010000003e-02, 3.3975108016354002e+03 },
    { 2.7999999999999998e-09, 6.0684121696300002e+00, 3.2837140517642001e+03 },
    { 2.7999999999999998e-09, 1.8381693499500000e+00, 6.6882710900975999e+03 },
    { 2.7799999999999999e-09, 4.6747238298399996e+00, 7.3221024607817999e+03 },
    { 2.7400000000000001e-09, 1.0611041590000000e-02, 1.3936794505133999e+04 },
    { 2.7200000000000001e-09, 5.8541247641599998e+00, 1.4712317116458000e+04 },
    { 2.7099999999999999e-09, 2.0291223128100002e+00, 1.1780490358516199e+04 },
    { 2.6799999999999998e-09, 1.6822498113600000e+00, 1.1766263264514600e+04 },
    { 2.6500000000000002e-09, 1.4491931645300000e+00, 7.9101869667217998e+03 },
    { 2.5899999999999999e-09, 1.5600946939300000e+00, 5.0327780962021998e+03 },
    { 2.5300000000000002e-09, 1.2054819282000000e-01, 1.7101211136907201e+04 },
    { 2.5300000000000002e-09, 1.9524001154299999e+00, 1.6703044879846799e+04 },
    { 2.5300000000000002e-09, 5.7937478483699998e+00, 6.9478318951213996e+03 },
    { 2.5300000000000002e-09, 3.9831721579199999e+00, 1.6703079387151189e+04 },
    { 2.5200000000000000e-09, 4.7234201034699996e+00, 5.3321408344359997e+02 },
    { 2.4899999999999999e-09, 3.2126804547600001e+00, 6.9927114822759995e+02 },
    { 2.4800000000000001e-09, 2.9121817627500000e+00, 2.0750449480734001e+03 },
    { 2.4600000000000002e-09, 1.1595000332900001e+00, 3.3040070613956000e+03 },
    { 2.4600000000000002e-09, 4.9642668853499998e+00, 3.3772177920039999e+03 },
    { 2.4600000000000002e-09, 2.0203315627200000e+00, 1.6276463942623001e+04 },
    { 2.4300000000000001e-09, 4.5963799228899997e+00, 5.2440492392010001e+03 },
    { 2.4300000000000001e-09, 5.8909221208200000e+00, 3.0065511840298201e+04 },
    { 2.4199999999999999e-09, 2.2578947000400000e+00, 6.6218509914859997e+03 },
    { 2.3899999999999998e-09, 1.5815702293500000e+00, 5.3054510535538002e+03 },
    { 2.3899999999999998e-09, 5.2779114611800004e+00, 6.1068800550648002e+03 },
    { 2.3499999999999999e-09, 4.6761271603300001e+00, 1.6858482532933202e+04 },
    { 2.3400000000000002e-09, 5.4780219977700000e+00, 3.1852593375684000e+03 },
    { 2.2999999999999999e-09, 8.6938411789000003e-01, 1.0309277421860000e+02 },
    { 2.2800000000000000e-09, 1.4203544099800001e+00, 1.7924910699820401e+04 },
    { 2.2600000000000001e-09, 2.1103680848600002e+00, 4.0398835749273999e+03 },
    { 2.2499999999999999e-09, 2.0747221093900001e+00, 1.0018246851447600e+04 },
    { 2.2400000000000001e-09, 2.5066693451300002e+00, 1.0025427708751200e+04 },
    { 2.2200000000000002e-09, 2.2033923392700001e+00, 1.2323423096008801e+04 },
    { 2.2200000000000002e-09, 4.8962551079000000e-01, 1.4314168113049800e+04 },
    { 2.2200000000000002e-09, 6.1578505780000004e-01, 3.4959655158311998e+03 },
    { 2.2200000000000002e-09, 1.7603242805899999e+00, 8.9828106693089994e+03 },
    { 2.2100000000000000e-09, 4.0061057008600001e+00, 4.0053684718298000e+03 },
    { 2.1799999999999999e-09, 1.7785911961300001e+00, 3.0297716278314001e+03 },
    { 2.1600000000000000e-09, 2.6726893364099999e+00, 1.5664035522708589e+04 },
    { 2.1600000000000000e-09, 4.1467994128600001e+00, 1.0037089751284400e+04 },
    { 2.1499999999999998e-09, 2.6187546673600002e+00, 3.2125935833623998e+03 },
    { 2.1499999999999998e-09, 5.9641587470099999e+00, 2.3878437747800000e+01 },
    { 2.1299999999999999e-09, 3.7075545701900001e+00, 1.0448435470975401e+04 },
    { 2.1299999999999999e-09, 1.5485340460899999e+00, 1.5453539829788001e+03 },
    { 2.1299999999999999e-09, 5.3888331766600004e+00, 7.7477203305896001e+03 },
    { 2.1200000000000001e-09, 3.2318920711499999e+00, 1.3363430027867400e+04 },
    { 2.1100000000000000e-09, 2.3315880495000001e+00, 6.9443087767724001e+03 },
    { 2.0900000000000000e-09, 3.7469825984599998e+00, 3.2304061054804001e+03 },
    { 2.0500000000000002e-09, 6.2786045733200000e+00, 2.9459865416406001e+03 },
    { 2.0400000000000000e-09, 3.5048996332000000e+00, 3.4686312700372000e+03 },
    { 2.0200000000000001e-09, 6.2000417643199999e+00, 1.2012582297140400e+04 },
    { 2.0099999999999999e-09, 5.9954600932700002e+00, 1.8353246176068001e+03 },
    { 2.0000000000000001e-09, 2.5087044335200002e+00, 6.6773435180416000e+03 },
    { 2.0000000000000001e-09, 1.4196975135500001e+00, 1.0001481960700610e+04 },
    { 1.9800000000000002e-09, 4.9375355624999999e+00, 6.5781320791810003e+03 },
    { 1.9500000000000001e-09, 3.2021480096100001e+00, 1.9034368125011999e+03 },
    { 1.9500000000000001e-09, 5.1406636577600002e+00, 6.5461597733642002e+03 },
    { 1.9300000000000002e-09, 4.3439364909700000e+00, 3.6514532255682002e+03 },
    { 1.9200000000000000e-09, 1.7087854666400000e+00, 2.7515475996916002e+03 },
    { 1.9000000000000001e-09, 5.2807642845000002e-01, 2.4803024979470001e+03 },
    { 1.8899999999999999e-09, 2.8469399344999999e+00, 2.4092493398480001e+03 },
    { 1.8600000000000000e-09, 1.4314113995900000e+00, 4.0256564809258002e+03 },
    { 1.8400000000000001e-09, 2.3672152630699999e+00, 7.2180293654950001e+03 },
    { 1.8300000000000001e-09, 5.8252958636000001e+00, 1.5508615123274400e+04 },
    { 1.8300000000000001e-09, 4.3959037465000002e+00, 1.3320548875408001e+03 },
    { 1.8199999999999999e-09, 3.9876712984200000e+00, 8.5429707060349992e+03 },
    { 1.8199999999999999e-09, 5.8098777041699998e+00, 6.8874104018367998e+03 },
    { 1.7800000000000001e-09, 3.4481929929600001e+00, 2.2847536148596000e+03 },
    { 1.7700000000000001e-09, 3.6401666914500002e+00, 6.6867477777069998e+03 },
    { 1.7700000000000001e-09, 1.0410066671000000e-01, 8.9589322315612008e+03 },
    { 1.7700000000000001e-09, 4.2251562820200004e+00, 4.6726673142405998e+03 },
    { 1.7500000000000000e-09, 4.8768023144199999e+00, 2.0775395492400001e+01 },
    { 1.7500000000000000e-09, 3.2904657010700000e+00, 8.9659784682591999e+03 },
    { 1.7400000000000000e-09, 1.6720756982500000e+00, 6.4181409300267997e+03 },
    { 1.7100000000000001e-09, 2.8666425552999999e+00, 6.7560064519669004e+03 },
    { 1.7100000000000001e-09, 5.8606273364400003e+00, 5.6183198048614004e+03 },
    { 1.7100000000000001e-09, 2.9243177012200001e+00, 1.0629050485381999e+03 },
    { 1.6999999999999999e-09, 4.7309942523900004e+00, 6.4568800576977001e+03 },
    { 1.6900000000000000e-09, 5.0893504245400001e+00, 2.6555683724738001e+03 },
    { 1.6900000000000000e-09, 1.5684943581999999e-01, 1.9645525556790599e+04 },
    { 1.6900000000000000e-09, 1.7124873423400000e+00, 1.4227094001599999e+01 },
    { 1.6900000000000000e-09, 4.9765109229300002e+00, 9.3890535407862008e+03 },
    { 1.6800000000000000e-09, 6.1824536975099997e+00, 1.3575748802237200e+04 },
    { 1.6800000000000000e-09, 6.0853944281199999e+00, 7.3140085927128002e+03 },
    { 1.6700000000000000e-09, 3.0866806063499999e+00, 9.3136308685179995e+02 },
    { 1.6600000000000001e-09, 2.2567946287199998e+00, 1.5650793750310600e+04 },
    { 1.6500000000000001e-09, 3.2692875608400001e+00, 1.9004647949408401e+04 },
    { 1.6500000000000001e-09, 1.9947686835200000e+00, 3.4508187479192002e+03 },
    { 1.6100000000000000e-09, 6.0216902494599998e+00, 2.4945295919485998e+03 },
    { 1.5900000000000001e-09, 1.0645166181100001e+00, 2.0085575391590000e+03 },
    { 1.5799999999999999e-09, 3.0391916026699999e+00, 1.4556896717023799e+04 },
    { 1.5799999999999999e-09, 4.0259682923900000e+00, 1.6484467571974001e+03 },
    { 1.5700000000000000e-09, 5.6913629905400001e+00, 5.8209149246467996e+03 },
    { 1.5700000000000000e-09, 1.7397913596000000e-01, 1.2722552420485201e+04 },
    { 1.5600000000000000e-09, 1.9188078357600000e+00, 1.9402796952816600e+04 },
    { 1.5500000000000000e-09, 7.9719420494000004e-01, 6.1550570066540004e+03 },
    { 1.5400000000000001e-09, 4.1279284727999999e-01, 3.4995559444830001e+03 },
    { 1.5400000000000001e-09, 4.6704906665000001e-01, 3.1339116894031999e+03 },
    { 1.5300000000000001e-09, 3.2872437606500000e+00, 3.3472960905739001e+03 },
    { 1.5300000000000001e-09, 2.8365231579899999e+00, 3.3339287628256998e+03 },
    { 1.5300000000000001e-09, 4.4348140597599999e+00, 3.9296772537080001e+03 },
    { 1.5100000000000000e-09, 2.9394299556000001e+00, 3.0775285033270002e+03 },
    { 1.5000000000000000e-09, 5.7527586431200000e+00, 7.3587651353180001e+02 },
    { 1.4800000000000001e-09, 1.4317570170200000e+00, 3.0536500655792001e+03 },
    { 1.4700000000000001e-09, 1.0760670017400000e+00, 6.5105519827804001e+03 },
    { 1.4599999999999999e-09, 5.2279831790999998e-01, 3.3974156888696002e+03 },
    { 1.4599999999999999e-09, 5.6009686007299999e+00, 3.2838091645300001e+03 },
    { 1.4599999999999999e-09, 2.7689169920499999e+00, 1.4788665740644001e+03 },
    { 1.4500000000000000e-09, 3.1310338458899998e+00, 1.3745346239022399e+04 },
    { 1.4500000000000000e-09, 1.8338154855300000e+00, 9.8032106819999998e-01 },
    { 1.4400000000000000e-09, 2.5785278927199999e+00, 2.0608178540718000e+03 },
    { 1.4300000000000000e-09, 3.7490671197999998e+00, 1.3553897972910800e+04 },
    { 1.3999999999999999e-09, 2.9967887542699998e+00, 8.5846616659007996e+03 },
    { 1.3999999999999999e-09, 5.6168920359500003e+00, 6.6918615187493997e+03 },
    { 1.3999999999999999e-09, 4.5277817392899999e+00, 6.6705881880498000e+03 },
    { 1.3900000000000000e-09, 5.6597638526300003e+00, 3.5473131639963999e+03 },
    { 1.3800000000000000e-09, 6.1865739236800001e+00, 2.0047197678547800e+04 },
    { 1.3800000000000000e-09, 4.5873840069699998e+00, 2.2334736210149999e+03 },
    { 1.3700000000000000e-09, 4.2443002717400002e+00, 6.6741786167015998e+03 },
    { 1.3600000000000001e-09, 5.4559423126300004e+00, 9.3748264467846002e+03 },
    { 1.3500000000000001e-09, 2.5313329078500000e+00, 2.6047359131680000e+03 },
    { 1.3500000000000001e-09, 3.5924340107899999e+00, 4.0764889402315998e+03 },
    { 1.3500000000000001e-09, 3.1778939193900002e+00, 3.6036963500726001e+03 },
    { 1.3300000000000000e-09, 4.5267907957100002e+00, 1.8606498946000200e+04 },
    { 1.3300000000000000e-09, 5.7455059890699998e+00, 5.1952447323344004e+03 },
    { 1.3300000000000000e-09, 5.8338625470099998e+00, 1.2839872288705399e+04 },
    { 1.3300000000000000e-09, 4.6927826634100001e+00, 3.6275747878203902e+03 },
    { 1.3200000000000000e-09, 4.0765469361499997e+00, 1.5265886519300400e+04 },
    { 1.3200000000000000e-09, 4.1753008952400004e+00, 3.0501269472302001e+03 },
    { 1.3100000000000000e-09, 6.4621084098000003e-01, 3.1910492295652002e+03 },
    { 1.3100000000000000e-09, 2.8429914616600001e+00, 1.1670284037296800e+04 },
    { 1.3100000000000000e-09, 5.2502555204799997e+00, 6.9016374958384004e+03 },
    { 1.2900000000000001e-09, 1.3861571491300000e+00, 7.3662689076256001e+03 },
    { 1.2799999999999999e-09, 3.5899748309400001e+00, 8.8593625756916008e+03 },
    { 1.2700000000000000e-09, 3.3156737409699999e+00, 4.2719755135515998e+03 },
    { 1.2600000000000000e-09, 3.2132251687400002e+00, 6.1480107699560003e+03 },
    { 1.2600000000000000e-09, 1.4324217407399999e+00, 7.3804960016271998e+03 },
    { 1.2500000000000000e-09, 5.0164955630000001e-02, 1.0213285546211000e+04 },
    { 1.2400000000000001e-09, 6.1167034703000001e+00, 6.5939166488598003e+03 },
    { 1.2400000000000001e-09, 3.1798319190500002e+00, 2.6316226244231998e+03 },
    { 1.2300000000000001e-09, 4.0352769609000000e-01, 1.4859801210651999e+03 },
    { 1.2300000000000001e-09, 3.5917750656599998e+00, 1.1925274092600601e+04 },
    { 1.2199999999999999e-09, 2.8479323579500000e+00, 3.3464696289993999e+03 },
    { 1.2199999999999999e-09, 3.2758345606899999e+00, 3.3347552244001999e+03 },
    { 1.2000000000000000e-09, 7.3600487057999997e-01, 1.4054607308025999e+04 },
    { 1.2000000000000000e-09, 1.4093833529699999e+00, 3.3701042450032000e+03 },
    { 1.2000000000000000e-09, 4.7143835656700004e+00, 3.3111206083963998e+03 },
    { 1.1900000000000000e-09, 5.1751257999400000e+00, 1.6511613867387401e+04 },
    { 1.1900000000000000e-09, 3.9612529251100002e+00, 1.5052878090929901e+03 },
    { 1.1900000000000000e-09, 1.9480874373799999e+00, 3.6310979061694002e+03 },
    { 1.1800000000000001e-09, 4.2034616222899999e+00, 9.8867722000640006e+03 },
    { 1.1800000000000001e-09, 3.6174053658099998e+00, 1.6958018313299999e+02 },
    { 1.1800000000000001e-09, 4.7909764819699996e+00, 3.3178439300904001e+03 },
    { 1.1800000000000001e-09, 1.3327904366700001e+00, 3.3633809233092002e+03 },
    { 1.1800000000000001e-09, 8.3092751776999996e-01, 6.6064432548323002e+03 },
    { 1.1700000000000001e-09, 9.1596190605000005e-01, 3.3371065620029999e+03 },
    { 1.1700000000000001e-09, 2.0662123590000001e+00, 3.3441182913966099e+03 },
    { 1.1700000000000001e-09, 4.0969839904900001e+00, 3.3441527987009999e+03 },
    { 1.1700000000000001e-09, 5.1683755817400003e+00, 3.3370720546985999e+03 },
    { 1.1700000000000001e-09, 1.0617952969000000e+00, 3.1842117061974000e+03 },
    { 1.1700000000000001e-09, 5.0619716217399997e+00, 3.4970131472021999e+03 },
    { 1.1700000000000001e-09, 4.6522496065299999e+00, 1.9054647649404001e+03 },
    { 1.1700000000000001e-09, 4.5107756699600001e+00, 5.5187501489918004e+03 },
    { 1.1599999999999999e-09, 1.2222210526299999e+00, 1.5906764126682599e+04 },
    { 1.1500000000000000e-09, 1.8584873052399999e+00, 6.7249437657046001e+03 },
    { 1.1500000000000000e-09, 3.2295985804299998e+00, 6.9976167230561996e+03 },
    { 1.1500000000000000e-09, 2.2040645940500001e+00, 1.1250799393421599e+04 },
    { 1.1500000000000000e-09, 1.9498409171600000e+00, 9.8156517316621994e+03 },
    { 1.1500000000000000e-09, 4.9288894799499996e+00, 2.1947111372700001e+04 },
    { 1.1400000000000000e-09, 1.5268402653099999e+00, 3.9876232600146000e+03 },
    { 1.1300000000000000e-09, 1.0132160965799999e+00, 5.4014302807716003e+03 },
    { 1.1300000000000000e-09, 4.9312188701400004e+00, 4.9225717749828000e+03 },
    { 1.1200000000000001e-09, 2.3998370042700001e+00, 2.5748193622534000e+03 },
    { 1.1100000000000001e-09, 6.2107506352300001e+00, 8.7308204539810006e+01 },
    { 1.1100000000000001e-09, 6.1318385382200002e+00, 9.0655481241287998e+03 },
    { 1.1100000000000001e-09, 2.1555950569300002e+00, 6.3577194367421998e+03 },
    { 1.1100000000000001e-09, 6.0015220095000004e+00, 1.0042192599498199e+04 },
    { 1.1100000000000001e-09, 2.4653634590900002e+00, 2.1795214091614791e+04 },
    { 1.0900000000000000e-09, 1.1106188868800000e+00, 7.6579306444639997e+02 },
    { 1.0800000000000000e-09, 6.1250867933000004e+00, 3.7352383117590002e+03 },
    { 1.0700000000000000e-09, 2.7003587091300001e+00, 2.9733880977374001e+03 },
    { 1.0700000000000000e-09, 2.2163493976000001e-01, 6.2865989683403996e+03 },
    { 1.0700000000000000e-09, 5.7869639852999999e+00, 4.8459002357928002e+03 },
    { 1.0600000000000001e-09, 1.6426574878000000e-01, 1.2509253325047201e+04 },
    { 1.0600000000000001e-09, 4.2030355248399998e+00, 2.6413412784722000e+03 },
    { 1.0600000000000001e-09, 5.1067796455699996e+00, 7.2144389368432003e+03 },
    { 1.0500000000000001e-09, 2.9461604691300001e+00, 1.0184303916231600e+04 },
    { 1.0399999999999999e-09, 3.5230601699000003e-01, 4.2969015826703999e+03 },
    { 1.0300000000000000e-09, 3.3151845467999999e+00, 6.2795527316424004e+03 },
    { 1.0200000000000000e-09, 2.8347715970700000e+00, 1.7468855197945399e+04 },
    { 1.0200000000000000e-09, 3.0318074433600000e+00, 4.0496022289764001e+03 },
    { 1.0200000000000000e-09, 3.2870966399600001e+00, 3.2827337306959998e+03 },
    { 1.0200000000000000e-09, 2.8366702786800002e+00, 3.3984911227036000e+03 },
    { 1.0100000000000000e-09, 2.1973097163499999e+00, 1.0124930054318000e+04 },
    { 1.0100000000000000e-09, 1.6116863669399999e+00, 2.1710241752912002e+03 },
    { 1.0000000000000001e-09, 8.9179712509999995e-02, 2.6887366049730601e+04 },
    { 1.0000000000000001e-09, 1.2337994791500000e+00, 6.6538232973027998e+03 },
    { 9.9000000000000010e-10, 1.0110408147000001e+00, 1.7917797152819589e+04 },
    { 9.9000000000000010e-10, 2.3174868901600001e+00, 1.0713994881326200e+04 },
    { 9.7999999999999992e-10, 5.7127925128000001e+00, 2.6793794999187999e+03 },
    { 9.7999999999999992e-10, 1.6459077876399999e+00, 5.9373861913600003e+01 },
    { 9.7999999999999992e-10, 7.0463876405000003e-01, 6.7685330579394004e+03 },
    { 9.6999999999999996e-10, 4.8036737833799998e+00, 7.4781598567299994e+01 },
    { 9.6999999999999996e-10, 2.1305934808500000e+00, 2.9221081038928000e+03 },
    { 9.6999999999999996e-10, 4.6583347676300004e+00, 5.8351420186484002e+03 },
    { 9.6999999999999996e-10, 1.8993329005699999e+00, 1.7654780539749601e+04 },
    { 9.6999999999999996e-10, 2.1157168319000000e-01, 6.7843176276182003e+03 },
    { 9.5999999999999999e-10, 3.2694534326900002e+00, 1.0006584808914400e+04 },
    { 9.5999999999999999e-10, 1.8450757606599999e+00, 1.1236572299420000e+04 },
    { 9.4000000000000006e-10, 3.1908692186200001e+00, 3.5464981478933398e+04 },
    { 9.4000000000000006e-10, 3.2506831898500002e+00, 1.2726810256271999e+03 },
    { 9.4000000000000006e-10, 4.4776783024600002e+00, 1.2942965062924000e+04 },
    { 9.4000000000000006e-10, 5.4332508600500002e+00, 1.6555603041981999e+03 },
    { 9.2999999999999999e-10, 3.5720589850900000e+00, 3.1639186965660002e+02 },
    { 9.2999999999999999e-10, 4.2345827251000001e+00, 3.5285375744623998e+03 },
    { 9.2999999999999999e-10, 2.1481062202699999e+00, 2.7717905526723998e+03 },
    { 9.2999999999999999e-10, 2.3047883796999999e-01, 1.0706881334325401e+04 },
    { 9.2999999999999999e-10, 1.2656042608800000e+00, 2.5382485042536000e+03 },
    { 9.2999999999999999e-10, 3.3944249778700000e+00, 6.8504405422599996e+02 },
    { 9.2999999999999999e-10, 1.7994940098200001e+00, 1.0050286467567201e+04 },
    { 9.2000000000000003e-10, 3.3994411635500001e+00, 3.7078367556622002e+03 },
    { 9.2000000000000003e-10, 3.9507892289999999e-02, 2.2777073781616000e+03 },
    { 9.2000000000000003e-10, 9.2188445819999998e-01, 3.4925097077850000e+03 },
    { 8.9999999999999999e-10, 3.6572145190600001e+00, 6.6475604512999996e+02 },
    { 8.9999999999999999e-10, 3.5846785410600002e+00, 6.7405987153132000e+03 },
    { 8.7999999999999996e-10, 4.4824196874800002e+00, 1.1883583132734801e+04 },
    { 8.7999999999999996e-10, 1.5286854643200001e+00, 3.0679394693482000e+03 },
    { 8.6999999999999999e-10, 4.4089706496699996e+00, 2.5407913015344002e+03 },
    { 8.6999999999999999e-10, 5.4529346930999996e+00, 3.1816689089165998e+03 },
    { 8.6999999999999999e-10, 1.3874166545200000e+00, 4.3340653342557598e+04 },
    { 8.6999999999999999e-10, 3.4820996285199999e+00, 9.8593706439672005e+03 },
    { 8.4999999999999996e-10, 7.7318635299999994e-02, 2.1548962369291799e+04 },
    { 8.4999999999999996e-10, 6.1209173605800000e+00, 1.6489763038060999e+04 },
    { 8.4999999999999996e-10, 5.9160013148899999e+00, 1.8208349942592002e+04 },
    { 8.4999999999999996e-10, 3.0449261990999998e+00, 3.6498907493555998e+03 },
    { 8.4999999999999996e-10, 3.0788407195400000e+00, 3.0313341040440000e+03 },
    { 8.4999999999999996e-10, 3.8235995373099998e+00, 4.2146206332332993e+04 },
    { 8.4999999999999996e-10, 2.3775770639999999e-02, 1.8546323056346000e+03 },
    { 8.3999999999999999e-10, 5.1190267264399996e+00, 3.1526872789372101e+03 },
    { 8.3999999999999999e-10, 4.8318830872299996e+00, 1.1527125089192399e+04 },
    { 8.3999999999999999e-10, 4.6228000344800000e+00, 3.6132853840513999e+03 },
    { 8.3999999999999999e-10, 3.1304828962000002e+00, 5.2023582793351998e+03 },
    { 8.3000000000000003e-10, 3.8162071415100001e+00, 2.2345260376108199e+04 },
    { 8.3000000000000003e-10, 3.9931669061699999e+00, 3.7591167495067998e+03 },
    { 8.3000000000000003e-10, 3.9798652040100002e+00, 4.2153969003049002e+04 },
    { 8.3000000000000003e-10, 1.1445724165000000e+00, 4.4477512323846004e+03 },
    { 8.1999999999999996e-10, 3.6932206436100001e+00, 2.0597243963041201e+04 },
    { 8.1999999999999996e-10, 1.7905650588599999e+00, 4.8859664096786000e+03 },
    { 8.0999999999999999e-10, 5.0650790580000005e-01, 3.3263853326981998e+03 },
    { 8.0999999999999999e-10, 5.6172590128400000e+00, 3.3548395207014000e+03 },
    { 8.0999999999999999e-10, 3.2999206154000000e+00, 1.5636566656309000e+04 },
    { 8.0000000000000003e-10, 5.4798761722000000e-01, 1.4422612087601999e+03 },
    { 8.0000000000000003e-10, 2.7478580938200001e+00, 8.7562698014729995e+03 },
    { 8.0000000000000003e-10, 5.8312506632299996e+00, 9.1757544453482005e+03 },
    { 8.0000000000000003e-10, 3.8271518404100000e+00, 1.2168002696574600e+04 },
    { 8.0000000000000003e-10, 9.9981693234000002e-01, 2.7417057014825201e+04 },
    { 8.0000000000000003e-10, 1.2902439393800000e+00, 1.4047493761025200e+04 },
    { 7.8999999999999996e-10, 1.5097569937499999e+00, 4.7777880408984001e+03 },
    { 7.8999999999999996e-10, 1.0051355661300001e+00, 1.1187557921027999e+03 },
    { 7.8999999999999996e-10, 2.6982680938899999e+00, 3.3787454623376002e+03 },
    { 7.8999999999999996e-10, 3.4254988247500000e+00, 3.3024793910620001e+03 },
    { 7.8999999999999996e-10, 1.1808803799200001e+00, 9.9470556815320997e+03 },
    { 7.8999999999999996e-10, 1.6779565128100000e+00, 5.5624690612967997e+03 },
    { 7.8999999999999996e-10, 5.2900738831199998e+00, 1.1695882514085999e+03 },
    { 7.8999999999999996e-10, 6.0888253438300000e+00, 3.3295667264358999e+03 },
    { 7.8999999999999996e-10, 3.4941574810000001e-02, 3.3516581269636999e+03 },
    { 7.7999999999999999e-10, 5.3095278606800003e+00, 2.6357675084636001e+04 },
    { 7.7999999999999999e-10, 5.0780163765499999e+00, 3.4363365597556003e+04 },
    { 7.7999999999999999e-10, 2.4772971270399999e+00, 1.4128242771245599e+04 },
    { 7.7999999999999999e-10, 4.8011444894500004e+00, 3.5309561079499203e+04 },
    { 7.7999999999999999e-10, 5.4750549855399999e+00, 1.4867737515892190e+04 },
    { 7.7999999999999999e-10, 1.6408389657000000e+00, 2.4606135553220000e+04 },
    { 7.7000000000000003e-10, 4.8016044263000002e-01, 4.0018453534808000e+03 },
    { 7.7000000000000003e-10, 3.9723786580099998e+00, 3.9094343007272000e+03 },
    { 7.7000000000000003e-10, 5.6830961075799999e+00, 2.0199027649330190e+04 },
    { 7.7000000000000003e-10, 8.6279166829999998e-01, 6.5496828917131998e+03 },
    { 7.7000000000000003e-10, 5.1850257951599996e+00, 5.4664898340789996e+03 },
    { 7.5999999999999996e-10, 3.1977379901199998e+00, 2.0809467624645200e+04 },
    { 7.5999999999999996e-10, 1.1543737155700000e+00, 1.9403777273884800e+04 },
    { 7.5000000000000000e-10, 1.3691452248300000e+00, 6.7106543619355998e+03 },
    { 7.5000000000000000e-10, 1.3026686561800001e+00, 7.4470179178460003e+03 },
    { 7.5000000000000000e-10, 2.4190895619399999e+00, 3.2124369052233589e+04 },
    { 7.5000000000000000e-10, 2.2661301048800002e+00, 1.5110466119866200e+04 },
    { 7.5000000000000000e-10, 5.8043929235900000e+00, 4.0398189919265998e+04 },
    { 7.4000000000000003e-10, 5.2876512352200002e+00, 1.1071388056848000e+03 },
    { 7.4000000000000003e-10, 4.2941159859500004e+00, 2.0111003364398000e+03 },
    { 7.4000000000000003e-10, 2.6486174897999999e+00, 2.5135826518314600e+04 },
    { 7.4000000000000003e-10, 7.0811376981999996e-01, 1.9800945956224801e+04 },
    { 7.4000000000000003e-10, 1.1595867971400000e+00, 1.7395219734725801e+04 },
    { 7.2999999999999996e-10, 2.5775180604100001e+00, 4.2942504339149389e+04 },
    { 7.2999999999999996e-10, 3.2352706125599999e+00, 2.0679314010725998e+03 },
    { 7.2999999999999996e-10, 3.7822309935899998e+00, 1.3369563253800001e+04 },
    { 7.2999999999999996e-10, 7.1840007075000001e-01, 6.3648329837430001e+03 },
    { 7.2999999999999996e-10, 4.9344366469000001e+00, 1.0730827082375990e+04 },
    { 7.2999999999999996e-10, 1.4291958571100001e+00, 4.7757600884592002e+03 },
    { 7.2000000000000000e-10, 1.2986754670200000e+00, 2.0206141196330998e+04 },
    { 7.2000000000000000e-10, 5.3255897203200000e+00, 2.5287723799399799e+04 },
    { 7.2000000000000000e-10, 4.7301045978000000e-01, 2.4889574795991601e+04 },
    { 7.2000000000000000e-10, 1.0487011096000001e+00, 3.3391279539914999e+03 },
    { 7.2000000000000000e-10, 5.0750658090399998e+00, 3.3420968994080999e+03 },
    { 7.1000000000000003e-10, 1.8271516226800000e+00, 1.5121102785216000e+04 },
    { 6.9999999999999996e-10, 3.9852574719000000e-01, 1.2725453434775000e+04 },
    { 6.9999999999999996e-10, 5.5862139735999996e+00, 3.5707710082907397e+04 },
    { 6.9999999999999996e-10, 1.6263364804300000e+00, 2.1265523126520198e+04 },
    { 6.9999999999999996e-10, 6.2659249939199997e+00, 4.0000040915857790e+04 },
    { 6.9000000000000000e-10, 4.4688257915299996e+00, 1.2797945726280000e+03 },
    { 6.9000000000000000e-10, 3.4769924880000000e+00, 1.1353892167640201e+04 },
    { 6.9000000000000000e-10, 2.2782599069999999e-02, 4.3738802345965800e+04 },
    { 6.9000000000000000e-10, 4.5288984584999997e-01, 1.0177190369230801e+04 },
    { 6.9000000000000000e-10, 3.5279777024999998e-01, 1.6858415222630389e+04 },
    { 6.9000000000000000e-10, 7.1335370910000001e-02, 1.0721108428326999e+04 },
    { 6.8000000000000003e-10, 3.3194254534500001e+00, 4.1064054911461999e+03 },
    { 6.8000000000000003e-10, 3.2850981308499998e+00, 1.2808880303956799e+04 },
    { 6.8000000000000003e-10, 4.2906740810999999e-01, 6.6147374444852003e+03 },
    { 6.8000000000000003e-10, 4.3585016969900003e+00, 8.1865126624925997e+03 },
    { 6.8000000000000003e-10, 5.1488267565900001e+00, 4.1429763491459998e+03 },
    { 6.6999999999999996e-10, 3.0915717232400000e+00, 9.3296796788726006e+03 },
    { 6.6999999999999996e-10, 3.8828142228200000e+00, 4.1427486983178802e+04 },
    { 6.6000000000000000e-10, 6.1825361829799999e+00, 1.3517802795930600e+04 },
    { 6.6000000000000000e-10, 5.7948870187399999e+00, 3.4142133826149998e+03 },
    { 6.6000000000000000e-10, 3.2887989989999999e-01, 3.2670114707846001e+03 },
    { 6.6000000000000000e-10, 4.7197857946799999e+00, 9.9979588423515997e+03 },
    { 6.5000000000000003e-10, 5.4758379352800004e+00, 6.7086264094963999e+03 },
    { 6.5000000000000003e-10, 4.8192608136299997e+00, 4.1404335518651997e+03 },
    { 6.5000000000000003e-10, 5.6518801728300003e+00, 6.6808047773059998e+03 },
    { 6.5000000000000003e-10, 5.3302236602299997e+00, 2.0213254743331789e+04 },
    { 6.3999999999999996e-10, 2.7819919859900000e+00, 9.8014246376606006e+03 },
    { 6.3999999999999996e-10, 5.4161822504200003e+00, 3.1938007401762002e+03 },
    { 6.3999999999999996e-10, 7.0758466821999999e-01, 3.4874241132234001e+03 },
    { 6.3999999999999996e-10, 4.4345729345600002e+00, 4.0796338922674193e+04 },
    { 6.3999999999999996e-10, 2.0396563407700001e+00, 1.0787630344545800e+04 },
    { 6.3999999999999996e-10, 5.6107266994199998e+00, 5.2236939198022001e+03 },
    { 6.3999999999999996e-10, 1.9653996391199999e+00, 7.7999806455024000e+03 },
    { 6.3999999999999996e-10, 1.4760244930599999e+00, 1.3295120902599999e+03 },
    { 6.3000000000000000e-10, 6.2831050139000002e+00, 2.0742945708426400e+04 },
    { 6.3000000000000000e-10, 3.2856760942000002e-01, 1.2082655134171200e+04 },
    { 6.2000000000000003e-10, 1.6755697300000001e+00, 1.8461715211915791e+04 },
    { 6.2000000000000003e-10, 1.4712984754500000e+00, 1.8451078546565990e+04 },
    { 6.2000000000000003e-10, 6.1927789333000005e-01, 2.0040151441849790e+04 },
    { 6.2000000000000003e-10, 4.4445207603499997e+00, 7.9982112516539996e+02 },
    { 6.2000000000000003e-10, 2.8551837216500000e+00, 1.2988111830836000e+04 },
    { 6.2000000000000003e-10, 1.3940814153600001e+00, 3.3574446277495899e+03 },
    { 6.2000000000000003e-10, 4.7296855032799998e+00, 3.3237802256499999e+03 },
    { 6.0999999999999996e-10, 1.4690329502899999e+00, 1.4591411820121401e+04 },
    { 6.0999999999999996e-10, 8.7700826571000001e-01, 3.6605365304199999e+01 },
    { 6.0999999999999996e-10, 2.8066486029500002e+00, 2.6555010621709998e+03 },
    { 6.0999999999999996e-10, 2.3725967736800002e+00, 1.9617076369322789e+04 },
    { 6.0000000000000000e-10, 4.0242239432600000e+00, 1.0016314355791999e+04 },
    { 6.0000000000000000e-10, 1.1058561538800000e+00, 5.5116366019910001e+03 },
    { 6.0000000000000000e-10, 3.2631347357000000e+00, 4.6283116765849190e+04 },
    { 5.9000000000000003e-10, 4.2163716105999999e-01, 5.9453483398678000e+03 },
    { 5.9000000000000003e-10, 4.2564724210399998e+00, 1.2516366872048000e+04 },
    { 5.9000000000000003e-10, 4.6116095408900000e+00, 4.3562754445840001e+03 },
    { 5.9000000000000003e-10, 5.6447699815999997e-01, 4.2897880356695996e+03 },
    { 5.9000000000000003e-10, 3.5499667537800002e+00, 2.5239591004846002e+03 },
    { 5.7999999999999996e-10, 5.3219837864799997e+00, 4.1748057328924799e+04 },
    { 5.7999999999999996e-10, 4.8659635216400003e+00, 1.5423267560871000e+04 },
    { 5.7999999999999996e-10, 1.3644789395200001e+00, 8.2265788363783995e+03 },
    { 5.7999999999999996e-10, 2.2442483444000000e-01, 3.4126681600545999e+03 },
    { 5.7999999999999996e-10, 5.8993420841999997e+00, 3.2685566933450000e+03 },
    { 5.7999999999999996e-10, 1.9061110078800001e+00, 2.9580054632350191e+04 },
    { 5.7999999999999996e-10, 1.5557996647100001e+00, 1.0156630178842000e+03 },
    { 5.7999999999999996e-10, 3.3842400693600001e+00, 5.6253660415594004e+03 },
    { 5.7000000000000000e-10, 4.8596840447499998e+00, 1.2928737968922400e+04 },
    { 5.7000000000000000e-10, 1.8737049942999999e+00, 2.4491425792583399e+04 },
    { 5.7000000000000000e-10, 2.7105593392700000e+00, 4.2544355335741202e+04 },
    { 5.7000000000000000e-10, 2.8605730302699999e+00, 5.4085438277723997e+03 },
    { 5.6000000000000003e-10, 1.1281613229100000e+00, 1.6063164847185000e+04 },
    { 5.6000000000000003e-10, 1.6121742866100000e+00, 2.9784912899689998e+03 },
    { 5.6000000000000003e-10, 4.5115926320400002e+00, 3.7027335634306000e+03 },
    { 5.6000000000000003e-10, 5.2972940042800003e+00, 1.7660303464057000e+04 },
    { 5.6000000000000003e-10, 5.5114739150699998e+00, 2.3539707386332801e+04 },
    { 5.6000000000000003e-10, 1.8326792187100001e+00, 1.6328724257535790e+04 },
    { 5.4999999999999996e-10, 3.9857816295299999e+00, 1.2715438873484400e+04 },
    { 5.4999999999999996e-10, 5.8672057015099996e+00, 3.3442028553516002e+03 },
    { 5.4999999999999996e-10, 1.9083697686300001e+00, 1.9719521738548001e+03 },
    { 5.4999999999999996e-10, 2.4760733724600001e+00, 4.4035174752379999e+03 },
    { 5.4000000000000000e-10, 2.7191025325900000e+00, 5.6076158292088003e+03 },
    { 5.4000000000000000e-10, 5.8067082075899998e+00, 1.8849227549974199e+04 },
    { 5.4000000000000000e-10, 5.9024312066500002e+00, 1.3532029889932201e+04 },
    { 5.4000000000000000e-10, 1.4138793935500000e+00, 2.1150813365883601e+04 },
    { 5.4000000000000000e-10, 2.0625745906100001e+00, 2.7357626859950001e+03 },
    { 5.4000000000000000e-10, 6.0638309736100000e+00, 2.0426571092422000e+04 },
    { 5.4000000000000000e-10, 5.7563887352499998e+00, 1.0558588118401999e+03 },
    { 5.3000000000000003e-10, 3.0969373893499998e+00, 6.7888883773382004e+03 },
    { 5.3000000000000003e-10, 2.1968979179499999e+00, 1.4481205498902000e+04 },
    { 5.3000000000000003e-10, 1.5615144356999999e-01, 2.5841293202242188e+04 },
    { 5.3000000000000003e-10, 1.4067614419000001e+00, 2.3914368177299998e+03 },
    { 5.3000000000000003e-10, 4.6935053803999999e-01, 4.4959547675990201e+04 },
    { 5.3000000000000003e-10, 2.9350448824800002e+00, 3.7895426290367403e+04 },
    { 5.3000000000000003e-10, 6.0717412230500001e+00, 3.8086874556479001e+04 },
    { 5.3000000000000003e-10, 5.2738105023499999e+00, 8.3367851575978002e+03 },
    { 5.1999999999999996e-10, 3.1501022084499999e+00, 6.8127668150860000e+03 },
    { 5.1999999999999996e-10, 4.4534317866800004e+00, 4.5884967762441003e+04 },
    { 5.1999999999999996e-10, 1.4556479817700001e+00, 2.2487371692841600e+04 },
    { 5.1999999999999996e-10, 3.8550282604900001e+00, 3.2765246659615801e+04 },
    { 5.1999999999999996e-10, 6.0554758880000004e-01, 2.9632388187840002e+03 },
    { 5.1999999999999996e-10, 5.5182193298400000e+00, 3.7179860346156001e+03 },
    { 5.1999999999999996e-10, 4.3977336729800003e+00, 6.6669977593980002e+03 },
    { 5.1999999999999996e-10, 3.2912116500500002e+00, 3.3821632176846001e+03 },
    { 5.1999999999999996e-10, 2.8325552685900002e+00, 3.2990616357150002e+03 },
    { 5.1999999999999996e-10, 7.8418608919999999e-01, 1.4610540519464000e+03 },
    { 5.1000000000000000e-10, 4.5973209352400000e+00, 1.2410731300548599e+04 },
    { 5.1000000000000000e-10, 3.1363613911999999e+00, 5.7498617665477996e+03 },
    { 5.1000000000000000e-10, 1.0193948277000000e+00, 1.7810200939183800e+04 },
    { 5.0000000000000003e-10, 9.5426876070000000e-02, 1.7276610936682000e+03 },
    { 5.0000000000000003e-10, 1.7054940736399999e+00, 3.5649572224017002e+03 },
    { 5.0000000000000003e-10, 6.1305007884700000e+00, 4.4768099409878603e+04 },
    { 5.0000000000000003e-10, 2.1936092777399998e+00, 3.7422845484569998e+03 },
    { 5.0000000000000003e-10, 5.8915391622800000e+00, 7.7883636590843998e+03 },
    { 5.0000000000000003e-10, 2.2094418625299999e+00, 5.2616784674559995e+02 },
    { 4.8999999999999996e-10, 6.0880038382799997e+00, 1.6865528769631201e+04 },
    { 4.8999999999999996e-10, 1.4789096228400000e+00, 5.5516418709848200e+04 },
    { 4.8999999999999996e-10, 2.8000482201399999e+00, 3.1214201436967800e+04 },
    { 4.8999999999999996e-10, 5.7353022724700002e+00, 1.5849865751747000e+04 },
    { 4.8999999999999996e-10, 3.3186258360499998e+00, 4.0257237912286000e+03 },
    { 4.8000000000000000e-10, 3.1237370396099999e+00, 1.8830814931023999e+03 },
    { 4.8000000000000000e-10, 6.2350780583300001e+00, 1.9190478178446800e+04 },
    { 4.8000000000000000e-10, 6.0926519192399997e+00, 2.4341528317456599e+04 },
    { 4.8000000000000000e-10, 1.3701747753500000e+00, 8.4608283475120004e+02 },
    { 4.8000000000000000e-10, 7.5363318362999998e-01, 3.2643463554241998e+03 },
    { 4.8000000000000000e-10, 5.0504780260400004e+00, 1.4955045720431999e+04 },
    { 4.8000000000000000e-10, 5.6426993725800001e+00, 1.8592030553546001e+03 },
    { 4.8000000000000000e-10, 4.2051619653900003e+00, 2.1258100970763999e+03 },
    { 4.8000000000000000e-10, 3.8320382112000001e-01, 1.2825645194703800e+04 },
    { 4.8000000000000000e-10, 3.2582553759500001e+00, 3.9601891912449602e+04 },
    { 4.7000000000000003e-10, 5.4813102994999996e+00, 1.1876469585733999e+04 },
    { 4.7000000000000003e-10, 4.2344784487400000e+00, 2.2573520587628000e+03 },
    { 4.7000000000000003e-10, 2.2369396979500000e+00, 2.7832038219283200e+04 },
    { 4.7000000000000003e-10, 5.5329946131599996e+00, 2.1000915890756802e+04 },
    { 4.7000000000000003e-10, 1.0916208217800001e+00, 2.0735832161425591e+04 },
    { 4.7000000000000003e-10, 4.1637997800899997e+00, 8.0132797409404002e+03 },
    { 4.7000000000000003e-10, 5.9490003666200000e+00, 4.5088669755624600e+04 },
    { 4.6000000000000001e-10, 4.9894902101399996e+00, 1.7952584437210000e+03 },
    { 4.6000000000000001e-10, 4.8225160264099998e+00, 5.8814037282341997e+03 },
    { 4.6000000000000001e-10, 3.9705030142900002e+00, 1.8052929543157799e+04 },
    { 4.6000000000000001e-10, 4.2675039162000000e-01, 2.3249494088155998e+03 },
    { 4.6000000000000001e-10, 2.0101213210000002e+00, 1.9247376553382401e+04 },
    { 4.5000000000000000e-10, 1.5821675827699999e+00, 3.9733961660129999e+03 },
    { 4.5000000000000000e-10, 3.0989961367899999e+00, 1.2655674786264001e+03 },
    { 4.5000000000000000e-10, 4.8075671546200001e+00, 3.9048322509607198e+04 },
    { 4.5000000000000000e-10, 8.6869575513999997e-01, 2.8230187222691391e+04 },
    { 4.5000000000000000e-10, 4.2233240451400000e+00, 3.1162676309979001e+03 },
    { 4.5000000000000000e-10, 6.0030640858900002e+00, 2.2747613278899999e+02 },
    { 4.5000000000000000e-10, 5.8646283625700004e+00, 3.9662826190122000e+03 },
    { 4.5000000000000000e-10, 2.5913855607000003e-01, 2.7149422343873998e+03 },
    { 4.5000000000000000e-10, 5.9000930611299998e+00, 3.3716965065866403e+04 },
    { 4.5000000000000000e-10, 2.5745674673000001e+00, 4.1572657529149901e+03 },
    { 4.3999999999999998e-10, 5.4252101928999996e-01, 1.4469588512484001e+04 },
    { 4.3999999999999998e-10, 3.9621574967000001e-01, 1.4061720855026801e+04 },
    { 4.3999999999999998e-10, 4.2384871155999999e+00, 3.4365916539176001e+03 },
    { 4.3999999999999998e-10, 1.8852798030400000e+00, 3.2446331994819998e+03 },
    { 4.3999999999999998e-10, 4.4985718102199996e+00, 1.7499360140315399e+04 },
    { 4.3999999999999998e-10, 5.6468376175300001e+00, 2.9698287511335799e+04 },
    { 4.3000000000000001e-10, 1.1849141753200001e+00, 9.7670387712897180e+04 },
    { 4.3000000000000001e-10, 3.7416970481299998e+00, 1.2199975002391400e+04 },
    { 4.3000000000000001e-10, 1.9555197446699999e+00, 1.5106875691214400e+04 },
    { 4.3000000000000001e-10, 3.3738122271999998e-01, 8.1234860621598000e+03 },
    { 4.3000000000000001e-10, 4.9386631705499999e+00, 3.8265844794894001e+03 },
    { 4.3000000000000001e-10, 1.1851037480900000e+00, 2.8546403739102002e+03 },
    { 4.3000000000000001e-10, 1.8467441633399999e+00, 4.6681265769257399e+04 },
    { 4.3000000000000001e-10, 4.4673415488000003e+00, 4.8835193856448590e+04 },
    { 4.3000000000000001e-10, 4.0070595587000002e-01, 1.6872642316631998e+04 },
    { 4.2000000000000000e-10, 4.3412645645200003e+00, 1.2729665967486000e+04 },
    { 4.2000000000000000e-10, 5.5227572894200003e+00, 2.6519779438219998e+03 },
    { 4.2000000000000000e-10, 6.2187667622999998e-01, 2.4076444588125400e+04 },
    { 4.2000000000000000e-10, 4.1661355281000001e-01, 1.0191417463232399e+04 },
    { 4.2000000000000000e-10, 3.1895024470000002e+00, 2.4093276789175201e+04 },
    { 4.2000000000000000e-10, 1.1876954261399999e+00, 2.0235122826310399e+04 },
    { 4.2000000000000000e-10, 2.2264878015299998e+00, 1.2979553174576000e+04 },
    { 4.0999999999999998e-10, 5.4322526507799997e+00, 3.3852662599400001e+03 },
    { 4.0999999999999998e-10, 6.9151426786000003e-01, 3.2959585934595998e+03 },
    { 4.0999999999999998e-10, 1.9422422403699999e+00, 4.6701245169597996e+03 },
    { 4.0999999999999998e-10, 5.6292691230100003e+00, 3.0220932239732399e+04 },
    { 4.0999999999999998e-10, 3.6816181119700002e+00, 1.7013902932367400e+04 },
    { 4.0999999999999998e-10, 6.1072175062299996e+00, 1.1128976085784199e+04 },
    { 4.0000000000000001e-10, 1.4266487249599999e+00, 2.6239442205650390e+04 },
    { 4.0000000000000001e-10, 1.1193016176899999e+00, 1.4577184726119800e+04 },
    { 4.0000000000000001e-10, 4.7215873715400001e+00, 2.8144445799542000e+03 },
    { 4.0000000000000001e-10, 3.6674922708000002e-01, 2.0728718614424790e+04 },
    { 4.0000000000000001e-10, 8.3205528676999996e-01, 3.8252024502790802e+04 },
    { 4.0000000000000001e-10, 4.0604189480899997e+00, 3.9454621674046002e+03 },
    { 4.0000000000000001e-10, 4.3032827579999999e+00, 5.2175806283148399e+04 },
    { 4.0000000000000001e-10, 2.6433287747400001e+00, 6.8508050365325998e+03 },
    { 4.0000000000000001e-10, 3.0733029530999998e+00, 3.5244820136017902e+03 },
    { 4.0000000000000001e-10, 3.0504639655400001e+00, 3.1567428397978001e+03 },
    { 3.9000000000000000e-10, 3.6179445759100002e+00, 2.7433889215874991e+04 },
    { 3.9000000000000000e-10, 2.0988063507399999e+00, 7.4299006053259995e+02 },
    { 3.9000000000000000e-10, 1.2395568176400000e+00, 3.3163395663023999e+04 },
    { 3.9000000000000000e-10, 1.8287390407499999e+00, 2.7149872905026000e+03 },
    { 3.9000000000000000e-10, 2.8379205504699998e+00, 3.3371566186536002e+03 },
    { 3.9000000000000000e-10, 1.4425371458000000e-01, 3.3440682347460001e+03 },
    { 3.9000000000000000e-10, 1.9824242407399999e+00, 1.1567191263078201e+04 },
    { 3.9000000000000000e-10, 5.0891895187899996e+00, 4.9225580189140797e+04 },
    { 3.7999999999999998e-10, 2.8229064898699998e+00, 4.1194487926082402e+04 },
    { 3.7999999999999998e-10, 4.5230071452100002e+00, 8.7420427074714007e+03 },
    { 3.7999999999999998e-10, 4.6008508979700000e+00, 1.4849324896942001e+03 },
    { 3.7999999999999998e-10, 5.2553634040699997e+00, 2.8628336226099600e+04 },
    { 3.7999999999999998e-10, 5.5869201636000003e+00, 3.0704822666290002e+03 },
    { 3.7999999999999998e-10, 5.3684675504000001e-01, 3.6107425867706002e+03 },
    { 3.7999999999999998e-10, 4.4443260976200003e+00, 3.4115114069274598e+04 },
    { 3.7999999999999998e-10, 4.7836625030800004e+00, 5.3517127631395997e+03 },
    { 3.7000000000000001e-10, 2.5331549199899999e+00, 3.0171070100423999e+03 },
    { 3.7000000000000001e-10, 2.8632368348299999e+00, 3.6261279485749801e+04 },
    { 3.7000000000000001e-10, 4.0219797359999997e-02, 4.8827431185732603e+04 },
    { 3.7000000000000001e-10, 6.6378563361999998e-01, 9.3848410080752001e+03 },
    { 3.7000000000000001e-10, 5.7098931703900000e+00, 2.0995392966449399e+04 },
    { 3.7000000000000001e-10, 5.5320188307200002e+00, 4.5486818759032787e+04 },
    { 3.6000000000000000e-10, 5.5820934439999998e+00, 2.0897822303990001e+03 },
    { 3.6000000000000000e-10, 5.3702209901300000e+00, 1.9676450231236398e+04 },
    { 3.6000000000000000e-10, 6.6244555675000005e-01, 3.7626398678557998e+03 },
    { 3.6000000000000000e-10, 5.4613213618899996e+00, 2.9185849855438000e+03 },
    { 3.6000000000000000e-10, 6.0533442921200002e+00, 2.7682140744156401e+04 },
    { 3.6000000000000000e-10, 1.2181747874400000e+00, 4.5554147563232000e+03 },
    { 3.6000000000000000e-10, 2.1485705089099998e+00, 6.6446194880953999e+03 },
    { 3.4999999999999998e-10, 1.4187010393099999e+00, 1.6850521225016000e+03 },
    { 3.4999999999999998e-10, 4.9923431768300004e+00, 4.7163862265456000e+03 },
    { 3.4999999999999998e-10, 3.0978766162700002e+00, 1.5010896463996600e+04 },
    { 3.4999999999999998e-10, 7.0040109787000004e-01, 1.7932024246821202e+04 },
    { 3.4999999999999998e-10, 4.9122737059499997e+00, 2.7035740212466801e+04 },
    { 3.4999999999999998e-10, 6.2795902728400002e+00, 2.3017062657936200e+04 },
    { 3.4999999999999998e-10, 2.0435016955799998e+00, 3.3036168360888000e+03 },
    { 3.4999999999999998e-10, 4.0802652230699996e+00, 3.3776080173107998e+03 },
    { 3.4000000000000001e-10, 3.2723184080899999e+00, 2.9181905628942000e+04 },
    { 3.4000000000000001e-10, 6.0904693499700002e+00, 2.1850829326400000e+01 },
    { 3.4000000000000001e-10, 1.8441403552200000e+00, 1.3355336159798400e+04 },
    { 3.4000000000000001e-10, 1.3970294522800000e+00, 6.8365779425310002e+03 },
    { 3.4000000000000001e-10, 4.5367448458300004e+00, 3.3704944703100000e+03 },
    { 3.4000000000000001e-10, 1.5870220728100000e+00, 3.3107303830895999e+03 },
    { 3.4000000000000001e-10, 1.2135437925000001e+00, 2.3937856389740999e+04 },
    { 3.4000000000000001e-10, 2.8993538672999999e+00, 7.4835887758458002e+03 },
    { 3.4000000000000001e-10, 3.0455295160000000e-02, 1.6702994823196201e+04 },
    { 3.4000000000000001e-10, 4.2126352580099997e+00, 3.3370219980480001e+03 },
    { 3.3000000000000000e-10, 4.7394319938700002e+00, 1.2502139778046399e+04 },
    { 3.3000000000000000e-10, 5.9700860392999999e+00, 3.7234917052708001e+03 },
    { 3.3000000000000000e-10, 1.5368087933999999e-01, 2.9577331481288002e+03 },
    { 3.3000000000000000e-10, 2.7421561739899998e+00, 1.4164813629245400e+04 },
    { 3.3000000000000000e-10, 4.8939984437200001e+00, 3.7455726495974399e+04 },
    { 3.3000000000000000e-10, 1.1915163974700000e+00, 1.6342951351537400e+04 },
    { 3.3000000000000000e-10, 6.1783437545800002e+00, 4.9535637597313998e+03 },
    { 3.3000000000000000e-10, 2.2540099511400000e+00, 2.4532976583568201e+04 },
    { 3.3000000000000000e-10, 5.5011585007799999e+00, 1.9956366355659000e+04 },
    { 3.1999999999999998e-10, 1.8867707787100001e+00, 4.4238727946367999e+03 },
    { 3.1999999999999998e-10, 3.3072198927200001e+00, 5.4156573747732000e+03 },
    { 3.1999999999999998e-10, 3.9012105411500002e+00, 2.0354515359067202e+04 },
    { 3.1999999999999998e-10, 2.9956942153699999e+00, 1.1979701285736001e+03 },
    { 3.1999999999999998e-10, 3.0410862257000000e+00, 3.6105859086315599e+04 },
    { 3.1999999999999998e-10, 6.0384612281000005e-01, 4.0292469095776000e+03 },
    { 3.1999999999999998e-10, 2.7150066713099998e+00, 4.1236038717067197e+04 },
    { 3.1999999999999998e-10, 3.3838806731200002e+00, 4.5030871311075998e+03 },
    { 3.1999999999999998e-10, 3.1779665852700001e+00, 3.3561544666432201e+04 },
    { 3.1999999999999998e-10, 5.8443662081900003e+00, 6.8307880505342000e+03 },
    { 3.1999999999999998e-10, 1.3863870306799999e+00, 3.6659428489158003e+04 },
    { 3.1000000000000002e-10, 2.0199573234299999e+00, 3.1014069410748002e+03 },
    { 3.1000000000000002e-10, 4.1038095952100004e+00, 3.5798179123248001e+03 },
    { 3.1000000000000002e-10, 5.0017802841000003e-01, 1.3605178310773201e+04 },
    { 3.1000000000000002e-10, 1.7140688402900000e+00, 9.0726616711295992e+03 },
    { 3.0000000000000000e-10, 1.8721594706200000e+00, 3.2769127994973802e+04 },
    { 3.0000000000000000e-10, 4.8379354672400003e+00, 9.9187445058808007e+03 },
    { 3.0000000000000000e-10, 4.3136817153899996e+00, 3.9662375628969999e+03 },
    { 3.0000000000000000e-10, 6.3974818582000004e-01, 1.7232753098593599e+04 },
    { 3.0000000000000000e-10, 4.7691051204000001e-01, 4.8220217980449997e+03 },
    { 3.0000000000000000e-10, 1.2959058539899999e+00, 1.2995225377836799e+04 },
    { 2.8999999999999998e-10, 5.3775707851199996e+00, 3.0376352639166598e+04 },
    { 2.8999999999999998e-10, 3.0154541535399999e+00, 2.6880319813032602e+04 },
    { 2.8999999999999998e-10, 3.3271254368999997e-01, 4.1990785932898798e+04 },
    { 2.8999999999999998e-10, 5.8910694177000000e-01, 3.3318816062458201e+04 },
    { 2.8000000000000002e-10, 3.3692094535599999e+00, 3.8813356576349201e+04 },
    { 2.8000000000000002e-10, 3.8926435489600002e+00, 3.3483750974158002e+03 },
    { 2.8000000000000002e-10, 2.2311233696800001e+00, 3.3328497559838002e+03 },
    { 2.8000000000000002e-10, 3.9015533956700001e+00, 7.3459808985296004e+03 },
    { 2.8000000000000002e-10, 4.5077221068000001e-01, 3.5173061568336002e+03 },
    { 2.8000000000000002e-10, 5.6729947079600000e+00, 3.1639186965660001e+03 },
    { 2.7000000000000000e-10, 9.5617807815000000e-01, 7.5344212351515998e+03 },
};

static const VSOP87Term VSOP87A_MARS_Y2[384] = {
    { 3.5396765000000001e-04, 3.1415926535900001e+00, 0.0000000000000000e+00 },
    { 2.1950759000000000e-04, 1.9629159494599999e+00, 6.6812248533995999e+03 },
    { 6.0759900000000001e-05, 2.3586432100099999e+00, 1.0021837280099400e+04 },
    { 2.5714249999999999e-05, 5.6479574532700001e+00, 3.3406124266997999e+03 },
    { 1.1290990000000000e-05, 2.7157624896299999e+00, 1.3362449706799200e+04 },
    { 1.8244300000000000e-06, 3.0633505046199998e+00, 1.6703062133498999e+04 },
    { 1.6835700000000001e-06, 3.0101787807300000e+00, 3.3370893083507999e+03 },
    { 1.6774700000000001e-06, 3.1092270291099999e+00, 3.3441355450487999e+03 },
    { 9.3890000000000002e-07, 5.6591303845800001e+00, 3.1851920272655998e+03 },
    { 8.9243999999999996e-07, 4.6137270695999999e-01, 3.4960328261340001e+03 },
    { 4.0563000000000001e-07, 3.4593107099900000e+00, 6.6847479717486003e+03 },
    { 3.6610999999999999e-07, 1.9723333298000001e-01, 1.0593819301891999e+03 },
    { 2.7525999999999999e-07, 3.4069758413200000e+00, 2.0043674560198800e+04 },
    { 2.4573999999999998e-07, 2.0570926173499999e+00, 3.1491641605882000e+03 },
    { 2.4401000000000003e-07, 4.2019785867900001e+00, 2.5443144198834002e+03 },
    { 2.3302000000000000e-07, 4.0605470345799999e+00, 3.5320606928113998e+03 },
    { 2.0298999999999999e-07, 1.9114799033700001e+00, 4.1369104335162001e+03 },
    { 1.8006000000000001e-07, 1.8519872894800000e+00, 3.0978838227257902e+03 },
    { 1.7221000000000000e-07, 2.7416186758199999e+00, 2.9424634232916001e+03 },
    { 1.6901999999999999e-07, 4.2513618631999996e+00, 3.5833410306738001e+03 },
    { 1.6812000000000000e-07, 7.7954286997000000e-01, 2.7870430238573999e+03 },
    { 1.5856000000000001e-07, 6.8419184100000002e-02, 1.1944470102246000e+03 },
    { 1.5776999999999999e-07, 5.9171878113099998e+00, 5.6218429232103999e+03 },
    { 1.5264000000000000e-07, 3.3525493118899998e+00, 3.7387614301079998e+03 },
    { 1.5019000000000000e-07, 4.9704749309100000e+00, 1.5925960136327999e+03 },
    { 1.3747000000000000e-07, 5.3111641976800001e+00, 3.8941818295421999e+03 },
    { 1.3171000000000001e-07, 3.5787733540000000e+00, 1.9907450170410000e+03 },
    { 1.2704999999999999e-07, 4.2580274785899999e+00, 1.5542039943419999e+02 },
    { 1.2592999999999999e-07, 2.1801331083700002e+00, 2.3888940204492001e+03 },
    { 1.2302000000000000e-07, 5.5471365171100002e+00, 3.3396321056316001e+03 },
    { 1.2284000000000001e-07, 5.7411575365000000e-01, 3.3415927477680002e+03 },
    { 1.1289000000000000e-07, 6.0129097903300002e+00, 6.5258044539654002e+03 },
    { 1.1117000000000000e-07, 1.4053037854499999e+00, 7.9629800681639995e+02 },
    { 1.0577000000000000e-07, 5.9752271503999999e-01, 4.3999943568890003e+03 },
    { 1.0097999999999999e-07, 5.4774769122500002e+00, 2.1461654164751999e+03 },
    { 8.8629999999999994e-08, 3.8816963761699999e+00, 4.2923308329503998e+03 },
    { 8.8629999999999994e-08, 5.1480663562999995e-01, 4.5350594369244000e+03 },
    { 8.3220000000000000e-08, 1.1126421154899999e+00, 5.0886288397668004e+03 },
    { 8.3080000000000000e-08, 5.9498076679400000e+00, 2.9577158944766002e+03 },
    { 8.1209999999999997e-08, 3.7826060157499999e+00, 1.0025360398448400e+04 },
    { 8.1199999999999999e-08, 2.4813249766299998e+00, 4.6904798363585996e+03 },
    { 8.0610000000000005e-08, 6.0260395888299998e+00, 5.4867778431750003e+03 },
    { 7.5460000000000002e-08, 7.2982074611000003e-01, 1.5890728952838001e+03 },
    { 7.3059999999999994e-08, 1.5133585090000001e-01, 3.7235089589230001e+03 },
    { 6.1210000000000003e-08, 3.8054769491300000e+00, 7.0793738568077997e+03 },
    { 5.7000000000000001e-08, 3.2366845409299998e+00, 7.1135470007999997e+00 },
    { 5.2940000000000003e-08, 2.4942428089400002e+00, 6.4897765872880000e+03 },
    { 5.2920000000000000e-08, 2.9115959261600000e+00, 5.5356940284240000e+02 },
    { 5.2600000000000001e-08, 4.6173705080999996e+00, 5.8849268465832001e+03 },
    { 5.1809999999999999e-08, 3.7865208419099998e+00, 2.4272860397400001e+02 },
    { 4.9940000000000002e-08, 3.7768723127200001e+00, 2.2741169495098002e+03 },
    { 4.7880000000000003e-08, 5.0894678442799997e+00, 3.3334988796990001e+03 },
    { 4.6089999999999998e-08, 2.2950377358399998e+00, 7.4775228602159996e+03 },
    { 4.4510000000000000e-08, 6.2002380534299997e+00, 2.2883440435113998e+03 },
    { 4.3100000000000002e-08, 5.3838577330300001e+00, 4.9332084403325998e+03 },
    { 4.1680000000000000e-08, 1.6329144282500001e+00, 9.5171840625059997e+02 },
    { 4.1600000000000002e-08, 1.6376281669999999e-02, 1.7480164130670000e+03 },
    { 4.1080000000000001e-08, 1.0933274107399999e+00, 3.3477259737006002e+03 },
    { 3.9860000000000001e-08, 3.7491132892499999e+00, 2.3384286986898602e+04 },
    { 3.9069999999999998e-08, 5.3489210373100002e+00, 5.0921519581158000e+03 },
    { 3.8530000000000002e-08, 2.5032726934100000e+00, 3.8703033917943999e+03 },
    { 3.5089999999999999e-08, 5.7679733808000000e-01, 1.3498674096587999e+03 },
    { 3.2210000000000002e-08, 8.0341863927000001e-01, 4.2659819087599999e+02 },
    { 3.0759999999999999e-08, 2.1992194872500002e+00, 6.4384962494255997e+03 },
    { 2.6860000000000000e-08, 3.8756914884999998e+00, 3.8289653222319998e+02 },
    { 2.4960000000000000e-08, 3.9688613885800001e+00, 5.3313574437407997e+03 },
    { 2.3989999999999998e-08, 5.7179819508899996e+00, 6.3989728631399998e+02 },
    { 2.3770000000000000e-08, 3.8140430638000000e-01, 6.8366452528338004e+03 },
    { 2.2539999999999999e-08, 5.3232661720300003e+00, 2.2812304965106000e+03 },
    { 2.2340000000000000e-08, 3.3403410508900002e+00, 6.6777017350506003e+03 },
    { 2.2080000000000000e-08, 4.3219223299899996e+00, 9.4921463150048003e+03 },
    { 2.1740000000000001e-08, 1.1414038602300001e+00, 6.1276554505572003e+03 },
    { 2.1290000000000001e-08, 9.6593329639000003e-01, 7.7406067835887998e+03 },
    { 2.0079999999999998e-08, 9.7233813710000006e-01, 6.6822051744678001e+03 },
    { 1.9659999999999999e-08, 5.6627709440400000e+00, 6.6812075997474003e+03 },
    { 1.9659999999999999e-08, 1.4093438420500000e+00, 6.6812421070518003e+03 },
    { 1.9589999999999999e-08, 4.5996706760699997e+00, 6.9239534573736000e+03 },
    { 1.8860000000000001e-08, 8.8359569638000002e-01, 4.9296853219836003e+03 },
    { 1.8559999999999999e-08, 2.5564794844300001e+00, 5.7295064471490005e+03 },
    { 1.7269999999999998e-08, 2.8556195214600000e+00, 6.6741113063987996e+03 },
    { 1.7090000000000000e-08, 3.1548522818200002e+00, 3.5539115221378001e+03 },
    { 1.7070000000000000e-08, 1.5805550752499999e+00, 1.9144826611159999e+02 },
    { 1.6230000000000000e-08, 2.9117685893599998e+00, 3.1273133312618002e+03 },
    { 1.6130000000000000e-08, 8.7522364052000001e-01, 7.8756718636242003e+03 },
    { 1.5630000000000002e-08, 6.0482531251999996e+00, 8.9695688969109997e+03 },
    { 1.5620000000000000e-08, 3.6060103089300002e+00, 8.9553418029093991e+03 },
    { 1.5209999999999999e-08, 1.2469971346900000e+00, 2.1187638603783998e+03 },
    { 1.4890000000000000e-08, 7.7639728980000003e-02, 9.8664168806651996e+03 },
    { 1.4790000000000000e-08, 4.1932862508099999e+00, 1.0419986283507600e+04 },
    { 1.4670000000000000e-08, 4.1052556038899999e+00, 1.3365972825148199e+04 },
    { 1.2770000000000000e-08, 4.1455611697499997e+00, 2.8109214616052000e+03 },
    { 1.2410000000000000e-08, 2.4015475121400001e+00, 6.4087760738220004e+02 },
    { 1.2159999999999999e-08, 5.3266638222499996e+00, 6.2546266625236003e+03 },
    { 1.2089999999999999e-08, 4.0272455673699996e+00, 5.2969096509459996e+02 },
    { 1.2010000000000000e-08, 1.3502781297999999e+00, 3.7672106175757999e+03 },
    { 1.1770000000000001e-08, 3.1185850988400001e+00, 2.6997348193175999e+03 },
    { 1.1630000000000001e-08, 2.6803523810000000e-02, 6.2983283211764001e+03 },
    { 1.1269999999999999e-08, 4.7470234782400000e+00, 1.2832758741704600e+04 },
    { 1.1100000000000000e-08, 6.2876661717000004e-01, 1.2303067776610000e+04 },
    { 1.1069999999999999e-08, 3.4359818294400002e+00, 7.2109158184941998e+03 },
    { 1.0200000000000000e-08, 2.8438358082300002e+00, 3.9814900340820000e+03 },
    { 9.9599999999999995e-09, 4.7452677574099997e+00, 4.5624609930212000e+03 },
    { 9.9200000000000005e-09, 2.8954312778600002e+00, 9.8303890139878004e+03 },
    { 9.8000000000000001e-09, 4.1018930172400001e+00, 1.7515395314160000e+03 },
    { 9.8000000000000001e-09, 2.3663941983300001e+00, 6.1515338883049999e+03 },
    { 9.1999999999999997e-09, 3.9289007555000000e-01, 6.0413275670856001e+03 },
    { 8.7899999999999996e-09, 2.6539650507400001e+00, 1.0818135286915800e+04 },
    { 8.5000000000000000e-09, 1.4788174634200000e+00, 6.6883384004004001e+03 },
    { 8.1699999999999997e-09, 1.8016383159600000e+00, 1.0021854533751601e+04 },
    { 8.1599999999999999e-09, 6.0541450646300001e+00, 1.0021820026447200e+04 },
    { 8.0100000000000003e-09, 3.1224704730099999e+00, 7.2347942562420003e+03 },
    { 7.5200000000000005e-09, 3.7599815514700001e+00, 9.6236882766912004e+03 },
    { 7.4999999999999993e-09, 5.4733257188300000e+00, 1.0018314161750401e+04 },
    { 7.2200000000000003e-09, 4.6290687637500003e+00, 8.5319638175199998e+02 },
    { 7.1699999999999998e-09, 2.0751983909699998e+00, 3.1781457905676002e+03 },
    { 6.9999999999999998e-09, 1.6753012988200000e+00, 7.6329432596502002e+03 },
    { 6.8500000000000001e-09, 5.7519320974400001e+00, 8.2738208670324002e+03 },
    { 6.7299999999999997e-09, 4.0482174341099997e+00, 3.5030790628320001e+03 },
    { 6.5199999999999998e-09, 3.3498155770100002e+00, 6.6812921637024001e+03 },
    { 6.5199999999999998e-09, 3.7025362694899999e+00, 6.8945239488376001e+03 },
    { 6.4700000000000002e-09, 3.7490881366900002e+00, 6.6811575430967996e+03 },
    { 6.1700000000000000e-09, 9.9680091352000000e-01, 3.2533042221599999e+03 },
    { 5.9600000000000001e-09, 5.1269984690700001e+00, 3.4279206312396000e+03 },
    { 5.8800000000000004e-09, 3.9627846902499999e+00, 2.1329909543799999e+02 },
    { 5.7699999999999997e-09, 1.0407076237199999e+00, 1.5643680203309799e+04 },
    { 5.6100000000000003e-09, 4.0907073673300003e+00, 2.6724899413598399e+04 },
    { 5.4500000000000000e-09, 3.4152213710399999e+00, 2.7662676283649998e+03 },
    { 5.4500000000000000e-09, 5.7515187280099997e+00, 6.6802445323313996e+03 },
    { 5.3599999999999997e-09, 3.5232759902300002e+00, 6.0403472460173998e+03 },
    { 4.9499999999999997e-09, 1.0202572390100000e+00, 1.0390266107903999e+03 },
    { 4.9399999999999999e-09, 4.4965544796000003e+00, 2.9140142358237999e+03 },
    { 4.7500000000000003e-09, 2.7257730049000001e-01, 8.0310922630584000e+03 },
    { 4.5200000000000001e-09, 3.9241373339800001e+00, 7.0641213856227996e+03 },
    { 4.2400000000000002e-09, 1.7022325704300001e+00, 2.6484548254729998e+03 },
    { 4.1000000000000003e-09, 1.1292048440000001e-02, 3.9805097130138001e+03 },
    { 4.0700000000000002e-09, 3.7145744800400000e+00, 3.9814900340819997e+02 },
    { 4.0599999999999996e-09, 2.5875939731300002e+00, 3.9149572250346000e+03 },
    { 4.0199999999999998e-09, 4.9349210668200003e+00, 8.9624553499102003e+03 },
    { 3.9400000000000000e-09, 4.1679305085299996e+00, 1.2964300703391000e+04 },
    { 3.8000000000000001e-09, 1.3220667095500001e+00, 1.1081219210288600e+04 },
    { 3.6899999999999999e-09, 1.1751405707500000e+00, 8.2702977486834006e+03 },
    { 3.6199999999999999e-09, 9.0947170770999997e-01, 2.7007151403858002e+03 },
    { 3.5300000000000000e-09, 4.3267545220199999e+00, 8.6719698704406001e+03 },
    { 3.4800000000000000e-09, 4.2381697714000000e-01, 5.6147293762095996e+03 },
    { 3.2799999999999998e-09, 2.7521017111799999e+00, 6.8726731195111997e+03 },
    { 3.2599999999999999e-09, 1.3471014482100001e+00, 1.0022817601167601e+04 },
    { 3.1500000000000001e-09, 5.1016612424799996e+00, 1.6173371168404399e+04 },
    { 3.0000000000000000e-09, 1.2828247952700000e+00, 5.4593762870782002e+03 },
    { 2.9699999999999999e-09, 1.3720616976700000e+00, 5.8280284716475999e+03 },
    { 2.9699999999999999e-09, 4.5617763821799997e+00, 4.0327700279266001e+03 },
    { 2.8900000000000002e-09, 1.1523060708000001e-01, 5.7434479833479998e+02 },
    { 2.8499999999999999e-09, 1.5954661285999999e-01, 1.2310181323610799e+04 },
    { 2.8100000000000000e-09, 4.5648681042800003e+00, 1.3760598710207400e+04 },
    { 2.7700000000000002e-09, 1.2500468219100000e+00, 1.1216284290324000e+04 },
    { 2.7200000000000001e-09, 4.2820934812599996e+00, 1.0551528245194000e+04 },
    { 2.7099999999999999e-09, 3.9282073954699999e+00, 8.4327643848156004e+03 },
    { 2.6799999999999998e-09, 3.7293251525000000e+00, 1.0021904590402200e+04 },
    { 2.6700000000000001e-09, 4.1377410491699997e+00, 1.0021769969796600e+04 },
    { 2.6599999999999999e-09, 1.3917474694300001e+00, 1.8984292630009601e+04 },
    { 2.6500000000000002e-09, 2.2531924319500001e+00, 2.4874160449477999e+03 },
    { 2.6299999999999998e-09, 1.4781163323600000e+00, 9.4682678772569998e+03 },
    { 2.5300000000000002e-09, 3.2599663081900001e+00, 1.0664954771900000e+03 },
    { 2.4600000000000002e-09, 4.9480491688999999e+00, 1.0264565884073400e+04 },
    { 2.4399999999999998e-09, 3.9945502976100000e+00, 1.2218485663214001e+03 },
    { 2.4199999999999999e-09, 5.3778168264500001e+00, 1.3358926588450200e+04 },
    { 2.3899999999999998e-09, 4.4336225431300003e+00, 1.6706585251847999e+04 },
    { 2.3300000000000000e-09, 2.0521722670100000e+00, 9.7791086761253991e+03 },
    { 2.2600000000000001e-09, 3.4927551858700001e+00, 1.0575406682941801e+04 },
    { 2.1200000000000001e-09, 1.8445793918300000e+00, 7.1078230442756003e+03 },
    { 2.1100000000000000e-09, 1.7675451742399999e+00, 6.2830758499914000e+03 },
    { 2.0400000000000000e-09, 5.6802927471600002e+00, 1.0177257679533601e+04 },
    { 2.0200000000000001e-09, 4.0598459347500002e+00, 2.3015858159093900e+03 },
    { 2.0000000000000001e-09, 5.3890464243200000e+00, 8.4292412664666008e+03 },
    { 1.9900000000000000e-09, 2.1628241128100001e+00, 1.3362466960451400e+04 },
    { 1.9900000000000000e-09, 1.3210292706999999e-01, 1.3362432453146999e+04 },
    { 1.9800000000000002e-09, 4.8326283304700004e+00, 5.6421982426092000e+03 },
    { 1.9599999999999998e-09, 4.2643861998999999e-01, 1.3207029307364999e+04 },
    { 1.9300000000000002e-09, 2.8684766629500000e+00, 9.0701188738487999e+03 },
    { 1.7700000000000001e-09, 3.7846184022000001e-01, 9.6389407478762005e+03 },
    { 1.7200000000000001e-09, 3.2768979260100002e+00, 1.3171001440687600e+04 },
    { 1.6999999999999999e-09, 1.4247788770200001e+00, 4.3796390374902003e+03 },
    { 1.6700000000000000e-09, 4.1792207347999999e-01, 1.0522683831884001e+03 },
    { 1.5500000000000000e-09, 3.9161867574300002e+00, 1.2295954229609200e+04 },
    { 1.5300000000000001e-09, 1.9091098536800000e+00, 1.0973555686350001e+04 },
    { 1.4800000000000001e-09, 4.1828092681999998e+00, 1.0235136375537400e+04 },
    { 1.4100000000000001e-09, 1.8525956321399999e+00, 1.0028950827100200e+04 },
    { 1.3600000000000001e-09, 5.8992298709800002e+00, 1.4371756141986000e+03 },
    { 1.3600000000000001e-09, 8.3727417600999998e-01, 1.5113989238215199e+04 },
    { 1.3100000000000000e-09, 2.5161165723000001e-01, 3.0297716278314001e+03 },
    { 1.3100000000000000e-09, 5.9093847141499998e+00, 6.6608695340007998e+03 },
    { 1.2900000000000001e-09, 2.3369951720700000e+00, 6.6964773245846000e+03 },
    { 1.2100000000000000e-09, 5.2212198647200001e+00, 3.3406296803519999e+03 },
    { 1.2100000000000000e-09, 3.1927298092899998e+00, 3.3405951730475999e+03 },
    { 1.2000000000000000e-09, 3.6731997743300000e+00, 1.4956319713459999e+02 },
    { 1.1700000000000001e-09, 4.2729944499499997e+00, 6.7015801727983999e+03 },
    { 1.1599999999999999e-09, 5.8720500268500002e+00, 3.6514532255682002e+03 },
    { 1.1400000000000000e-09, 3.1868472374899999e+00, 7.3221024607817999e+03 },
    { 1.1400000000000000e-09, 2.2468155563600001e+00, 9.1686408983473993e+03 },
    { 1.1400000000000000e-09, 2.1494022133200001e+00, 6.5187582172674001e+03 },
    { 1.1400000000000000e-09, 1.1654964631900000e+00, 1.1773376811515400e+04 },
    { 1.0700000000000000e-09, 3.5743186937000000e+00, 4.4071079038897997e+03 },
    { 1.0399999999999999e-09, 1.5640809826800000e+00, 6.6659723822145997e+03 },
    { 1.0399999999999999e-09, 5.0360213697100003e+00, 8.8273902698748007e+03 },
    { 1.0100000000000000e-09, 2.8948052979200001e+00, 1.4158747713615599e+04 },
    { 1.0000000000000001e-09, 5.7415157210999999e+00, 1.7924910699820401e+04 },
    { 1.0000000000000001e-09, 1.5839418796900000e+00, 6.4679257579615996e+03 },
    { 9.5999999999999999e-10, 4.5249839774899998e+00, 1.6304913130090799e+04 },
    { 9.5000000000000003e-10, 5.0185914443500002e+00, 6.8162899334350004e+03 },
    { 9.4000000000000006e-10, 3.7322658094099999e+00, 9.8085381846614000e+03 },
    { 9.4000000000000006e-10, 3.6377778931600000e+00, 7.9030734197210004e+03 },
    { 9.2000000000000003e-10, 3.1642284430299998e+00, 1.0014723733098601e+04 },
    { 9.0999999999999996e-10, 6.8664247875999995e-01, 9.3819399937853996e+03 },
    { 8.9000000000000003e-10, 1.6636618973000000e-01, 1.1371704689758200e+04 },
    { 8.7999999999999996e-10, 4.4918062900600004e+00, 1.8353246176068001e+03 },
    { 8.7999999999999996e-10, 2.8956834699000000e-01, 7.2555696517344004e+03 },
    { 8.4999999999999996e-10, 5.1667883018899996e+00, 4.1938088084518004e+03 },
    { 8.1999999999999996e-10, 1.6445432952000001e+00, 2.0752664362475400e+04 },
    { 8.0000000000000003e-10, 4.0216866971699998e+00, 1.0404733812322600e+04 },
    { 8.0000000000000003e-10, 8.5355076616000003e-01, 1.3119721102825190e+04 },
    { 8.0000000000000003e-10, 7.8152072903000003e-01, 2.1795214091614791e+04 },
    { 7.8999999999999996e-10, 3.7563978651099998e+00, 9.2255392732830005e+03 },
    { 7.7999999999999999e-10, 2.0667732304199999e+00, 2.7433889215874991e+04 },
    { 7.7999999999999999e-10, 5.3945121588599996e+00, 1.5052878090929901e+03 },
    { 7.5000000000000000e-10, 2.2800607548200000e+00, 9.3809596727171993e+03 },
    { 7.5000000000000000e-10, 5.7767592354000001e+00, 1.3517870106233400e+04 },
    { 7.4000000000000003e-10, 4.8547338430100000e+00, 1.9034368125011999e+03 },
    { 7.2999999999999996e-10, 5.5956408817499996e+00, 6.6604494579071998e+03 },
    { 7.2000000000000000e-10, 2.2271495095999999e-01, 1.9800945956224801e+04 },
    { 6.9999999999999996e-10, 3.0309698245600001e+00, 5.6289564702112002e+03 },
    { 6.9000000000000000e-10, 4.4328627585599998e+00, 3.0065511840298201e+04 },
    { 6.6999999999999996e-10, 1.0543129344000000e-01, 1.2935851515923199e+04 },
    { 6.6999999999999996e-10, 6.0310142017299997e+00, 5.2440492392010001e+03 },
    { 6.6999999999999996e-10, 3.0799291608199999e+00, 2.2334736210149999e+03 },
    { 6.6000000000000000e-10, 1.8945955683400000e+00, 4.8459002357928002e+03 },
    { 6.5000000000000003e-10, 2.2754062624100002e+00, 1.2721572099417001e+04 },
    { 6.5000000000000003e-10, 1.6735676760600000e+00, 2.6316226244231998e+03 },
    { 6.5000000000000003e-10, 5.7482888296799999e+00, 2.0199094959632999e+04 },
    { 6.5000000000000003e-10, 3.7706594897299999e+00, 6.1068800550648002e+03 },
    { 6.3999999999999996e-10, 5.4734934184400004e+00, 6.7685330579394004e+03 },
    { 6.3000000000000000e-10, 4.0861987572800000e+00, 1.3362517017102000e+04 },
    { 6.3000000000000000e-10, 1.0426818351500000e+00, 1.4128242771245599e+04 },
    { 6.2000000000000003e-10, 1.6619336096599999e+00, 5.9890672521728002e+03 },
    { 6.2000000000000003e-10, 5.9894610449399996e+00, 8.4256508378147992e+03 },
    { 6.0000000000000000e-10, 2.7371149874500000e+00, 1.6062184526116800e+04 },
    { 5.9000000000000003e-10, 4.5766074093300002e+00, 2.5287723799399799e+04 },
    { 5.7999999999999996e-10, 1.2199561916099999e+00, 2.7832038219283200e+04 },
    { 5.7999999999999996e-10, 2.4784605031700000e+00, 2.2743409379516401e+04 },
    { 5.7999999999999996e-10, 8.0841286660000000e-01, 1.8454601664915001e+04 },
    { 5.7000000000000000e-10, 3.9404248003400002e+00, 2.7035740212466801e+04 },
    { 5.7000000000000000e-10, 5.4671396680499997e+00, 1.6699539015149989e+04 },
    { 5.7000000000000000e-10, 1.0939066557399999e+00, 1.7412051935775598e+04 },
    { 5.7000000000000000e-10, 3.2351678207800001e+00, 8.9828106693089994e+03 },
    { 5.6000000000000003e-10, 4.0901904934499997e+00, 9.4992598620055996e+03 },
    { 5.6000000000000003e-10, 4.4943344609000002e+00, 1.3362382396496399e+04 },
    { 5.6000000000000003e-10, 1.4744992079999999e+00, 6.7020002488919999e+03 },
    { 5.1999999999999996e-10, 3.1997283882200001e+00, 4.4477512323846004e+03 },
    { 5.1000000000000000e-10, 5.4752345594399996e+00, 1.9513983595104201e+04 },
    { 5.0000000000000003e-10, 2.9464065389099998e+00, 1.4584298273120599e+04 },
    { 5.0000000000000003e-10, 1.3094874029100001e+00, 6.5939166488598003e+03 },
    { 5.0000000000000003e-10, 4.4497550604500002e+00, 4.0496022289764001e+03 },
    { 5.0000000000000003e-10, 6.0489139787799999e+00, 2.1265523126520198e+04 },
    { 5.0000000000000003e-10, 5.4785491573999998e+00, 8.1865126624925997e+03 },
    { 5.0000000000000003e-10, 4.8787322473600003e+00, 1.3892140671893800e+04 },
    { 4.8999999999999996e-10, 5.0814741761000004e+00, 1.1128976085784199e+04 },
    { 4.8999999999999996e-10, 1.6691601630299999e+00, 1.4421831636988400e+04 },
    { 4.8999999999999996e-10, 2.9151460179500002e+00, 6.6573464156518003e+03 },
    { 4.8999999999999996e-10, 1.4889949253700001e+00, 1.1610910175383200e+04 },
    { 4.8000000000000000e-10, 1.2559852906500000e+00, 2.3141558382924599e+04 },
    { 4.8000000000000000e-10, 8.0944904899000003e-01, 1.0596182078434200e+04 },
    { 4.8000000000000000e-10, 1.2480285573400001e+00, 1.6460333529524989e+04 },
    { 4.8000000000000000e-10, 2.9910781152100001e+00, 1.7654780539749601e+04 },
    { 4.7000000000000003e-10, 4.6146406745000002e-01, 7.3211221397135996e+03 },
    { 4.7000000000000003e-10, 3.6536617797700002e+00, 1.2323423096008801e+04 },
    { 4.6000000000000001e-10, 4.3227765710000003e+00, 1.3149150611361199e+04 },
    { 4.6000000000000001e-10, 9.9074391134999995e-01, 1.0713994881326200e+04 },
    { 4.6000000000000001e-10, 4.7454168475599996e+00, 3.3335661900017999e+03 },
    { 4.6000000000000001e-10, 4.5199427246699999e+00, 3.3476586633978000e+03 },
    { 4.5000000000000000e-10, 4.4282021406099998e+00, 8.5846616659007996e+03 },
    { 4.5000000000000000e-10, 4.5006111589700000e+00, 3.0376352639166598e+04 },
    { 4.3999999999999998e-10, 4.3603680784499996e+00, 2.5685872802808000e+04 },
    { 4.3000000000000001e-10, 1.5180397326099999e+00, 8.7999887137780006e+03 },
    { 4.3000000000000001e-10, 6.1818181916000003e-01, 6.6848152820513997e+03 },
    { 4.3000000000000001e-10, 9.9542400693999999e-01, 6.7051032911474003e+03 },
    { 4.3000000000000001e-10, 6.2384443428000003e+00, 6.7096740408674004e+03 },
    { 4.3000000000000001e-10, 1.8705839537999999e-01, 6.6776344247478000e+03 },
    { 4.3000000000000001e-10, 3.4130984905599999e+00, 7.3733824546264004e+03 },
    { 4.3000000000000001e-10, 1.8806765253000000e+00, 3.1570799649391200e+04 },
    { 4.2000000000000000e-10, 4.6765191148899996e+00, 9.4850327680040009e+03 },
    { 4.2000000000000000e-10, 1.7088885869599999e+00, 1.3363430027867400e+04 },
    { 4.0000000000000001e-10, 2.1390587501799998e+00, 1.3553897972910800e+04 },
    { 3.9000000000000000e-10, 1.2807261412800000e+00, 3.3405451163970001e+03 },
    { 3.7999999999999998e-10, 4.1376954399599999e+00, 1.5664035522708589e+04 },
    { 3.7999999999999998e-10, 8.7505173892999999e-01, 3.3406797370026002e+03 },
    { 3.7999999999999998e-10, 2.4463614606799999e+00, 6.5316616562649997e+03 },
    { 3.7999999999999998e-10, 1.7112950159000001e-01, 5.0992655051166003e+03 },
    { 3.7999999999999998e-10, 5.8072440181099996e+00, 1.1614433293732200e+04 },
    { 3.7999999999999998e-10, 1.7588702948999999e+00, 2.2324905056709398e+04 },
    { 3.7999999999999998e-10, 2.8437615959400002e+00, 1.0213285546211000e+04 },
    { 3.7000000000000001e-10, 5.9992177809099996e+00, 1.8606498946000200e+04 },
    { 3.7000000000000001e-10, 4.5942379954900003e+00, 1.9004647949408401e+04 },
    { 3.6000000000000000e-10, 3.1214044128700000e+00, 6.1586474353058002e+03 },
    { 3.6000000000000000e-10, 3.0699789941000000e+00, 2.6084021806216198e+04 },
    { 3.6000000000000000e-10, 6.1139464907900001e+00, 2.4606135553220000e+04 },
    { 3.4999999999999998e-10, 5.1775812900900000e+00, 2.0735832161425591e+04 },
    { 3.4999999999999998e-10, 1.2378304174000000e-01, 3.5231183490000002e+00 },
    { 3.4000000000000001e-10, 4.3723568347199997e+00, 1.1769853693166400e+04 },
    { 3.4000000000000001e-10, 7.9000828719999999e-02, 2.8628336226099600e+04 },
    { 3.3000000000000000e-10, 1.4393075400000000e-02, 4.7777880408984001e+03 },
    { 3.1999999999999998e-10, 4.8897334684200002e+00, 2.0597243963041201e+04 },
    { 3.1000000000000002e-10, 5.2511763191499998e+00, 4.1064054911461999e+03 },
    { 3.1000000000000002e-10, 8.7259059948999995e-01, 2.5748193622534000e+03 },
    { 3.1000000000000002e-10, 9.8186586658999997e-01, 2.8230187222691391e+04 },
    { 3.1000000000000002e-10, 3.1164236538300001e+00, 1.9402796952816600e+04 },
    { 3.1000000000000002e-10, 3.8537314987800002e+00, 6.8436914895317996e+03 },
    { 3.0000000000000000e-10, 5.1571196940700004e+00, 8.4398779318163997e+03 },
    { 3.0000000000000000e-10, 5.3798126979100003e+00, 2.4076444588125400e+04 },
    { 3.0000000000000000e-10, 5.0408919592399997e+00, 1.1925274092600601e+04 },
    { 2.8999999999999998e-10, 5.0244127168999997e-01, 1.6276463942623001e+04 },
    { 2.8000000000000002e-10, 1.0419361626000000e-01, 1.2509253325047201e+04 },
    { 2.8000000000000002e-10, 5.9603557039999999e-01, 1.5650793750310600e+04 },
    { 2.8000000000000002e-10, 5.5072387731400001e+00, 1.5265886519300400e+04 },
    { 2.7000000000000000e-10, 6.2604873768099996e+00, 1.1527125089192399e+04 },
    { 2.7000000000000000e-10, 1.2026593100700000e+00, 2.0809467624645200e+04 },
    { 2.7000000000000000e-10, 4.8211899460899996e+00, 1.7101211136907201e+04 },
    { 2.5999999999999998e-10, 1.5596672950399999e+00, 2.1548962369291799e+04 },
    { 2.5999999999999998e-10, 4.1155985862500000e+00, 1.3916019109641600e+04 },
    { 2.5999999999999998e-10, 5.4444812833200000e+00, 1.3745346239022399e+04 },
    { 2.5999999999999998e-10, 2.5236022336400001e+00, 1.6066065861474801e+04 },
    { 2.5999999999999998e-10, 6.0412257218000001e+00, 1.2722552420485201e+04 },
    { 2.5999999999999998e-10, 1.0826980097000000e-01, 2.1947111372700001e+04 },
    { 2.5000000000000002e-10, 3.7379378704800001e+00, 2.3695127785766999e+04 },
    { 2.5000000000000002e-10, 4.9108501809999998e+00, 2.9026485229507791e+04 },
    { 2.4000000000000000e-10, 5.5482436698799997e+00, 3.8774169387952002e+03 },
    { 2.4000000000000000e-10, 2.6293436196999997e-01, 2.3539707386332801e+04 },
    { 2.3000000000000001e-10, 2.8758703643999999e+00, 3.1172650645983002e+04 },
    { 2.3000000000000001e-10, 4.1340095641200003e+00, 4.3928808098882000e+03 },
    { 2.3000000000000001e-10, 5.6008287680000004e+00, 3.7455726495974399e+04 },
    { 2.1999999999999999e-10, 5.9265034235000003e-01, 2.2747290714874402e+04 },
    { 2.1999999999999999e-10, 5.9502221518500003e+00, 1.6858482532933202e+04 },
    { 2.1000000000000000e-10, 5.5493343914900004e+00, 1.7444259844152000e+03 },
    { 2.1000000000000000e-10, 2.4046949717900001e+00, 2.8180350086059998e+03 },
    { 2.0000000000000001e-10, 8.2610997827999999e-01, 3.3318816062458201e+04 },
    { 2.0000000000000001e-10, 3.6583164248100002e+00, 3.0774501642574800e+04 },
    { 2.0000000000000001e-10, 5.6109897399399999e+00, 2.6637591209058592e+04 },
    { 2.0000000000000001e-10, 1.7140409406899999e+00, 1.0629050485381999e+03 },
    { 2.0000000000000001e-10, 4.0523568978200002e+00, 3.3716965065866403e+04 },
    { 2.0000000000000001e-10, 2.3710871100399999e+00, 3.4911412076091001e+04 },
    { 2.0000000000000001e-10, 5.6057892002999998e-01, 1.0787630344545800e+04 },
    { 2.0000000000000001e-10, 1.9197717710000000e+00, 2.4889574795991601e+04 },
    { 2.0000000000000001e-10, 4.9300849233299999e+00, 2.2345260376108199e+04 },
    { 2.0000000000000001e-10, 2.7282391369999999e-02, 2.9978203635758400e+04 },
    { 1.8999999999999999e-10, 2.5488145016199999e+00, 4.0000040915857790e+04 },
    { 1.8999999999999999e-10, 2.8519547320000001e-02, 5.0850384111149997e+03 },
    { 1.8999999999999999e-10, 2.8130174915400001e+00, 2.8038079146044001e+03 },
    { 1.8999999999999999e-10, 9.9992364561000002e-01, 1.8208349942592002e+04 },
    { 1.8999999999999999e-10, 3.2057853572599999e+00, 2.0354515359067202e+04 },
    { 1.8999999999999999e-10, 1.6724995554700000e+00, 3.6659428489158003e+04 },
    { 1.8000000000000000e-10, 1.1198521668500001e+00, 2.3937856389740999e+04 },
    { 1.8000000000000000e-10, 6.1053937437899997e+00, 3.8631898447936001e+03 },
    { 1.8000000000000000e-10, 8.4708374415999999e-01, 1.4071439509075790e+04 },
    { 1.8000000000000000e-10, 1.6672027110200001e+00, 1.4469588512484001e+04 },
    { 1.8000000000000000e-10, 2.2640504866000000e+00, 3.4925097077850000e+03 },
    { 1.8000000000000000e-10, 7.1812377844999997e-01, 3.1887151456145998e+03 },
    { 1.8000000000000000e-10, 6.5926982751999996e-01, 1.8451078546565990e+04 },
    { 1.8000000000000000e-10, 6.5056538407999998e-01, 1.4867737515892190e+04 },
    { 1.8000000000000000e-10, 3.9991109387599999e+00, 1.7468855197945399e+04 },
    { 1.7000000000000001e-10, 3.2955848147500002e+00, 2.4491425792583399e+04 },
    { 1.7000000000000001e-10, 1.1017094703000001e+00, 1.3936794505133999e+04 },
    { 1.7000000000000001e-10, 1.2763204727999999e+00, 5.6183198048614004e+03 },
    { 1.5999999999999999e-10, 3.9558818795000001e+00, 3.9601891912449602e+04 },
    { 1.5999999999999999e-10, 2.2082058065500001e+00, 1.0448435470975401e+04 },
    { 1.5999999999999999e-10, 2.1370793975900000e+00, 2.1150813365883601e+04 },
    { 1.5999999999999999e-10, 1.8404197823999999e+00, 1.2808880303956799e+04 },
    { 1.5000000000000000e-10, 4.3978873734599997e+00, 2.3546753623030800e+04 },
    { 1.5000000000000000e-10, 1.0039703826699999e+00, 1.7277406931833801e+04 },
    { 1.5000000000000000e-10, 2.7344874405700001e+00, 3.6261279485749801e+04 },
    { 1.5000000000000000e-10, 4.3226403725799996e+00, 2.9389403049426001e+03 },
    { 1.5000000000000000e-10, 4.9427191996500000e+00, 3.7422845484569998e+03 },
    { 1.5000000000000000e-10, 2.6808818459300001e+00, 1.7013902932367400e+04 },
    { 1.5000000000000000e-10, 1.2342432722100001e+00, 2.9580054632350191e+04 },
    { 1.5000000000000000e-10, 1.1814963122000000e-01, 2.6239442205650390e+04 },
    { 1.4000000000000001e-10, 4.6397355590000000e+00, 2.7490692478044799e+04 },
    { 1.4000000000000001e-10, 5.8458510269800001e+00, 1.1243685846420800e+04 },
    { 1.4000000000000001e-10, 3.6912356168899998e+00, 9.5952390892233998e+03 },
};

static const VSOP87Term VSOP87A_MARS_Y3[136] = {
    { 1.4487780000000001e-05, 0.0000000000000000e+00, 0.0000000000000000e+00 },
    { 5.4827699999999999e-06, 3.3360423381999998e-01, 6.6812248533995999e+03 },
    { 3.1642199999999998e-06, 8.1609547752000000e-01, 1.0021837280099400e+04 },
    { 1.2186399999999999e-06, 4.2128144875700002e+00, 3.3406124266997999e+03 },
    { 8.8558000000000002e-07, 1.2027699307099999e+00, 1.3362449706799200e+04 },
    { 1.9088000000000000e-07, 1.5656041046500000e+00, 1.6703062133498999e+04 },
    { 1.7293000000000000e-07, 1.9750377557800001e+00, 3.4960328261340001e+03 },
    { 1.7050000000000000e-07, 4.1487908647499996e+00, 3.1851920272655998e+03 },
    { 7.9129999999999995e-08, 4.6248264399699996e+00, 3.3370893083507999e+03 },
    { 7.8790000000000006e-08, 1.4880183433900001e+00, 3.3441355450487999e+03 },
    { 3.7240000000000001e-08, 2.7426317782299998e+00, 3.5833410306738001e+03 },
    { 3.7060000000000002e-08, 3.3795284710500000e+00, 3.0978838227257902e+03 },
    { 3.5980000000000002e-08, 1.9197105596299999e+00, 2.0043674560198800e+04 },
    { 2.8590000000000000e-08, 1.9351516861300000e+00, 6.6847479717486003e+03 },
    { 2.3919999999999998e-08, 4.5084957215100001e+00, 6.5258044539654002e+03 },
    { 2.3890000000000001e-08, 5.7456638903000001e-01, 3.8941818295421999e+03 },
    { 2.3569999999999999e-08, 5.5495731955299998e+00, 2.7870430238573999e+03 },
    { 2.1249999999999998e-08, 5.7597936077399998e+00, 1.5542039943419999e+02 },
    { 1.3010000000000000e-08, 5.4634307374600004e+00, 4.2923308329503998e+03 },
    { 1.2650000000000001e-08, 6.5832369520000000e-01, 2.3888940204492001e+03 },
    { 1.1600000000000000e-08, 5.6375119057500003e+00, 3.5320606928113998e+03 },
    { 1.1440000000000000e-08, 4.7525689968000001e-01, 3.1491641605882000e+03 },
    { 1.1400000000000000e-08, 3.6648108967999998e-01, 4.1369104335162001e+03 },
    { 1.0330000000000001e-08, 5.7465228557800003e+00, 2.5443144198834002e+03 },
    { 9.9900000000000005e-09, 8.3738148713000005e-01, 3.3396321056316001e+03 },
    { 9.9300000000000002e-09, 5.2817060505900004e+00, 3.3415927477680002e+03 },
    { 9.0300000000000005e-09, 4.0628941432500003e+00, 4.6904798363585996e+03 },
    { 8.6599999999999995e-09, 2.0395718218600001e+00, 1.9907450170410000e+03 },
    { 7.9500000000000001e-09, 1.6881028192900001e+00, 3.7235089589230001e+03 },
    { 7.8100000000000001e-09, 4.4287884070899999e+00, 2.9577158944766002e+03 },
    { 7.8000000000000004e-09, 2.2991584006900001e+00, 1.0025360398448400e+04 },
    { 7.1799999999999996e-09, 5.2621651852199998e+00, 4.5350594369244000e+03 },
    { 6.3000000000000002e-09, 2.6659481654900001e+00, 5.0886288397668004e+03 },
    { 6.2600000000000003e-09, 2.2672246408299999e+00, 2.3384286986898602e+04 },
    { 6.1900000000000003e-09, 8.7185440690000005e-01, 2.1461654164751999e+03 },
    { 5.9099999999999997e-09, 3.4196173012000002e+00, 1.5925960136327999e+03 },
    { 4.8900000000000003e-09, 3.1001545463500002e+00, 6.9239534573736000e+03 },
    { 4.4999999999999998e-09, 3.5535293147000002e-01, 3.3334988796990001e+03 },
    { 4.2599999999999998e-09, 1.7441324432900001e+00, 3.7387614301079998e+03 },
    { 4.1299999999999996e-09, 5.7818427215500003e+00, 3.3477259737006002e+03 },
    { 4.1100000000000001e-09, 4.2923540277100001e+00, 2.9424634232916001e+03 },
    { 3.9899999999999997e-09, 9.3216820628999997e-01, 6.4897765872880000e+03 },
    { 3.9799999999999999e-09, 5.3476041421399998e+00, 4.3999943568890003e+03 },
    { 3.7200000000000000e-09, 1.8352663239700000e+00, 6.8366452528338004e+03 },
    { 3.5699999999999999e-09, 4.8635856647000004e+00, 9.8664168806651996e+03 },
    { 3.5600000000000001e-09, 5.9135881377899997e+00, 6.1276554505572003e+03 },
    { 3.4800000000000000e-09, 4.9219836333400000e+00, 1.0593819301891999e+03 },
    { 3.3799999999999999e-09, 7.7428089130000000e-01, 7.4775228602159996e+03 },
    { 2.9300000000000000e-09, 2.2515258920400001e+00, 7.0793738568077997e+03 },
    { 2.7999999999999998e-09, 1.1407128882999999e+00, 5.6218429232103999e+03 },
    { 2.3800000000000001e-09, 3.6430294360300000e+00, 6.4384962494255997e+03 },
    { 2.1499999999999998e-09, 1.0328136250500000e+00, 5.7295064471490005e+03 },
    { 2.1400000000000001e-09, 5.7002808004799999e+00, 6.6822051744678001e+03 },
    { 1.7599999999999999e-09, 2.4282997483400002e+00, 5.3313574437407997e+03 },
    { 1.7300000000000000e-09, 2.6428441279500001e+00, 1.3365972825148199e+04 },
    { 1.5700000000000000e-09, 3.8139231262900002e+00, 4.9332084403325998e+03 },
    { 1.5600000000000000e-09, 5.7430056904400004e+00, 7.7406067835887998e+03 },
    { 1.5300000000000001e-09, 5.6415747770999998e+00, 7.8756718636242003e+03 },
    { 1.4900000000000000e-09, 4.8062169083399997e+00, 6.2983283211764001e+03 },
    { 1.4400000000000000e-09, 8.4473821033999996e-01, 7.2347942562420003e+03 },
    { 1.4200000000000001e-09, 4.5350095035400004e+00, 6.6777017350506003e+03 },
    { 1.0800000000000000e-09, 2.6686970821700000e+00, 1.0419986283507600e+04 },
    { 1.0000000000000001e-09, 1.3424795950199999e+00, 9.8303890139878004e+03 },
    { 9.9000000000000010e-10, 2.6155113774199998e+00, 2.6724899413598399e+04 },
    { 8.3999999999999999e-10, 1.1492984294399999e+00, 1.0818135286915800e+04 },
    { 8.3999999999999999e-10, 3.8593613673900000e+00, 7.9629800681639995e+02 },
    { 8.3000000000000003e-10, 5.7318173827100001e+00, 4.9296853219836003e+03 },
    { 8.1999999999999996e-10, 6.1874134937800003e+00, 6.6883384004004001e+03 },
    { 7.5999999999999996e-10, 5.7395555289700004e+00, 7.6329432596502002e+03 },
    { 6.5000000000000003e-10, 3.4498070282600000e+00, 1.0264565884073400e+04 },
    { 5.7000000000000000e-10, 6.2648323529000001e+00, 9.4682678772569998e+03 },
    { 5.7000000000000000e-10, 3.2256992823599999e+00, 1.2832758741704600e+04 },
    { 5.7000000000000000e-10, 1.6423706324500000e+00, 7.2109158184941998e+03 },
    { 5.6000000000000003e-10, 2.6758933050799998e+00, 9.4921463150048003e+03 },
    { 5.3000000000000003e-10, 5.4392276458700000e+00, 3.9814900340819997e+02 },
    { 5.1000000000000000e-10, 4.2569386400699996e+00, 8.2738208670324002e+03 },
    { 5.0000000000000003e-10, 2.5124081809000001e+00, 1.1944470102246000e+03 },
    { 5.0000000000000003e-10, 5.2136271932399998e+00, 1.3207029307364999e+04 },
    { 4.7000000000000003e-10, 4.0150400413399998e+00, 6.6812075997474003e+03 },
    { 4.7000000000000003e-10, 6.0458130519199997e+00, 6.6812421070518003e+03 },
    { 4.6000000000000001e-10, 4.3285799330600003e+00, 8.0310922630584000e+03 },
    { 4.3999999999999998e-10, 5.3837644030099998e+00, 1.2303067776610000e+04 },
    { 4.3000000000000001e-10, 1.1118327011300000e+00, 3.8703033917943999e+03 },
    { 4.3000000000000001e-10, 6.0898412169900000e+00, 1.0022817601167601e+04 },
    { 4.2000000000000000e-10, 2.6848832014000001e-01, 1.0021854533751601e+04 },
    { 4.2000000000000000e-10, 4.5210460047699996e+00, 1.0021820026447200e+04 },
    { 3.7000000000000001e-10, 4.0817889184100000e+00, 1.3498674096587999e+03 },
    { 3.7000000000000001e-10, 4.2710683804700000e+00, 1.0018314161750401e+04 },
    { 3.7000000000000001e-10, 1.8807111948299999e+00, 7.0641213856227996e+03 },
    { 3.6000000000000000e-10, 6.1091083155100003e+00, 1.1081219210288600e+04 },
    { 3.4999999999999998e-10, 4.8709172962600000e+00, 8.8273902698748007e+03 },
    { 3.4999999999999998e-10, 1.3978945409800001e+00, 9.0701188738487999e+03 },
    { 3.3000000000000000e-10, 5.7199239162900000e+00, 1.5643680203309799e+04 },
    { 3.3000000000000000e-10, 2.8193081647699998e+00, 8.6719698704406001e+03 },
    { 3.1000000000000002e-10, 5.5026950034400004e+00, 9.5171840625059997e+02 },
    { 3.1000000000000002e-10, 9.4089475775999998e-01, 6.6802445323313996e+03 },
    { 3.0000000000000000e-10, 6.3584180470999996e-01, 5.5356940284240000e+02 },
    { 2.8000000000000002e-10, 4.0036192213600001e+00, 8.4327643848156004e+03 },
    { 2.7000000000000000e-10, 2.9689810419099998e+00, 1.6706585251847999e+04 },
    { 2.5999999999999998e-10, 6.0110899562300002e+00, 1.1216284290324000e+04 },
    { 2.5999999999999998e-10, 1.5519190415499999e+00, 1.0575406682941801e+04 },
    { 2.5999999999999998e-10, 2.1439796225499999e+00, 9.6236882766912004e+03 },
    { 2.5999999999999998e-10, 3.1161447148799999e+00, 1.7480164130670000e+03 },
    { 2.5000000000000002e-10, 3.0490988808899999e+00, 1.3760598710207400e+04 },
    { 2.5000000000000002e-10, 5.3199314789300001e+00, 8.9624553499102003e+03 },
    { 2.5000000000000002e-10, 2.1275738788199998e+00, 6.8945239488376001e+03 },
    { 2.4000000000000000e-10, 6.0325995894000002e+00, 8.2702977486834006e+03 },
    { 2.3000000000000001e-10, 1.2198032175000000e-01, 1.0973555686350001e+04 },
    { 2.0000000000000001e-10, 5.9432454258899998e+00, 3.7672106175757999e+03 },
    { 2.0000000000000001e-10, 5.1627101197199998e+00, 9.6389407478762005e+03 },
    { 1.8999999999999999e-10, 1.1970977687600000e+00, 5.4867778431750003e+03 },
    { 1.8999999999999999e-10, 7.1454311342999999e-01, 2.8109214616052000e+03 },
    { 1.8000000000000000e-10, 2.6249452595700000e+00, 2.2812304965106000e+03 },
    { 1.8000000000000000e-10, 5.4862494404999995e-01, 6.6741113063987996e+03 },
    { 1.7000000000000001e-10, 1.7112242345599999e+00, 1.3171001440687600e+04 },
    { 1.7000000000000001e-10, 5.9059224110699997e+00, 2.4272860397400001e+02 },
    { 1.5999999999999999e-10, 2.1381426598200002e+00, 6.6811575430967996e+03 },
    { 1.5999999999999999e-10, 2.4582252072599999e+00, 1.0551528245194000e+04 },
    { 1.5999999999999999e-10, 1.7296462729800000e+00, 6.6812921637024001e+03 },
    { 1.5000000000000000e-10, 6.1824214074699997e+00, 5.4593762870782002e+03 },
    { 1.4000000000000001e-10, 2.8512193613999998e-01, 1.0028950827100200e+04 },
    { 1.4000000000000001e-10, 5.3142413548300000e+00, 1.9144826611159999e+02 },
    { 1.4000000000000001e-10, 4.9593860784599997e+00, 1.2310181323610799e+04 },
    { 1.2999999999999999e-10, 1.7475977078100000e+00, 1.0177257679533601e+04 },
    { 1.2000000000000000e-10, 3.2795728157399999e+00, 6.8726731195111997e+03 },
    { 1.0999999999999999e-10, 3.2342748734500000e+00, 8.4292412664666008e+03 },
    { 1.0999999999999999e-10, 1.6632368916300000e+00, 3.8289653222319998e+02 },
    { 1.0000000000000000e-10, 1.6634705617100001e+00, 1.0014723733098601e+04 },
    { 1.0000000000000000e-10, 3.6206445272100001e+00, 1.6173371168404399e+04 },
    { 8.9999999999999999e-11, 5.1534562536599999e+00, 1.1773376811515400e+04 },
    { 8.9999999999999999e-11, 2.5350001864900000e+00, 1.2295954229609200e+04 },
    { 8.9999999999999999e-11, 3.4728902114800002e+00, 3.3406296803519999e+03 },
    { 8.9999999999999999e-11, 1.4422544543500000e+00, 3.3405951730475999e+03 },
    { 7.9999999999999995e-11, 3.3244932376100000e+00, 2.9140142358237999e+03 },
    { 7.9999999999999995e-11, 6.2370445093600004e+00, 1.7515395314160000e+03 },
    { 7.0000000000000004e-11, 1.5128456652900000e+00, 1.4158747713615599e+04 },
};

static const VSOP87Term VSOP87A_MARS_Y4[44] = {
    { 1.4558000000000001e-07, 0.0000000000000000e+00, 0.0000000000000000e+00 },
    { 1.2136000000000000e-07, 5.5317109692299997e+00, 1.0021837280099400e+04 },
    { 9.3970000000000000e-08, 5.0002313405500001e+00, 6.6812248533995999e+03 },
    { 5.1760000000000002e-08, 5.9499444981799998e+00, 1.3362449706799200e+04 },
    { 3.7280000000000000e-08, 2.3174430937300001e+00, 3.3406124266997999e+03 },
    { 2.4240000000000001e-08, 3.4821298382000001e+00, 3.4960328261340001e+03 },
    { 2.4080000000000001e-08, 2.6386985774300000e+00, 3.1851920272655998e+03 },
    { 1.4950000000000002e-08, 4.9846487129999999e-02, 1.6703062133498999e+04 },
    { 5.9600000000000001e-09, 4.8854266619300004e+00, 3.0978838227257902e+03 },
    { 5.9099999999999997e-09, 1.2333831829099999e+00, 3.5833410306738001e+03 },
    { 3.8000000000000001e-09, 2.9976135160699999e+00, 6.5258044539654002e+03 },
    { 3.5400000000000002e-09, 4.1910666274000002e-01, 2.0043674560198800e+04 },
    { 2.8600000000000000e-09, 2.0745118337999999e+00, 3.8941818295421999e+03 },
    { 2.8499999999999999e-09, 4.0474458630500001e+00, 2.7870430238573999e+03 },
    { 2.5599999999999998e-09, 6.2523656989200003e+00, 3.3370893083507999e+03 },
    { 2.5300000000000002e-09, 6.1301378877200001e+00, 3.3441355450487999e+03 },
    { 1.5199999999999999e-09, 3.7871347738000000e-01, 6.6847479717486003e+03 },
    { 1.2799999999999999e-09, 3.6871475343900002e+00, 6.8366452528338004e+03 },
    { 8.7999999999999996e-10, 1.5966000274800001e+00, 6.9239534573736000e+03 },
    { 7.1000000000000003e-10, 7.7284614162999998e-01, 2.3384286986898602e+04 },
    { 6.3000000000000000e-10, 3.3594448742199998e+00, 9.8664168806651996e+03 },
    { 5.6000000000000003e-10, 7.7975710465000003e-01, 1.0025360398448400e+04 },
    { 4.8999999999999996e-10, 4.4092178379000000e+00, 6.1276554505572003e+03 },
    { 3.4000000000000001e-10, 5.1101126183199996e+00, 6.4384962494255997e+03 },
    { 1.8000000000000000e-10, 5.6727370549999998e+00, 6.4897765872880000e+03 },
    { 1.5000000000000000e-10, 5.5412408900500001e+00, 7.4775228602159996e+03 },
    { 1.4000000000000001e-10, 2.2569147758599999e+00, 7.2347942562420003e+03 },
    { 1.2999999999999999e-10, 4.1504675949600003e+00, 1.0177257679533601e+04 },
    { 1.2999999999999999e-10, 1.1530457487300001e+00, 1.3365972825148199e+04 },
    { 1.2000000000000000e-10, 4.1849603548900003e+00, 6.6822051744678001e+03 },
    { 1.2000000000000000e-10, 3.7997749697100001e+00, 4.3999943568890003e+03 },
    { 1.2000000000000000e-10, 5.8313725715000002e+00, 5.7295064471490005e+03 },
    { 1.0999999999999999e-10, 1.9485356572700001e+00, 1.0264565884073400e+04 },
    { 1.0999999999999999e-10, 6.9449787915000005e-01, 7.0793738568077997e+03 },
    { 1.0999999999999999e-10, 1.1045194994500001e+00, 2.6724899413598399e+04 },
    { 1.0000000000000000e-10, 9.3942138709000000e-01, 5.3313574437407997e+03 },
    { 7.9999999999999995e-11, 3.3179497644999998e+00, 6.2983283211764001e+03 },
    { 7.0000000000000004e-11, 4.1560600232200002e+00, 7.8756718636242003e+03 },
    { 7.0000000000000004e-11, 2.3350886492399998e+00, 4.9332084403325998e+03 },
    { 3.9999999999999998e-11, 1.3967649726700000e+00, 1.1944470102246000e+03 },
    { 3.9999999999999998e-11, 2.2089390534400000e+00, 1.3498674096587999e+03 },
    { 3.9999999999999998e-11, 5.9471585769299997e+00, 3.8289653222319998e+02 },
    { 3.9999999999999998e-11, 3.7124615892200001e+00, 1.3207029307364999e+04 },
    { 3.9999999999999998e-11, 4.7667879476100001e+00, 9.4682678772569998e+03 },
};

static const VSOP87Term VSOP87A_MARS_Y5[21] = {
    { 3.5300000000000000e-09, 3.9483084942600000e+00, 1.0021837280099400e+04 },
    { 2.7400000000000001e-09, 4.9924038897100003e+00, 3.4960328261340001e+03 },
    { 2.7200000000000001e-09, 1.1230895164900001e+00, 3.1851920272655998e+03 },
    { 2.3800000000000001e-09, 4.4006541240499999e+00, 1.3362449706799200e+04 },
    { 1.6999999999999999e-09, 3.1415926535900001e+00, 0.0000000000000000e+00 },
    { 1.5100000000000000e-09, 1.0801049393800000e+00, 3.3406124266997999e+03 },
    { 9.2999999999999999e-10, 4.8120108455299997e+00, 1.6703062133498999e+04 },
    { 7.7000000000000003e-10, 1.1215015674999999e-01, 3.0978838227257902e+03 },
    { 7.7000000000000003e-10, 6.0116167618900000e+00, 3.5833410306738001e+03 },
    { 5.9000000000000003e-10, 2.6799740747800000e+00, 6.6812248533995999e+03 },
    { 4.8999999999999996e-10, 1.4938509887600000e+00, 6.5258044539654002e+03 },
    { 2.7000000000000000e-10, 5.1858231489800000e+00, 2.0043674560198800e+04 },
    { 2.5999999999999998e-10, 3.6029803699200000e+00, 3.8941818295421999e+03 },
    { 2.5999999999999998e-10, 2.5207865487199999e+00, 2.7870430238573999e+03 },
    { 1.2999999999999999e-10, 5.1599404438500001e+00, 6.8366452528338004e+03 },
    { 1.2999999999999999e-10, 9.0009619299999996e-02, 6.9239534573736000e+03 },
    { 7.0000000000000004e-11, 1.8506403664700000e+00, 9.8664168806651996e+03 },
    { 6.0000000000000000e-11, 5.0987428988500003e+00, 6.6847479717486003e+03 },
    { 5.0000000000000002e-11, 2.8963266195399999e+00, 6.1276554505572003e+03 },
    { 5.0000000000000002e-11, 5.5580438783000004e+00, 2.3384286986898602e+04 },
    { 3.9999999999999998e-11, 3.1530831962000000e-01, 6.4384962494255997e+03 },
};

static const VSOP87Term VSOP87A_MARS_Z0[355] = {
    { 4.9012072199999999e-02, 3.7671232428599999e+00, 3.3406124266997999e+03 },
    { 6.6066954100000002e-03, 0.0000000000000000e+00, 0.0000000000000000e+00 },
    { 2.2833390400000000e-03, 4.1054402226600004e+00, 6.6812248533995999e+03 },
    { 1.5958402000000000e-04, 4.4436705826100003e+00, 1.0021837280099400e+04 },
    { 1.3219760000000001e-05, 4.7818660411399998e+00, 1.3362449706799200e+04 },
    { 6.7966000000000003e-06, 5.6510997781299999e+00, 3.3370893083507999e+03 },
    { 6.7921900000000001e-06, 5.0252703089899997e+00, 3.3441355450487999e+03 },
    { 5.3113999999999997e-06, 3.8674839004499999e+00, 2.2812304965106000e+03 },
    { 3.7499299999999999e-06, 4.3333821677300000e+00, 5.2969096509459996e+02 },
    { 3.2531499999999999e-06, 2.2456250821700001e+00, 1.0593819301891999e+03 },
    { 2.1876200000000002e-06, 1.1809484970199999e+00, 3.3405951730475999e+03 },
    { 2.1876099999999999e-06, 3.2117201285200001e+00, 3.3406296803519999e+03 },
    { 1.5244299999999999e-06, 2.1325453584999998e+00, 5.6218429232103999e+03 },
    { 1.5233300000000001e-06, 6.0810956613000000e+00, 6.1515338883049999e+03 },
    { 1.4663099999999999e-06, 5.0992702285500000e+00, 3.9814900340819997e+02 },
    { 1.2601600000000000e-06, 3.6780347547300001e+00, 7.9629800681639995e+02 },
    { 1.2032800000000001e-06, 5.1201066398300004e+00, 1.6703062133498999e+04 },
    { 1.1676600000000000e-06, 4.8691358301000003e+00, 2.9424634232916001e+03 },
    { 1.1292500000000000e-06, 5.7654839768099997e+00, 3.7387614301079998e+03 },
    { 7.5858000000000001e-07, 4.5164513609399997e+00, 3.1491641605882000e+03 },
    { 7.4290000000000001e-07, 2.4350823493800000e+00, 8.9624553499102003e+03 },
    { 7.1989000000000002e-07, 5.5421767927500003e+00, 3.3405451163970001e+03 },
    { 7.1987000000000000e-07, 5.1336804059699999e+00, 3.3406797370026002e+03 },
    { 6.8644000000000005e-07, 5.5453247154799996e+00, 6.2830758499914000e+03 },
    { 6.3827999999999997e-07, 5.9878682349899996e+00, 6.6777017350506003e+03 },
    { 6.3501999999999998e-07, 4.1674838573999997e+00, 1.7515395314160000e+03 },
    { 6.3219000000000000e-07, 5.3659096709799998e+00, 6.6847479717486003e+03 },
    { 5.9355000000000002e-07, 7.2879549961000001e-01, 5.8849268465832001e+03 },
    { 5.1292000000000002e-07, 2.6476905643299999e+00, 2.9140142358237999e+03 },
    { 5.0332000000000004e-07, 1.9603382326300001e+00, 2.8109214616052000e+03 },
    { 4.5526000000000001e-07, 2.2741577631099998e+00, 1.1944470102246000e+03 },
    { 4.5097999999999998e-07, 4.0820011507400000e+00, 4.1369104335162001e+03 },
    { 4.5000999999999999e-07, 2.2994238962000000e+00, 1.5890728952838001e+03 },
    { 4.4615000000000001e-07, 5.8895638017099996e+00, 3.5320606928113998e+03 },
    { 3.3967999999999998e-07, 4.6851841759000001e-01, 2.5443144198834002e+03 },
    { 3.3148000000000000e-07, 2.0160906301599999e+00, 5.4867778431750003e+03 },
    { 3.0473000000000000e-07, 4.9878205181900004e+00, 3.8703033917943999e+03 },
    { 2.9003000000000001e-07, 2.6493755575900000e+00, 4.2659819087599999e+02 },
    { 2.7697000000000002e-07, 5.5431986509400000e+00, 1.9144826611159999e+02 },
    { 2.4101000000000001e-07, 2.5737199177400001e+00, 4.3999943568890003e+03 },
    { 2.2415000000000001e-07, 2.8079235336999999e+00, 3.4960328261340001e+03 },
    { 2.2261999999999999e-07, 1.9101027283999999e-01, 9.4921463150048003e+03 },
    { 2.1565999999999999e-07, 2.4505920189200001e+00, 3.3334988796990001e+03 },
    { 2.0382000000000001e-07, 3.5500473697900001e+00, 6.6812421070518003e+03 },
    { 2.0382000000000001e-07, 1.5192757383000000e+00, 6.6812075997474003e+03 },
    { 1.9900000000000000e-07, 4.7849359762599999e+00, 2.1329909543799999e+02 },
    { 1.9445000000000000e-07, 3.3673371835300001e+00, 5.0886288397668004e+03 },
    { 1.9188000000000000e-07, 1.5164081594200001e+00, 3.1851920272655998e+03 },
    { 1.9109000000000000e-07, 8.6825937304000000e-01, 1.5925960136327999e+03 },
    { 1.3491999999999999e-07, 1.5279118483000000e+00, 3.3396321056316001e+03 },
    { 1.3488000000000000e-07, 2.8647589158700000e+00, 3.3415927477680002e+03 },
    { 1.2741000000000000e-07, 6.0555922974899996e+00, 7.0793738568077997e+03 },
    { 1.1906999999999999e-07, 1.6503351322199999e+00, 6.2546266625236003e+03 },
    { 1.1780000000000000e-07, 4.7530742127399996e+00, 4.6904798363585996e+03 },
    { 1.1777000000000000e-07, 4.7977227997399998e+00, 3.5539115221378001e+03 },
    { 1.1629000000000000e-07, 5.4583223769000000e+00, 2.0043674560198800e+04 },
    { 1.1511000000000000e-07, 2.7583530538100001e+00, 1.2303067776610000e+04 },
    { 1.1208000000000000e-07, 2.6476342109700002e+00, 8.4327643848156004e+03 },
    { 1.0586999999999999e-07, 6.1412567080900002e+00, 7.1135470007999997e+00 },
    { 9.5749999999999994e-08, 2.6669816734000001e-01, 1.0213285546211000e+04 },
    { 9.2999999999999999e-08, 5.7442641236399998e+00, 1.9907450170410000e+03 },
    { 9.0989999999999996e-08, 4.1473462728800001e+00, 3.1273133312618002e+03 },
    { 9.0100000000000006e-08, 5.9103165021299997e+00, 9.6236882766912004e+03 },
    { 8.7460000000000004e-08, 1.0753337205500000e+00, 9.2255392732830005e+03 },
    { 8.7320000000000004e-08, 5.9445639165099999e+00, 5.0921519581158000e+03 },
    { 8.4250000000000003e-08, 5.6780634671200003e+00, 6.4679257579615996e+03 },
    { 8.3990000000000000e-08, 2.6594277273000002e+00, 4.5350594369244000e+03 },
    { 8.2339999999999995e-08, 3.6863071074799998e+00, 2.7007151403858002e+03 },
    { 7.8790000000000006e-08, 6.1630131841800004e+00, 4.2923308329503998e+03 },
    { 7.2139999999999996e-08, 1.1164759309800001e+00, 3.3198370312073998e+03 },
    { 7.1569999999999998e-08, 3.2759362683500002e+00, 3.3613878221922000e+03 },
    { 7.0130000000000007e-08, 4.8316009311299997e+00, 6.4897765872880000e+03 },
    { 6.9860000000000005e-08, 1.3159942588600000e+00, 3.8941818295421999e+03 },
    { 6.7119999999999995e-08, 4.2234883750000000e-02, 1.0018314161750401e+04 },
    { 6.7070000000000004e-08, 5.4720104819800000e+00, 6.6812921637024001e+03 },
    { 6.7070000000000004e-08, 5.8805068688200004e+00, 6.6811575430967996e+03 },
    { 6.6269999999999997e-08, 5.7048858138499998e+00, 1.0025360398448400e+04 },
    { 6.4270000000000006e-08, 6.1094102584599996e+00, 4.5624609930212000e+03 },
    { 6.1280000000000003e-08, 4.5174184556199997e+00, 1.2218485663214001e+03 },
    { 5.6669999999999997e-08, 1.8457232492000000e+00, 2.9577158944766002e+03 },
    { 5.6410000000000000e-08, 2.4352733124000001e+00, 2.1187638603783998e+03 },
    { 5.4329999999999997e-08, 4.3376055511700002e+00, 2.3888940204492001e+03 },
    { 4.9159999999999998e-08, 2.9304696014500000e+00, 2.7870430238573999e+03 },
    { 4.7419999999999997e-08, 1.9792037391899999e+00, 3.3370219980480001e+03 },
    { 4.7220000000000002e-08, 2.4104190808300001e+00, 3.3442028553516002e+03 },
    { 4.6499999999999999e-08, 1.7127688830600001e+00, 6.3989728631399998e+02 },
    { 4.5970000000000001e-08, 4.4100287797700002e+00, 7.4775228602159996e+03 },
    { 4.3630000000000001e-08, 2.2432079971399999e+00, 1.1773376811515400e+04 },
    { 4.1269999999999998e-08, 2.0441188173399998e+00, 2.1461654164751999e+03 },
    { 3.7140000000000000e-08, 1.7641896562199999e+00, 3.8289653222319998e+02 },
    { 3.6130000000000000e-08, 3.6108198479700002e+00, 8.4292412664666008e+03 },
    { 3.5800000000000003e-08, 3.9047099453600000e+00, 7.9030734197210004e+03 },
    { 3.5089999999999999e-08, 3.9175847818400000e+00, 3.2055473466643998e+03 },
    { 3.4930000000000002e-08, 2.2566678451600000e+00, 3.7235089589230001e+03 },
    { 3.4779999999999998e-08, 2.6466096897500000e+00, 4.9296853219836003e+03 },
    { 3.3510000000000000e-08, 4.7359753009000000e-01, 3.4756775067352000e+03 },
    { 3.3309999999999999e-08, 6.1775343162100000e+00, 7.2109158184941998e+03 },
    { 3.1109999999999999e-08, 2.0340680933500002e+00, 1.5542039943419999e+02 },
    { 3.0250000000000003e-08, 1.9108327097700000e+00, 9.5952390892233998e+03 },
    { 3.0069999999999997e-08, 8.1247341819999996e-01, 2.2741169495098002e+03 },
    { 2.9819999999999998e-08, 3.6825700400499999e+00, 2.2883440435113998e+03 },
    { 2.7719999999999999e-08, 2.4396896877200001e+00, 8.8273902698748007e+03 },
    { 2.7330000000000001e-08, 5.1395284394999996e-01, 1.2832758741704600e+04 },
    { 2.6490000000000001e-08, 2.9874248386500000e+00, 3.7672106175757999e+03 },
    { 2.6140000000000001e-08, 4.4569407352699999e+00, 6.6741113063987996e+03 },
    { 2.5670000000000001e-08, 4.9609253305100003e+00, 8.0310922630584000e+03 },
    { 2.5130000000000001e-08, 1.0458597053000000e-01, 3.5833410306738001e+03 },
    { 2.3219999999999999e-08, 1.2324959234199999e+00, 4.9332084403325998e+03 },
    { 2.2329999999999999e-08, 2.9475800096100002e+00, 7.7406067835887998e+03 },
    { 2.1620000000000001e-08, 3.0182837651300001e+00, 6.0413275670856001e+03 },
    { 2.1360000000000001e-08, 3.8884222335000000e+00, 1.0021854533751601e+04 },
    { 2.1360000000000001e-08, 1.8576506020000001e+00, 1.0021820026447200e+04 },
    { 2.1039999999999998e-08, 2.4580992701100000e+00, 1.0522683831884001e+03 },
    { 2.0910000000000000e-08, 3.8048905987700001e+00, 6.8726731195111997e+03 },
    { 1.9300000000000001e-08, 5.3075910068700001e+00, 1.0664954771900000e+03 },
    { 1.9110000000000000e-08, 2.9410763003499998e+00, 1.7480164130670000e+03 },
    { 1.8609999999999999e-08, 4.1352213418500003e+00, 1.3498674096587999e+03 },
    { 1.8399999999999999e-08, 2.0978853292999999e-01, 4.0327700279266001e+03 },
    { 1.7660000000000001e-08, 1.8566146745800001e+00, 6.5258044539654002e+03 },
    { 1.7280000000000000e-08, 2.9558420153300000e+00, 1.2168002696574600e+04 },
    { 1.7039999999999999e-08, 4.5075781513299997e+00, 3.0978838227257902e+03 },
    { 1.6689999999999999e-08, 5.5049715579100003e+00, 9.5171840625059997e+02 },
    { 1.6130000000000000e-08, 6.2341029242000001e-01, 5.5356940284240000e+02 },
    { 1.6099999999999999e-08, 2.6567107994899999e+00, 3.3121632392319998e+03 },
    { 1.5889999999999999e-08, 1.7360129719999999e+00, 3.3690616141676001e+03 },
    { 1.5799999999999999e-08, 4.4866517630000002e+00, 3.5231183490000002e+00 },
    { 1.5790000000000001e-08, 3.0905333702300002e+00, 1.5643680203309799e+04 },
    { 1.4600000000000000e-08, 4.1014541150000000e-02, 7.6329432596502002e+03 },
    { 1.4419999999999999e-08, 1.8773123749000000e+00, 3.0775285033270002e+03 },
    { 1.4190000000000000e-08, 3.2932000506999998e-01, 3.2643463554241998e+03 },
    { 1.3920000000000000e-08, 2.7504136646999999e-01, 3.3253599555147998e+03 },
    { 1.3840000000000000e-08, 4.1171963324799998e+00, 3.3558648978848000e+03 },
    { 1.3790000000000000e-08, 9.0130235269999998e-02, 1.0419986283507600e+04 },
    { 1.3760000000000000e-08, 8.1116680513999995e-01, 3.5030790628320001e+03 },
    { 1.3200000000000001e-08, 3.2381205000400000e+00, 3.3477259737006002e+03 },
    { 1.2970000000000000e-08, 5.4848169993899996e+00, 1.1371704689758200e+04 },
    { 1.2919999999999999e-08, 2.5182913605600001e+00, 3.6036963500726001e+03 },
    { 1.2740000000000000e-08, 5.5350948231999997e-01, 3.1910492295652002e+03 },
    { 1.2610000000000000e-08, 1.8673047473300000e+00, 6.6802445323313996e+03 },
    { 1.2569999999999999e-08, 3.2032647331400002e+00, 6.6822051744678001e+03 },
    { 1.2420000000000000e-08, 5.7581650609199997e+00, 1.4956319713459999e+02 },
    { 1.2299999999999999e-08, 1.1094815434000000e+00, 5.3680451209540001e+02 },
    { 1.2159999999999999e-08, 2.9830664936200000e+00, 6.8366452528338004e+03 },
    { 1.1710000000000000e-08, 5.7964612925200001e+00, 2.3384286986898602e+04 },
    { 1.1609999999999999e-08, 4.6050501340599999e+00, 1.6484467571974001e+03 },
    { 1.1250000000000000e-08, 6.2734196924500001e+00, 1.2964300703391000e+04 },
    { 1.1180000000000000e-08, 6.0542925268300003e+00, 9.8085381846614000e+03 },
    { 1.1030000000000000e-08, 5.7956520571899999e+00, 6.8945239488376001e+03 },
    { 1.1010000000000001e-08, 4.6226322775700002e+00, 5.2257741809380002e+02 },
    { 1.0140000000000000e-08, 5.5883970460999999e-01, 1.0973555686350001e+04 },
    { 1.0039999999999999e-08, 4.7138044972699999e+00, 2.4874160449477999e+03 },
    { 9.6899999999999994e-09, 2.6568919415600001e+00, 2.6484548254729998e+03 },
    { 9.6799999999999997e-09, 4.4365005121500003e+00, 1.0309277421860000e+02 },
    { 9.6400000000000006e-09, 5.9868464522300000e+00, 2.4272860397400001e+02 },
    { 9.5800000000000004e-09, 1.4530086207899999e+00, 1.2566151699982800e+04 },
    { 9.5000000000000007e-09, 2.9820175790399999e+00, 3.1781457905676002e+03 },
    { 9.4600000000000000e-09, 1.8893672207900001e+00, 5.6289564702112002e+03 },
    { 9.3499999999999994e-09, 4.9493384531200002e+00, 1.7586530784168001e+03 },
    { 9.2500000000000001e-09, 5.4613190532500004e+00, 5.6147293762095996e+03 },
    { 9.0099999999999993e-09, 5.6792882339000004e+00, 2.9211277828246002e+03 },
    { 8.5299999999999993e-09, 4.8108633780799996e+00, 1.5110466119866200e+04 },
    { 8.4399999999999998e-09, 2.9790638487400001e+00, 7.8756718636242003e+03 },
    { 8.4000000000000008e-09, 6.1179904485099996e+00, 5.3313574437407997e+03 },
    { 8.2000000000000006e-09, 4.3782463752399998e+00, 7.3733824546264004e+03 },
    { 8.1199999999999993e-09, 4.1963978558699999e+00, 3.3350895023923999e+03 },
    { 8.1099999999999995e-09, 1.9379046861999999e-01, 3.3461353510071999e+03 },
    { 7.8500000000000008e-09, 4.5999088060700002e+00, 6.4384962494255997e+03 },
    { 7.7400000000000002e-09, 1.3869643920500001e+00, 7.2347942562420003e+03 },
    { 7.7099999999999992e-09, 2.4725270137500002e+00, 1.5113989238215199e+04 },
    { 7.6700000000000002e-09, 1.3544179235700000e+00, 3.3202571073009999e+03 },
    { 7.6399999999999993e-09, 4.3758191255399996e+00, 1.1769853693166400e+04 },
    { 7.6100000000000000e-09, 1.9922108118099999e+00, 2.6087903141574199e+04 },
    { 7.6000000000000002e-09, 3.0350105883600000e+00, 3.3609677460985899e+03 },
    { 7.4199999999999996e-09, 3.7908087242999999e-01, 1.3358926588450200e+04 },
    { 7.3200000000000004e-09, 5.1678494800800001e+00, 9.8303890139878004e+03 },
    { 7.3200000000000004e-09, 6.0423721904600001e+00, 1.3365972825148199e+04 },
    { 7.0399999999999997e-09, 5.8090829900800003e+00, 1.0021904590402200e+04 },
    { 7.0399999999999997e-09, 6.2175793769199998e+00, 1.0021769969796600e+04 },
    { 6.9200000000000001e-09, 1.4671825711499999e+00, 6.6604494579071998e+03 },
    { 6.7100000000000002e-09, 5.2522440660500003e+00, 7.6266071275599998e+01 },
    { 6.6899999999999999e-09, 5.1352618116000004e-01, 7.0641213856227996e+03 },
    { 6.6800000000000001e-09, 3.6113037794600000e+00, 6.7020002488919999e+03 },
    { 6.6100000000000001e-09, 2.7928619986899998e+00, 2.9069006888230001e+03 },
    { 6.5499999999999999e-09, 2.6000227598099999e+00, 1.1243685846420800e+04 },
    { 6.4599999999999996e-09, 3.0191943685300000e+00, 9.3819399937853996e+03 },
    { 6.4100000000000000e-09, 1.7326962778899999e+00, 1.6921656695024001e+03 },
    { 6.3000000000000002e-09, 1.5838683025800000e+00, 1.5819593482830001e+03 },
    { 6.2900000000000004e-09, 1.0988020263300000e+00, 1.4314168113049800e+04 },
    { 6.2300000000000002e-09, 4.8149184650699999e+00, 3.3299757613500001e+03 },
    { 6.2099999999999999e-09, 5.8569084290399998e+00, 3.3512490920496002e+03 },
    { 6.1200000000000004e-09, 3.4075081970399999e+00, 1.0713994881326200e+04 },
    { 6.0500000000000004e-09, 1.9380401186400000e+00, 1.0575406682941801e+04 },
    { 6.0000000000000000e-09, 5.0454765114600004e+00, 6.9215760122680001e+02 },
    { 5.8600000000000000e-09, 7.9051565116000000e-01, 8.5319638175199998e+02 },
    { 5.6999999999999998e-09, 4.6130789892299999e+00, 3.0740053849780002e+03 },
    { 5.2800000000000000e-09, 3.1261193309599999e+00, 7.4781598567299994e+01 },
    { 5.2400000000000001e-09, 6.0577013557199999e+00, 3.6072194684216001e+03 },
    { 5.2400000000000001e-09, 2.1752989810400001e+00, 6.2983283211764001e+03 },
    { 5.1600000000000004e-09, 2.4362964957700002e+00, 1.3916019109641600e+04 },
    { 5.0700000000000001e-09, 2.0293950289399998e+00, 3.3335661900017999e+03 },
    { 5.0600000000000003e-09, 1.1846182373400000e+00, 1.7444259844152000e+03 },
    { 4.8799999999999997e-09, 4.6781256484900000e+00, 5.7295064471490005e+03 },
    { 4.6699999999999998e-09, 3.0281240678799999e+00, 6.1444203413041996e+03 },
    { 4.6699999999999998e-09, 5.8107479137100002e+00, 6.1586474353058002e+03 },
    { 4.6500000000000003e-09, 5.5622476658000002e+00, 8.9553418029093991e+03 },
    { 4.5699999999999997e-09, 2.7648217207000001e+00, 5.4593762870782002e+03 },
    { 4.5399999999999996e-09, 5.9650223046899997e+00, 4.1948464387519999e+02 },
    { 4.5299999999999999e-09, 6.2663424924099997e+00, 1.4712317116458000e+04 },
    { 4.5200000000000001e-09, 4.7486857957300002e+00, 5.8280284716475999e+03 },
    { 4.5200000000000001e-09, 2.1866753755700001e+00, 1.2935851515923199e+04 },
    { 4.4999999999999998e-09, 4.9754971878200003e+00, 6.5316616562649997e+03 },
    { 4.4900000000000000e-09, 4.2790591395400002e+00, 6.5461597733642002e+03 },
    { 4.4400000000000004e-09, 2.3112675382400001e+00, 6.6776344247478000e+03 },
    { 4.4200000000000000e-09, 2.2127925095799998e+00, 3.3476586633978000e+03 },
    { 4.4200000000000000e-09, 3.5835774849000002e-01, 1.8052929543157799e+04 },
    { 4.3999999999999997e-09, 2.2802455371999999e+00, 8.9695688969109997e+03 },
    { 4.3899999999999999e-09, 3.2682391318400001e+00, 6.1276554505572003e+03 },
    { 4.3699999999999996e-09, 2.7463491343599999e+00, 6.6848152820513997e+03 },
    { 4.2999999999999996e-09, 4.7879344373799997e+00, 3.8631898447936001e+03 },
    { 4.2700000000000004e-09, 4.2766619939000000e-01, 1.0551528245194000e+04 },
    { 4.1800000000000000e-09, 4.8839586149200001e+00, 3.4437052009183999e+03 },
    { 4.1700000000000003e-09, 2.6241800971200000e+00, 4.3371173787679999e+02 },
    { 4.1000000000000003e-09, 4.5439293138299996e+00, 3.3444937620577998e+03 },
    { 4.1000000000000003e-09, 6.1379021912800003e+00, 3.3367310913418000e+03 },
    { 4.0899999999999997e-09, 1.9983607163599999e+00, 3.9805097130138001e+03 },
    { 4.0300000000000004e-09, 5.3693782688200002e+00, 1.5961864422845999e+03 },
    { 4.0300000000000004e-09, 1.8242339021900000e+00, 3.3999862886134001e+03 },
    { 4.0000000000000002e-09, 2.7786529913500000e+00, 4.9890591838971995e+03 },
    { 4.0000000000000002e-09, 1.6206326759300000e+00, 1.4584298273120599e+04 },
    { 3.9899999999999997e-09, 4.7425525843300003e+00, 1.0818135286915800e+04 },
    { 3.8700000000000001e-09, 2.2062421390100000e+00, 3.2812385647862002e+03 },
    { 3.8700000000000001e-09, 5.5055591494799998e+00, 3.9733961660129999e+03 },
    { 3.8000000000000001e-09, 4.9781884912000001e+00, 1.0014723733098601e+04 },
    { 3.7099999999999998e-09, 3.3340596600199999e+00, 1.0177257679533601e+04 },
    { 3.6600000000000002e-09, 5.4436256148999995e-01, 1.1020632121940000e+02 },
    { 3.6100000000000001e-09, 5.2408870368700002e+00, 2.1781377222920000e+03 },
    { 3.6000000000000000e-09, 8.1576341527999996e-01, 6.8162899334350004e+03 },
    { 3.5100000000000001e-09, 8.9374033083000004e-01, 1.6173371168404399e+04 },
    { 3.4900000000000001e-09, 4.7428206685700003e+00, 3.3167339889519999e+03 },
    { 3.4600000000000000e-09, 5.9387660771500004e+00, 3.3644908644476000e+03 },
    { 3.4299999999999999e-09, 1.0252547982200000e+00, 3.2304061054804001e+03 },
    { 3.4100000000000000e-09, 3.7652634102799998e+00, 6.6883384004004001e+03 },
    { 3.3799999999999999e-09, 3.0141100633900000e+00, 8.2702977486834006e+03 },
    { 3.3299999999999999e-09, 2.9739119878000002e+00, 1.7256631536341400e+04 },
    { 3.3000000000000002e-09, 3.8142183219099999e+00, 1.3517870106233400e+04 },
    { 3.2799999999999998e-09, 1.6070657338000001e+00, 3.5467979751369999e+03 },
    { 3.1000000000000000e-09, 1.7690517457299999e+00, 1.0404733812322600e+04 },
    { 3.0300000000000001e-09, 4.3932586534500002e+00, 2.8180350086059998e+03 },
    { 3.0199999999999999e-09, 5.7602082390299998e+00, 2.3843232707292000e+03 },
    { 3.0000000000000000e-09, 3.6103046616599999e+00, 3.3045845600224002e+03 },
    { 3.0000000000000000e-09, 7.0241789966000001e-01, 3.1344268782626000e+03 },
    { 2.9499999999999999e-09, 7.8856158129999998e-01, 3.3766402933772001e+03 },
    { 2.8999999999999999e-09, 4.3059501832500002e+00, 1.6858482532933202e+04 },
    { 2.8699999999999998e-09, 5.8967769083299997e+00, 3.3401923506061898e+03 },
    { 2.8699999999999998e-09, 4.7850373588000004e+00, 3.3410325027934000e+03 },
    { 2.8499999999999999e-09, 1.0171887281000000e-01, 1.7085958665722199e+04 },
    { 2.8400000000000001e-09, 2.4952921105699999e+00, 6.7843176276182003e+03 },
    { 2.7900000000000001e-09, 1.5295518242000000e-01, 1.3506508003540000e+02 },
    { 2.7700000000000002e-09, 3.2126661646499999e+00, 8.3296716105970008e+03 },
    { 2.7400000000000001e-09, 2.6065032353799999e+00, 3.6570042963564001e+03 },
    { 2.6900000000000000e-09, 1.9698472851000001e-01, 3.0242205570432002e+03 },
    { 2.6400000000000000e-09, 5.4585926658000004e+00, 6.6049587821240002e+03 },
    { 2.6400000000000000e-09, 1.8214690020099999e+00, 1.7654780539749601e+04 },
    { 2.5399999999999999e-09, 2.9381903137799998e+00, 5.0850384111149997e+03 },
    { 2.4699999999999999e-09, 4.7468757267499999e+00, 9.7791086761253991e+03 },
    { 2.4600000000000002e-09, 2.6479528886199999e+00, 3.3040070613956000e+03 },
    { 2.4500000000000000e-09, 1.0025430721599999e+00, 3.2090704650133998e+03 },
    { 2.4199999999999999e-09, 4.1755696146999999e-01, 1.8617458526354001e+03 },
    { 2.4100000000000002e-09, 3.3323575714899998e+00, 1.1081219210288600e+04 },
    { 2.4100000000000002e-09, 1.7483276125600000e+00, 3.3772177920039999e+03 },
    { 2.3800000000000001e-09, 4.3806562773000002e-01, 6.9239534573736000e+03 },
    { 2.3699999999999999e-09, 2.1973729123500001e+00, 1.3362432453146999e+04 },
    { 2.3600000000000001e-09, 4.2304263802299999e+00, 1.3362466960451400e+04 },
    { 2.2699999999999998e-09, 2.8804810884799998e+00, 3.4153940252671000e+03 },
    { 2.2499999999999999e-09, 3.2450219510000000e-01, 1.3149150611361199e+04 },
    { 2.2200000000000002e-09, 3.3962755938400000e+00, 3.4721543883862000e+03 },
    { 2.2200000000000002e-09, 1.3704917170699999e+00, 3.1699395560806001e+03 },
    { 2.1999999999999998e-09, 1.4383620967999999e+00, 6.6064432548323002e+03 },
    { 2.1799999999999999e-09, 3.3151885058000002e-01, 1.6304913130090799e+04 },
    { 2.1700000000000002e-09, 2.1853050308499999e+00, 2.0995392966449399e+04 },
    { 2.1499999999999998e-09, 5.2645704783599996e+00, 1.3757737998458001e+03 },
    { 2.1100000000000000e-09, 4.3813675995599999e+00, 6.4087760738220004e+02 },
    { 2.0799999999999998e-09, 3.0273872094400001e+00, 3.5112852973190002e+03 },
    { 2.0799999999999998e-09, 3.3480936197900002e+00, 3.3410423098265001e+03 },
    { 2.0799999999999998e-09, 6.1849634862900000e+00, 1.4788665740644001e+03 },
    { 2.0799999999999998e-09, 1.0457695038999999e+00, 3.3401825435730998e+03 },
    { 2.0700000000000001e-09, 3.6059117833299998e+00, 7.1078230442756003e+03 },
    { 2.0500000000000002e-09, 4.2015795269299998e+00, 6.8436914895317996e+03 },
    { 2.0500000000000002e-09, 5.0194013820799999e+00, 1.1187557921027999e+03 },
    { 2.0400000000000000e-09, 6.0046379183500003e+00, 1.6246663613219999e+02 },
    { 2.0000000000000001e-09, 3.0011278137000001e+00, 1.1624747044077999e+03 },
    { 1.9800000000000002e-09, 9.7311347960000005e-01, 3.2658308281324998e+03 },
    { 1.9599999999999998e-09, 3.3450547679899998e+00, 8.4398779318163997e+03 },
    { 1.9399999999999999e-09, 4.5250741445099996e+00, 2.2041264243879999e+02 },
    { 1.9399999999999999e-09, 3.4153467139800000e+00, 1.8984292630009601e+04 },
    { 1.9300000000000002e-09, 5.8896882498799998e+00, 2.8038079146044001e+03 },
    { 1.9200000000000000e-09, 5.0847541507100003e+00, 1.3119721102825190e+04 },
    { 1.9000000000000001e-09, 6.1763748490000001e+00, 1.3745346239022399e+04 },
    { 1.8800000000000001e-09, 5.5624189917500004e+00, 1.6460333529524989e+04 },
    { 1.8600000000000000e-09, 1.7594095856100000e+00, 2.2218566345969998e+03 },
    { 1.8400000000000001e-09, 2.0944984533000000e+00, 9.4917560896980001e+02 },
    { 1.8400000000000001e-09, 1.5801792875200000e+00, 8.2738208670324002e+03 },
    { 1.8300000000000001e-09, 3.6296227926100002e+00, 9.1686408983473993e+03 },
    { 1.8300000000000001e-09, 2.1904167858000001e+00, 9.8664168806651996e+03 },
    { 1.7500000000000000e-09, 4.0416512873799997e+00, 3.7600970705750001e+03 },
    { 1.7500000000000000e-09, 3.8064797784999999e-01, 9.5628915597059995e+02 },
    { 1.7400000000000000e-09, 4.5111792723299997e+00, 2.3539707386332801e+04 },
    { 1.7400000000000000e-09, 5.5877831812999998e+00, 2.7662676283649998e+03 },
    { 1.7300000000000000e-09, 3.9052074944399999e+00, 5.0992655051166003e+03 },
    { 1.7100000000000001e-09, 4.7557001137899997e+00, 6.2617402095243997e+03 },
    { 1.7100000000000001e-09, 3.6060029411899999e+00, 2.0597243963041201e+04 },
    { 1.6999999999999999e-09, 3.3740623265899998e+00, 1.5508615123274400e+04 },
    { 1.6999999999999999e-09, 2.0005095742400001e+00, 2.2434479570190001e+02 },
    { 1.6900000000000000e-09, 3.7112716520300002e+00, 1.0184303916231600e+04 },
    { 1.6700000000000000e-09, 9.4707767481000005e-01, 3.2741250177853999e+03 },
    { 1.6500000000000001e-09, 5.4493966973400001e+00, 1.2289621133221999e+03 },
    { 1.6399999999999999e-09, 2.2052905390399999e+00, 3.8133035637799999e+01 },
    { 1.5799999999999999e-09, 2.9857481271299999e+00, 1.3524916342931399e+04 },
    { 1.5799999999999999e-09, 1.7220610476800000e+00, 3.2968935143948001e+03 },
    { 1.5600000000000000e-09, 3.6805817708199999e+00, 1.3553897972910800e+04 },
    { 1.5600000000000000e-09, 4.9062016220800002e+00, 2.0199094959632999e+04 },
    { 1.5600000000000000e-09, 4.3599559375000002e-01, 4.4593682188025996e+03 },
    { 1.5600000000000000e-09, 1.2569141303200000e+00, 3.3843313390048002e+03 },
    { 1.5400000000000001e-09, 2.1725503179999999e+00, 1.9648386268540000e+03 },
    { 1.5100000000000000e-09, 1.5980901207600000e+00, 1.3892140671893800e+04 },
    { 1.4500000000000000e-09, 6.1716114054000002e+00, 1.9800945956224801e+04 },
    { 1.4500000000000000e-09, 2.8858103705499998e+00, 4.3928808098882000e+03 },
    { 1.4500000000000000e-09, 5.6586698568299996e+00, 4.4071079038897997e+03 },
    { 1.4400000000000000e-09, 2.2567806033000001e-01, 2.6308392337279997e+02 },
    { 1.4300000000000000e-09, 1.9892133284999999e-01, 1.0235136375537400e+04 },
    { 1.4300000000000000e-09, 4.4862113833399997e+00, 6.3278373931320004e+02 },
    { 1.3999999999999999e-09, 1.5411097902899999e+00, 2.1116503133776000e+03 },
    { 1.3300000000000000e-09, 2.8675923544900002e+00, 2.0618019358533598e+04 },
    { 1.3300000000000000e-09, 6.1491880700200001e+00, 1.7277406931833801e+04 },
    { 1.2900000000000001e-09, 2.2019822355100001e+00, 3.4508187479192002e+03 },
    { 1.2700000000000000e-09, 2.6865231951399999e+00, 7.3211221397135996e+03 },
    { 1.2600000000000000e-09, 4.5428658284400001e+00, 3.9149572250346000e+03 },
    { 1.2500000000000000e-09, 2.2039708812300001e+00, 3.0220932239732399e+04 },
    { 1.2400000000000001e-09, 2.7319740085099999e+00, 3.4070998356141999e+03 },
    { 1.1700000000000001e-09, 4.9615307992400002e+00, 3.9814900340820000e+03 },
    { 1.1700000000000001e-09, 2.1376364003499999e+00, 2.9424634232916000e+04 },
    { 1.1599999999999999e-09, 2.9046095700000000e-02, 2.6482170809624400e+04 },
    { 1.1500000000000000e-09, 5.2485095314999999e+00, 3.2375196524812000e+03 },
    { 1.1500000000000000e-09, 3.8603739543999999e-01, 1.2721572099417001e+04 },
    { 1.1300000000000000e-09, 3.0685404443700000e+00, 2.0618554843720000e+02 },
    { 1.1300000000000000e-09, 1.3655807676800000e+00, 6.7096740408674004e+03 },
    { 1.1200000000000001e-09, 6.2635810282500000e+00, 2.1258774073792001e+03 },
    { 1.1200000000000001e-09, 4.6786161397699999e+00, 2.8073983432562000e+03 },
    { 1.0999999999999999e-09, 1.8586807658000001e-01, 2.9822783236324201e+04 },
    { 1.0900000000000000e-09, 5.2718730795199997e+00, 5.5187501489918004e+03 },
    { 1.0800000000000000e-09, 6.2021980254700004e+00, 6.6659723822145997e+03 },
    { 1.0700000000000000e-09, 5.0226646577000000e-01, 2.2743409379516401e+04 },
    { 1.0700000000000000e-09, 3.5985604686400001e+00, 5.9890672521728002e+03 },
    { 1.0600000000000001e-09, 5.6996616603400003e+00, 3.3561544666432201e+04 },
    { 1.0500000000000001e-09, 3.9737597608000002e+00, 9.3809596727171993e+03 },
};

static const VSOP87Term VSOP87A_MARS_Z1[232] = {
    { 3.3184295800000002e-03, 6.0502777349199999e+00, 3.3406124266997999e+03 },
    { 4.7930411000000000e-04, 3.1415926535900001e+00, 0.0000000000000000e+00 },
    { 9.8965009999999999e-05, 1.6115584471500000e+00, 6.6812248533995999e+03 },
    { 1.7001470000000001e-05, 2.6370324206500002e+00, 1.0021837280099400e+04 },
    { 2.4017600000000002e-06, 3.1271230341399998e+00, 1.3362449706799200e+04 },
    { 1.1490800000000000e-06, 9.5987621952000002e-01, 3.3370893083507999e+03 },
    { 4.5173999999999999e-07, 3.3996514756699998e+00, 3.3441355450487999e+03 },
    { 4.1578999999999999e-07, 2.2274883923700002e+00, 2.2812304965106000e+03 },
    { 3.1105999999999998e-07, 3.5276010730700000e+00, 1.6703062133498999e+04 },
    { 2.4694000000000001e-07, 5.6373112342999998e-01, 5.2969096509459996e+02 },
    { 2.2408999999999999e-07, 3.7341509000999999e+00, 5.6218429232103999e+03 },
    { 2.0211000000000000e-07, 6.1372296655099996e+00, 1.0593819301891999e+03 },
    { 1.6941999999999999e-07, 1.9191717796500001e+00, 3.5320606928113998e+03 },
    { 1.6081000000000000e-07, 7.4382156250999998e-01, 2.9424634232916001e+03 },
    { 1.4812999999999999e-07, 5.4949954264100001e+00, 3.3406296803519999e+03 },
    { 1.4812999999999999e-07, 3.4642235856600001e+00, 3.3405951730475999e+03 },
    { 1.3077000000000000e-07, 2.0111362483700002e+00, 2.5443144198834002e+03 },
    { 1.2609000000000001e-07, 4.3990127176099998e+00, 3.4960328261340001e+03 },
    { 1.1579000000000001e-07, 7.6965497834999996e-01, 3.9814900340819997e+02 },
    { 1.1449000000000000e-07, 6.2290060924899997e+00, 2.8109214616052000e+03 },
    { 9.9830000000000001e-08, 6.2296142658500004e+00, 3.1851920272655998e+03 },
    { 9.6279999999999999e-08, 2.1799873563999999e+00, 6.1515338883049999e+03 },
    { 9.1129999999999996e-08, 3.1308123000700001e+00, 3.7387614301079998e+03 },
    { 9.1119999999999998e-08, 3.8369936040399999e+00, 6.6847479717486003e+03 },
    { 8.7019999999999994e-08, 2.5017319659899999e+00, 5.8849268465832001e+03 },
    { 8.1409999999999999e-08, 2.2883367245500001e+00, 3.1491641605882000e+03 },
    { 5.8649999999999999e-08, 1.2118973083000000e+00, 6.6777017350506003e+03 },
    { 5.4319999999999999e-08, 1.1152893794200001e+00, 7.9629800681639995e+02 },
    { 5.3540000000000002e-08, 2.1784461286900001e+00, 4.1369104335162001e+03 },
    { 5.1109999999999999e-08, 2.4843603187799999e+00, 1.7515395314160000e+03 },
    { 4.9780000000000000e-08, 4.2028381997999997e-01, 1.1944470102246000e+03 },
    { 4.9369999999999998e-08, 4.5470634793700002e+00, 8.9624553499102003e+03 },
    { 4.8779999999999997e-08, 1.1327648133699999e+00, 3.3406797370026002e+03 },
    { 4.8760000000000001e-08, 1.5389850940600001e+00, 3.3405451163970001e+03 },
    { 4.8170000000000001e-08, 9.9936257716999999e-01, 2.9140142358237999e+03 },
    { 4.6740000000000000e-08, 3.9343101932700000e+00, 5.4867778431750003e+03 },
    { 4.5970000000000001e-08, 1.5637908738699999e+00, 6.2830758499914000e+03 },
    { 4.3880000000000000e-08, 4.0037689251300002e+00, 5.0921519581158000e+03 },
    { 4.0159999999999997e-08, 3.2529141122200000e+00, 3.3396321056316001e+03 },
    { 3.9069999999999998e-08, 3.8995433539100000e+00, 2.0043674560198800e+04 },
    { 3.5950000000000001e-08, 5.4521931081300004e+00, 1.5925960136327999e+03 },
    { 3.3750000000000001e-08, 3.4463583737099999e+00, 2.1461654164751999e+03 },
    { 3.3360000000000002e-08, 5.2050325996500000e+00, 5.0886288397668004e+03 },
    { 3.3130000000000000e-08, 2.5803824385900000e+00, 3.8703033917943999e+03 },
    { 2.9950000000000000e-08, 7.4761803992999998e-01, 4.3999943568890003e+03 },
    { 2.8870000000000000e-08, 2.9489117555000002e+00, 3.8941818295421999e+03 },
    { 2.7070000000000000e-08, 2.1082942855000000e-01, 4.6904798363585996e+03 },
    { 2.6689999999999999e-08, 4.0877211523000001e-01, 1.5890728952838001e+03 },
    { 2.6569999999999999e-08, 1.0597333513300000e+00, 3.3415927477680002e+03 },
    { 2.4369999999999999e-08, 1.5503788770900000e+00, 4.2923308329503998e+03 },
    { 2.4249999999999999e-08, 4.1064383912100002e+00, 1.9907450170410000e+03 },
    { 2.3750000000000001e-08, 5.4457953608900000e+00, 3.1273133312618002e+03 },
    { 2.1540000000000000e-08, 1.3419653203399999e+00, 2.7870430238573999e+03 },
    { 1.9289999999999999e-08, 2.7382445163400000e+00, 2.3888940204492001e+03 },
    { 1.7529999999999999e-08, 3.9238817809400000e+00, 7.1135470007999997e+00 },
    { 1.6499999999999999e-08, 4.2481548414299999e+00, 3.7235089589230001e+03 },
    { 1.5950000000000001e-08, 9.1889240925000004e-01, 4.5350594369244000e+03 },
    { 1.5530000000000001e-08, 5.9580007480799999e+00, 3.0978838227257902e+03 },
    { 1.5230000000000001e-08, 4.0427253021200000e+00, 7.0793738568077997e+03 },
    { 1.5200000000000000e-08, 4.8258032162999998e+00, 3.5833410306738001e+03 },
    { 1.4710000000000001e-08, 4.2049579840100000e+00, 1.0025360398448400e+04 },
    { 1.4530000000000000e-08, 5.7347650924000004e-01, 4.2659819087599999e+02 },
    { 1.4440000000000001e-08, 1.5895080264000000e-01, 2.9577158944766002e+03 },
    { 1.4359999999999999e-08, 3.0131937397900002e+00, 6.2546266625236003e+03 },
    { 1.4050000000000000e-08, 3.6484960397699999e+00, 1.5542039943419999e+02 },
    { 1.3469999999999999e-08, 1.6383699177700000e+00, 1.9144826611159999e+02 },
    { 1.3180000000000000e-08, 4.0890394626999998e-01, 2.1329909543799999e+02 },
    { 1.2070000000000000e-08, 2.9434164548199999e+00, 6.4897765872880000e+03 },
    { 1.1780000000000000e-08, 4.8481314470700001e+00, 1.7480164130670000e+03 },
    { 1.0680000000000000e-08, 4.7808396461999996e+00, 8.4327643848156004e+03 },
    { 1.0560000000000000e-08, 3.0211067524000002e-01, 6.5258044539654002e+03 },
    { 1.0130000000000000e-08, 4.0159005199799997e+00, 9.4921463150048003e+03 },
    { 1.0039999999999999e-08, 2.0864632842200002e+00, 2.2883440435113998e+03 },
    { 9.6899999999999994e-09, 1.3174292533500001e+00, 3.3334988796990001e+03 },
    { 9.2199999999999992e-09, 2.7170826324599999e+00, 7.4775228602159996e+03 },
    { 8.9999999999999995e-09, 2.9043699778100001e+00, 3.5539115221378001e+03 },
    { 8.8400000000000001e-09, 5.3084018045299999e+00, 6.6812075997474003e+03 },
    { 8.8400000000000001e-09, 1.0559879826300000e+00, 6.6812421070518003e+03 },
    { 8.2100000000000004e-09, 5.0918826839999998e-01, 6.6741113063987996e+03 },
    { 7.6600000000000004e-09, 2.7521660676000002e+00, 9.2255392732830005e+03 },
    { 7.2099999999999997e-09, 4.4554887234800002e+00, 4.5624609930212000e+03 },
    { 7.1099999999999996e-09, 2.2884818150599999e+00, 5.5356940284240000e+02 },
    { 7.0999999999999999e-09, 5.9797616539099998e+00, 6.3989728631399998e+02 },
    { 6.9600000000000000e-09, 3.4605140976600000e+00, 3.8289653222319998e+02 },
    { 6.6700000000000003e-09, 6.0635554739200002e+00, 1.3498674096587999e+03 },
    { 6.5700000000000003e-09, 3.5758423713999998e+00, 1.0664954771900000e+03 },
    { 6.3700000000000001e-09, 2.6514085316300000e+00, 2.2741169495098002e+03 },
    { 6.0799999999999997e-09, 5.8605193969800000e+00, 4.9332084403325998e+03 },
    { 6.0399999999999998e-09, 4.3876718470700000e+00, 1.0522683831884001e+03 },
    { 5.9099999999999997e-09, 1.9202976292900000e+00, 3.3253599555147998e+03 },
    { 5.7800000000000003e-09, 4.5919067442700001e+00, 3.3609677460985899e+03 },
    { 5.7299999999999999e-09, 9.9737613449999996e-01, 9.5171840625059997e+02 },
    { 5.4599999999999998e-09, 2.6493926183999998e+00, 1.2218485663214001e+03 },
    { 5.4100000000000001e-09, 4.5325043390799999e+00, 6.8366452528338004e+03 },
    { 5.1600000000000004e-09, 6.0845899059899997e+00, 3.3202571073009999e+03 },
    { 5.1499999999999998e-09, 5.5144608834700000e+00, 3.3613878221922000e+03 },
    { 5.1099999999999999e-09, 4.3913642474400003e+00, 2.4272860397400001e+02 },
    { 4.9300000000000001e-09, 1.5145335644900000e+00, 6.4679257579615996e+03 },
    { 4.8699999999999999e-09, 5.7637076730199999e+00, 3.2055473466643998e+03 },
    { 4.8399999999999998e-09, 4.2589735329599998e+00, 2.3384286986898602e+04 },
    { 4.7900000000000002e-09, 1.9705724948300001e+00, 1.0213285546211000e+04 },
    { 4.7699999999999999e-09, 1.2913263500600001e+00, 3.3477259737006002e+03 },
    { 4.7699999999999999e-09, 3.4212243439000001e-01, 2.7007151403858002e+03 },
    { 4.6900000000000001e-09, 3.9448877554299999e+00, 8.8273902698748007e+03 },
    { 4.6299999999999999e-09, 9.9068109708999996e-01, 4.9296853219836003e+03 },
    { 4.6200000000000002e-09, 3.4579256367000002e+00, 3.3198370312073998e+03 },
    { 4.4699999999999997e-09, 2.4480701543999999e+00, 3.3558648978848000e+03 },
    { 4.4200000000000000e-09, 4.1487045739199999e+00, 6.0413275670856001e+03 },
    { 4.3800000000000002e-09, 1.2903510383300001e+00, 7.7406067835887998e+03 },
    { 4.3500000000000001e-09, 3.6977250222200002e+00, 7.2109158184941998e+03 },
    { 4.2400000000000002e-09, 1.7271442573000001e-01, 5.2257741809380002e+02 },
    { 4.1499999999999999e-09, 3.0661716597999999e-01, 1.2303067776610000e+04 },
    { 3.9600000000000004e-09, 1.1071298014600000e+00, 2.1187638603783998e+03 },
    { 3.9499999999999998e-09, 7.4904985729999995e-01, 5.6147293762095996e+03 },
    { 3.8799999999999998e-09, 6.2580013239300003e+00, 6.4384962494255997e+03 },
    { 3.4100000000000000e-09, 1.3794108673500001e+00, 3.7672106175757999e+03 },
    { 3.4100000000000000e-09, 1.4996230968299999e+00, 6.6822051744678001e+03 },
    { 3.3799999999999999e-09, 5.5590607435999999e+00, 5.3680451209540001e+02 },
    { 3.1700000000000000e-09, 4.4164010021099998e+00, 3.3370219980480001e+03 },
    { 3.1599999999999998e-09, 4.8474926061400003e+00, 3.3442028553516002e+03 },
    { 3.1099999999999998e-09, 4.4733659949800000e+00, 5.3313574437407997e+03 },
    { 2.9800000000000000e-09, 5.0233577463900003e+00, 1.2832758741704600e+04 },
    { 2.9100000000000001e-09, 3.3871649001000002e+00, 6.6811575430967996e+03 },
    { 2.9100000000000001e-09, 2.9787035212499999e+00, 6.6812921637024001e+03 },
    { 2.8600000000000000e-09, 4.6133737225999996e+00, 1.0419986283507600e+04 },
    { 2.8299999999999999e-09, 3.1865916767800000e+00, 9.6236882766912004e+03 },
    { 2.8100000000000000e-09, 5.0912204199900000e+00, 7.9030734197210004e+03 },
    { 2.7799999999999999e-09, 3.9637490209899999e+00, 2.9211277828246002e+03 },
    { 2.7099999999999999e-09, 3.6402171459599999e+00, 6.6802445323313996e+03 },
    { 2.4000000000000000e-09, 4.1037780465399996e+00, 1.1773376811515400e+04 },
    { 2.3800000000000001e-09, 1.6950009101900001e+00, 6.1276554505572003e+03 },
    { 2.2900000000000002e-09, 1.2592658595799999e+00, 7.8756718636242003e+03 },
    { 2.2699999999999998e-09, 2.0795930934100002e+00, 1.0021854533751601e+04 },
    { 2.2699999999999998e-09, 4.8823396450000001e-02, 1.0021820026447200e+04 },
    { 2.2499999999999999e-09, 4.5316155567300003e+00, 1.3365972825148199e+04 },
    { 2.2100000000000000e-09, 4.5313904163399998e+00, 3.3121632392319998e+03 },
    { 2.1900000000000001e-09, 1.3194362445300001e+00, 3.3644908644476000e+03 },
    { 2.1700000000000002e-09, 3.0743980113000000e+00, 5.7295064471490005e+03 },
    { 2.1200000000000001e-09, 2.4971087475200000e+00, 3.5030790628320001e+03 },
    { 2.1100000000000000e-09, 3.1897560476400000e+00, 1.7586530784168001e+03 },
    { 2.0700000000000001e-09, 2.8064114140199998e+00, 6.8726731195111997e+03 },
    { 2.0400000000000000e-09, 4.5255901559099998e+00, 1.2168002696574600e+04 },
    { 1.9900000000000000e-09, 3.9120319518900000e+00, 6.8945239488376001e+03 },
    { 1.9700000000000000e-09, 3.2915462269100000e+00, 1.7654780539749601e+04 },
    { 1.9099999999999998e-09, 3.0614794168700001e+00, 3.3167339889519999e+03 },
    { 1.8700000000000000e-09, 3.3987638729400000e+00, 9.8303890139878004e+03 },
    { 1.8500000000000000e-09, 1.7684126571900001e+00, 3.5231183490000002e+00 },
    { 1.8400000000000001e-09, 5.7826487554500003e+00, 1.1769853693166400e+04 },
    { 1.8100000000000000e-09, 5.0210362291499999e+00, 6.1444203413041996e+03 },
    { 1.7900000000000000e-09, 5.2534002867999996e-01, 6.2983283211764001e+03 },
    { 1.7300000000000000e-09, 6.1770069918899999e+00, 5.6289564702112002e+03 },
    { 1.6999999999999999e-09, 3.8207610785799999e+00, 9.5952390892233998e+03 },
    { 1.6999999999999999e-09, 2.3047559080000002e+00, 3.3461353510071999e+03 },
    { 1.6800000000000000e-09, 5.2225480505400004e+00, 8.5319638175199998e+02 },
    { 1.6399999999999999e-09, 5.1656581241700001e+00, 6.9239534573736000e+03 },
    { 1.6399999999999999e-09, 8.7273677570999997e-01, 2.4874160449477999e+03 },
    { 1.6100000000000000e-09, 1.2930458619400000e+00, 1.5643680203309799e+04 },
    { 1.5600000000000000e-09, 4.2553346137899997e+00, 3.4756775067352000e+03 },
    { 1.5199999999999999e-09, 2.1281816683299999e+00, 3.1781457905676002e+03 },
    { 1.4900000000000000e-09, 3.0996476674000002e+00, 1.0818135286915800e+04 },
    { 1.4700000000000001e-09, 1.1394946698999999e+00, 8.9553418029093991e+03 },
    { 1.4700000000000001e-09, 3.6589690743799999e+00, 1.5961864422845999e+03 },
    { 1.4599999999999999e-09, 9.9908711720999999e-01, 4.3371173787679999e+02 },
    { 1.4300000000000000e-09, 3.6304973352999999e+00, 1.5819593482830001e+03 },
    { 1.4200000000000001e-09, 9.1426251678000003e-01, 2.6997348193175999e+03 },
    { 1.4100000000000001e-09, 7.8662586076999996e-01, 1.0018314161750401e+04 },
    { 1.3999999999999999e-09, 4.0941198366500000e+00, 6.1586474353058002e+03 },
    { 1.3900000000000000e-09, 1.4872093804800000e+00, 1.4712317116458000e+04 },
    { 1.3000000000000001e-09, 1.3137787681499999e+00, 7.6329432596502002e+03 },
    { 1.3000000000000001e-09, 2.6177874228900002e+00, 7.2347942562420003e+03 },
    { 1.3000000000000001e-09, 5.2294678055499997e+00, 3.3045845600224002e+03 },
    { 1.2700000000000000e-09, 4.4063508970900003e+00, 1.2964300703391000e+04 },
    { 1.2500000000000000e-09, 6.4246695672999998e-01, 9.8664168806651996e+03 },
    { 1.2000000000000000e-09, 1.5636750158499999e+00, 3.3367310913418000e+03 },
    { 1.1900000000000000e-09, 2.9760218935600000e+00, 6.4087760738220004e+02 },
    { 1.1800000000000001e-09, 1.9771386808700000e+00, 6.6883384004004001e+03 },
    { 1.1599999999999999e-09, 1.7702482960199999e+00, 3.3350895023923999e+03 },
    { 1.0999999999999999e-09, 2.9631373151000000e+00, 1.0575406682941801e+04 },
    { 1.0999999999999999e-09, 3.4606761114000001e-01, 8.9695688969109997e+03 },
    { 1.0800000000000000e-09, 1.4275830710599999e+00, 7.0641213856227996e+03 },
    { 1.0800000000000000e-09, 1.5171667705800000e+00, 4.1948464387519999e+02 },
    { 1.0300000000000000e-09, 3.5376408608999999e+00, 1.7444259844152000e+03 },
    { 1.0200000000000000e-09, 5.4346609774400001e+00, 3.3766402933772001e+03 },
    { 1.0000000000000001e-09, 1.7115286882900000e+00, 2.6484548254729998e+03 },
    { 9.7999999999999992e-10, 1.8721764890000001e-01, 3.9805097130138001e+03 },
    { 9.7999999999999992e-10, 1.7235130355599999e+00, 1.8052929543157799e+04 },
    { 9.6999999999999996e-10, 4.3107220168799998e+00, 3.0775285033270002e+03 },
    { 9.5999999999999999e-10, 1.4160811948700001e+00, 5.0850384111149997e+03 },
    { 8.9000000000000003e-10, 3.8926679494999999e-01, 5.4593762870782002e+03 },
    { 8.6999999999999999e-10, 4.9536041243400000e+00, 3.6036963500726001e+03 },
    { 8.3999999999999999e-10, 2.7734376147200002e+00, 3.3444937620577998e+03 },
    { 8.3000000000000003e-10, 3.4804485176400002e+00, 1.3916019109641600e+04 },
    { 8.1999999999999996e-10, 3.9654108454800001e+00, 1.0021904590402200e+04 },
    { 8.1999999999999996e-10, 4.3717495884100002e+00, 1.0021769969796600e+04 },
    { 8.0999999999999999e-10, 4.5198439279000002e+00, 9.3819399937853996e+03 },
    { 8.0000000000000003e-10, 4.5778987234299997e+00, 1.1243685846420800e+04 },
    { 8.0000000000000003e-10, 5.4786387167299999e+00, 3.3690616141676001e+03 },
    { 8.0000000000000003e-10, 1.3241217705399999e+00, 2.1393541969857601e+04 },
    { 7.7999999999999999e-10, 1.3687472191800001e+00, 1.1081219210288600e+04 },
    { 7.7000000000000003e-10, 3.0886632738000003e-01, 3.8631898447936001e+03 },
    { 7.7000000000000003e-10, 4.7762296566900000e+00, 4.0327700279266001e+03 },
    { 7.5999999999999996e-10, 4.5941210724100001e+00, 1.0177257679533601e+04 },
    { 7.5999999999999996e-10, 6.1536424712600004e+00, 8.2738208670324002e+03 },
    { 7.5999999999999996e-10, 4.8616451121899997e+00, 2.0597243963041201e+04 },
    { 7.4000000000000003e-10, 4.6347570602999999e-01, 8.4292412664666008e+03 },
    { 7.4000000000000003e-10, 2.7777342748399998e+00, 3.9814900340820000e+03 },
    { 7.2000000000000000e-10, 4.4542398675400001e+00, 1.7256631536341400e+04 },
    { 7.1000000000000003e-10, 9.1306168511999997e-01, 8.2702977486834006e+03 },
    { 7.1000000000000003e-10, 4.6249816560700001e+00, 1.0551528245194000e+04 },
    { 6.9000000000000000e-10, 4.0745857251200004e+00, 4.4071079038897997e+03 },
    { 6.6999999999999996e-10, 2.6225280922100001e+00, 6.9215760122680001e+02 },
    { 6.6000000000000000e-10, 1.3337760474999999e-01, 1.5110466119866200e+04 },
    { 6.6000000000000000e-10, 2.3936248717000002e+00, 7.6266071275599998e+01 },
    { 6.6000000000000000e-10, 2.1946862186999999e-01, 8.0310922630584000e+03 },
    { 6.5000000000000003e-10, 2.4955372209400002e+00, 1.0973555686350001e+04 },
    { 6.5000000000000003e-10, 1.8346101533600001e+00, 1.4314168113049800e+04 },
    { 6.3000000000000000e-10, 5.0337339801800001e+00, 1.6921656695024001e+03 },
    { 6.3000000000000000e-10, 4.3880962128499998e+00, 7.3733824546264004e+03 },
    { 6.2000000000000003e-10, 3.4568900511999998e+00, 1.3517870106233400e+04 },
    { 6.0999999999999996e-10, 3.1673418239900002e+00, 2.7662676283649998e+03 },
    { 6.0999999999999996e-10, 3.3571173373500001e+00, 2.0995392966449399e+04 },
    { 5.9000000000000003e-10, 2.4222793545800001e+00, 1.2566151699982800e+04 },
    { 5.9000000000000003e-10, 4.9664910322900004e+00, 3.1910492295652002e+03 },
    { 5.7000000000000000e-10, 5.0195851150399999e+00, 5.8280284716475999e+03 },
    { 5.6000000000000003e-10, 3.5573007261499998e+00, 1.0713994881326200e+04 },
    { 5.6000000000000003e-10, 4.7902707293700004e+00, 1.6173371168404399e+04 },
    { 5.4999999999999996e-10, 6.2300485605900002e+00, 3.9149572250346000e+03 },
    { 5.4999999999999996e-10, 5.3445242959100003e+00, 1.3358926588450200e+04 },
    { 5.4999999999999996e-10, 8.0155920043999995e-01, 1.1371704689758200e+04 },
    { 5.4000000000000000e-10, 5.1358052230999995e-01, 6.7020002488919999e+03 },
    { 5.3000000000000003e-10, 6.2458910177700000e+00, 2.0199094959632999e+04 },
    { 5.1999999999999996e-10, 4.9495785867700004e+00, 2.4734154396557398e+04 },
};

static const VSOP87Term VSOP87A_MARS_Z2[122] = {
    { 1.3705360000000001e-04, 1.0421285259799999e+00, 3.3406124266997999e+03 },
    { 5.9315960000000003e-05, 3.1415926535900001e+00, 0.0000000000000000e+00 },
    { 7.1672800000000003e-06, 1.2154825255000000e-01, 6.6812248533995999e+03 },
    { 1.3851399999999999e-06, 7.8090653398999998e-01, 1.0021837280099400e+04 },
    { 2.5745000000000002e-07, 1.3907744907699999e+00, 1.3362449706799200e+04 },
    { 1.2181000000000000e-07, 2.5873283490799999e+00, 3.3370893083507999e+03 },
    { 4.5359999999999997e-08, 1.3677000168300000e+00, 3.3441355450487999e+03 },
    { 4.3709999999999999e-08, 1.8736885953500000e+00, 1.6703062133498999e+04 },
    { 3.7370000000000003e-08, 6.0028090279499997e+00, 3.4960328261340001e+03 },
    { 2.5869999999999999e-08, 4.6286856505499996e+00, 3.1851920272655998e+03 },
    { 1.7220000000000001e-08, 9.8795977798000001e-01, 2.2812304965106000e+03 },
    { 1.6149999999999999e-08, 5.6679133309600003e+00, 5.6218429232103999e+03 },
    { 1.5200000000000000e-08, 3.5434460337800000e+00, 3.5320606928113998e+03 },
    { 1.0680000000000000e-08, 4.6653632603100004e+00, 1.0593819301891999e+03 },
    { 1.0410000000000000e-08, 3.8870172622100001e+00, 2.5443144198834002e+03 },
    { 1.0370000000000000e-08, 2.6334245915599999e+00, 2.9424634232916001e+03 },
    { 9.7599999999999994e-09, 1.9849346501000000e+00, 5.2969096509459996e+02 },
    { 9.0500000000000000e-09, 2.0466939932299999e+00, 6.6847479717486003e+03 },
    { 7.6999999999999995e-09, 5.1011303627700002e+00, 2.8109214616052000e+03 },
    { 7.1300000000000000e-09, 4.1632804017299998e+00, 5.8849268465832001e+03 },
    { 6.9600000000000000e-09, 5.3998389563100000e+00, 5.4867778431750003e+03 },
    { 6.8900000000000000e-09, 2.2946554111399999e+00, 2.0043674560198800e+04 },
    { 6.8800000000000002e-09, 4.5908536059599996e+00, 3.8941818295421999e+03 },
    { 6.4700000000000002e-09, 1.1782296028699999e+00, 3.0978838227257902e+03 },
    { 6.2199999999999996e-09, 4.9395262414300003e+00, 3.3396321056316001e+03 },
    { 6.1000000000000000e-09, 4.9406493230999998e-01, 3.3406296803519999e+03 },
    { 6.1000000000000000e-09, 4.7464786079900003e+00, 3.3405951730475999e+03 },
    { 5.6699999999999997e-09, 5.1314316796000004e-01, 5.0886288397668004e+03 },
    { 5.1499999999999998e-09, 5.2495804247000000e-01, 3.1491641605882000e+03 },
    { 5.0300000000000002e-09, 2.8396117472700002e+00, 6.6777017350506003e+03 },
    { 4.6200000000000002e-09, 6.0103156857100002e+00, 2.7870430238573999e+03 },
    { 4.4599999999999999e-09, 3.2195930638100001e+00, 4.2923308329503998e+03 },
    { 4.3899999999999999e-09, 2.9884669602000002e-01, 4.1369104335162001e+03 },
    { 4.2999999999999996e-09, 3.3478310848300001e+00, 3.5833410306738001e+03 },
    { 3.9300000000000003e-09, 5.1560001582400004e+00, 2.1461654164751999e+03 },
    { 3.9000000000000002e-09, 1.9125581950299999e+00, 4.6904798363585996e+03 },
    { 3.8400000000000000e-09, 4.8719358771300003e+00, 5.0921519581158000e+03 },
    { 3.7499999999999997e-09, 3.6608542336599998e+00, 1.5925960136327999e+03 },
    { 3.7399999999999999e-09, 4.8008500837000003e+00, 1.1944470102246000e+03 },
    { 3.6100000000000001e-09, 9.4089120900000001e-01, 2.3888940204492001e+03 },
    { 3.4800000000000000e-09, 3.8871229031099999e+00, 6.1515338883049999e+03 },
    { 3.4200000000000002e-09, 2.5897430907399999e+00, 3.9814900340819997e+02 },
    { 3.3700000000000001e-09, 5.8433757193700000e+00, 3.7235089589230001e+03 },
    { 3.3200000000000001e-09, 5.8549974357999996e+00, 7.9629800681639995e+02 },
    { 3.2500000000000002e-09, 2.3808531957299999e+00, 1.9907450170410000e+03 },
    { 3.2299999999999998e-09, 5.2842958635899997e+00, 1.5542039943419999e+02 },
    { 3.1500000000000001e-09, 1.3987811526000000e+00, 3.7387614301079998e+03 },
    { 3.0899999999999999e-09, 4.9984733654599998e+00, 6.5258044539654002e+03 },
    { 2.9499999999999999e-09, 5.5380334352600000e+00, 3.3415927477680002e+03 },
    { 2.6500000000000002e-09, 1.0458265918000000e-01, 1.7480164130670000e+03 },
    { 2.5700000000000000e-09, 5.2218843850800001e+00, 4.3999943568890003e+03 },
    { 2.2200000000000002e-09, 5.9252721834999997e+00, 8.9624553499102003e+03 },
    { 2.2200000000000002e-09, 4.4580044569700004e+00, 3.3334988796990001e+03 },
    { 2.0200000000000001e-09, 4.7484048883899996e+00, 2.9577158944766002e+03 },
    { 2.0099999999999999e-09, 2.4160265381100001e+00, 3.3406797370026002e+03 },
    { 2.0099999999999999e-09, 2.8245229248500001e+00, 3.3405451163970001e+03 },
    { 1.9000000000000001e-09, 5.7199116141799999e+00, 2.9140142358237999e+03 },
    { 1.8899999999999999e-09, 2.0179038049000000e+00, 7.1135470007999997e+00 },
    { 1.8899999999999999e-09, 4.9933357747300002e+00, 1.5890728952838001e+03 },
    { 1.8700000000000000e-09, 2.5478951001499999e+00, 6.2830758499914000e+03 },
    { 1.8700000000000000e-09, 2.5557586123900000e+00, 1.0025360398448400e+04 },
    { 1.8100000000000000e-09, 5.4059073408699998e+00, 4.5350594369244000e+03 },
    { 1.7500000000000000e-09, 1.3804610814200000e+00, 1.3498674096587999e+03 },
    { 1.6700000000000000e-09, 6.6523343049000006e-01, 2.2883440435113998e+03 },
    { 1.4900000000000000e-09, 4.8067602956400002e+00, 2.2741169495098002e+03 },
    { 1.4700000000000001e-09, 2.7190489846100001e+00, 2.4272860397400001e+02 },
    { 1.4599999999999999e-09, 2.6851513601299999e+00, 9.5171840625059997e+02 },
    { 1.4500000000000000e-09, 3.9280852280100000e+00, 5.5356940284240000e+02 },
    { 1.4400000000000000e-09, 5.1451501296200002e+00, 6.2546266625236003e+03 },
    { 1.2900000000000001e-09, 2.2155031959799998e+00, 7.0793738568077997e+03 },
    { 1.2199999999999999e-09, 1.1550804083799999e+00, 6.4897765872880000e+03 },
    { 1.1900000000000000e-09, 1.5509533826799999e+00, 6.4384962494255997e+03 },
    { 1.1800000000000001e-09, 1.2919663835099999e+00, 3.1273133312618002e+03 },
    { 1.0399999999999999e-09, 4.2038236642299998e+00, 4.9332084403325998e+03 },
    { 1.0200000000000000e-09, 6.0813657573200004e+00, 6.8366452528338004e+03 },
    { 1.0000000000000001e-09, 2.6708992136799998e+00, 2.3384286986898602e+04 },
    { 9.9000000000000010e-10, 6.1577079695499997e+00, 3.3477259737006002e+03 },
    { 9.7999999999999992e-10, 3.7829296009000002e-01, 3.8703033917943999e+03 },
    { 9.5999999999999999e-10, 5.1171018560899997e+00, 4.2659819087599999e+02 },
    { 9.5000000000000003e-10, 9.7194852729000003e-01, 7.4775228602159996e+03 },
    { 8.7999999999999996e-10, 2.2872671236900000e+00, 6.6741113063987996e+03 },
    { 8.1999999999999996e-10, 8.8675647815000003e-01, 1.7515395314160000e+03 },
    { 8.0999999999999999e-10, 2.9532447060200000e+00, 2.1329909543799999e+02 },
    { 7.7000000000000003e-10, 2.5829922839299999e+00, 9.4921463150048003e+03 },
    { 6.6999999999999996e-10, 5.1745394192800003e+00, 3.8289653222319998e+02 },
    { 6.6000000000000000e-10, 2.7975222807500000e+00, 5.3313574437407997e+03 },
    { 6.3999999999999996e-10, 3.8212022583900001e+00, 6.6812075997474003e+03 },
    { 6.3999999999999996e-10, 5.8517377305300000e+00, 6.6812421070518003e+03 },
    { 6.0999999999999996e-10, 7.7101226780000004e-02, 6.1276554505572003e+03 },
    { 6.0999999999999996e-10, 5.0096146984400001e+00, 4.5624609930212000e+03 },
    { 5.7000000000000000e-10, 2.8394341918400001e+00, 8.9553418029093991e+03 },
    { 5.4000000000000000e-10, 3.6196616921899998e+00, 6.9239534573736000e+03 },
    { 5.1999999999999996e-10, 6.1630241150000001e-02, 1.0664954771900000e+03 },
    { 5.1999999999999996e-10, 1.3996785015700000e+00, 5.7295064471490005e+03 },
    { 5.1000000000000000e-10, 2.7102630025000001e+00, 1.9144826611159999e+02 },
    { 5.1000000000000000e-10, 5.9842724430600001e+00, 6.6822051744678001e+03 },
    { 5.1000000000000000e-10, 4.1190783359300003e+00, 6.3989728631399998e+02 },
    { 5.1000000000000000e-10, 5.7556457756699997e+00, 7.7406067835887998e+03 },
    { 5.0000000000000003e-10, 3.7788454707799999e+00, 7.2109158184941998e+03 },
    { 4.7000000000000003e-10, 1.9662345765200000e+00, 1.0419986283507600e+04 },
    { 4.6000000000000001e-10, 5.0349630837600001e+00, 8.9695688969109997e+03 },
    { 4.3000000000000001e-10, 7.6139880953000005e-01, 6.6812921637024001e+03 },
    { 3.9000000000000000e-10, 5.8795068144600000e+00, 7.8756718636242003e+03 },
    { 3.9000000000000000e-10, 1.4073128561099999e+00, 3.5539115221378001e+03 },
    { 3.7999999999999998e-10, 2.5088803108800000e+00, 1.0018314161750401e+04 },
    { 3.7000000000000001e-10, 2.3090942371200001e+00, 3.3202571073009999e+03 },
    { 3.7000000000000001e-10, 6.7405360149000004e-01, 3.3609677460985899e+03 },
    { 3.7000000000000001e-10, 4.5490842851700002e+00, 9.8664168806651996e+03 },
    { 3.6000000000000000e-10, 5.6938523608300002e+00, 4.9296853219836003e+03 },
    { 3.6000000000000000e-10, 5.0100954994200002e+00, 1.2303067776610000e+04 },
    { 3.6000000000000000e-10, 2.9169996193399998e+00, 1.3365972825148199e+04 },
    { 3.4999999999999998e-10, 4.5122114069000002e-01, 1.0522683831884001e+03 },
    { 3.4999999999999998e-10, 2.0843595568800000e+00, 2.6997348193175999e+03 },
    { 3.4999999999999998e-10, 4.2498281711599999e+00, 3.3253599555147998e+03 },
    { 3.4999999999999998e-10, 5.0180023774900002e+00, 3.3558648978848000e+03 },
    { 3.4000000000000001e-10, 5.9992429230899997e+00, 3.7672106175757999e+03 },
    { 3.3000000000000000e-10, 5.9509015383200001e+00, 6.0413275670856001e+03 },
    { 3.1999999999999998e-10, 6.0244035709999999e-01, 6.4087760738220004e+02 },
    { 3.1000000000000002e-10, 4.2889773433000000e+00, 6.2983283211764001e+03 },
    { 2.8999999999999998e-10, 3.6132145044600001e+00, 1.0021820026447200e+04 },
    { 2.8999999999999998e-10, 5.6439580161700000e+00, 1.0021854533751601e+04 },
    { 2.8000000000000002e-10, 3.2159445294800002e+00, 1.2832758741704600e+04 },
};

static const VSOP87Term VSOP87A_MARS_Z3[51] = {
    { 4.8982199999999997e-06, 2.0639288683100001e+00, 3.3406124266997999e+03 },
    { 2.1257499999999999e-06, 0.0000000000000000e+00, 0.0000000000000000e+00 },
    { 8.2450000000000000e-08, 4.9182799375900004e+00, 6.6812248533995999e+03 },
    { 7.5090000000000006e-08, 5.3964144831300001e+00, 1.0021837280099400e+04 },
    { 2.0120000000000001e-08, 5.9775101373700004e+00, 1.3362449706799200e+04 },
    { 8.9399999999999993e-09, 4.1416076550999996e+00, 3.3370893083507999e+03 },
    { 7.3799999999999997e-09, 1.3158572236099999e+00, 3.4960328261340001e+03 },
    { 4.3800000000000002e-09, 2.1206488997000000e-01, 1.6703062133498999e+04 },
    { 4.3299999999999997e-09, 3.0144275220500001e+00, 3.1851920272655998e+03 },
    { 1.8899999999999999e-09, 6.1972076002399996e+00, 3.3441355450487999e+03 },
    { 1.5400000000000001e-09, 2.7244756210899999e+00, 3.0978838227257902e+03 },
    { 1.1800000000000001e-09, 5.1396423769500004e+00, 3.5320606928113998e+03 },
    { 1.1300000000000000e-09, 5.3273488041199997e+00, 2.5443144198834002e+03 },
    { 1.0399999999999999e-09, 6.2424095123100001e+00, 3.8941818295421999e+03 },
    { 1.0100000000000000e-09, 3.8422186576400001e+00, 2.9424634232916001e+03 },
    { 9.5999999999999999e-10, 1.6749609829600001e+00, 3.5833410306738001e+03 },
    { 9.5000000000000003e-10, 7.1711229332000004e-01, 5.6218429232103999e+03 },
    { 8.6000000000000003e-10, 6.6972591038999996e-01, 2.0043674560198800e+04 },
    { 6.6999999999999996e-10, 3.8442120869000002e-01, 6.6847479717486003e+03 },
    { 6.5000000000000003e-10, 3.0552815689000001e-01, 3.3396321056316001e+03 },
    { 6.0999999999999996e-10, 3.4312018663499999e+00, 6.5258044539654002e+03 },
    { 6.0000000000000000e-10, 4.8914740592900001e+00, 4.2923308329503998e+03 },
    { 5.7000000000000000e-10, 6.0718635812999999e-01, 1.5542039943419999e+02 },
    { 5.4999999999999996e-10, 4.3400491378400003e+00, 2.7870430238573999e+03 },
    { 4.8999999999999996e-10, 4.1339413590000001e-01, 2.1461654164751999e+03 },
    { 4.8000000000000000e-10, 2.8885786868799999e+00, 5.2969096509459996e+02 },
    { 4.8000000000000000e-10, 3.5542200998700002e+00, 4.6904798363585996e+03 },
    { 4.0999999999999998e-10, 1.2301542943399999e+00, 3.7235089589230001e+03 },
    { 4.0000000000000001e-10, 2.3043076419399999e+00, 5.0886288397668004e+03 },
    { 2.8000000000000002e-10, 4.9584999492400001e+00, 3.1491641605882000e+03 },
    { 2.7000000000000000e-10, 5.0432067813500003e+00, 4.1369104335162001e+03 },
    { 2.7000000000000000e-10, 5.5555754017899996e+00, 2.3888940204492001e+03 },
    { 2.4000000000000000e-10, 3.8128117738000000e+00, 3.3415927477680002e+03 },
    { 2.3000000000000001e-10, 2.8320385866999997e-01, 2.2812304965106000e+03 },
    { 2.1000000000000000e-10, 3.2087488984000001e-01, 1.9907450170410000e+03 },
    { 2.1000000000000000e-10, 2.7831826700000000e+00, 3.9814900340819997e+02 },
    { 1.8999999999999999e-10, 9.3486196403999999e-01, 3.3406296803519999e+03 },
    { 1.8999999999999999e-10, 5.1872756397200002e+00, 3.3405951730475999e+03 },
    { 1.8999999999999999e-10, 3.0104793975100002e+00, 2.9577158944766002e+03 },
    { 1.8000000000000000e-10, 1.3210361733400000e+00, 1.5925960136327999e+03 },
    { 1.8000000000000000e-10, 1.6695819492399999e+00, 1.1944470102246000e+03 },
    { 1.8000000000000000e-10, 6.0423250313899999e+00, 7.0793738568077997e+03 },
    { 1.8000000000000000e-10, 3.8251910733400001e+00, 4.5350594369244000e+03 },
    { 1.8000000000000000e-10, 1.0892564114100001e+00, 5.4867778431750003e+03 },
    { 1.7000000000000001e-10, 1.6564148066000001e-01, 2.3384286986898602e+04 },
    { 1.5999999999999999e-10, 9.5206290089000001e-01, 1.0025360398448400e+04 },
    { 1.5000000000000000e-10, 5.8664199643500003e+00, 5.8849268465832001e+03 },
    { 1.5000000000000000e-10, 4.7743108983999996e+00, 6.4897765872880000e+03 },
    { 1.5000000000000000e-10, 4.1595367501700000e+00, 6.6777017350506003e+03 },
    { 1.5000000000000000e-10, 5.8299261785500001e+00, 3.3334988796990001e+03 },
    { 1.5000000000000000e-10, 1.4830351746099999e+00, 1.0593819301891999e+03 },
};

static const VSOP87Term VSOP87A_MARS_Z4[16] = {
    { 1.5141000000000000e-07, 0.0000000000000000e+00, 0.0000000000000000e+00 },
    { 1.0395000000000000e-07, 3.3327150554300000e+00, 3.3406124266997999e+03 },
    { 2.5599999999999998e-09, 3.7624317358999999e+00, 1.0021837280099400e+04 },
    { 2.0099999999999999e-09, 7.8166958626000005e-01, 6.6812248533995999e+03 },
    { 1.1700000000000001e-09, 4.3248929475600004e+00, 1.3362449706799200e+04 },
    { 1.1599999999999999e-09, 2.8820937580599999e+00, 3.4960328261340001e+03 },
    { 6.0000000000000000e-10, 1.4322731948700000e+00, 3.1851920272655998e+03 },
    { 4.7000000000000003e-10, 5.6798935453199997e+00, 3.3370893083507999e+03 },
    { 3.3000000000000000e-10, 4.8549420496800000e+00, 1.6703062133498999e+04 },
    { 2.5999999999999998e-10, 4.3201901805200000e+00, 3.0978838227257902e+03 },
    { 1.5000000000000000e-10, 4.5538346930000001e-02, 3.5833410306738001e+03 },
    { 1.0000000000000000e-10, 1.7311748525799999e+00, 6.5258044539654002e+03 },
    { 8.9999999999999999e-11, 5.3452884360699997e+00, 2.0043674560198800e+04 },
    { 7.9999999999999995e-11, 1.1587567192199999e+00, 3.8941818295421999e+03 },
    { 7.9999999999999995e-11, 1.8288499757200001e+00, 2.7870430238573999e+03 },
    { 7.9999999999999995e-11, 4.2112051900200003e+00, 6.6847479717486003e+03 },
};

static const VSOP87Term VSOP87A_MARS_Z5[7] = {
    { 5.0499999999999997e-09, 4.6255815077799998e+00, 3.3406124266997999e+03 },
    { 8.0000000000000003e-10, 3.1415926535900001e+00, 0.0000000000000000e+00 },
    { 2.7000000000000000e-10, 5.1664990799700004e+00, 6.6812248533995999e+03 },
    { 1.5000000000000000e-10, 4.4391991634199996e+00, 3.4960328261340001e+03 },
    { 6.0000000000000000e-11, 6.1627106396200002e+00, 3.1851920272655998e+03 },
    { 3.9999999999999998e-11, 1.8984457741600000e+00, 1.0021837280099400e+04 },
    { 3.9999999999999998e-11, 2.6663452370799998e+00, 1.3362449706799200e+04 },
};

static const VSOP87Block VSOP87A_MARS_X[6] = {
    { VSOP87A_MARS_X0, 1584 },
    { VSOP87A_MARS_X1, 956 },
    { VSOP87A_MARS_X2, 387 },
    { VSOP87A_MARS_X3, 135 },
    { VSOP87A_MARS_X4, 41 },
    { VSOP87A_MARS_X5, 21 },
};

static const VSOP87Block VSOP87A_MARS_Y[6] = {
    { VSOP87A_MARS_Y0, 1612 },
    { VSOP87A_MARS_Y1, 969 },
    { VSOP87A_MARS_Y2, 384 },
    { VSOP87A_MARS_Y3, 136 },
    { VSOP87A_MARS_Y4, 44 },
    { VSOP87A_MARS_Y5, 21 },
};

static const VSOP87Block VSOP87A_MARS_Z[6] = {
    { VSOP87A_MARS_Z0, 355 },
    { VSOP87A_MARS_Z1, 232 },
    { VSOP87A_MARS_Z2, 122 },
    { VSOP87A_MARS_Z3, 51 },
    { VSOP87A_MARS_Z4, 16 },
    { VSOP87A_MARS_Z5, 7 },
};

static const int32 VSOP87A_MARS_TRUNC_COUNT[11][3][6] = {
    {
        { 1584, 956, 387, 135, 41, 21 },
        { 1612, 969, 384, 136, 44, 21 },
        { 355, 232, 122, 51, 16, 7 },
    },
    {
        { 540, 252, 100, 24, 8, 0 },
        { 546, 257, 101, 24, 8, 0 },
        { 151, 73, 16, 5, 2, 0 },
    },
    {
        { 299, 136, 54, 10, 4, 0 },
        { 303, 137, 50, 10, 4, 0 },
        { 82, 30, 6, 4, 2, 0 },
    },
    {
        { 233, 106, 35, 8, 3, 0 },
        { 237, 106, 35, 8, 2, 0 },
        { 59, 20, 6, 2, 2, 0 },
    },
    {
        { 118, 51, 10, 5, 0, 0 },
        { 117, 49, 10, 5, 0, 0 },
        { 30, 6, 4, 2, 0, 0 },
    },
    {
        { 96, 31, 8, 4, 0, 0 },
        { 98, 30, 8, 4, 0, 0 },
        { 19, 6, 4, 2, 0, 0 },
    },
    {
        { 47, 9, 5, 2, 0, 0 },
        { 47, 9, 5, 2, 0, 0 },
        { 8, 4, 3, 0, 0, 0 },
    },
    {
        { 35, 8, 5, 0, 0, 0 },
        { 36, 8, 5, 1, 0, 0 },
        { 5, 4, 2, 0, 0, 0 },
    },
    {
        { 14, 5, 3, 0, 0, 0 },
        { 14, 5, 3, 0, 0, 0 },
        { 4, 3, 2, 0, 0, 0 },
    },
    {
        { 9, 4, 2, 0, 0, 0 },
        { 9, 4, 2, 0, 0, 0 },
        { 4, 2, 1, 0, 0, 0 },
    },
    {
        { 4, 2, 0, 0, 0, 0 },
        { 4, 2, 0, 0, 0, 0 },
        { 3, 1, 0, 0, 0, 0 },
    },
};

