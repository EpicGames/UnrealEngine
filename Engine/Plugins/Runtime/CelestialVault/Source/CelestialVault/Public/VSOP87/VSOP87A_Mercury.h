// Copyright Epic Games, Inc. All Rights Reserved.

// Auto-generated VSOP87A header for MERCURY (rectangular XYZ, J2000)
// Source: IMCCE/BDL VSOP87A.mercury file
// Generated by vsop87a_header_gen_imcce.py

#pragma once

#include "VSOP87A_Common.h"

static const VSOP87Term VSOP87A_MERCURY_X0[1449] = {
    { 3.7546291727999997e-01, 4.3965150694200004e+00, 2.6087903141574199e+04 },
    { 3.8257466720000001e-02, 1.1648560433899999e+00, 5.2175806283148399e+04 },
    { 2.6256159630000001e-02, 3.1415926535900001e+00, 0.0000000000000000e+00 },
    { 5.8426133300000004e-03, 4.2159939475700003e+00, 7.8263709424722590e+04 },
    { 1.0571669499999999e-03, 9.8379033181999997e-01, 1.0435161256629678e+05 },
    { 2.1011730000000000e-04, 4.0346935392300001e+00, 1.3043951570787099e+05 },
    { 4.4333730000000001e-05, 8.0236674526999996e-01, 1.5652741884944518e+05 },
    { 9.7496700000000005e-06, 3.8531967453600000e+00, 1.8261532199101939e+05 },
    { 7.0032700000000000e-06, 4.4547872536700002e+00, 2.4978524589480799e+04 },
    { 6.2646799999999998e-06, 1.1856349200100000e+00, 2.7197281693667599e+04 },
    { 4.4698900000000003e-06, 2.9750718150300002e+00, 1.0593819301891999e+03 },
    { 3.9840100000000002e-06, 1.8648789504900001e+00, 2.0426571092422000e+04 },
    { 2.7721600000000001e-06, 3.7790954834199999e+00, 3.1749235190726398e+04 },
    { 2.2102800000000001e-06, 6.2082250658000004e-01, 2.0870322513259359e+05 },
    { 1.9441799999999998e-06, 6.7806013045000002e-01, 4.5519534970588002e+03 },
    { 1.9071299999999999e-06, 1.1738521268600000e+00, 5.6613320491521999e+03 },
    { 1.9065699999999999e-06, 4.2720180194099999e+00, 5.3285184835241802e+04 },
    { 1.8179000000000000e-06, 4.9485713821699999e+00, 1.1093785520934000e+03 },
    { 1.5169300000000000e-06, 2.6760456688600001e+00, 5.1116424352959199e+04 },
    { 1.3849200000000000e-06, 1.2244642197300000e+00, 5.1066427731055002e+04 },
    { 1.1433800000000000e-06, 5.6002737806000003e-01, 5.7837138332300601e+04 },
    { 9.5481000000000001e-07, 1.6753795386000001e-01, 5.2969096509459996e+02 },
    { 8.3763999999999998e-07, 3.1866688301799999e+00, 1.0213285546211000e+04 },
    { 7.5178999999999998e-07, 6.0695401270299998e+00, 2.7147285071763388e+04 },
    { 7.4527999999999996e-07, 4.9245470921300001e+00, 4.6514474233996203e+04 },
    { 6.8378000000000000e-07, 1.9832816517999999e-01, 2.5132303399965600e+04 },
    { 6.5033999999999999e-07, 4.9098277024500003e+00, 2.1535949644515400e+04 },
    { 6.0503000000000001e-07, 4.9965296452199999e+00, 4.7623852786089599e+04 },
    { 5.8576999999999998e-07, 5.4167194012200000e+00, 2.7043502883182799e+04 },
    { 5.7815000000000003e-07, 5.7020041601200004e+00, 1.4765239043269799e+04 },
    { 5.4815999999999995e-07, 5.3090904600800002e+00, 1.2566151699982800e+04 },
    { 5.2661999999999999e-07, 2.2980165054300001e+00, 2.6617594106668799e+04 },
    { 5.1287000000000001e-07, 3.6716214949400001e+00, 2.3479112827416777e+05 },
    { 5.1223000000000003e-07, 3.5362782680299998e+00, 4.2659819087599999e+02 },
    { 4.8571000000000003e-07, 5.7291551427499998e+00, 7.7204327494533383e+04 },
    { 4.8375999999999999e-07, 3.5189095472099998e+00, 1.5890728952838001e+03 },
    { 4.8307999999999996e-07, 3.6525982520400002e+00, 2.5558212176479599e+04 },
    { 4.7830000000000004e-07, 1.0429680053099999e+00, 7.9373087976815994e+04 },
    { 4.4563000000000000e-07, 2.4756679699499999e+00, 4.1962520736937397e+04 },
    { 3.4348000000000000e-07, 5.6794095710199999e+00, 5.1646115318053788e+04 },
    { 3.2947000000000000e-07, 2.9949649589499998e+00, 9.5559974160859997e+02 },
    { 3.1953000000000000e-07, 3.6131338418099999e+00, 8.3925041473874793e+04 },
    { 3.1389000000000001e-07, 4.2759821340700004e+00, 7.7154330872629187e+04 },
    { 2.8369999999999998e-07, 7.0313546811000005e-01, 3.0639856638632999e+04 },
    { 2.7626000000000001e-07, 6.1484612795800002e+00, 3.7410567239878597e+04 },
    { 2.7160000000000002e-07, 1.7809143084500001e+00, 6.2830758499914000e+03 },
    { 2.7137999999999999e-07, 3.3631401741000000e-01, 3.9609654583165597e+04 },
    { 2.6666999999999999e-07, 6.2561835482700001e+00, 3.6301188687785187e+04 },
    { 2.5544999999999997e-07, 3.7148603410500001e+00, 1.1322664098304400e+04 },
    { 2.3509000000000000e-07, 3.9971435028700002e+00, 1.3521751441591399e+04 },
    { 2.3211999999999999e-07, 2.2968362382100000e+00, 5.3131406024757001e+04 },
    { 2.1075999999999999e-07, 1.7653297974900000e+00, 7.3711755927663791e+04 },
    { 2.0743000000000000e-07, 5.6802480783000000e+00, 1.5874617595363199e+04 },
    { 1.7093000000000001e-07, 2.1161877283499999e+00, 5.1749208092272391e+04 },
    { 1.6621000000000000e-07, 2.1629934103299999e+00, 5.0586733387864588e+04 },
    { 1.6406000000000000e-07, 1.6945190286400000e+00, 7.2602377375570388e+04 },
    { 1.6077000000000000e-07, 3.2249236245200001e+00, 9.1039069941176003e+03 },
    { 1.5400999999999999e-07, 3.8409642418000001e-01, 7.2386755916000002e+03 },
    { 1.5073000000000000e-07, 2.8119637296900000e+00, 1.8849227549974199e+04 },
    { 1.3833000000000000e-07, 2.8606025243900000e+00, 5.3235188213337591e+04 },
    { 1.3750999999999999e-07, 3.0170025304200001e+00, 6.3498470381452789e+04 },
    { 1.3561000000000001e-07, 3.2535568934299999e+00, 5.1220206541539788e+04 },
    { 1.3481000000000000e-07, 2.4974395072300002e+00, 1.0329223063610759e+05 },
    { 1.3461000000000001e-07, 5.2167490239500003e+00, 5.2705497248242988e+04 },
    { 1.3311999999999999e-07, 5.8088573376100001e+00, 2.5028521211384999e+04 },
    { 1.3012000000000000e-07, 1.7728495466800001e+00, 2.6068233380674399e+04 },
    { 1.2986000000000000e-07, 3.8784807918299999e+00, 2.6107572902473989e+04 },
    { 1.2384000000000000e-07, 3.8849136445400001e+00, 4.5892730433156990e+04 },
    { 1.2122000000000000e-07, 4.3923849384000002e-01, 2.6087903141574195e+05 },
    { 1.2080000000000000e-07, 4.0956644096100003e+00, 1.0546099111839019e+05 },
    { 1.2067000000000001e-07, 3.3997912053700001e+00, 6.5697557724739789e+04 },
    { 1.1952000000000000e-07, 5.5300520982299997e+00, 6.8050423878511589e+04 },
    { 1.1389999999999999e-07, 1.9772095302399999e+00, 1.9317192540328600e+04 },
    { 1.0111999999999999e-07, 2.3887988805800000e+00, 7.7734018459627987e+04 },
    { 9.5259999999999994e-08, 2.1068557905000000e+00, 3.8654054841556987e+04 },
    { 9.1870000000000001e-08, 2.4789909631399998e+00, 4.0853142184844000e+04 },
    { 8.9299999999999999e-08, 4.7010685402000002e-01, 2.6084021806216198e+04 },
    { 8.9270000000000005e-08, 5.1813472003200003e+00, 2.6091784476932200e+04 },
    { 8.8800000000000001e-08, 7.4693819700999997e-01, 3.4425749449653999e+03 },
    { 8.8559999999999994e-08, 3.3573269124999999e-01, 2.6514501332450200e+04 },
    { 8.7680000000000002e-08, 4.8669243723599998e+00, 3.2370978991565600e+04 },
    { 8.4730000000000005e-08, 3.8165820379999998e-01, 1.1001294461544899e+05 },
    { 8.1499999999999995e-08, 3.2531424487999999e-01, 2.7676976036857999e+04 },
    { 7.6269999999999994e-08, 3.7272450706600000e+00, 5.6727759780207198e+04 },
    { 7.5339999999999998e-08, 1.0440777314900000e+00, 1.0324223401420339e+05 },
    { 7.3729999999999999e-08, 4.5427071437100004e+00, 2.3869146037387400e+04 },
    { 7.2100000000000004e-08, 1.1577623261200001e+00, 6.7707106012455997e+03 },
    { 7.0490000000000005e-08, 2.5314777910799999e+00, 3.3326578733174203e+04 },
    { 6.2660000000000007e-08, 5.2264344127999998e+00, 2.5874604046136199e+04 },
    { 6.2660000000000007e-08, 2.7460337283400000e+00, 6.3989728631399998e+02 },
    { 6.0740000000000000e-08, 3.5249199207699999e+00, 3.2858613742819791e+04 },
    { 6.0440000000000004e-08, 4.8176554623000003e+00, 9.9799659069237983e+04 },
    { 6.0010000000000006e-08, 5.9753409823699997e+00, 5.9414481874748402e+04 },
    { 5.9610000000000002e-08, 3.0405452225700000e+00, 6.2389091829359393e+04 },
    { 5.9499999999999997e-08, 5.3465249989699997e+00, 7.9219309166331193e+04 },
    { 5.8320000000000002e-08, 1.0705186629000001e+00, 2.8306660245760992e+04 },
    { 5.8239999999999997e-08, 7.2194174801000000e-01, 1.9804827291582798e+04 },
    { 5.6570000000000003e-08, 4.8291974083199998e+00, 2.2645328196608789e+04 },
    { 5.4399999999999997e-08, 5.1787410360999999e+00, 7.6674636529438780e+04 },
    { 5.4230000000000003e-08, 2.2709464322400001e+00, 4.3071899289030800e+04 },
    { 5.3200000000000000e-08, 5.1694266911400000e+00, 7.7837111233846590e+04 },
    { 5.0029999999999999e-08, 3.3033193603000000e-01, 2.6301202237012199e+04 },
    { 4.9940000000000002e-08, 1.0827081566200001e+00, 2.1329909543799999e+02 },
    { 4.6030000000000003e-08, 4.1081882123600000e+00, 2.6095016688575000e+04 },
    { 4.4180000000000003e-08, 1.6001230123900001e+00, 2.6080789594573391e+04 },
    { 4.0919999999999998e-08, 5.5410491255000005e-01, 5.8946516884393990e+04 },
    { 4.0170000000000002e-08, 4.0627085317000002e+00, 2.1187638603783998e+03 },
    { 3.9029999999999999e-08, 7.5356452004999996e-01, 2.9530478086539599e+04 },
    { 3.8880000000000001e-08, 4.7472160549200000e+00, 9.8690280517144580e+04 },
    { 3.6970000000000000e-08, 6.0725148626600003e+00, 8.9586373523026981e+04 },
    { 3.6190000000000002e-08, 5.5484785090099997e+00, 1.2938013377768178e+05 },
    { 3.5929999999999998e-08, 1.7002529302000000e-01, 9.1785460866313981e+04 },
    { 3.5299999999999998e-08, 1.5964305075100000e+00, 7.1135470007999997e+00 },
    { 3.4620000000000002e-08, 6.5717640119999998e-01, 7.1980633574731182e+04 },
    { 3.3090000000000001e-08, 1.9506216712000000e+00, 7.8793400389817180e+04 },
    { 3.2490000000000002e-08, 2.7004450324000001e-01, 1.6983996147456601e+04 },
    { 3.2390000000000001e-08, 2.7067086485999998e+00, 2.5934124331089399e+04 },
    { 3.2080000000000000e-08, 2.9450921825599998e+00, 2.6241681952059000e+04 },
    { 3.1949999999999998e-08, 4.8766248370699996e+00, 2.2187571041868000e+03 },
    { 3.0939999999999998e-08, 8.6497244959999997e-01, 1.3154889425996438e+05 },
    { 3.0820000000000001e-08, 2.2983987793299998e+00, 9.4138327020085781e+04 },
    { 3.0769999999999997e-08, 2.2569037380000000e-02, 7.7308109683113988e+04 },
    { 3.0549999999999999e-08, 5.9211038832199998e+00, 7.9323091354911783e+04 },
    { 3.0540000000000001e-08, 5.5549851378500001e+00, 6.9159802430604992e+04 },
    { 2.9080000000000000e-08, 3.4899857745700000e+00, 2.8696693455731618e+05 },
    { 2.7269999999999999e-08, 1.9052309532000000e-01, 1.4956319713459999e+02 },
    { 2.7220000000000002e-08, 5.8825714674300000e+00, 4.4937130691548402e+04 },
    { 2.7140000000000000e-08, 5.4199036435599997e+00, 1.0382192160120218e+05 },
    { 2.6700000000000001e-08, 6.4762614763000004e-01, 5.2195476044048191e+04 },
    { 2.6499999999999999e-08, 4.8243794096099997e+00, 5.2156136522248598e+04 },
    { 2.5300000000000002e-08, 6.0197576566800004e+00, 1.0664954771900000e+03 },
    { 2.4850000000000001e-08, 3.1231374650800001e+00, 1.0522683831884001e+03 },
    { 2.4809999999999998e-08, 5.3123627252899999e+00, 2.4498830246290399e+04 },
    { 2.4120000000000000e-08, 4.2318915682800000e+00, 5.4394563387335191e+04 },
    { 2.2250000000000001e-08, 3.4331458532900001e+00, 1.3610084775702318e+05 },
    { 2.1150000000000001e-08, 2.9069790525100001e+00, 5.1535908996834391e+04 },
    { 1.9690000000000000e-08, 5.1658518332499996e+00, 6.4741957983131193e+04 },
    { 1.9650000000000001e-08, 5.0348440254800000e+00, 4.5405095681902800e+04 },
    { 1.9420000000000001e-08, 1.6843116630299999e+00, 5.8458882133139792e+04 },
    { 1.9250000000000000e-08, 5.7853147953199997e+00, 1.3655860491176400e+04 },
    { 1.8690000000000001e-08, 4.8618542978000001e-01, 8.2815662921781390e+04 },
    { 1.8660000000000000e-08, 4.0950196573500000e+00, 1.2933013715577759e+05 },
    { 1.8320000000000001e-08, 5.5298182853500002e+00, 6.6941045326418185e+04 },
    { 1.8250000000000001e-08, 1.9493067032800000e+00, 5.2179687618506403e+04 },
    { 1.8200000000000001e-08, 3.5216819755000000e+00, 5.2171924947790401e+04 },
    { 1.7780000000000001e-08, 1.9389900540900000e+00, 2.5619938151219802e+04 },
    { 1.7050000000000000e-08, 3.7142308339999999e+00, 2.6555868131928601e+04 },
    { 1.6730000000000002e-08, 4.3225204941699999e+00, 5.2168692736147590e+04 },
    { 1.6639999999999999e-08, 3.8311937956700000e+00, 5.3274761083827998e+03 },
    { 1.6540000000000001e-08, 6.1278907946899999e+00, 7.6266071275599998e+01 },
    { 1.6499999999999999e-08, 1.5861201731200001e+00, 1.2588756221081219e+05 },
    { 1.6390000000000000e-08, 3.3995373703300000e+00, 5.2602404474024399e+04 },
    { 1.6269999999999999e-08, 4.7935223979800003e+00, 4.8733231338182988e+04 },
    { 1.5609999999999999e-08, 1.5279860860800001e+00, 6.6812248533995999e+03 },
    { 1.5390000000000001e-08, 2.1201758232199999e+00, 1.0530721230790539e+05 },
    { 1.5370000000000001e-08, 3.4321010467000002e+00, 5.2389105378586399e+04 },
    { 1.5360000000000000e-08, 1.9631934356700000e+00, 1.0276253967101299e+05 },
    { 1.5290000000000000e-08, 2.7336415120500002e+00, 8.5502385016322587e+04 },
    { 1.5139999999999999e-08, 3.3950014844299998e+00, 5.3764879178432202e+04 },
    { 1.4580000000000001e-08, 1.9379637018700000e+00, 1.0392501437542078e+05 },
    { 1.4450000000000000e-08, 6.1006004165399998e+00, 8.8476994970933592e+04 },
    { 1.4110000000000000e-08, 1.3130997330400001e+00, 4.9957049178961599e+04 },
    { 1.4110000000000000e-08, 1.6698652255699999e+00, 5.0057042422769999e+04 },
    { 1.3930000000000000e-08, 1.2558735472700000e+00, 2.6081274582674188e+04 },
    { 1.3930000000000000e-08, 4.3957451789000004e+00, 2.6094531700474210e+04 },
    { 1.2660000000000001e-08, 6.2407212057699999e+00, 3.5191810135691798e+04 },
    { 1.2470000000000000e-08, 4.0524749250800003e+00, 5.1962507187710398e+04 },
    { 1.2320000000000000e-08, 8.4010473770000005e-01, 5.2182919830149200e+04 },
    { 1.0680000000000000e-08, 5.8260164388100000e+00, 2.6727800427888200e+04 },
    { 1.0140000000000000e-08, 1.8162082958000001e+00, 7.4821134479757180e+04 },
    { 9.9100000000000007e-09, 3.2222442366799999e+00, 1.1787336400788819e+05 },
    { 9.6299999999999992e-09, 2.3164930905299999e+00, 1.5546803691925600e+05 },
    { 9.5599999999999992e-09, 1.5157738787299999e+00, 1.2477818365871879e+05 },
    { 9.3900000000000000e-09, 3.6953688984299999e+00, 1.2432042650397800e+04 },
    { 9.2900000000000008e-09, 3.7991767973499999e+00, 5.5618381228113802e+04 },
    { 9.2300000000000006e-09, 2.8366826577699999e+00, 1.1567427666460119e+05 },
    { 9.0699999999999995e-09, 3.7081691655700002e+00, 9.8068536716305389e+04 },
    { 9.0300000000000005e-09, 5.4595366742299998e+00, 5.2026243086013797e+04 },
    { 8.5899999999999995e-09, 5.9146445630600004e+00, 5.1109310805958390e+04 },
    { 8.5099999999999998e-09, 3.4785644863299998e+00, 8.5034420025968182e+04 },
    { 8.4599999999999993e-09, 2.5247270519999998e+00, 5.1123537899959993e+04 },
    { 8.3400000000000006e-09, 5.9500130802099997e+00, 3.3406124266997999e+03 },
    { 8.2700000000000006e-09, 4.9853440608500001e+00, 1.0488130353139139e+05 },
    { 8.2700000000000006e-09, 4.7071404533800001e+00, 2.5661304950698199e+04 },
    { 8.0200000000000000e-09, 3.9172132334300001e+00, 1.5763679740153858e+05 },
    { 7.9400000000000003e-09, 5.3480306076000002e+00, 1.2022623016165999e+05 },
    { 7.7300000000000004e-09, 2.3213438634400001e+00, 9.5247705572179184e+04 },
    { 7.3900000000000003e-09, 3.0742844236800000e+00, 1.0339601282468818e+05 },
    { 7.3000000000000000e-09, 3.4190750669300001e+00, 4.3371173787679999e+02 },
    { 7.2300000000000001e-09, 2.6940760505200001e+00, 1.0541099449648599e+05 },
    { 7.1600000000000001e-09, 2.1775498152199999e+00, 1.2990982474277639e+05 },
    { 7.0600000000000000e-09, 2.5739782783999998e-01, 3.1305483769889036e+05 },
    { 6.7700000000000004e-09, 7.0216648530000003e-01, 4.1948464387519999e+02 },
    { 6.7400000000000003e-09, 5.8116997217300002e+00, 3.8519945791972001e+04 },
    { 6.7299999999999997e-09, 8.6671401460999997e-01, 2.8206667001952599e+04 },
    { 6.7299999999999997e-09, 5.9951416591999998e+00, 5.2329585093633192e+04 },
    { 6.6100000000000001e-09, 5.9609613827499999e+00, 7.7623812138408583e+04 },
    { 6.5899999999999998e-09, 5.7591472597099997e+00, 5.2022027472663598e+04 },
    { 6.5000000000000003e-09, 3.1313182987200001e+00, 6.4607848933546193e+04 },
    { 6.3300000000000003e-09, 4.1347807230800004e+00, 4.5494581429748789e+04 },
    { 6.1399999999999999e-09, 3.6989597192899999e+00, 7.8283379185622383e+04 },
    { 6.0699999999999999e-09, 1.5922067878799999e+00, 7.8244039663822798e+04 },
    { 5.9200000000000002e-09, 3.3085271513999999e+00, 7.9945284420241996e+03 },
    { 5.8999999999999999e-09, 2.6560179084700000e+00, 7.1025033833122594e+04 },
    { 5.8999999999999999e-09, 1.9283357445100000e+00, 8.5319638175199998e+02 },
    { 5.8500000000000003e-09, 2.0238616925000000e-01, 1.6218875089859738e+05 },
    { 5.5200000000000000e-09, 2.3971176087799999e+00, 1.6921656695024001e+03 },
    { 5.4500000000000000e-09, 5.8060215935799997e+00, 5.2099540211872802e+04 },
    { 5.2400000000000001e-09, 2.2240638357200000e+00, 1.0021837280099400e+04 },
    { 5.2100000000000000e-09, 8.6596201957999996e-01, 8.0482466528909383e+04 },
    { 5.1700000000000001e-09, 4.9480465040099997e+00, 2.5455119402261000e+04 },
    { 5.0600000000000003e-09, 4.7887108992599998e+00, 2.3754706748702189e+04 },
    { 5.0199999999999996e-09, 8.1791649247999998e-01, 2.6720686880887391e+04 },
    { 4.7200000000000002e-09, 8.6276697767999999e-01, 1.5541804029735178e+05 },
    { 4.6900000000000001e-09, 3.2906822826800002e+00, 2.6237466338708789e+04 },
    { 4.6900000000000001e-09, 3.5362225020100002e+00, 1.0890356606335558e+05 },
    { 4.6600000000000000e-09, 4.7546005052700000e+00, 8.4546785274713984e+04 },
    { 4.5600000000000000e-09, 1.9415104557700000e+00, 9.0829861124705378e+04 },
    { 4.5399999999999996e-09, 2.0071547466799999e+00, 2.6037906519669988e+04 },
    { 4.5299999999999999e-09, 3.6441874018800000e+00, 2.6137899763478399e+04 },
    { 4.4699999999999997e-09, 1.0408789203800000e+00, 7.8256595877721789e+04 },
    { 4.4699999999999997e-09, 4.6288975222500000e+00, 7.6144945564344191e+04 },
    { 4.4599999999999999e-09, 5.8249813898899996e+00, 2.5984810367355600e+04 },
    { 4.4400000000000004e-09, 4.6386912012200003e+00, 1.5197546535238638e+05 },
    { 4.4100000000000003e-09, 6.1955975200599998e+00, 2.6190995915792791e+04 },
    { 4.3999999999999997e-09, 6.2653253566400000e+00, 2.7140171524762591e+04 },
    { 4.3800000000000002e-09, 1.0206385526599999e+00, 7.8050410329284583e+04 },
    { 4.3599999999999998e-09, 8.2989514901999994e-01, 2.3331963928720002e+03 },
    { 4.2899999999999999e-09, 2.3135896302300001e+00, 2.9428515568274001e+04 },
    { 4.2599999999999998e-09, 2.8308590552299999e+00, 2.7154398618764200e+04 },
    { 4.2500000000000000e-09, 5.1185865772500003e+00, 6.3278373931320004e+02 },
    { 4.2299999999999997e-09, 1.3857632104700000e+00, 4.6848330174765601e+04 },
    { 4.2199999999999999e-09, 2.3040736644800002e+00, 9.3028948467992392e+04 },
    { 4.1899999999999998e-09, 5.0008724111999996e+00, 7.8267590760080580e+04 },
    { 4.1800000000000000e-09, 5.0331052013799997e+00, 1.2885044281258718e+05 },
    { 4.1800000000000000e-09, 2.9020498002000000e-01, 7.8259828089364586e+04 },
    { 4.1700000000000003e-09, 1.8027492070100000e+00, 7.1492998823476984e+04 },
    { 4.1100000000000001e-09, 2.5560489631000000e-01, 1.5961864422845999e+03 },
    { 4.0499999999999999e-09, 2.5759405606599999e+00, 1.5819593482830001e+03 },
    { 4.0300000000000004e-09, 5.1786789685499999e+00, 1.3139511544947958e+05 },
    { 4.0000000000000002e-09, 8.3625504702999998e-01, 2.8421099534446199e+04 },
    { 3.9300000000000003e-09, 2.0924522981999999e-01, 7.8477008520160583e+04 },
    { 3.9000000000000002e-09, 4.9907248274200002e+00, 1.3001291751699499e+05 },
    { 3.7200000000000000e-09, 1.4914363958100001e+00, 1.9406678288174600e+04 },
    { 3.7000000000000000e-09, 3.5733703865300002e+00, 2.6162684740141500e+04 },
    { 3.6800000000000001e-09, 5.5633790128299996e+00, 2.5565325723480400e+04 },
    { 3.6600000000000002e-09, 3.5404918857899998e+00, 2.2747290714874402e+04 },
    { 3.6600000000000002e-09, 5.7674857667200001e+00, 1.1159028815789679e+05 },
    { 3.6399999999999998e-09, 1.7059200526000001e-01, 7.8690307615598591e+04 },
    { 3.6000000000000000e-09, 2.8700287066599999e+00, 1.1456489811250778e+05 },
    { 3.5600000000000001e-09, 6.0742564503600001e+00, 4.8835193856448590e+04 },
    { 3.5199999999999998e-09, 4.8243014753000002e-01, 5.2643771273502804e+04 },
    { 3.4299999999999999e-09, 2.0338059926500001e+00, 2.6013121543006899e+04 },
    { 3.3700000000000001e-09, 1.7003651408000001e-01, 7.9852782320006387e+04 },
    { 3.3499999999999998e-09, 2.0667111125100002e+00, 1.8207813988235212e+04 },
    { 3.2799999999999998e-09, 5.1213926186499998e+00, 7.2936233316339785e+04 },
    { 3.2400000000000000e-09, 6.1360545384699998e+00, 2.0760427033191401e+04 },
    { 3.2099999999999999e-09, 5.5503914529999997e-02, 2.6610480559667991e+04 },
    { 3.2099999999999999e-09, 1.5703187980800000e+00, 2.5771511271917600e+04 },
    { 3.2000000000000001e-09, 4.0888185180800001e+00, 2.6404295011230799e+04 },
    { 3.1599999999999998e-09, 4.3644892666299997e+00, 7.6044952320535798e+04 },
    { 3.0899999999999999e-09, 4.0904055993999999e-01, 7.4781598567299994e+01 },
    { 3.0500000000000000e-09, 3.8820719626700000e+00, 7.8270822971723392e+04 },
    { 3.0300000000000001e-09, 4.9798893411499998e+00, 5.1707841292793993e+04 },
    { 3.0000000000000000e-09, 2.5863393129999999e+00, 2.2434479570190001e+02 },
    { 2.9899999999999998e-09, 4.6350014440300003e+00, 2.5973463852888959e+04 },
    { 2.9699999999999999e-09, 1.0215680567500001e+00, 2.6202342430259410e+04 },
    { 2.9699999999999999e-09, 4.6004600750700000e+00, 2.6484548254729998e+03 },
    { 2.9499999999999999e-09, 2.6849314493200001e+00, 1.0309277421860000e+02 },
    { 2.9300000000000000e-09, 4.6757347205500004e+00, 2.3969139281195799e+04 },
    { 2.8699999999999998e-09, 2.9465225138400002e+00, 2.6164169212849800e+04 },
    { 2.8499999999999999e-09, 1.1680775244099999e+00, 5.2182434842048402e+04 },
    { 2.8499999999999999e-09, 4.3075015784800001e+00, 5.2169177724248388e+04 },
    { 2.8200000000000002e-09, 2.6287715507999998e+00, 5.3680451209540001e+02 },
    { 2.7900000000000001e-09, 2.6860122914400000e+00, 7.7197213947532582e+04 },
    { 2.7900000000000001e-09, 2.2308263029500002e+00, 7.8114146227587989e+04 },
    { 2.7700000000000002e-09, 1.4270392645100001e+00, 1.1624747044077999e+03 },
    { 2.7700000000000002e-09, 4.6603996156900003e+00, 2.6507387785449391e+04 },
    { 2.7400000000000001e-09, 5.5827775656799998e+00, 7.7211441041534184e+04 },
    { 2.7299999999999999e-09, 1.1065725453199999e+00, 9.5628915597059995e+02 },
    { 2.6700000000000001e-09, 6.2748448846500002e+00, 1.4396126714946239e+05 },
    { 2.6500000000000002e-09, 2.5578124321600000e+00, 3.9743763632750597e+04 },
    { 2.5899999999999999e-09, 4.4771100200399996e+00, 3.2769127994973802e+04 },
    { 2.5800000000000002e-09, 5.3726850999500000e+00, 1.8155594006083018e+05 },
    { 2.5200000000000000e-09, 5.9000537548999998e-01, 2.5668418497699000e+04 },
    { 2.4699999999999999e-09, 8.8342639890999997e-01, 7.1582484571322988e+04 },
    { 2.4399999999999998e-09, 4.8355613482299997e+00, 5.1756321639273199e+04 },
    { 2.4399999999999998e-09, 2.2340899293300001e+00, 5.1742094545271590e+04 },
    { 2.4300000000000001e-09, 1.2380146752500001e+00, 1.1610551958374201e+04 },
    { 2.4100000000000002e-09, 4.5715013040599999e+00, 1.5086608680029298e+05 },
    { 2.3699999999999999e-09, 4.7343036140000000e-01, 1.2415643985787958e+05 },
    { 2.2999999999999999e-09, 5.8845422064299999e+00, 1.4176217980617538e+05 },
    { 2.2900000000000002e-09, 9.3581945851000004e-01, 1.1020632121940000e+02 },
    { 2.2200000000000002e-09, 5.6159148263000003e-01, 8.1706284369687986e+04 },
    { 2.1999999999999998e-09, 2.1393446500999999e-01, 1.1112232316754239e+05 },
    { 2.1999999999999998e-09, 1.0881075585700000e+00, 7.8800891533389904e+03 },
    { 2.1799999999999999e-09, 4.7255615517000003e+00, 1.0090903762133139e+05 },
    { 2.1299999999999999e-09, 6.0035314724000000e-01, 1.6066065861474801e+04 },
    { 2.1200000000000001e-09, 5.3351044087999999e-01, 1.4788665740644001e+03 },
    { 2.1200000000000001e-09, 3.5340307978899999e+00, 4.2153969003049002e+04 },
    { 2.1100000000000000e-09, 1.7474725443100001e+00, 1.3096920667296558e+05 },
    { 2.1000000000000002e-09, 6.9225590128000003e-01, 1.8372470054311279e+05 },
    { 2.0799999999999998e-09, 2.1108338732499998e+00, 1.4631413330323418e+05 },
    { 2.0700000000000001e-09, 5.8129042910499997e+00, 1.3757737998458001e+03 },
    { 2.0500000000000002e-09, 3.7284923721899998e+00, 5.1322609901396390e+04 },
    { 2.0200000000000001e-09, 2.3337058157300001e+00, 2.5881717593137000e+04 },
    { 1.9700000000000000e-09, 2.6029108283400002e+00, 5.2815703569462399e+04 },
    { 1.9599999999999998e-09, 3.2808466421500002e+00, 2.6294088690011391e+04 },
    { 1.9599999999999998e-09, 5.3684458078599997e+00, 1.2133560871375339e+05 },
    { 1.9200000000000000e-09, 3.2598734402899998e+00, 5.0483640613645999e+04 },
    { 1.9099999999999998e-09, 5.2297847516399996e+00, 1.5599772788435058e+05 },
    { 1.8500000000000000e-09, 2.7333318781900000e+00, 1.0371171527998279e+05 },
    { 1.8400000000000001e-09, 6.1295569351400001e+00, 1.2948391596626239e+05 },
    { 1.7800000000000001e-09, 5.7488434368099997e+00, 1.3149889763806018e+05 },
    { 1.7700000000000001e-09, 1.9779739523700000e+00, 1.4477351183200000e+04 },
    { 1.7400000000000000e-09, 3.3113014678699999e+00, 3.3914274084046460e+05 },
    { 1.7300000000000000e-09, 5.1882146901299997e+00, 2.6624707653669589e+04 },
    { 1.7200000000000001e-09, 9.7217516384000002e-01, 5.1543022543835199e+04 },
    { 1.6999999999999999e-09, 2.5789315092999998e+00, 7.8187443353446986e+04 },
    { 1.6900000000000000e-09, 4.1121772445599998e+00, 1.0657036967048359e+05 },
    { 1.6700000000000000e-09, 3.7589979746000002e+00, 4.6796499035440002e+02 },
    { 1.6200000000000000e-09, 9.2049935752000001e-01, 2.5780345520604598e+04 },
    { 1.5900000000000001e-09, 5.6752365323999996e+00, 7.0269180982698381e+04 },
    { 1.5900000000000001e-09, 4.7382754106300000e+00, 2.6395460762543789e+04 },
    { 1.5799999999999999e-09, 2.9493045932299999e+00, 6.1279713277265997e+04 },
    { 1.5799999999999999e-09, 2.7657835167800000e+00, 7.8417488235207391e+04 },
    { 1.5500000000000000e-09, 3.2565629711700002e+00, 1.8827665404017159e+05 },
    { 1.5300000000000001e-09, 1.9550261263299999e+00, 4.4181277841124189e+04 },
    { 1.5199999999999999e-09, 2.5308171781400000e+00, 7.8109930614237790e+04 },
    { 1.5199999999999999e-09, 5.2842376586900004e+00, 5.2101024684581091e+04 },
    { 1.5000000000000000e-09, 4.0932669208799997e+00, 2.3888815798287189e+04 },
    { 1.4900000000000000e-09, 3.3052566028600001e+00, 3.3967992294913187e+04 },
    { 1.4900000000000000e-09, 4.6707747399999999e-01, 1.0437128232719658e+05 },
    { 1.4900000000000000e-09, 2.9397844837900000e+00, 1.3362449706799200e+04 },
    { 1.4800000000000001e-09, 4.6435056768800003e+00, 1.0433194280539699e+05 },
    { 1.4599999999999999e-09, 5.8342798298999998e-01, 6.0055895436487393e+04 },
    { 1.4500000000000000e-09, 1.3087699859199999e+00, 3.1639186965660002e+02 },
    { 1.4300000000000000e-09, 6.4505147056000001e-01, 2.5551098629478791e+04 },
    { 1.4200000000000001e-09, 3.0951554275700000e+00, 5.0593846934865389e+04 },
    { 1.4100000000000001e-09, 5.4185644299100000e+00, 5.0579619840863787e+04 },
    { 1.3900000000000000e-09, 2.4706459367600000e+00, 5.2278899057366987e+04 },
    { 1.3900000000000000e-09, 3.8230381893100001e+00, 5.2808590022461591e+04 },
    { 1.3900000000000000e-09, 5.7194152559200004e+00, 9.7112936974696786e+04 },
    { 1.3300000000000000e-09, 1.4364310762500001e+00, 1.0223284870591838e+05 },
    { 1.3200000000000000e-09, 2.6311812872500000e+00, 6.4701083331480004e+02 },
    { 1.3000000000000001e-09, 4.6432061812100001e+00, 2.2759767485294011e+04 },
    { 1.2700000000000000e-09, 3.9290870894699998e+00, 5.4294570143526791e+04 },
    { 1.2700000000000000e-09, 2.1479421217000000e-01, 2.6521614879451001e+04 },
    { 1.2700000000000000e-09, 4.1321370456200004e+00, 1.0413831347085879e+05 },
    { 1.2500000000000000e-09, 1.6513811398500000e+00, 6.6653157466348392e+04 },
    { 1.2199999999999999e-09, 2.5444615732700000e+00, 5.2072713508929788e+04 },
    { 1.2100000000000000e-09, 3.9139738874000001e+00, 1.8150594343892598e+05 },
    { 1.2100000000000000e-09, 1.5709482920300000e+00, 2.8286990484861199e+04 },
    { 1.2000000000000000e-09, 3.0678874175999998e-01, 1.3499146920492980e+05 },
    { 1.1900000000000000e-09, 1.4136047976599999e+00, 1.7806336849396059e+05 },
    { 1.1700000000000001e-09, 1.5418277346600000e+00, 1.1063468841628819e+05 },
    { 1.1500000000000000e-09, 4.0718576967900004e+00, 1.0434449901929598e+05 },
    { 1.1300000000000000e-09, 8.1015998025000002e-01, 2.5754047200804798e+04 },
    { 1.1200000000000001e-09, 1.8173387273399999e+00, 1.5493834595416137e+05 },
    { 1.1100000000000001e-09, 5.0939039069000003e-01, 1.8093374699549990e+04 },
    { 1.0999999999999999e-09, 4.8405115199299997e+00, 2.6421759082343600e+04 },
    { 1.0999999999999999e-09, 5.0017630051499999e+00, 1.1691776426627958e+05 },
    { 1.0999999999999999e-09, 1.6237016300000001e-01, 2.4356780788641601e+04 },
    { 1.0999999999999999e-09, 1.1922609850899999e+00, 4.9527351457675388e+04 },
    { 1.0900000000000000e-09, 3.1429685262899998e+00, 5.1639001771052986e+04 },
    { 1.0800000000000000e-09, 5.7694111877400003e+00, 2.5448005855260191e+04 },
    { 1.0800000000000000e-09, 2.3565771645800000e+00, 2.4176703658357001e+04 },
    { 1.0600000000000001e-09, 1.9543065395699999e+00, 1.5748301859105378e+05 },
    { 1.0399999999999999e-09, 1.3113326845000001e-01, 6.8241872144623194e+04 },
    { 1.0399999999999999e-09, 1.7584992754300000e+00, 1.5610082065856917e+05 },
    { 1.0300000000000000e-09, 5.2841559907900004e+00, 5.5516418709848200e+04 },
    { 1.0200000000000000e-09, 1.7681968478900001e+00, 1.0435549390165479e+05 },
    { 1.0200000000000000e-09, 5.3630729788300000e+00, 1.1911685160956658e+05 },
    { 1.0200000000000000e-09, 3.3406045612000002e+00, 1.0434773123093879e+05 },
    { 1.0100000000000000e-09, 4.1411534514800001e+00, 2.6198109462793600e+04 },
    { 1.0100000000000000e-09, 2.8445154003100002e+00, 7.4923096998022782e+04 },
    { 1.0100000000000000e-09, 5.0021091482799998e+00, 2.6941099523326200e+04 },
    { 1.0100000000000000e-09, 3.7463492787000002e-01, 5.2250587881715699e+04 },
    { 1.0000000000000001e-09, 5.4613770914600002e+00, 2.5035634758385801e+04 },
    { 1.0000000000000001e-09, 3.2657760589099998e+00, 1.0456491166173479e+05 },
    { 1.0000000000000001e-09, 3.7157933173000002e-01, 5.2225802905052602e+04 },
    { 9.9000000000000010e-10, 3.0581638086599998e+00, 5.1951461487446490e+04 },
    { 9.9000000000000010e-10, 4.8641658134499997e+00, 9.7580901965051191e+04 },
    { 9.7999999999999992e-10, 4.0293133382000001e-01, 3.1415379249957001e+04 },
    { 9.6999999999999996e-10, 3.4807625718400002e+00, 7.4299006053259995e+02 },
    { 9.5999999999999999e-10, 5.1369911562999997e+00, 3.2350541665740002e+02 },
    { 9.5999999999999999e-10, 5.4902322494600000e+00, 2.7819025494506801e+04 },
    { 9.2999999999999999e-10, 5.0610076859599999e+00, 5.2125809661244188e+04 },
    { 9.2999999999999999e-10, 5.4780257914900004e+00, 2.7780068811076591e+04 },
    { 9.2999999999999999e-10, 2.9654371644899999e+00, 2.2218566345969998e+03 },
    { 9.2999999999999999e-10, 3.1212307617800001e+00, 5.2698383701242194e+04 },
    { 9.2000000000000003e-10, 9.3585280840999996e-01, 2.9416038797854391e+04 },
    { 9.2000000000000003e-10, 5.9238136531899999e+00, 1.4065280125408198e+05 },
    { 9.2000000000000003e-10, 4.5487310888400003e+00, 5.1219517127177787e+04 },
    { 9.0999999999999996e-10, 6.0161799308799999e+00, 1.2725453434775000e+04 },
    { 9.0999999999999996e-10, 4.5835698190700001e+00, 5.1859414413491788e+04 },
    { 8.9999999999999999e-10, 3.8862811902600001e+00, 5.4509002676020398e+04 },
    { 8.9999999999999999e-10, 4.2302657903399998e+00, 5.1013331578740588e+04 },
    { 8.9000000000000003e-10, 8.5475471665000002e-01, 1.9111994832171999e+03 },
    { 8.9000000000000003e-10, 2.5105376433100002e+00, 1.3767819129947099e+05 },
    { 8.7999999999999996e-10, 5.2385693173499996e+00, 3.6109740421673603e+04 },
    { 8.7999999999999996e-10, 8.4909576957999999e-01, 5.2492198152804987e+04 },
    { 8.6000000000000003e-10, 4.7142029005200001e+00, 5.2065599961928987e+04 },
    { 8.6000000000000003e-10, 3.2210029254300001e+00, 1.0477821075717278e+05 },
    { 8.4999999999999996e-10, 7.4911935880000002e-02, 5.2325369480282992e+04 },
    { 8.3000000000000003e-10, 3.0670136789900000e+00, 5.3228074666336790e+04 },
    { 8.3000000000000003e-10, 6.7057706395000005e-01, 5.2257741809380002e+02 },
    { 8.1999999999999996e-10, 3.5286565488099999e+00, 7.8731674415076996e+04 },
    { 8.1999999999999996e-10, 4.5386787262299997e+00, 8.1942753332085995e+03 },
    { 8.0000000000000003e-10, 3.1672322869400000e+00, 2.7999102624791401e+04 },
    { 8.0000000000000003e-10, 5.2892633217100000e+00, 7.7829997686845789e+04 },
    { 7.8999999999999996e-10, 3.2219280728199999e+00, 4.0565254324774200e+04 },
    { 7.8999999999999996e-10, 3.2217380091100001e+00, 1.0594068546158058e+05 },
    { 7.8999999999999996e-10, 3.7994668187200000e+00, 2.5446489579835201e+04 },
    { 7.7999999999999999e-10, 5.8971671573300002e+00, 5.3242301760338400e+04 },
    { 7.7999999999999999e-10, 5.7331577243699998e+00, 1.0328511708910679e+05 },
    { 7.7999999999999999e-10, 5.5023801469600002e+00, 2.6086418668865899e+04 },
    { 7.7000000000000003e-10, 5.2800151558800001e+00, 1.0420204936916218e+05 },
    { 7.7000000000000003e-10, 2.3542802490899999e+00, 1.0329934418310839e+05 },
    { 7.7000000000000003e-10, 1.5680563848000001e-01, 2.6089387614282488e+04 },
    { 7.5999999999999996e-10, 1.1315310608800000e+00, 1.0213285546210999e+05 },
    { 7.5999999999999996e-10, 2.7756734987699998e+00, 2.5021407664384202e+04 },
    { 7.5999999999999996e-10, 5.2959208457900004e+00, 2.5953794091989192e+04 },
    { 7.5000000000000000e-10, 4.7865700126000004e+00, 3.3281356562801898e+03 },
    { 7.5000000000000000e-10, 1.6596485677499999e+00, 7.7844224780847391e+04 },
    { 7.5000000000000000e-10, 6.3857154573999997e-01, 1.0435872611329758e+05 },
    { 7.4000000000000003e-10, 1.5479931282499999e+00, 5.2595290927023590e+04 },
    { 7.4000000000000003e-10, 1.8536841740100001e+00, 2.6729316703313190e+04 },
    { 7.4000000000000003e-10, 5.4023517436299997e+00, 9.0695752075120385e+04 },
    { 7.2999999999999996e-10, 5.1260534242600002e+00, 5.0696939709083992e+04 },
    { 7.2999999999999996e-10, 3.7194429846999999e-01, 2.6222012191159200e+04 },
    { 7.2999999999999996e-10, 3.9232698254099998e+00, 9.7670387712897180e+04 },
    { 7.2000000000000000e-10, 3.0520306582000001e+00, 1.7004917029103660e+05 },
    { 7.2000000000000000e-10, 6.1246448005399996e+00, 5.0800032483302588e+04 },
    { 7.2000000000000000e-10, 1.6673083466800001e+00, 2.1990873432869998e+03 },
    { 7.1000000000000003e-10, 1.8194969620799999e+00, 9.9024136457913992e+04 },
    { 6.9999999999999996e-10, 3.9745991951900002e+00, 2.4998194350380589e+04 },
    { 6.9999999999999996e-10, 3.3322402959400002e+00, 2.7684089583858800e+04 },
    { 6.9999999999999996e-10, 4.9391225968199999e+00, 1.7893627808365600e+04 },
    { 6.9999999999999996e-10, 4.7269453564599999e+00, 2.4864085300795588e+04 },
    { 6.9000000000000000e-10, 5.6739331250799996e+00, 2.7669862489857191e+04 },
    { 6.8000000000000003e-10, 1.3308717949100000e+00, 2.5977696820354791e+04 },
    { 6.8000000000000003e-10, 2.1353480477300000e+00, 2.0764384320240439e+05 },
    { 6.8000000000000003e-10, 2.8287897079700000e+00, 2.1116503133776000e+03 },
    { 6.6999999999999996e-10, 1.6814388496699999e+00, 2.0085575391590000e+03 },
    { 6.6999999999999996e-10, 1.3661000693300001e+00, 1.9669760899789999e+01 },
    { 6.6000000000000000e-10, 4.2101025113699997e+00, 7.8270337983622594e+04 },
    { 6.6000000000000000e-10, 1.7413716240099999e+00, 7.7795744434368185e+04 },
    { 6.6000000000000000e-10, 1.0757816151599999e+00, 7.8257080865822587e+04 },
    { 6.5000000000000003e-10, 5.8715067110600003e+00, 1.2546481939083000e+04 },
    { 6.3999999999999996e-10, 2.6101335250000000e-02, 7.6571543755220191e+04 },
    { 6.3000000000000000e-10, 1.6848798074200000e+00, 2.7177611932767799e+04 },
    { 6.3000000000000000e-10, 4.9748028885000001e-01, 7.7410513042970590e+04 },
    { 6.3000000000000000e-10, 9.2748869513999999e-01, 2.7311720982352810e+04 },
    { 6.2000000000000003e-10, 3.4984048911299999e+00, 1.5024434299945380e+05 },
    { 6.2000000000000003e-10, 4.0724977702300000e+00, 5.2290245571833613e+04 },
    { 6.2000000000000003e-10, 3.7049116131400002e+00, 9.4917560896980001e+02 },
    { 6.2000000000000003e-10, 1.3356427903999999e+00, 1.7695398994186718e+05 },
    { 6.0999999999999996e-10, 2.9909906162900000e+00, 3.8813356576349201e+04 },
    { 6.0000000000000000e-10, 4.3589295980899996e+00, 8.4608283475120004e+02 },
    { 6.0000000000000000e-10, 3.2544034250400000e+00, 1.3721022630911658e+05 },
    { 5.9000000000000003e-10, 1.3997100580399999e+00, 5.2061366994463169e+04 },
    { 5.9000000000000003e-10, 5.0018829726799998e+00, 2.2041264243879999e+02 },
    { 5.7999999999999996e-10, 2.6436680307099998e+00, 1.6785008294774959e+05 },
    { 5.7999999999999996e-10, 1.5414193673600001e+00, 2.5508215554575399e+04 },
    { 5.7999999999999996e-10, 5.0059508707099996e+00, 2.4448833624386200e+04 },
    { 5.7000000000000000e-10, 5.0088217969600004e+00, 3.8133035637799999e+01 },
    { 5.6000000000000003e-10, 3.7531720409800000e+00, 2.0981260368468694e+05 },
    { 5.6000000000000003e-10, 5.4121384212599999e+00, 2.2625658435709000e+04 },
    { 5.4999999999999996e-10, 5.1184697384700000e+00, 4.4295717129809404e+04 },
    { 5.4999999999999996e-10, 5.1612880669100001e+00, 1.7240203644480839e+05 },
    { 5.4999999999999996e-10, 3.6065446461200001e+00, 1.0779418751126219e+05 },
    { 5.4999999999999996e-10, 4.7766876656199999e+00, 1.5705710981453978e+05 },
    { 5.4000000000000000e-10, 4.2521966311800004e+00, 6.0370081616356991e+04 },
    { 5.4000000000000000e-10, 8.3991353377000000e-01, 2.5938339944439598e+04 },
    { 5.4000000000000000e-10, 5.7916930497300001e+00, 2.3919142659291600e+04 },
    { 5.4000000000000000e-10, 2.0680001647999999e-01, 1.5377881048480000e+02 },
    { 5.3000000000000003e-10, 6.0139212394300001e+00, 5.2252072354423988e+04 },
    { 5.3000000000000003e-10, 2.3746311564000000e-01, 2.7037078750543991e+04 },
    { 5.3000000000000003e-10, 2.9722112113999999e-01, 5.4087005766365597e+04 },
    { 5.3000000000000003e-10, 2.1378946179099998e+00, 1.4742351185532758e+05 },
    { 5.3000000000000003e-10, 1.2507223620999999e-01, 5.2381991831585590e+04 },
    { 5.1999999999999996e-10, 5.6738965818400002e+00, 2.6312247937276101e+04 },
    { 5.1999999999999996e-10, 4.1266529854899998e+00, 2.6667590728572999e+04 },
    { 5.1000000000000000e-10, 4.2061661106199999e+00, 5.5503941939428587e+04 },
    { 5.1000000000000000e-10, 1.3893288027799999e+00, 1.2699694076290558e+05 },
    { 5.1000000000000000e-10, 1.3920904516099999e+00, 2.8736357967047199e+04 },
    { 5.1000000000000000e-10, 1.9880186826499999e+00, 1.8208563102592478e+05 },
    { 5.0000000000000003e-10, 1.2289887572900000e+00, 5.8857031136547987e+04 },
    { 5.0000000000000003e-10, 6.2769089958000002e-01, 2.7726972658762199e+04 },
    { 5.0000000000000003e-10, 3.3310607736000000e+00, 1.4844727083000000e+00 },
    { 4.8999999999999996e-10, 5.7713752861699996e+00, 1.2979961842155698e+05 },
    { 4.8999999999999996e-10, 5.1628105246700002e+00, 2.6709646942413401e+04 },
    { 4.8999999999999996e-10, 1.9171408600600000e+00, 7.5930513031850584e+04 },
    { 4.8000000000000000e-10, 3.8515120551800002e+00, 7.7630925685409384e+04 },
    { 4.8000000000000000e-10, 5.6038578425400001e+00, 6.5831666774324796e+04 },
    { 4.8000000000000000e-10, 5.4568349426599996e+00, 2.5138727532604400e+04 },
    { 4.8000000000000000e-10, 5.0977666152000001e-01, 2.5466159340735001e+04 },
    { 4.7000000000000003e-10, 5.6801808527700004e+00, 5.1969620734711192e+04 },
    { 4.7000000000000003e-10, 6.0759035630399998e+00, 7.6681750076439581e+04 },
    { 4.7000000000000003e-10, 4.1667736372300004e+00, 2.7044192297544800e+04 },
    { 4.7000000000000003e-10, 2.8952313371899998e+00, 1.5557181910783658e+05 },
    { 4.7000000000000003e-10, 2.1689217457000001e+00, 7.6667522982437978e+04 },
    { 4.6000000000000001e-10, 2.4060913722999999e-01, 2.9550147847439392e+04 },
    { 4.6000000000000001e-10, 5.6265303905500001e+00, 1.0427534649502118e+05 },
    { 4.5000000000000000e-10, 7.9959361589000000e-01, 2.1258774073792001e+03 },
    { 4.3999999999999998e-10, 8.8444106920000001e-02, 3.6523064398203877e+05 },
    { 4.3999999999999998e-10, 3.4632654999999998e-02, 2.1436455718174577e+05 },
    { 4.3999999999999998e-10, 1.8507909092699999e+00, 5.2712610795243789e+04 },
    { 4.3999999999999998e-10, 6.1535127867600004e+00, 2.8256663623856792e+04 },
    { 4.3999999999999998e-10, 2.5337525042900002e+00, 1.5758680077963436e+05 },
    { 4.3000000000000001e-10, 5.6584433174699997e+00, 7.8903606711036584e+04 },
    { 4.3000000000000001e-10, 4.3565351045099998e+00, 1.2726810256271999e+03 },
    { 4.3000000000000001e-10, 3.6666296175599999e+00, 1.6703062133498999e+04 },
    { 4.3000000000000001e-10, 5.3342092891700004e+00, 2.5867490499135391e+04 },
    { 4.3000000000000001e-10, 3.0208777150100001e+00, 5.1528795449833589e+04 },
    { 4.2000000000000000e-10, 2.0337813145500001e+00, 1.4859801210651999e+03 },
    { 4.2000000000000000e-10, 2.0821946079600000e+00, 7.8188927826155283e+04 },
    { 4.2000000000000000e-10, 5.1972252175799998e+00, 2.5169972855592401e+04 },
    { 4.2000000000000000e-10, 5.6126000399999998e-01, 2.6308315784013001e+04 },
    { 4.0999999999999998e-10, 6.2469553837099996e+00, 2.0618554843720000e+02 },
    { 4.0999999999999998e-10, 3.3587194781699998e+00, 1.7952584437210000e+03 },
    { 4.0999999999999998e-10, 1.7963734408700001e+00, 1.2655674786264001e+03 },
    { 4.0000000000000001e-10, 8.1354779862000004e-01, 1.3265827281205778e+05 },
    { 4.0000000000000001e-10, 1.6481831154400000e+00, 2.6823779655105991e+04 },
    { 4.0000000000000001e-10, 4.0188434221699998e+00, 2.5352026628042389e+04 },
    { 4.0000000000000001e-10, 4.4092705709000004e+00, 1.4807872442630000e+02 },
    { 3.9000000000000000e-10, 4.6607609111999998e-01, 2.7005833427555990e+04 },
    { 3.9000000000000000e-10, 5.4035479384300000e+00, 1.5104766984290001e+02 },
    { 3.7999999999999998e-10, 5.8136521270600001e+00, 1.0450539137678158e+05 },
    { 3.7999999999999998e-10, 3.9477294750600000e+00, 2.6102130235575802e+04 },
    { 3.7999999999999998e-10, 3.5195619618600000e+00, 1.3045918546877075e+05 },
    { 3.7999999999999998e-10, 1.6642784577300000e+00, 2.6402089321443800e+04 },
    { 3.7999999999999998e-10, 5.1353178269199997e+00, 9.3848410080752001e+03 },
    { 3.7999999999999998e-10, 1.4191178983499999e+00, 1.3041984594697120e+05 },
    { 3.7999999999999998e-10, 5.5037722593799998e+00, 6.4901259717923393e+04 },
    { 3.7000000000000001e-10, 5.5789925114500001e+00, 1.0419783375581198e+05 },
    { 3.7000000000000001e-10, 3.0219826668300001e+00, 2.6118230002578599e+04 },
    { 3.7000000000000001e-10, 1.6707094739099999e+00, 2.4203001978156801e+04 },
    { 3.7000000000000001e-10, 5.4274772034099996e+00, 7.8366802198941179e+04 },
    { 3.7000000000000001e-10, 4.0081159434600000e+00, 2.5773716961704591e+04 },
    { 3.7000000000000001e-10, 4.2908436709700002e+00, 3.7698455099948398e+04 },
    { 3.6000000000000000e-10, 4.5451100482499998e+00, 1.2832075184749259e+05 },
    { 3.6000000000000000e-10, 2.6504117343600000e+00, 2.6057576280569789e+04 },
    { 3.6000000000000000e-10, 9.4812382604000001e-01, 1.3022621661243298e+05 },
    { 3.4999999999999998e-10, 5.6576540054000002e-01, 7.8896493164035783e+04 },
    { 3.4999999999999998e-10, 4.0050606097200001e+00, 2.7972804304991590e+04 },
    { 3.4999999999999998e-10, 2.6191697008200001e+00, 2.7463676941419999e+04 },
    { 3.4999999999999998e-10, 3.1617510624400000e+00, 5.1653228865054589e+04 },
    { 3.4999999999999998e-10, 1.4968430187699999e+00, 5.2483363904117992e+04 },
    { 3.4000000000000001e-10, 3.5646272093400002e+00, 2.7566769715638591e+04 },
    { 3.4000000000000001e-10, 1.7528783406299999e+00, 2.6073676047572590e+04 },
    { 3.4000000000000001e-10, 3.6770135519500000e+00, 2.5985940623308590e+04 },
    { 3.4000000000000001e-10, 3.9685364408599999e+00, 5.1868248662178798e+04 },
    { 3.4000000000000001e-10, 1.3376406002300001e+00, 7.7307420268751986e+04 },
    { 3.4000000000000001e-10, 1.2534287625200000e+00, 2.2003914634869801e+04 },
    { 3.4000000000000001e-10, 5.7667612800000002e-03, 1.9054647649404001e+03 },
    { 3.3000000000000000e-10, 5.3966934587000004e-01, 5.1852300866490987e+04 },
    { 3.3000000000000000e-10, 1.9953808492400000e+00, 2.6189865659839801e+04 },
    { 3.3000000000000000e-10, 6.2028445495900000e+00, 7.7726904912627186e+04 },
    { 3.3000000000000000e-10, 2.7564930616500001e+00, 4.9953949648551388e+04 },
    { 3.3000000000000000e-10, 6.1242250033100003e+00, 7.8039364629020682e+04 },
    { 3.3000000000000000e-10, 2.4849892517200001e+00, 1.2320084011627098e+05 },
    { 3.1999999999999998e-10, 6.9003641909000002e-01, 2.0759384658050019e+05 },
    { 3.1999999999999998e-10, 4.4601313862599996e+00, 2.0415127163553477e+05 },
    { 3.1999999999999998e-10, 2.1800559844700000e+00, 5.1432816222615787e+04 },
    { 3.1999999999999998e-10, 1.1963444152700000e+00, 2.6011637070298599e+04 },
    { 3.1999999999999998e-10, 3.3593795056700002e+00, 1.6107937234650398e+05 },
    { 3.1000000000000002e-10, 5.5742833531800002e+00, 7.8160616650503987e+04 },
    { 3.1000000000000002e-10, 4.1780967254099997e+00, 2.6083970988311092e+04 },
    { 3.1000000000000002e-10, 3.6213300334900000e+00, 1.3541421202491190e+04 },
    { 3.1000000000000002e-10, 6.0127812315100000e+00, 8.7367616418840189e+04 },
    { 3.1000000000000002e-10, 4.0791058460600000e+00, 7.5615254599249587e+04 },
    { 3.0000000000000000e-10, 8.2681675218999995e-01, 1.3043240216087017e+05 },
    { 3.0000000000000000e-10, 2.7812059079900000e+00, 1.1678365521669458e+05 },
    { 3.0000000000000000e-10, 4.8161695722499998e+00, 1.8218872380014337e+05 },
    { 3.0000000000000000e-10, 4.5876251608500001e+00, 4.9842609890276392e+04 },
    { 3.0000000000000000e-10, 3.2577372628600001e+00, 9.4329775286197386e+04 },
    { 3.0000000000000000e-10, 9.7454681171000002e-01, 7.7101234720314780e+04 },
    { 3.0000000000000000e-10, 3.2859851833999998e-01, 2.4395737472071800e+04 },
    { 3.0000000000000000e-10, 4.4241408729099998e+00, 3.0171891648278601e+04 },
    { 3.0000000000000000e-10, 2.2705611831599999e+00, 5.3906928636080993e+04 },
    { 3.0000000000000000e-10, 2.8930038044500002e+00, 1.7311223529326001e+03 },
    { 2.8999999999999998e-10, 4.8857675744400000e+00, 1.8102624909573558e+05 },
    { 2.8999999999999998e-10, 4.0056231762800003e+00, 2.3439448316101189e+04 },
    { 2.8999999999999998e-10, 4.4192487042900002e+00, 1.1439910691340000e+02 },
    { 2.8999999999999998e-10, 1.5097855175999999e+00, 2.6091835294837289e+04 },
    { 2.8999999999999998e-10, 4.5965972113199998e+00, 1.3672259155786238e+05 },
    { 2.8999999999999998e-10, 2.0605268044099998e+00, 5.1226630674178588e+04 },
    { 2.8999999999999998e-10, 8.9787685569999998e-02, 2.6575537892828390e+04 },
    { 2.8999999999999998e-10, 5.5851070742399997e+00, 2.5600268390320001e+04 },
    { 2.8999999999999998e-10, 4.6494181053299997e+00, 3.9321532631000000e+00 },
    { 2.8999999999999998e-10, 6.2646737278400000e+00, 6.1292970395066004e+03 },
    { 2.8999999999999998e-10, 1.6280010268500000e+00, 1.4227094001599999e+01 },
    { 2.8000000000000002e-10, 7.0524581752000004e-01, 8.0382473285100990e+04 },
    { 2.8000000000000002e-10, 4.6272281350000002e+00, 5.4374893626435398e+04 },
    { 2.8000000000000002e-10, 5.0373778388900003e+00, 1.8357092173262799e+05 },
    { 2.8000000000000002e-10, 5.9384243381699999e+00, 3.4282178474782799e+04 },
    { 2.8000000000000002e-10, 3.1497981475799999e+00, 6.5717227485639582e+04 },
    { 2.7000000000000000e-10, 5.9197642912699999e+00, 1.0101100013959699e+05 },
    { 2.7000000000000000e-10, 1.6262875937500001e+00, 5.2509662223917803e+04 },
    { 2.7000000000000000e-10, 5.2784028449999999e-02, 1.3065281480330898e+05 },
    { 2.7000000000000000e-10, 9.1518492017999997e-01, 1.2238178407786099e+03 },
    { 2.7000000000000000e-10, 7.3776364995999999e-01, 2.7170983373867792e+04 },
    { 2.7000000000000000e-10, 4.9258613812499998e+00, 2.5004822909280600e+04 },
    { 2.7000000000000000e-10, 1.7655329782200000e+00, 1.4300566740785379e+05 },
    { 2.7000000000000000e-10, 1.2981817373900000e+00, 2.4072921469776400e+04 },
    { 2.7000000000000000e-10, 2.9010679493299998e+00, 7.6887935624876787e+04 },
    { 2.5999999999999998e-10, 5.9968174687899998e+00, 2.6049770105936401e+04 },
    { 2.5999999999999998e-10, 1.9017651592600000e+00, 7.6784842850658184e+04 },
    { 2.5999999999999998e-10, 5.9726058886300004e+00, 3.9450352848373397e+04 },
    { 2.5999999999999998e-10, 1.5061489275500000e+00, 7.8153503103503186e+04 },
    { 2.5999999999999998e-10, 4.3785927421300004e+00, 2.8102884813371998e+04 },
    { 2.5000000000000002e-10, 3.4399804923800001e+00, 7.8338491023289884e+04 },
    { 2.5000000000000002e-10, 2.8869747024499999e+00, 5.0064155969770793e+04 },
    { 2.5000000000000002e-10, 5.0011564315900001e+00, 2.9912639426919998e+02 },
    { 2.5000000000000002e-10, 1.4112259027600000e+00, 4.8847670626868203e+04 },
    { 2.5000000000000002e-10, 5.9387150192500000e+00, 2.6724899413598399e+04 },
    { 2.5000000000000002e-10, 6.0080953191399997e+00, 2.5450906869549999e+04 },
    { 2.5000000000000002e-10, 1.1282707963999999e-01, 1.3043563437251298e+05 },
    { 2.5000000000000002e-10, 2.2970680875400000e+00, 5.2174321810440088e+04 },
    { 2.5000000000000002e-10, 2.0124499988200002e+00, 8.1604321851422384e+04 },
    { 2.5000000000000002e-10, 4.8236665733599997e+00, 1.3044339704322898e+05 },
    { 2.5000000000000002e-10, 3.3612008451399999e+00, 7.8313706046626787e+04 },
    { 2.4000000000000000e-10, 2.6694542793100000e+00, 2.4824745778995999e+04 },
    { 2.4000000000000000e-10, 3.2456464221600001e+00, 1.0451548361876000e+03 },
    { 2.4000000000000000e-10, 2.4921599578400002e+00, 1.2937302023068098e+05 },
    { 2.4000000000000000e-10, 9.7156468917000005e-01, 5.2286012604367788e+04 },
    { 2.4000000000000000e-10, 6.1914558140500002e+00, 7.8786286842816378e+04 },
    { 2.4000000000000000e-10, 3.8654620953699999e+00, 5.1841950342379001e+04 },
    { 2.4000000000000000e-10, 2.1282854684300001e+00, 1.4520475475114078e+05 },
    { 2.4000000000000000e-10, 3.0467575454500002e+00, 5.2177290755856688e+04 },
    { 2.4000000000000000e-10, 4.2377329661600003e+00, 2.4925428437166400e+04 },
    { 2.3000000000000001e-10, 3.0024283656600002e+00, 2.7351060504152389e+04 },
    { 2.3000000000000001e-10, 3.2721658935100000e+00, 5.2609518021025193e+04 },
    { 2.3000000000000001e-10, 1.3274773883300000e+00, 7.7947317555065980e+04 },
    { 2.3000000000000001e-10, 1.6307977087700001e+00, 1.2366880510662538e+05 },
    { 2.3000000000000001e-10, 2.0532778504000002e+00, 1.0391790082841998e+05 },
    { 2.3000000000000001e-10, 4.9670965401200000e+00, 1.6342582585717601e+04 },
    { 2.3000000000000001e-10, 5.4049135492799998e+00, 1.2938724732468258e+05 },
    { 2.3000000000000001e-10, 3.9787434198299998e+00, 3.4622447058652001e+03 },
    { 2.3000000000000001e-10, 2.6976865878699998e+00, 1.6674070439565618e+05 },
    { 2.3000000000000001e-10, 4.8404969014799999e+00, 2.7223580013467399e+04 },
    { 2.1999999999999999e-10, 1.8324573184799999e+00, 7.8213712802818380e+04 },
    { 2.1999999999999999e-10, 5.4189779710100003e+00, 5.0264606799931200e+04 },
    { 2.1999999999999999e-10, 3.4013471257800001e+00, 6.8851498899308099e+03 },
    { 2.1999999999999999e-10, 3.2133587692900001e+00, 5.0444683930215797e+04 },
    { 2.1999999999999999e-10, 3.5184805114800000e+00, 1.0349841618454478e+05 },
    { 2.1999999999999999e-10, 3.8889557505800001e+00, 7.8580101294379187e+04 },
    { 2.1999999999999999e-10, 6.5838685945999997e-01, 8.0596905817594597e+04 },
    { 2.1999999999999999e-10, 2.2831739621199998e+00, 2.4491716699289591e+04 },
    { 2.1999999999999999e-10, 3.9813492102399999e+00, 5.0167248743989388e+04 },
    { 2.1999999999999999e-10, 6.6692365929999997e-02, 2.6126036177212001e+04 },
    { 2.1999999999999999e-10, 5.7163606077900004e+00, 2.6734913974889001e+04 },
    { 2.1999999999999999e-10, 8.3649888959999996e-01, 2.4952226269680999e+04 },
    { 2.1999999999999999e-10, 4.7500138667399998e+00, 1.0393212792242158e+05 },
    { 2.1999999999999999e-10, 6.0845881068000001e+00, 1.0554497769261000e+03 },
    { 2.1000000000000000e-10, 9.4006911216000000e-01, 5.1955393640709590e+04 },
    { 2.1000000000000000e-10, 2.3734827097900002e+00, 3.5472744149649399e+04 },
    { 2.1000000000000000e-10, 3.0139540547400001e+00, 1.0633140834523001e+03 },
    { 2.1000000000000000e-10, 6.7741466251000004e-01, 1.2375829085447139e+05 },
    { 2.1000000000000000e-10, 5.5332559104800003e+00, 1.6376609444104519e+05 },
    { 2.1000000000000000e-10, 2.0545908205500001e+00, 1.3028995251073639e+05 },
    { 2.1000000000000000e-10, 6.1205369639999996e-01, 9.6357084124272587e+04 },
    { 2.1000000000000000e-10, 6.2799029728100004e+00, 1.3086611389874699e+05 },
    { 2.1000000000000000e-10, 5.1308004310099999e+00, 3.1781457905676002e+03 },
    { 2.0000000000000001e-10, 1.3107685389299999e+00, 5.1329723448397199e+04 },
    { 2.0000000000000001e-10, 1.7177676496800001e+00, 1.0897087911936001e+03 },
    { 2.0000000000000001e-10, 5.1352529087500001e+00, 7.3587651353180001e+02 },
    { 2.0000000000000001e-10, 6.0994538441800001e+00, 1.9613707343261078e+05 },
    { 2.0000000000000001e-10, 5.5448226272100003e+00, 3.9629324344065390e+04 },
    { 2.0000000000000001e-10, 3.0552664551999997e-01, 1.0481957755665119e+05 },
    { 2.0000000000000001e-10, 5.9360066295799996e+00, 2.5907826011289599e+04 },
    { 2.0000000000000001e-10, 6.0108037088100001e+00, 2.6267980271858800e+04 },
    { 2.0000000000000001e-10, 5.1830563921700001e+00, 2.3373174634397859e+05 },
    { 2.0000000000000001e-10, 3.8279809837000001e-01, 1.2412372889497999e+04 },
    { 1.8999999999999999e-10, 4.6688489706700000e+00, 7.8683194068597775e+04 },
    { 1.8999999999999999e-10, 3.6932381942500001e+00, 1.3044662925487179e+05 },
    { 1.8999999999999999e-10, 6.2129874023299996e+00, 2.4505943793291190e+04 },
    { 1.8999999999999999e-10, 6.2331509206400000e+00, 2.6248310510959000e+04 },
    { 1.8999999999999999e-10, 5.7136594177499997e+00, 2.5927495772189392e+04 },
    { 1.8999999999999999e-10, 4.9343210895900000e+00, 5.2817219844887390e+04 },
    { 1.8999999999999999e-10, 3.0846312292700002e+00, 1.0265944689679438e+05 },
    { 1.8999999999999999e-10, 6.2821067810400004e+00, 1.3202858860315479e+05 },
    { 1.8999999999999999e-10, 2.0173190298699999e+00, 8.6143798578061585e+04 },
    { 1.8000000000000000e-10, 7.2295496626000000e-01, 1.1223170171963578e+05 },
    { 1.8000000000000000e-10, 4.1893174099400001e+00, 1.2822075860368418e+05 },
    { 1.8000000000000000e-10, 1.7878574251199999e+00, 5.3029002664900399e+04 },
    { 1.8000000000000000e-10, 1.8789134494699999e+00, 2.6411408558231600e+04 },
    { 1.8000000000000000e-10, 4.9066614582900003e+00, 2.7573193848277399e+04 },
    { 1.8000000000000000e-10, 8.1718560103000004e-01, 5.2137673247510589e+04 },
    { 1.8000000000000000e-10, 3.1553670823300002e+00, 7.8413272621857192e+04 },
    { 1.8000000000000000e-10, 6.1581696674500002e+00, 7.9315977807910982e+04 },
    { 1.8000000000000000e-10, 4.9434605365199999e+00, 5.0049928875769190e+04 },
    { 1.7000000000000001e-10, 2.2918299719399999e+00, 5.3867971952650791e+04 },
    { 1.7000000000000001e-10, 7.7039059906999996e-01, 2.4602612434870989e+04 },
    { 1.7000000000000001e-10, 3.3283518260600000e+00, 5.5638050989013587e+04 },
    { 1.7000000000000001e-10, 4.8837524285300002e+00, 1.2511203959948818e+05 },
    { 1.5999999999999999e-10, 2.6929300486000001e+00, 7.9330204901912584e+04 },
    { 1.5999999999999999e-10, 2.7851081358299998e+00, 9.2741060607922584e+04 },
    { 1.5999999999999999e-10, 1.3134140508100001e+00, 5.0903125257521198e+04 },
    { 1.5999999999999999e-10, 2.4436891949500001e+00, 3.8813353579999998e+00 },
    { 1.5999999999999999e-10, 3.6577455426499998e+00, 5.0689826162083191e+04 },
    { 1.5999999999999999e-10, 6.1012209457299997e+00, 2.8309759776171191e+04 },
    { 1.5999999999999999e-10, 4.2617837623400003e+00, 6.0442285813753997e+03 },
    { 1.5999999999999999e-10, 9.8987078340000001e-01, 1.0435824112519680e+05 },
    { 1.5999999999999999e-10, 2.8898343422000000e-01, 1.7633224614102798e+05 },
    { 1.5999999999999999e-10, 1.9981679881400001e+00, 6.2197643563247788e+04 },
    { 1.5999999999999999e-10, 4.5145796409900001e+00, 6.1560647291223591e+04 },
    { 1.5999999999999999e-10, 1.9157651442200001e+00, 7.7750543983900002e+01 },
    { 1.5999999999999999e-10, 4.1309179936999998e+00, 1.0434498400739678e+05 },
    { 1.5999999999999999e-10, 3.8661972599999999e+00, 5.0910238804522000e+04 },
    { 1.5999999999999999e-10, 3.7152747653299998e+00, 5.3814875800336391e+04 },
    { 1.5999999999999999e-10, 4.3906066697000004e+00, 2.0304189308344139e+05 },
    { 1.5999999999999999e-10, 5.6654525193000005e-01, 5.1534392721409400e+04 },
    { 1.5999999999999999e-10, 4.4352911898100000e+00, 1.6390695171880000e+03 },
    { 1.5999999999999999e-10, 6.0388943460000002e+00, 3.4082431583598402e+04 },
    { 1.5000000000000000e-10, 2.5196730570599999e+00, 1.5588752156313116e+05 },
    { 1.5000000000000000e-10, 5.2512621542000004e-01, 2.3590050682626115e+05 },
    { 1.5000000000000000e-10, 3.4273081317099998e+00, 5.2309915332733399e+04 },
    { 1.5000000000000000e-10, 2.6165813900499999e+00, 5.0007045800865802e+04 },
    { 1.5000000000000000e-10, 6.6107548299999999e-02, 2.6830893202106799e+04 },
    { 1.5000000000000000e-10, 5.2888794914300004e+00, 1.0275542612401219e+05 },
    { 1.5000000000000000e-10, 2.9233149063499999e+00, 1.0276965321801379e+05 },
    { 1.5000000000000000e-10, 5.2124686583099997e+00, 1.7351141499690176e+05 },
    { 1.5000000000000000e-10, 6.0788519222200001e+00, 7.7616698591407781e+04 },
    { 1.5000000000000000e-10, 4.7947224566599997e+00, 1.0388364757594238e+05 },
    { 1.5000000000000000e-10, 3.6794634600000001e-02, 1.6329812945069079e+05 },
    { 1.5000000000000000e-10, 5.0201953547600002e+00, 8.6457984757931190e+04 },
    { 1.4000000000000001e-10, 3.8206454808000001e-01, 1.3388209065283637e+05 },
    { 1.4000000000000001e-10, 5.6668871077100000e+00, 1.9393798608932379e+05 },
    { 1.4000000000000001e-10, 1.9512324370800000e+00, 2.6247204876366399e+04 },
    { 1.4000000000000001e-10, 3.7123925941299998e+00, 2.5928601406782000e+04 },
    { 1.4000000000000001e-10, 5.7896956275499996e+00, 1.1695882514085999e+03 },
    { 1.4000000000000001e-10, 4.3562985554200004e+00, 1.5308484390447979e+05 },
    { 1.4000000000000001e-10, 4.1808098651999998e-01, 2.5234706759822191e+04 },
    { 1.4000000000000001e-10, 4.4533755169899996e+00, 7.8149270136037361e+04 },
    { 1.4000000000000001e-10, 4.3915054031400000e+00, 5.2161579189146789e+04 },
    { 1.4000000000000001e-10, 8.4211332160999997e-01, 7.8378148713407805e+04 },
    { 1.4000000000000001e-10, 6.9241893677999999e-01, 3.5833223697430803e+04 },
    { 1.4000000000000001e-10, 3.2177197165200000e+00, 2.6076857441310302e+04 },
    { 1.4000000000000001e-10, 2.4459053146900001e+00, 2.6098948841838101e+04 },
    { 1.4000000000000001e-10, 4.4503321544299999e+00, 2.0043674560198800e+04 },
    { 1.4000000000000001e-10, 1.9351579894900000e+00, 1.9848993958638259e+05 },
    { 1.4000000000000001e-10, 4.7401582959999997e+00, 5.3265515074342002e+04 },
    { 1.4000000000000001e-10, 3.9812967426900001e+00, 5.3399624123927002e+04 },
    { 1.4000000000000001e-10, 4.3247770022100003e+00, 2.7250377845981991e+04 },
    { 1.4000000000000001e-10, 1.0085644603199999e+00, 5.2755493870147191e+04 },
    { 1.4000000000000001e-10, 2.0527268464700001e+00, 5.2041697233563391e+04 },
    { 1.4000000000000001e-10, 1.8565580505899999e+00, 8.6030992875280003e+02 },
    { 1.4000000000000001e-10, 1.5337479461000001e+00, 1.8314501295611399e+05 },
    { 1.4000000000000001e-10, 2.9395627729800000e+00, 5.4344566765430987e+04 },
    { 1.4000000000000001e-10, 3.2694554329400001e+00, 5.3124981892118187e+04 },
    { 1.2999999999999999e-10, 2.2822725763900000e+00, 5.0380547839427389e+04 },
    { 1.2999999999999999e-10, 4.9599266039699996e+00, 2.5654191403697390e+04 },
    { 1.2999999999999999e-10, 5.4283839652999999e-01, 1.0371882882698359e+05 },
    { 1.2999999999999999e-10, 2.5880243150999999e-01, 5.2024758613305487e+04 },
    { 1.2999999999999999e-10, 5.0169077270200004e+00, 2.0817353416749899e+05 },
    { 1.2999999999999999e-10, 1.2553016270099999e+00, 5.2027727558722087e+04 },
    { 1.2999999999999999e-10, 3.2086364940599998e+00, 7.8469894973159782e+04 },
    { 1.2999999999999999e-10, 1.3533616712500001e+00, 1.1045700263900001e+01 },
    { 1.2999999999999999e-10, 5.6546930874700001e+00, 5.0270341518207999e+04 },
    { 1.2999999999999999e-10, 9.5208862336000000e-01, 2.5241820306823000e+04 },
    { 1.2999999999999999e-10, 9.0383542595999999e-01, 2.6014606015715191e+04 },
    { 1.2999999999999999e-10, 4.7597896052499999e+00, 2.6161200267433189e+04 },
    { 1.2999999999999999e-10, 5.9571486517399999e+00, 1.0736090241908000e+03 },
    { 1.2000000000000000e-10, 5.6625486512999998e+00, 1.2244498726584678e+05 },
    { 1.2000000000000000e-10, 1.2624433462700000e+00, 2.5984120952993599e+04 },
    { 1.2000000000000000e-10, 4.4011816849400001e+00, 2.6191685330154789e+04 },
    { 1.2000000000000000e-10, 5.9498218007699997e+00, 1.8165972224941078e+05 },
    { 1.2000000000000000e-10, 1.3991027783000001e-01, 5.3771992725432989e+04 },
    { 1.2000000000000000e-10, 1.7251618542500000e+00, 4.3981530949939799e+04 },
    { 1.2000000000000000e-10, 5.9238675730299999e+00, 2.8199553454951791e+04 },
    { 1.2000000000000000e-10, 4.5911006957699998e+00, 5.1596118696149599e+04 },
    { 1.2000000000000000e-10, 2.4023088493300002e+00, 1.3036324963659538e+05 },
    { 1.2000000000000000e-10, 7.1489552499999998e-01, 5.1439929769616589e+04 },
    { 1.2000000000000000e-10, 4.2240138805300003e+00, 5.2179738436411491e+04 },
    { 1.2000000000000000e-10, 5.7948872490600003e+00, 1.9958606102067599e+04 },
    { 1.2000000000000000e-10, 2.5129454307999999e+00, 5.3757765631431394e+04 },
    { 1.2000000000000000e-10, 6.0614644545200003e+00, 1.4287155835826878e+05 },
    { 1.2000000000000000e-10, 1.3246244377300000e+00, 2.6149629116314391e+04 },
    { 1.2000000000000000e-10, 4.3390005934799998e+00, 2.6026177166834001e+04 },
    { 1.2000000000000000e-10, 5.8380507039699996e+00, 7.6041852790125587e+04 },
    { 1.2000000000000000e-10, 3.4715698387399998e+00, 5.2396218925587200e+04 },
    { 1.2000000000000000e-10, 5.3724337865500003e+00, 1.6992792165031999e+03 },
    { 1.0999999999999999e-10, 4.6476991642999996e+00, 5.2911682796680187e+04 },
    { 1.0999999999999999e-10, 1.8986389332600000e+00, 7.0383620271383596e+04 },
    { 1.0999999999999999e-10, 2.1797511628800001e+00, 4.8713561577283202e+04 },
    { 1.0999999999999999e-10, 5.1434816198500002e+00, 1.0427683096772949e+05 },
    { 1.0999999999999999e-10, 3.1807076758799999e+00, 5.7503282391531196e+04 },
    { 1.0999999999999999e-10, 2.3087510988600002e+00, 3.3385594076939998e+02 },
    { 1.0999999999999999e-10, 1.2505786901100000e+00, 2.0894536082776402e+04 },
    { 1.0999999999999999e-10, 5.6422235469600004e+00, 6.4141356173898998e+02 },
    { 1.0999999999999999e-10, 1.7045270323199999e+00, 5.0536736765960391e+04 },
    { 1.0999999999999999e-10, 4.8840598908699997e+00, 7.8800513936817981e+04 },
    { 1.0999999999999999e-10, 2.5312128182600002e+00, 7.8057523876285384e+04 },
    { 1.0999999999999999e-10, 5.2503332056799996e+00, 7.7520719364189979e+04 },
    { 1.0999999999999999e-10, 5.2796404499599996e+00, 3.9763433393650390e+04 },
    { 1.0999999999999999e-10, 5.5772374011699997e+00, 1.8367470392120857e+05 },
    { 1.0999999999999999e-10, 3.3804842091800000e+00, 1.9367189162232800e+04 },
    { 1.0999999999999999e-10, 5.6931429856299998e+00, 4.6046509243641798e+04 },
    { 1.0999999999999999e-10, 2.5875582347799999e+00, 2.5382485042536000e+03 },
    { 1.0999999999999999e-10, 4.7873519083399998e+00, 2.8096460680733191e+04 },
    { 1.0000000000000000e-10, 2.7970080459000002e+00, 7.8339975495998180e+04 },
    { 1.0000000000000000e-10, 4.2586052963799998e+00, 8.4944934278122193e+04 },
    { 1.0000000000000000e-10, 3.5593097312799999e+00, 5.3093736569130189e+04 },
    { 1.0000000000000000e-10, 1.3211954012900000e+00, 1.5440865498906677e+05 },
    { 1.0000000000000000e-10, 1.4386297102700001e+00, 2.5131613985603592e+04 },
    { 1.0000000000000000e-10, 2.3627860050899998e+00, 9.1919569915898988e+04 },
    { 1.0000000000000000e-10, 3.5628621775700000e+00, 7.7940204008065179e+04 },
    { 1.0000000000000000e-10, 4.1733410997700000e+00, 6.7510408403457996e+03 },
    { 1.0000000000000000e-10, 5.0269108149999997e-01, 2.8791519296249800e+04 },
    { 1.0000000000000000e-10, 6.6820373958000001e-01, 6.3699627202420004e+02 },
    { 1.0000000000000000e-10, 3.0625053036500001e+00, 2.4712129341728400e+04 },
    { 1.0000000000000000e-10, 2.4396789759600002e+00, 1.0499150985261079e+05 },
    { 1.0000000000000000e-10, 1.2434229230000000e+00, 5.2061407176234992e+04 },
    { 1.0000000000000000e-10, 5.1671693846000002e+00, 2.3384286986898602e+04 },
    { 1.0000000000000000e-10, 1.2578667828200001e+00, 2.3023917477710897e+05 },
    { 1.0000000000000000e-10, 1.2271016292100001e+00, 4.0222625319642002e+03 },
    { 1.0000000000000000e-10, 3.1078714492500001e+00, 3.9131854712361295e+05 },
    { 1.0000000000000000e-10, 7.6588647880000005e-01, 5.4060707446565793e+04 },
    { 1.0000000000000000e-10, 3.6931542750099999e+00, 2.1687604822826002e+03 },
    { 1.0000000000000000e-10, 3.5289404937400000e+00, 2.7515475996916002e+03 },
    { 1.0000000000000000e-10, 3.0761826519200000e+00, 2.4045246032331997e+05 },
    { 1.0000000000000000e-10, 4.4969499334699998e+00, 2.2065640609610000e+04 },
    { 1.0000000000000000e-10, 3.2781889804000000e+00, 4.4082528487759998e+02 },
    { 1.0000000000000000e-10, 3.3770148176300001e+00, 2.7036161546755998e+03 },
    { 8.9999999999999999e-11, 2.5829627063600000e+00, 1.3059329451835579e+05 },
    { 8.9999999999999999e-11, 1.9357025819800000e+00, 2.6513811918088199e+04 },
    { 8.9999999999999999e-11, 3.7279224492299998e+00, 2.5661994365060200e+04 },
    { 8.9999999999999999e-11, 5.9888174145799997e+00, 4.3718763667741996e+03 },
    { 8.9999999999999999e-11, 2.3719144413300000e+00, 8.0174908907939782e+04 },
    { 8.9999999999999999e-11, 3.9694496891900002e+00, 1.0318913786188899e+05 },
    { 8.9999999999999999e-11, 2.1796861385999999e-01, 1.5654708861034497e+05 },
    { 8.9999999999999999e-11, 2.3479114885499999e+00, 1.3028573689738619e+05 },
    { 8.9999999999999999e-11, 3.6111835778699999e+00, 1.0498439630560997e+05 },
    { 8.9999999999999999e-11, 2.3468503393400000e+00, 2.4609036567509800e+04 },
    { 8.9999999999999999e-11, 5.6949390862799998e+00, 7.6152059111344992e+04 },
    { 8.9999999999999999e-11, 7.4268878384000003e-01, 7.6255151885563580e+04 },
    { 8.9999999999999999e-11, 4.4164849242799997e+00, 3.1281270200372001e+04 },
    { 8.9999999999999999e-11, 2.4171255482400000e+00, 5.1130651446960786e+04 },
    { 8.9999999999999999e-11, 3.6442825256100000e+00, 1.5874617595363196e+05 },
    { 8.9999999999999999e-11, 1.9696987510699999e+00, 5.2797550083987589e+04 },
    { 8.9999999999999999e-11, 5.3858540405599999e+00, 2.4844415539895788e+04 },
    { 8.9999999999999999e-11, 2.7777099064999999e-01, 2.7331390743252599e+04 },
    { 8.9999999999999999e-11, 4.0397309188400001e+00, 1.5631411975400720e+05 },
    { 8.9999999999999999e-11, 5.8907704491499997e+00, 2.3866046506977189e+04 },
    { 8.9999999999999999e-11, 9.5857873026999996e-01, 1.1443928868521000e+02 },
    { 8.9999999999999999e-11, 7.4288994111999995e-01, 5.2190033377149994e+04 },
    { 8.9999999999999999e-11, 8.1265678038000000e-01, 1.0170315774082378e+05 },
    { 8.9999999999999999e-11, 2.5436731294900001e+00, 1.6850521225016000e+03 },
    { 8.9999999999999999e-11, 2.3665227131600002e+00, 1.4852907067031999e+03 },
    { 8.9999999999999999e-11, 3.8360221271300001e+00, 2.2595331574704589e+04 },
    { 8.9999999999999999e-11, 8.9908538699999996e-01, 5.2171874129885291e+04 },
    { 8.9999999999999999e-11, 2.1423598857199999e+00, 1.0445470534051539e+05 },
    { 8.9999999999999999e-11, 4.3944500759500000e+00, 1.5650774908854539e+05 },
    { 8.9999999999999999e-11, 1.0805381822300000e+00, 4.9996621904199998e+01 },
    { 8.9999999999999999e-11, 1.5139574615900000e+00, 3.2132131722949602e+04 },
    { 8.9999999999999999e-11, 2.2237568774700001e+00, 9.0989162859497592e+04 },
    { 8.9999999999999999e-11, 1.1542790648700001e+00, 5.6259794789852793e+04 },
    { 8.9999999999999999e-11, 2.8836059916800001e+00, 1.8073704938650200e+04 },
    { 8.9999999999999999e-11, 1.7317449958000000e+00, 2.0149816717977999e+03 },
    { 7.9999999999999995e-11, 3.7455700241700001e+00, 9.1805130627213788e+04 },
    { 7.9999999999999995e-11, 2.6566948980800000e+00, 3.8634385080657201e+04 },
    { 7.9999999999999995e-11, 2.1429885885200002e+00, 2.0965882487420217e+05 },
    { 7.9999999999999995e-11, 2.7479929428999998e+00, 2.3249494088155998e+03 },
    { 7.9999999999999995e-11, 3.4285402250499999e+00, 1.1789303376878797e+05 },
    { 7.9999999999999995e-11, 2.1682402885599998e+00, 1.5748458012822000e+03 },
    { 7.9999999999999995e-11, 6.6150337712999996e-01, 5.1876679888879189e+04 },
    { 7.9999999999999995e-11, 2.1684409094300001e+00, 1.2810631931499895e+05 },
    { 7.9999999999999995e-11, 7.4099606624000003e-01, 7.7956151803752990e+04 },
    { 7.9999999999999995e-11, 2.4621542990999998e+00, 5.2400151078850300e+04 },
    { 7.9999999999999995e-11, 2.4608172574200000e+00, 1.0681250536565400e+04 },
    { 7.9999999999999995e-11, 4.4625736995900001e+00, 5.4824261108621402e+04 },
    { 7.9999999999999995e-11, 5.5712719907999997e-01, 7.7741132006628788e+04 },
    { 7.9999999999999995e-11, 4.2508106268000001e-01, 4.2790965275073198e+04 },
    { 7.9999999999999995e-11, 4.7062536939499999e+00, 5.2489992463018003e+04 },
    { 7.9999999999999995e-11, 3.8674416509900000e+00, 1.5652030530244438e+05 },
    { 7.9999999999999995e-11, 2.1580076299700002e+00, 1.7098435436141812e+04 },
    { 7.9999999999999995e-11, 2.6502830697599999e+00, 5.1112492199696091e+04 },
    { 7.9999999999999995e-11, 5.3048607618499997e+00, 3.3084658953804001e+03 },
    { 7.9999999999999995e-11, 1.1827657719000000e-01, 1.8716727548807819e+05 },
    { 7.9999999999999995e-11, 4.4382848778000000e-01, 4.3053034413910001e+02 },
    { 7.9999999999999995e-11, 5.5361594436199999e+00, 1.4928874325784517e+05 },
    { 7.9999999999999995e-11, 5.0360039932899996e+00, 7.7314533815752788e+04 },
    { 7.9999999999999995e-11, 4.5561999130600004e+00, 7.8571267045692191e+04 },
    { 7.9999999999999995e-11, 4.9127231704299996e+00, 1.0287274599223239e+05 },
    { 7.9999999999999995e-11, 2.2835399331200001e+00, 3.2808617120915602e+04 },
    { 7.9999999999999995e-11, 1.7922499617600001e+00, 2.0711415223730978e+05 },
    { 7.9999999999999995e-11, 6.1620182924000000e-01, 4.8997660492580799e+04 },
    { 7.9999999999999995e-11, 1.6244340089500000e+00, 5.7968758699880004e+02 },
    { 7.9999999999999995e-11, 4.3738373292099997e+00, 1.0339532341032618e+05 },
    { 7.9999999999999995e-11, 2.9614940941599999e+00, 1.0381480805420138e+05 },
    { 7.9999999999999995e-11, 2.9861705172500002e+00, 1.0201841617342478e+05 },
    { 7.9999999999999995e-11, 1.9653503623099999e+00, 5.1257875997166600e+04 },
    { 7.9999999999999995e-11, 7.7567524247999997e-01, 5.1861620103278787e+04 },
    { 7.9999999999999995e-11, 3.5855257200700001e+00, 5.1554062482309193e+04 },
    { 7.9999999999999995e-11, 1.6602084844200000e+00, 4.0839885067044002e+03 },
    { 7.9999999999999995e-11, 1.1225715897999999e+00, 5.4190787954946187e+04 },
    { 7.9999999999999995e-11, 2.2164311054999999e-01, 1.2041767842777158e+05 },
    { 7.9999999999999995e-11, 5.8628323064299996e+00, 5.1120356506222292e+04 },
    { 7.9999999999999995e-11, 3.8047888831100001e+00, 5.3258886515441991e+04 },
    { 7.9999999999999995e-11, 3.4926399845299998e+00, 4.2266603761290003e+02 },
    { 7.0000000000000004e-11, 5.4240042026099999e+00, 2.6933985976325399e+04 },
    { 7.0000000000000004e-11, 6.0639579030500004e+00, 5.2206133144152802e+04 },
    { 7.0000000000000004e-11, 4.7215879129999996e+00, 5.0290905119730996e+04 },
    { 7.0000000000000004e-11, 1.1008096119999999e+00, 2.7360584167201388e+04 },
    { 7.0000000000000004e-11, 7.4218080415999999e-01, 5.1086097491954788e+04 },
    { 7.0000000000000004e-11, 3.7044147767000002e+00, 2.3368174972207437e+05 },
    { 7.0000000000000004e-11, 2.3400272085800000e+00, 1.0424851979207818e+05 },
    { 7.0000000000000004e-11, 1.5972962874100001e+00, 2.9396369036954598e+04 },
    { 7.0000000000000004e-11, 4.0663287437999998e+00, 2.2779437246193800e+04 },
    { 7.0000000000000004e-11, 5.9561429560199999e+00, 2.6290156536748291e+04 },
    { 7.0000000000000004e-11, 5.9906673823699998e+00, 2.5885649746400089e+04 },
    { 7.0000000000000004e-11, 3.0379213584200002e+00, 1.5674071794488319e+05 },
    { 7.0000000000000004e-11, 3.1437876300099998e+00, 5.2663441034402589e+04 },
    { 7.0000000000000004e-11, 4.3438210275099998e+00, 9.1793028598180001e+02 },
    { 7.0000000000000004e-11, 4.3580348098999999e-01, 5.2073843764882789e+04 },
    { 7.0000000000000004e-11, 3.6761937172199999e+00, 1.4984619399604559e+05 },
    { 7.0000000000000004e-11, 4.3085664172399998e+00, 7.7417626589971391e+04 },
    { 7.0000000000000004e-11, 2.7683273955999999e-01, 8.1591845081002786e+04 },
    { 7.0000000000000004e-11, 2.9165861520599998e+00, 2.3267071997448002e+04 },
    { 7.0000000000000004e-11, 2.7470388791499998e+00, 2.8908734285700401e+04 },
    { 7.0000000000000004e-11, 5.7011563405599999e+00, 5.2145479422143988e+04 },
    { 7.0000000000000004e-11, 1.5784102312699999e+00, 2.0827662694171758e+05 },
    { 7.0000000000000004e-11, 1.0018751150600000e+00, 8.9894677054323893e+03 },
    { 7.0000000000000004e-11, 1.0826063718100001e+00, 6.3786358241522597e+04 },
    { 7.0000000000000004e-11, 2.6451451499799998e+00, 1.2709890328117118e+05 },
    { 7.0000000000000004e-11, 5.9119122867499998e+00, 1.0297583876645098e+05 },
    { 7.0000000000000004e-11, 1.9686370234699999e+00, 4.5290656393217592e+04 },
    { 7.0000000000000004e-11, 1.6183720287800001e+00, 5.3311483155041598e+04 },
    { 7.0000000000000004e-11, 5.5329169066799997e+00, 1.8023719907218001e+03 },
    { 7.0000000000000004e-11, 5.3285171443700001e+00, 7.9994831777655185e+04 },
    { 7.0000000000000004e-11, 5.0461254554400004e+00, 5.2277768801414000e+04 },
    { 7.0000000000000004e-11, 1.8059324001599999e+00, 2.9580474708443799e+04 },
    { 7.0000000000000004e-11, 3.6791452774400000e+00, 2.7388729959779190e+04 },
    { 7.0000000000000004e-11, 1.9844797537700001e+00, 2.4787076323369201e+04 },
    { 7.0000000000000004e-11, 2.8886975396499999e+00, 1.0412726777059489e+05 },
    { 7.0000000000000004e-11, 3.4091991597999999e-01, 2.7623879884543599e+04 },
    { 7.0000000000000004e-11, 5.3227051152299998e+00, 2.4551926398604792e+04 },
    { 7.0000000000000004e-11, 4.5546735823500004e+00, 1.0767974822257696e+05 },
    { 7.0000000000000004e-11, 2.8465194908500000e+00, 5.2213939318786201e+04 },
    { 7.0000000000000004e-11, 5.5879751626700003e+00, 2.5572439270481191e+04 },
    { 7.0000000000000004e-11, 7.5649868539999998e-02, 2.6603367012667200e+04 },
    { 7.0000000000000004e-11, 5.3841209571600004e+00, 4.1555249061209997e+02 },
    { 7.0000000000000004e-11, 4.3035714957900000e+00, 4.8091817776443997e+04 },
    { 7.0000000000000004e-11, 4.2973397140999996e+00, 2.9746099546109999e+03 },
    { 7.0000000000000004e-11, 1.4150265693499999e+00, 1.8849011634174001e+03 },
    { 7.0000000000000004e-11, 1.3685687400100000e+00, 1.6281049469943656e+05 },
    { 7.0000000000000004e-11, 1.1630048767200001e+00, 3.0110165673538399e+04 },
    { 7.0000000000000004e-11, 2.7805696315900001e+00, 3.4102101344498187e+04 },
    { 7.0000000000000004e-11, 1.7757419461699999e+00, 1.5119994274106238e+05 },
    { 7.0000000000000004e-11, 2.6543590994700001e+00, 1.3675530252076200e+04 },
    { 7.0000000000000004e-11, 4.3752132474799996e+00, 9.7453205558566005e+03 },
    { 7.0000000000000004e-11, 4.8182952438899997e+00, 1.6909357054942797e+05 },
    { 7.0000000000000004e-11, 2.9746347620600000e+00, 2.6413412784722000e+03 },
    { 7.0000000000000004e-11, 4.2360494168400002e+00, 1.5773435424478000e+03 },
    { 7.0000000000000004e-11, 4.4707332572400000e+00, 1.0830802322935999e+03 },
    { 6.0000000000000000e-11, 9.5564679710000000e-01, 5.3132095439118988e+04 },
    { 6.0000000000000000e-11, 5.0779713485800002e+00, 2.7573883262639389e+04 },
    { 6.0000000000000000e-11, 3.8330705923999999e+00, 2.8774625236115400e+04 },
    { 6.0000000000000000e-11, 1.8305544388099999e+00, 2.3401181047032998e+04 },
    { 6.0000000000000000e-11, 1.3893010981000000e+00, 8.0462796768009590e+04 },
    { 6.0000000000000000e-11, 6.7582514581999997e-01, 1.0626281204951399e+05 },
    { 6.0000000000000000e-11, 3.6187064778100000e+00, 2.3704710126797989e+04 },
    { 6.0000000000000000e-11, 2.0449185534000001e+00, 2.8471096156350399e+04 },
    { 6.0000000000000000e-11, 6.0685577963900004e+00, 5.3438963645726588e+04 },
    { 6.0000000000000000e-11, 3.7715024694500001e+00, 1.0647037642667518e+05 },
    { 6.0000000000000000e-11, 6.1160131582500004e+00, 1.9144826611159999e+02 },
    { 6.0000000000000000e-11, 2.2944235380100002e+00, 1.1356768718932001e+03 },
    { 6.0000000000000000e-11, 1.2451435294099999e+00, 2.6867220945412000e+03 },
    { 6.0000000000000000e-11, 3.8428698525699998e+00, 7.8043296782283782e+04 },
    { 6.0000000000000000e-11, 1.7000928922300000e+00, 5.3661096989851598e+04 },
    { 6.0000000000000000e-11, 2.1405293390999999e-01, 1.0442639416486409e+05 },
    { 6.0000000000000000e-11, 5.3109566095099998e+00, 2.6298319799800002e+01 },
    { 6.0000000000000000e-11, 2.6081525757000001e-01, 2.7883161585295191e+04 },
    { 6.0000000000000000e-11, 3.0930720928500000e+00, 1.5652353751408719e+05 },
    { 6.0000000000000000e-11, 1.5247977663100001e+00, 2.2582926841020000e+02 },
    { 6.0000000000000000e-11, 4.6810952498600003e+00, 1.4975670824819960e+05 },
    { 6.0000000000000000e-11, 5.0703298012399998e+00, 1.3000580396999417e+05 },
    { 6.0000000000000000e-11, 2.2123447265700000e+00, 2.7441651886591000e+04 },
    { 6.0000000000000000e-11, 3.4512803046400000e+00, 2.4734154396557398e+04 },
    { 6.0000000000000000e-11, 5.5234478103000004e+00, 1.5546092337225520e+05 },
    { 6.0000000000000000e-11, 4.3863811626900002e+00, 5.0160824611350588e+04 },
    { 6.0000000000000000e-11, 5.3504156279500004e+00, 7.8262224952014280e+04 },
    { 6.0000000000000000e-11, 3.6766239974100001e+00, 2.2286032299359999e+02 },
    { 6.0000000000000000e-11, 3.7632144412700002e+00, 7.8225576389084788e+04 },
    { 6.0000000000000000e-11, 4.6770778322800002e+00, 7.8597565365491988e+04 },
    { 6.0000000000000000e-11, 2.8707862997300002e+00, 9.5979227217800002e+01 },
    { 6.0000000000000000e-11, 5.3657695324299999e+00, 7.6468450981001588e+04 },
    { 6.0000000000000000e-11, 4.4311785186000002e-01, 7.6777729303657383e+04 },
    { 6.0000000000000000e-11, 5.1786331989700001e+00, 1.7129265789271498e+05 },
    { 6.0000000000000000e-11, 4.2010275718900001e+00, 2.9864334027309000e+04 },
    { 6.0000000000000000e-11, 1.4625974593200000e+00, 2.2311472255839399e+04 },
    { 6.0000000000000000e-11, 4.3869498079799998e+00, 7.6991028399095390e+04 },
    { 6.0000000000000000e-11, 1.6936674066299999e+00, 5.1092726050854799e+04 },
    { 6.0000000000000000e-11, 5.5573788915500000e+00, 3.1903014001211191e+04 },
    { 6.0000000000000000e-11, 1.0624613966000000e-01, 2.0272792281937200e+04 },
    { 6.0000000000000000e-11, 2.8608384687499999e+00, 1.0487418998439058e+05 },
    { 6.0000000000000000e-11, 2.1508494679500001e+00, 3.4925715668695998e+03 },
    { 6.0000000000000000e-11, 1.7283739456300000e+00, 7.6137832017343389e+04 },
    { 6.0000000000000000e-11, 4.8497821374000001e+00, 2.5462232949261801e+04 },
    { 6.0000000000000000e-11, 8.1384289380999997e-01, 2.6713573333886590e+04 },
    { 6.0000000000000000e-11, 1.5207702488899999e+00, 1.5653130018480320e+05 },
    { 6.0000000000000000e-11, 3.8714916892900000e+00, 2.8213780548953389e+04 },
    { 6.0000000000000000e-11, 4.8352052538699999e+00, 4.3764389113990001e+02 },
    { 6.0000000000000000e-11, 5.2182273445799998e+00, 6.2174380083920005e+02 },
    { 6.0000000000000000e-11, 4.5614365481699997e+00, 1.0424140624507738e+05 },
    { 6.0000000000000000e-11, 2.3528635265900002e+00, 5.1688171531894201e+04 },
    { 5.0000000000000002e-11, 3.4883091627899998e+00, 5.8220034864523797e+04 },
    { 5.0000000000000002e-11, 6.8949263605000000e-01, 2.7410580789105599e+04 },
    { 5.0000000000000002e-11, 4.9741323951599998e+00, 2.4765225494042799e+04 },
    { 5.0000000000000002e-11, 2.1679588809700001e+00, 1.5547515046625677e+05 },
    { 5.0000000000000002e-11, 6.1411696287900002e+00, 1.2874735003836859e+05 },
    { 5.0000000000000002e-11, 5.0510004491000000e+00, 2.6500274238448590e+04 },
    { 5.0000000000000002e-11, 6.1262458211000004e-01, 2.5675532044699801e+04 },
    { 5.0000000000000002e-11, 5.6919713857199996e+00, 5.3551580082994202e+04 },
    { 5.0000000000000002e-11, 3.4624525969999997e-01, 6.0170334725172601e+04 },
    { 5.0000000000000002e-11, 1.9317169070700000e+00, 5.1933670587978004e+03 },
    { 5.0000000000000002e-11, 6.0565561406799997e+00, 2.7353470620200591e+04 },
    { 5.0000000000000002e-11, 2.0251757845000000e+00, 6.2885158605009997e+02 },
    { 5.0000000000000002e-11, 2.0090403597000001e-01, 9.1235767550174005e+03 },
    { 5.0000000000000002e-11, 2.7009741108199998e+00, 5.2812802555172602e+04 },
    { 5.0000000000000002e-11, 1.1991205146699999e+00, 2.6235981866000500e+04 },
    { 5.0000000000000002e-11, 5.7780383535900004e+00, 7.1378559534791784e+04 },
    { 5.0000000000000002e-11, 4.7375520261800004e+00, 2.6034806989259800e+04 },
    { 5.0000000000000002e-11, 9.2607300503000001e-01, 2.6140999293888592e+04 },
    { 5.0000000000000002e-11, 4.4047672764000001e-01, 1.2434876016784001e+03 },
    { 5.0000000000000002e-11, 4.3693494623499998e+00, 1.0403522069664019e+05 },
    { 5.0000000000000002e-11, 3.7481025689799998e+00, 5.2098055739164491e+04 },
    { 5.0000000000000002e-11, 2.7413281704000001e+00, 6.5538255989947589e+04 },
    { 5.0000000000000002e-11, 4.8832010919800002e+00, 1.0430161594439259e+05 },
    { 5.0000000000000002e-11, 4.8832940470000002e-02, 7.8697421162599392e+04 },
    { 5.0000000000000002e-11, 1.1277607968000000e-01, 1.0440160918820098e+05 },
    { 5.0000000000000002e-11, 3.2519606442999999e-01, 5.3654672857212790e+04 },
    { 5.0000000000000002e-11, 6.1433300142400000e+00, 5.1006218031739787e+04 },
    { 5.0000000000000002e-11, 6.5495415750999997e-01, 1.0466800443595338e+05 },
    { 5.0000000000000002e-11, 5.1699372779599999e+00, 2.3389451694197000e+04 },
    { 5.0000000000000002e-11, 4.9368775325000003e-01, 2.8786354588951399e+04 },
    { 5.0000000000000002e-11, 4.4640342203200003e+00, 1.1254588789950538e+05 },
    { 5.0000000000000002e-11, 6.0962093376000004e+00, 7.8265193897430887e+04 },
    { 5.0000000000000002e-11, 5.0897687033899999e+00, 2.5863558345872290e+04 },
    { 5.0000000000000002e-11, 3.8255290841799998e+00, 4.8763475125420001e+02 },
    { 5.0000000000000002e-11, 4.5995922975300001e+00, 2.5778624818918390e+04 },
    { 5.0000000000000002e-11, 1.0640327336800000e+00, 2.6397181464229991e+04 },
    { 5.0000000000000002e-11, 2.0993062858300000e+00, 1.8985399758261937e+05 },
    { 5.0000000000000002e-11, 4.0228347885500000e+00, 7.8373915745941980e+04 },
    { 5.0000000000000002e-11, 6.0115238757000000e-01, 7.0363318461740005e+02 },
    { 5.0000000000000002e-11, 2.3279879178199998e+00, 2.0156710861598001e+03 },
    { 5.0000000000000002e-11, 5.1244494243399998e+00, 2.6090872086990799e+04 },
    { 5.0000000000000002e-11, 5.3917560686999999e-01, 2.6084934196157588e+04 },
    { 5.0000000000000002e-11, 1.4678722877599999e+00, 5.0951988442369788e+04 },
    { 5.0000000000000002e-11, 8.4223070789999999e-01, 1.3705857113080599e+04 },
    { 5.0000000000000002e-11, 4.8213943233099998e+00, 3.8469949170067790e+04 },
    { 5.0000000000000002e-11, 4.6989617128900001e+00, 1.8227483749135001e+04 },
    { 5.0000000000000002e-11, 9.6466331831999996e-01, 3.3948322534013401e+04 },
    { 5.0000000000000002e-11, 6.0063521661800001e+00, 5.1102197258957589e+04 },
    { 5.0000000000000002e-11, 3.8074470210000002e+00, 5.1315496354395589e+04 },
    { 5.0000000000000002e-11, 5.5349527897200002e+00, 2.4822335662947800e+04 },
    { 5.0000000000000002e-11, 5.0255422082300001e+00, 1.0769222499299659e+05 },
    { 5.0000000000000002e-11, 3.0506929037699999e+00, 4.9637557778894792e+04 },
    { 5.0000000000000002e-11, 3.2266532082800001e+00, 3.5077370847006590e+04 },
    { 5.0000000000000002e-11, 6.3333030207999996e-01, 7.7929853483953193e+04 },
    { 5.0000000000000002e-11, 1.5466614962600000e+00, 1.3002003106399579e+05 },
    { 5.0000000000000002e-11, 1.5089449095999999e-01, 4.5455092303807003e+04 },
    { 5.0000000000000002e-11, 1.3562332779999999e+00, 1.0059506735726200e+04 },
    { 3.9999999999999998e-11, 6.2643746307499999e+00, 7.6532587071789982e+04 },
    { 3.9999999999999998e-11, 1.1351799827800000e+00, 3.1775533510526198e+04 },
    { 3.9999999999999998e-11, 4.5284450484300001e+00, 2.0400272772622200e+04 },
    { 3.9999999999999998e-11, 2.7381324042899999e+00, 3.5211479896591591e+04 },
    { 3.9999999999999998e-11, 2.9254926269200001e+00, 1.6964326386556801e+04 },
    { 3.9999999999999998e-11, 1.0848586011100001e+00, 2.7039979764833792e+04 },
    { 3.9999999999999998e-11, 4.5787664301000000e+00, 2.5135826518314600e+04 },
    { 3.9999999999999998e-11, 1.0424346431000000e+00, 1.9202753251643389e+04 },
    { 3.9999999999999998e-11, 7.5081825828000004e-01, 2.6309778851992876e+05 },
    { 3.9999999999999998e-11, 1.6791709572500000e+00, 3.3480357543659004e+04 },
    { 3.9999999999999998e-11, 3.9844540739599998e+00, 1.8695448739489399e+04 },
    { 3.9999999999999998e-11, 5.1069840536099997e+00, 1.5637785565231059e+05 },
    { 3.9999999999999998e-11, 6.1980953649000003e-01, 7.6998141946096192e+04 },
    { 3.9999999999999998e-11, 3.1901093327700001e+00, 1.3686602528450001e+03 },
    { 3.9999999999999998e-11, 1.2965867040000001e-01, 1.2958631932611899e+05 },
    { 3.9999999999999998e-11, 1.4919450030900001e+00, 4.5424765442802593e+04 },
    { 3.9999999999999998e-11, 5.3987391999399996e+00, 2.8928404046600190e+04 },
    { 3.9999999999999998e-11, 2.6488583126999998e-01, 2.3247402236548201e+04 },
    { 3.9999999999999998e-11, 2.3518805185199998e+00, 2.0452869412221800e+04 },
    { 3.9999999999999998e-11, 3.3117445126899998e+00, 3.1722936870926598e+04 },
    { 3.9999999999999998e-11, 2.3033178454900001e+00, 2.6238950811417089e+04 },
    { 3.9999999999999998e-11, 2.7783630078299999e+00, 5.2355883413433003e+04 },
    { 3.9999999999999998e-11, 3.4634238213300002e+00, 5.1538810011124187e+04 },
    { 3.9999999999999998e-11, 3.6994459080500000e+00, 3.2278926155820998e+04 },
    { 3.9999999999999998e-11, 1.9641791231600001e+00, 1.9896880127327400e+04 },
    { 3.9999999999999998e-11, 6.0349910643499998e+00, 2.3976252828196601e+04 },
    { 3.9999999999999998e-11, 1.7018802699000000e+00, 7.8905122986461574e+04 },
    { 3.9999999999999998e-11, 1.6650944477000000e+00, 8.7648550432797783e+04 },
    { 3.9999999999999998e-11, 4.4194912376300000e+00, 2.2318057631579356e+05 },
    { 3.9999999999999998e-11, 3.0007102196600002e+00, 5.2336213652533203e+04 },
    { 3.9999999999999998e-11, 5.6867601433899999e+00, 5.0912648920570202e+04 },
    { 3.9999999999999998e-11, 1.9655785900500000e+00, 1.9959931813847594e+05 },
    { 3.9999999999999998e-11, 3.3471183432400000e+00, 2.5923467230984788e+04 },
    { 3.9999999999999998e-11, 2.3165066879700000e+00, 2.6252339052163610e+04 },
    { 3.9999999999999998e-11, 4.3582033971299996e+00, 7.4935573768442395e+04 },
    { 3.9999999999999998e-11, 3.7589110708000001e-01, 2.6130786216477602e+04 },
    { 3.9999999999999998e-11, 5.2877339241300003e+00, 2.6045020066670801e+04 },
    { 3.9999999999999998e-11, 1.4847947795300001e+00, 1.0477109721017198e+05 },
    { 3.9999999999999998e-11, 7.5542038962000002e-01, 6.1165273988580790e+04 },
    { 3.9999999999999998e-11, 1.0703576963200001e+00, 2.4079345602415189e+04 },
    { 3.9999999999999998e-11, 3.3402545856799999e+00, 1.8319536584879599e+04 },
    { 3.9999999999999998e-11, 2.3233704455300002e+00, 3.3856269698268799e+04 },
    { 3.9999999999999998e-11, 3.2643869095800002e+00, 1.5996999379441058e+05 },
    { 3.9999999999999998e-11, 3.4412921882999998e+00, 7.3891833057948388e+04 },
    { 3.9999999999999998e-11, 3.7061702201100002e+00, 1.0668480895916879e+05 },
    { 3.9999999999999998e-11, 2.8732100738000002e+00, 2.5248933853823801e+04 },
    { 3.9999999999999998e-11, 2.7904149574099999e+00, 2.6926872429324590e+04 },
    { 3.9999999999999998e-11, 1.7396543981699999e+00, 4.2430485727291802e+04 },
    { 3.9999999999999998e-11, 3.2501801509599999e+00, 1.3090748069822539e+05 },
    { 3.9999999999999998e-11, 2.0635310669300000e+00, 7.6352509941505385e+04 },
    { 3.9999999999999998e-11, 3.8203911186499999e+00, 5.0690515576445192e+04 },
    { 3.9999999999999998e-11, 4.0278258619000001e-01, 2.7047026001531802e+04 },
    { 3.9999999999999998e-11, 5.2608424450199998e+00, 2.5128780281616589e+04 },
    { 3.9999999999999998e-11, 4.6356179400700004e+00, 5.2164760582884490e+04 },
    { 3.9999999999999998e-11, 6.1345975730599998e+00, 1.1882896374949679e+05 },
    { 3.9999999999999998e-11, 2.5561088409899999e+00, 2.1594190072419358e+05 },
    { 3.9999999999999998e-11, 5.4198688463200000e+00, 2.5991923914356390e+04 },
    { 3.9999999999999998e-11, 2.4375618489000000e-01, 2.6183882368791990e+04 },
    { 3.9999999999999998e-11, 4.3539187888399997e+00, 4.8153543751184188e+04 },
    { 3.9999999999999998e-11, 4.4819649651999999e-01, 1.6496361396202399e+04 },
    { 3.9999999999999998e-11, 5.2154285346900000e+00, 3.5679444886945988e+04 },
    { 3.9999999999999998e-11, 4.5944305539999997e+00, 8.0445250639284004e+03 },
    { 3.9999999999999998e-11, 1.9164109085400001e+00, 4.9424258683456792e+04 },
    { 3.9999999999999998e-11, 3.8876202502299999e+00, 5.1040129411255199e+04 },
    { 3.9999999999999998e-11, 2.4614554053100002e+00, 1.2884332926558638e+05 },
    { 3.9999999999999998e-11, 5.3951820045499996e+00, 2.3336355541882600e+04 },
    { 3.9999999999999998e-11, 2.4812187167699999e+00, 5.2015398913763587e+04 },
    { 3.9999999999999998e-11, 1.9979754653099999e+00, 2.5177512442162435e+05 },
    { 3.9999999999999998e-11, 7.8738841944000004e-01, 2.3543230504681789e+04 },
    { 3.9999999999999998e-11, 4.8762366117699996e+00, 2.8632575778466598e+04 },
    { 3.9999999999999998e-11, 4.1768657295900002e+00, 2.5998417393728199e+04 },
    { 3.9999999999999998e-11, 1.4867593016200000e+00, 2.6177388889420188e+04 },
    { 3.9999999999999998e-11, 6.1497272153100004e+00, 3.2217200181080800e+04 },
    { 3.9999999999999998e-11, 4.3344689130000003e-02, 1.4235548078175199e+04 },
    { 3.9999999999999998e-11, 5.6202803420800000e+00, 3.7940258204973186e+04 },
    { 3.9999999999999998e-11, 3.7228917821500001e+00, 2.6198840996783535e+05 },
    { 3.9999999999999998e-11, 3.0383956937800001e+00, 1.6895946149984296e+05 },
    { 3.9999999999999998e-11, 2.7035659285999998e+00, 5.1995729152863802e+04 },
    { 3.9999999999999998e-11, 1.1384488221600000e+00, 1.7917274704605396e+05 },
    { 3.9999999999999998e-11, 2.9292521588699998e+00, 2.4182438376633800e+04 },
    { 3.9999999999999998e-11, 1.8415160488900000e+00, 2.3962025734194991e+04 },
    { 3.9999999999999998e-11, 3.0543925024099998e+00, 1.5695401704032117e+05 },
    { 3.9999999999999998e-11, 5.7321218219899999e+00, 1.9282860753723036e+05 },
    { 3.9999999999999998e-11, 2.9131403039999998e-01, 5.0476527066645191e+04 },
    { 3.9999999999999998e-11, 3.1671476557300000e+00, 2.5014294117383390e+04 },
    { 3.9999999999999998e-11, 5.2793725339800002e+00, 2.7511467873537200e+04 },
    { 3.9999999999999998e-11, 3.8425249722999999e-01, 2.4664338409611199e+04 },
    { 3.9999999999999998e-11, 3.7638145811500001e+00, 6.1921126839004988e+04 },
    { 3.9999999999999998e-11, 3.0598171426500000e+00, 1.5811649174472899e+05 },
    { 3.9999999999999998e-11, 1.0577947981300000e+00, 2.5810868147832789e+04 },
    { 3.9999999999999998e-11, 4.6058302330799998e+00, 2.6364938135315599e+04 },
    { 3.0000000000000000e-11, 5.6891268218600004e+00, 7.6094948942439994e+04 },
    { 3.0000000000000000e-11, 4.4450292683499999e+00, 2.5980597834644592e+04 },
    { 3.0000000000000000e-11, 1.2185957628599999e+00, 2.6195208448503799e+04 },
    { 3.0000000000000000e-11, 3.1669082378900000e+00, 2.0225339517410001e+02 },
    { 3.0000000000000000e-11, 1.3501794324900001e+00, 2.3113293186963201e+04 },
    { 3.0000000000000000e-11, 4.3134455987200004e+00, 2.9062513096185201e+04 },
    { 3.0000000000000000e-11, 1.3992300676000000e-01, 2.5794492357196999e+04 },
    { 3.0000000000000000e-11, 5.5237020244500004e+00, 2.6381313925951399e+04 },
    { 3.0000000000000000e-11, 2.3467733512600000e+00, 2.6279351407685790e+04 },
    { 3.0000000000000000e-11, 3.3168516799500001e+00, 2.5896454875462601e+04 },
    { 3.0000000000000000e-11, 2.8372207946199999e+00, 4.9850856874332792e+04 },
    { 3.0000000000000000e-11, 3.1975149788900001e+00, 4.1494555746582992e+04 },
    { 3.0000000000000000e-11, 8.9067609208999998e-01, 1.5430866174525840e+05 },
    { 3.0000000000000000e-11, 5.6513686153000000e+00, 2.4292644697853189e+04 },
    { 3.0000000000000000e-11, 1.9486754591000000e-01, 7.8397818474307584e+04 },
    { 3.0000000000000000e-11, 3.2221843840800002e+00, 2.0242014928260216e+05 },
    { 3.0000000000000000e-11, 4.8533678064199997e+00, 7.9116905806474591e+04 },
    { 3.0000000000000000e-11, 2.2685292343099999e+00, 1.4853289040742096e+05 },
    { 3.0000000000000000e-11, 1.8234862858500001e+00, 2.6114201461373999e+04 },
    { 3.0000000000000000e-11, 3.8401387453600000e+00, 2.6061604821774388e+04 },
    { 3.0000000000000000e-11, 5.3248028171500001e+00, 4.7803929916374203e+04 },
    { 3.0000000000000000e-11, 3.6775623555500001e+00, 9.9979736199522595e+04 },
    { 3.0000000000000000e-11, 3.1107791631300001e+00, 2.7993367906514599e+04 },
    { 3.0000000000000000e-11, 3.9307779118799999e+00, 1.3044614426677099e+05 },
    { 3.0000000000000000e-11, 2.9895063710999997e-01, 2.5344913081041599e+04 },
    { 3.0000000000000000e-11, 3.6548906016200000e+00, 2.5764397724916791e+04 },
    { 3.0000000000000000e-11, 1.4367034299000001e+00, 2.3420850807932791e+04 },
    { 3.0000000000000000e-11, 4.2269216013099999e+00, 2.8754955475215600e+04 },
    { 3.0000000000000000e-11, 3.2909771124199998e+00, 2.2909757351006599e+04 },
    { 3.0000000000000000e-11, 4.8889443655299996e+00, 1.3376765136415116e+05 },
    { 3.0000000000000000e-11, 4.2145788401699997e+00, 5.2072024094567802e+04 },
    { 3.0000000000000000e-11, 3.6179232462300002e+00, 7.7622295862983592e+04 },
    { 3.0000000000000000e-11, 4.7995189314999998e-01, 5.2016504548356199e+04 },
    { 3.0000000000000000e-11, 2.6581023345600001e+00, 2.7331107406613970e+05 },
    { 3.0000000000000000e-11, 1.6876318583600001e+00, 2.5970583273353990e+04 },
    { 3.0000000000000000e-11, 3.9759931728500000e+00, 2.6205223009794390e+04 },
    { 3.0000000000000000e-11, 3.0844630585999999e+00, 5.4879422437824003e+04 },
    { 3.0000000000000000e-11, 2.3003275414800002e+00, 3.3917995673008991e+04 },
    { 3.0000000000000000e-11, 3.3632974897299999e+00, 1.8257810610139401e+04 },
    { 3.0000000000000000e-11, 2.8833562508599999e+00, 2.2222497657418498e+05 },
    { 3.0000000000000000e-11, 1.1589191101900000e+00, 7.8249482330720988e+04 },
    { 3.0000000000000000e-11, 5.9602473023600000e+00, 1.2355436581794015e+05 },
    { 3.0000000000000000e-11, 1.8833858288500001e+00, 2.6387029535843401e+04 },
    { 3.0000000000000000e-11, 5.8236000505899996e+00, 2.5440892308259390e+04 },
    { 3.0000000000000000e-11, 5.0479602474199998e+00, 2.6983982598229599e+04 },
    { 3.0000000000000000e-11, 6.1566478379000000e-01, 2.5191823684918800e+04 },
    { 3.0000000000000000e-11, 7.4835060715000001e-01, 7.9487527265501194e+04 },
    { 3.0000000000000000e-11, 1.6964267850100001e+00, 2.1092198223678037e+05 },
    { 3.0000000000000000e-11, 5.0518757577999995e-01, 1.2631573625781900e+03 },
    { 3.0000000000000000e-11, 3.1195994986300000e+00, 5.0490754160646793e+04 },
    { 3.0000000000000000e-11, 2.2731451067199999e+00, 7.3297125859000005e+01 },
    { 3.0000000000000000e-11, 4.6966753876400000e+00, 5.5484272178528787e+04 },
    { 3.0000000000000000e-11, 1.3977938123300000e+00, 1.3828873468465999e+03 },
    { 3.0000000000000000e-11, 3.5517816901999999e+00, 1.5843861833266239e+05 },
    { 3.0000000000000000e-11, 1.1094558535200001e+00, 2.4601923020508992e+04 },
    { 3.0000000000000000e-11, 7.6074196321999998e-01, 2.4298540360737636e+05 },
    { 3.0000000000000000e-11, 7.0803012799999998e-01, 1.4717530270636000e+03 },
    { 3.0000000000000000e-11, 4.7238629214600003e+00, 1.4650558156934578e+05 },
    { 3.0000000000000000e-11, 5.0449337878799998e+00, 2.4510559599126402e+04 },
    { 3.0000000000000000e-11, 6.1869124333000003e-01, 2.7665246684022000e+04 },
    { 3.0000000000000000e-11, 4.8167243915500002e+00, 2.2946365216578494e+05 },
    { 3.0000000000000000e-11, 3.4973661856399998e+00, 5.0600960481866197e+04 },
    { 3.0000000000000000e-11, 1.0398634094899999e+00, 1.8594345764729957e+05 },
    { 3.0000000000000000e-11, 6.9061740154999995e-01, 1.5406652605008790e+04 },
    { 3.0000000000000000e-11, 2.9778988754000002e-01, 1.7996686716554637e+05 },
    { 3.0000000000000000e-11, 1.3162472363100000e+00, 2.6202302248487598e+04 },
    { 3.0000000000000000e-11, 4.9248533806300001e+00, 2.4815222115946999e+04 },
    { 3.0000000000000000e-11, 6.0305414740999996e-01, 9.6340270297140000e+02 },
    { 3.0000000000000000e-11, 1.9403381822600001e+00, 2.0337574902421076e+05 },
    { 3.0000000000000000e-11, 3.4809585062799999e+00, 2.3528661537717999e+03 },
    { 3.0000000000000000e-11, 3.8363072728200001e+00, 2.8918569166150293e+05 },
    { 3.0000000000000000e-11, 1.4267317354400000e+00, 2.9000787121444999e+04 },
    { 3.0000000000000000e-11, 4.2368932957699998e+00, 2.3175019161703389e+04 },
    { 3.0000000000000000e-11, 2.7487800522799999e+00, 2.8208311441261999e+03 },
    { 3.0000000000000000e-11, 1.3590812879200000e+00, 4.9201196328537400e+04 },
    { 3.0000000000000000e-11, 2.3867323215699998e+00, 5.1734980998270788e+04 },
    { 3.0000000000000000e-11, 2.8453213270200002e+00, 2.5914454570189610e+04 },
    { 3.0000000000000000e-11, 2.8183037041899999e+00, 2.6261351712958789e+04 },
    { 3.0000000000000000e-11, 5.8207278511500000e+00, 1.4316496914264598e+05 },
    { 3.0000000000000000e-11, 5.2204168700700002e+00, 5.1745275939009291e+04 },
    { 3.0000000000000000e-11, 5.2075347864400001e+00, 2.5530452403373802e+04 },
    { 3.0000000000000000e-11, 4.5609024476999999e-01, 2.6645353879774601e+04 },
    { 3.0000000000000000e-11, 5.0356580859699998e+00, 2.5626566710119801e+04 },
    { 3.0000000000000000e-11, 6.2796694523999996e-01, 2.6549239573028590e+04 },
    { 3.0000000000000000e-11, 5.3436157772800001e+00, 2.0956262057516600e+04 },
    { 3.0000000000000000e-11, 3.2000925393000001e-01, 3.1219544225631798e+04 },
    { 3.0000000000000000e-11, 4.4413352343500003e+00, 2.8352435961235070e+05 },
    { 3.0000000000000000e-11, 2.4844359093900001e+00, 7.6358244659782184e+04 },
    { 3.0000000000000000e-11, 1.8789373272800001e+00, 5.4391805909619995e+02 },
    { 3.0000000000000000e-11, 8.3289676584000005e-01, 2.5361421121703999e+04 },
    { 3.0000000000000000e-11, 4.8307282653700003e+00, 2.6814385161444388e+04 },
    { 3.0000000000000000e-11, 5.4152553038000004e+00, 5.2186851983412300e+04 },
    { 3.0000000000000000e-11, 4.7312814921099999e+00, 2.1650388933200611e+04 },
    { 3.0000000000000000e-11, 9.3234353910000001e-01, 3.0525417349947791e+04 },
    { 3.0000000000000000e-11, 6.0673014610999996e+00, 1.2885755635958798e+05 },
    { 3.0000000000000000e-11, 4.3440266037199997e+00, 5.3338280987556187e+04 },
    { 3.0000000000000000e-11, 3.8205525875599999e+00, 2.1716026774800001e+04 },
    { 3.0000000000000000e-11, 3.3967643008200001e+00, 2.4971411042479991e+04 },
    { 3.0000000000000000e-11, 2.2668607303899999e+00, 2.7204395240668400e+04 },
    { 3.0000000000000000e-11, 3.1280988629399999e+00, 2.0658446127221518e+05 },
    { 3.0000000000000000e-11, 2.9720068764400001e+00, 1.8938603259226496e+05 },
    { 3.0000000000000000e-11, 2.6882400316799999e+00, 2.7161512165765002e+04 },
    { 3.0000000000000000e-11, 8.6347662918000001e-01, 4.9976718939861392e+04 },
    { 3.0000000000000000e-11, 1.6122935027600001e+00, 2.9050783743349199e+04 },
    { 3.0000000000000000e-11, 4.0513315284500004e+00, 2.3125022539799200e+04 },
    { 3.0000000000000000e-11, 2.5834751524900001e+00, 2.4611379237247215e+05 },
    { 3.0000000000000000e-11, 6.0146850161299996e+00, 7.7683665566946001e+03 },
    { 3.0000000000000000e-11, 4.3065432223600002e+00, 7.8115630700296286e+04 },
    { 3.0000000000000000e-11, 3.0513492076700000e+00, 5.8151108596369904e+03 },
    { 3.0000000000000000e-11, 3.5333837496999998e-01, 1.1731986822019999e+02 },
    { 3.0000000000000000e-11, 3.3116945691400002e+00, 7.8112661754879678e+04 },
    { 3.0000000000000000e-11, 4.3348211085999999e-01, 1.3410904958500001e+02 },
    { 3.0000000000000000e-11, 4.3072419303500000e+00, 4.2116299547422190e+04 },
    { 3.0000000000000000e-11, 5.8820711018800003e+00, 2.5949385644703500e+04 },
    { 3.0000000000000000e-11, 6.0647392365100004e+00, 2.6226420638444899e+04 },
    { 3.0000000000000000e-11, 2.0730336242599998e+00, 6.5851336535224589e+04 },
    { 3.0000000000000000e-11, 1.9032612200500001e+00, 2.6375791001644000e+04 },
    { 3.0000000000000000e-11, 3.7603638111600000e+00, 2.5800015281504398e+04 },
    { 3.0000000000000000e-11, 3.7979601583200000e+00, 2.3581258177317599e+04 },
    { 3.0000000000000000e-11, 1.8656648728900000e+00, 2.8594548105830800e+04 },
    { 3.0000000000000000e-11, 4.2295866043299997e+00, 2.6051254578644690e+04 },
    { 3.0000000000000000e-11, 1.4340384268799999e+00, 2.6124551704503690e+04 },
    { 3.0000000000000000e-11, 2.1716032615800001e+00, 5.1753140245535491e+04 },
    { 3.0000000000000000e-11, 6.1519942292499996e+00, 2.5632993775046889e+04 },
    { 3.0000000000000000e-11, 5.7948161091400001e+00, 2.6542812508101491e+04 },
    { 1.9999999999999999e-11, 6.2503756597800004e+00, 5.0199184874131997e+03 },
    { 1.9999999999999999e-11, 2.0136750077899999e+00, 2.7897240611529193e+05 },
    { 1.9999999999999999e-11, 3.0356616382500001e+00, 2.5554280023216488e+04 },
    { 1.9999999999999999e-11, 2.6279633929599999e+00, 2.6621526259931888e+04 },
    { 1.9999999999999999e-11, 4.5893182736000000e+00, 2.1202093703745999e+04 },
    { 1.9999999999999999e-11, 1.0743067576100001e+00, 3.0973712579402400e+04 },
    { 1.9999999999999999e-11, 1.4093886624900001e+00, 2.5743645647077652e+05 },
    { 1.9999999999999999e-11, 1.2449771770800000e+00, 5.7369173341946189e+04 },
    { 1.9999999999999999e-11, 2.6362096880100001e+00, 3.1127491389887189e+04 },
    { 1.9999999999999999e-11, 3.0274153431999999e+00, 2.1048314893261198e+04 },
    { 1.9999999999999999e-11, 6.0830550269999999e-02, 1.5215554248267098e+05 },
    { 1.9999999999999999e-11, 1.6837305459999999e-01, 2.7133057977761790e+04 },
    { 1.9999999999999999e-11, 3.2369088391999998e+00, 2.6764974201698753e+05 },
    { 1.9999999999999999e-11, 2.1816983609600000e+00, 2.9266048932141799e+04 },
    { 1.9999999999999999e-11, 7.1621395615000005e-01, 5.6350337293523999e+03 },
    { 1.9999999999999999e-11, 2.1160238297200000e+00, 1.3831960486120995e+05 },
    { 1.9999999999999999e-11, 3.2523055726500001e+00, 1.0212975593169978e+05 },
    { 1.9999999999999999e-11, 4.8228399968000000e+00, 5.6876303689520000e+03 },
    { 1.9999999999999999e-11, 6.3318713248000003e-01, 5.4862528377689603e+04 },
    { 1.9999999999999999e-11, 3.6727925688799998e+00, 1.4398093691036216e+05 },
    { 1.9999999999999999e-11, 2.2745373307499999e+00, 2.2002588923089797e+05 },
    { 1.9999999999999999e-11, 3.1877938400000000e-02, 6.7608757207956980e+04 },
    { 1.9999999999999999e-11, 5.0181781300699999e+00, 2.6165653685558089e+04 },
    { 1.9999999999999999e-11, 3.9913665768000001e-01, 2.4156183887541335e+05 },
    { 1.9999999999999999e-11, 3.6962686583999999e-01, 2.3670232775738495e+05 },
    { 1.9999999999999999e-11, 5.2423950714499998e+00, 1.8049655813064100e+05 },
    { 1.9999999999999999e-11, 6.1245748376700000e+00, 2.7151217225026488e+04 },
    { 1.9999999999999999e-11, 3.4207163980000002e+00, 3.2973053031505013e+04 },
    { 1.9999999999999999e-11, 1.5742783993600000e+00, 1.1039584114767218e+05 },
    { 1.9999999999999999e-11, 6.4120353128999996e-01, 7.8484122067161385e+04 },
    { 1.9999999999999999e-11, 1.3220595262000001e-01, 5.0373434292426587e+04 },
    { 1.9999999999999999e-11, 4.6042924320000000e-01, 1.5653453239644598e+05 },
    { 1.9999999999999999e-11, 2.8012264229000000e-01, 5.1760253792536292e+04 },
    { 1.9999999999999999e-11, 5.1260360416099999e+00, 3.0486077828148202e+04 },
    { 1.9999999999999999e-11, 5.3758898960000001e-01, 2.1689728455000190e+04 },
    { 1.9999999999999999e-11, 4.3657285579099998e+00, 1.8983512179396000e+03 },
    { 1.9999999999999999e-11, 5.2127727581999999e+00, 2.3574672801577637e+05 },
    { 1.9999999999999999e-11, 9.4406129310999998e-01, 2.6024167243270800e+04 },
    { 1.9999999999999999e-11, 4.7195637381099997e+00, 2.6151639039877598e+04 },
    { 1.9999999999999999e-11, 4.1409568144700000e+00, 2.1689750046580215e+05 },
    { 1.9999999999999999e-11, 1.1708192255800001e+00, 2.4926847945736776e+05 },
    { 1.9999999999999999e-11, 3.5577683982599999e+00, 1.5930050485469001e+03 },
    { 1.9999999999999999e-11, 3.3350074548199999e+00, 4.5256551772590001e+03 },
    { 1.9999999999999999e-11, 5.9962756472300001e+00, 2.4722317092456558e+05 },
    { 1.9999999999999999e-11, 5.1412049520000003e-02, 9.7466462676365962e+04 },
    { 1.9999999999999999e-11, 2.2454827199600000e+00, 4.2590877651400001e+02 },
    { 1.9999999999999999e-11, 4.1829675040999997e-01, 1.2797945726280000e+03 },
    { 1.9999999999999999e-11, 5.4857688240800000e+00, 5.2367254549259989e+04 },
    { 1.9999999999999999e-11, 4.1405659731300002e+00, 5.1949977014738201e+04 },
    { 1.9999999999999999e-11, 2.5594050330200000e+00, 2.8133211674376402e+04 },
    { 1.9999999999999999e-11, 3.1042199981900001e+00, 2.4042594608771989e+04 },
    { 1.9999999999999999e-11, 3.9751275783300000e+00, 2.4999299984973200e+04 },
    { 1.9999999999999999e-11, 1.6884974528800001e+00, 2.7176506298175191e+04 },
    { 1.9999999999999999e-11, 1.1691062351899999e+00, 1.9049541114435834e+05 },
    { 1.9999999999999999e-11, 5.1272792013100004e+00, 4.3981746865740097e+03 },
    { 1.9999999999999999e-11, 2.9120234714000000e+00, 2.7143352918500292e+04 },
    { 1.9999999999999999e-11, 3.0369055406499998e+00, 2.5024589058121888e+04 },
    { 1.9999999999999999e-11, 5.8217114862899999e+00, 1.9504736464141717e+05 },
    { 1.9999999999999999e-11, 2.3421279242200002e+00, 2.8546403739102002e+03 },
    { 1.9999999999999999e-11, 1.5191722367000000e+00, 2.0618019358533598e+04 },
    { 1.9999999999999999e-11, 4.1444527945100003e+00, 3.1557786924614789e+04 },
    { 1.9999999999999999e-11, 1.9883128525400000e+00, 5.1952945960154793e+04 },
    { 1.9999999999999999e-11, 6.2184328993399998e+00, 4.1520854066382803e+04 },
    { 1.9999999999999999e-11, 1.5718304467399999e+00, 5.2249103409007388e+04 },
    { 1.9999999999999999e-11, 3.3865138034500002e+00, 1.2606763934109679e+05 },
    { 1.9999999999999999e-11, 4.7143662252200000e+00, 5.2202104602948202e+04 },
    { 1.9999999999999999e-11, 5.8386244640999996e+00, 4.6019501189630000e+03 },
    { 1.9999999999999999e-11, 3.6716240322200000e+00, 3.3061896365820002e+02 },
    { 1.9999999999999999e-11, 1.3421869369599999e+00, 1.3863379104107959e+05 },
    { 1.9999999999999999e-11, 4.9292660619899999e+00, 2.3134855332920235e+05 },
    { 1.9999999999999999e-11, 2.4910956678599998e+00, 5.2822817116463193e+04 },
    { 1.9999999999999999e-11, 3.6321728961100002e+00, 1.5359767429694000e+03 },
    { 1.9999999999999999e-11, 4.7130483356799999e+00, 2.3245793188129575e+05 },
    { 1.9999999999999999e-11, 7.9025165000999997e-01, 2.4705705209089589e+04 },
    { 1.9999999999999999e-11, 4.8733733812000004e+00, 2.7470101074058799e+04 },
    { 1.9999999999999999e-11, 1.3351259420999999e+00, 2.6555683724738001e+03 },
    { 1.9999999999999999e-11, 1.2163545664000000e-01, 1.6032999892854000e+03 },
    { 1.9999999999999999e-11, 5.7150479927199997e+00, 3.0459779508348402e+04 },
    { 1.9999999999999999e-11, 6.1499950419699996e+00, 1.3537487450168001e+03 },
    { 1.9999999999999999e-11, 4.1989753260800002e+00, 3.2838943981919998e+04 },
    { 1.9999999999999999e-11, 1.3374340654000000e-01, 1.4297274052915400e+04 },
    { 1.9999999999999999e-11, 5.5298816246699998e+00, 3.7878532230233002e+04 },
    { 1.9999999999999999e-11, 6.2054288676500002e+00, 1.0450117576343138e+05 },
    { 1.9999999999999999e-11, 2.1559661096700000e+00, 2.5446726368924001e+03 },
    { 1.9999999999999999e-11, 1.0639535169300001e+00, 4.4131281219219993e+04 },
    { 1.9999999999999999e-11, 2.1301321066200001e+00, 7.7039891583943958e+04 },
    { 1.9999999999999999e-11, 1.1584451506300000e+00, 4.5782518168586003e+03 },
    { 1.9999999999999999e-11, 3.6395866830800001e+00, 5.1546954697098292e+04 },
    { 1.9999999999999999e-11, 3.6373802844699998e+00, 1.6440750800278416e+05 },
    { 1.9999999999999999e-11, 5.4269614553799999e+00, 7.3924544020847998e+03 },
    { 1.9999999999999999e-11, 6.2577504672700002e+00, 3.1107821628987400e+04 },
    { 1.9999999999999999e-11, 5.6890598711199996e+00, 2.1067984654160991e+04 },
    { 1.9999999999999999e-11, 2.9175659320500000e+00, 1.1345551956041438e+05 },
    { 1.9999999999999999e-11, 1.9857084746800000e+00, 2.7299244211933201e+04 },
    { 1.9999999999999999e-11, 3.6779165565300000e+00, 2.4876562071215201e+04 },
    { 1.9999999999999999e-11, 5.7429870318200003e+00, 1.0541810804348679e+05 },
    { 1.9999999999999999e-11, 2.6909961739799999e+00, 4.9534465004676204e+04 },
    { 1.9999999999999999e-11, 1.8070825507899999e+00, 2.4957749193988398e+04 },
    { 1.9999999999999999e-11, 3.8565424804199999e+00, 2.7218057089160000e+04 },
    { 1.9999999999999999e-11, 3.1167720069699998e+00, 2.7257491392982800e+04 },
    { 1.9999999999999999e-11, 3.4939973486000002e+00, 4.0738702896158793e+04 },
    { 1.9999999999999999e-11, 2.1234042745000001e-01, 3.0927832265580003e+02 },
    { 1.9999999999999999e-11, 2.3378178794900002e+00, 1.1553611574070001e+03 },
    { 1.9999999999999999e-11, 3.0324833833099998e+00, 1.0223996225291918e+05 },
    { 1.9999999999999999e-11, 1.6582857526499999e+00, 1.9709267317421938e+05 },
    { 1.9999999999999999e-11, 5.5661011791900004e+00, 2.0235122826310399e+04 },
    { 1.9999999999999999e-11, 9.7523852019999996e-02, 3.1940683456837989e+04 },
    { 1.9999999999999999e-11, 5.2879995877399999e+00, 2.2568722128005017e+05 },
    { 1.9999999999999999e-11, 3.4521072462000002e-01, 1.5851407420206999e+03 },
    { 1.9999999999999999e-11, 8.9847733803999996e-01, 1.3043288714897096e+05 },
    { 1.9999999999999999e-11, 5.3529504736800000e+00, 7.9955875094224975e+04 },
    { 1.9999999999999999e-11, 1.5743392494000000e-01, 8.3896928775039999e+02 },
    { 1.9999999999999999e-11, 2.8634966530299999e+00, 1.7196036977425378e+05 },
    { 1.9999999999999999e-11, 8.2971507937000000e-01, 5.1738162392008489e+04 },
    { 1.9999999999999999e-11, 4.4088685718100002e+00, 2.7953134544091801e+04 },
    { 1.9999999999999999e-11, 1.2547564594000000e+00, 2.4222671739056601e+04 },
    { 1.9999999999999999e-11, 5.6714244988800004e+00, 2.9939897720771394e+05 },
    { 1.9999999999999999e-11, 2.9639942930000000e-02, 2.8440769295345999e+04 },
    { 1.9999999999999999e-11, 5.6339850882800002e+00, 2.3735036987802388e+04 },
    { 1.9999999999999999e-11, 4.9405278696600003e+00, 2.6948213070327001e+04 },
    { 1.9999999999999999e-11, 7.2309716155000003e-01, 2.5227593212821390e+04 },
    { 1.9999999999999999e-11, 4.5138652293500003e+00, 1.6472169418265377e+05 },
    { 1.9999999999999999e-11, 2.1954394707299998e+00, 2.4918314890165591e+04 },
    { 1.9999999999999999e-11, 2.1285811066600000e+00, 2.5888831140137801e+04 },
    { 1.9999999999999999e-11, 3.5350439245500000e+00, 2.6286975143010590e+04 },
    { 1.9999999999999999e-11, 5.0541465511199997e+00, 4.2680941781935999e+03 },
    { 1.9999999999999999e-11, 5.7167202865800002e+00, 8.0432469907005187e+04 },
    { 1.9999999999999999e-11, 2.9162943835399999e+00, 1.3989716431945801e+04 },
    { 1.9999999999999999e-11, 2.7473306476700001e+00, 3.8186089851202603e+04 },
    { 1.9999999999999999e-11, 2.5531405364299999e+00, 1.2466374437003357e+05 },
    { 1.9999999999999999e-11, 3.2737086693699999e+00, 5.2575881183150000e+02 },
    { 1.9999999999999999e-11, 4.3511157175799999e+00, 3.7764308857348001e+03 },
    { 1.9999999999999999e-11, 2.9354111099200000e+00, 2.0453085328022000e+03 },
    { 1.9999999999999999e-11, 2.8110386695899998e+00, 2.2113526778299137e+05 },
    { 1.9999999999999999e-11, 7.9461762448999995e-01, 7.6064622081435577e+04 },
    { 1.9999999999999999e-11, 2.5323225356600001e+00, 1.5075164751160776e+05 },
    { 1.9999999999999999e-11, 1.4745731173000001e+00, 3.0831304904744600e+04 },
    { 1.9999999999999999e-11, 4.1890519139200002e+00, 2.1344501378403791e+04 },
    { 1.9999999999999999e-11, 2.9255361492000000e+00, 1.0540388094948517e+05 },
    { 1.9999999999999999e-11, 5.9897854840600004e+00, 1.9336862301228390e+04 },
    { 1.9999999999999999e-11, 6.0882634417499997e+00, 7.8300925314347996e+03 },
    { 1.9999999999999999e-11, 5.2384439474500004e+00, 8.4821631932784003e+03 },
    { 1.9999999999999999e-11, 4.2518108376000002e-01, 4.3693643089869991e+04 },
    { 1.9999999999999999e-11, 2.6216784699800000e+00, 9.5915417453718001e+03 },
    { 1.9999999999999999e-11, 1.2884993754100000e+00, 8.7253177130154960e+04 },
    { 1.9999999999999999e-11, 5.4917422032300003e+00, 2.1516279883615600e+04 },
    { 1.9999999999999999e-11, 1.7188282798000001e-01, 3.0659526399532791e+04 },
    { 1.9999999999999999e-11, 4.1087217447900004e+00, 2.2147430875962000e+03 },
    { 1.9999999999999999e-11, 3.3446329497999999e-01, 2.8839450741265791e+04 },
    { 1.9999999999999999e-11, 3.4805250813700002e+00, 1.4650799754584590e+04 },
    { 1.9999999999999999e-11, 1.5605989281199999e+00, 1.2997155071751658e+05 },
    { 1.9999999999999999e-11, 5.5010117698999998e+00, 1.0222573515891758e+05 },
    { 1.9999999999999999e-11, 1.9025373648099999e+00, 3.0755762096960001e+02 },
    { 1.9999999999999999e-11, 1.7683937063999999e-01, 2.3549654637320600e+04 },
    { 1.9999999999999999e-11, 5.0637199341599999e+00, 5.1472173098530991e+04 },
    { 9.9999999999999994e-12, 3.5287972008600001e+00, 6.6826606037732956e+04 },
    { 9.9999999999999994e-12, 1.7763443543599999e+00, 9.3696660349531187e+04 },
    { 9.9999999999999994e-12, 5.7809610748999996e-01, 2.4919004304527600e+04 },
    { 9.9999999999999994e-12, 5.0855289237200001e+00, 2.7256801978620799e+04 },
    { 9.9999999999999994e-12, 4.2261381277799996e+00, 9.5912285995759999e+02 },
    { 9.9999999999999994e-12, 4.9221469287700002e+00, 5.2499311699805789e+04 },
    { 9.9999999999999994e-12, 4.5067029057000002e-01, 2.6010152597590299e+04 },
    { 9.9999999999999994e-12, 1.4997489214799999e+00, 1.9080959732422797e+05 },
    { 9.9999999999999994e-12, 5.6879920653499996e+00, 2.3320205537888399e+05 },
    { 9.9999999999999994e-12, 4.6466019424700002e+00, 1.8452652147423659e+05 },
    { 9.9999999999999994e-12, 2.6246864329399999e+00, 5.0807146030303389e+04 },
    { 9.9999999999999994e-12, 9.7102998199999999e-02, 5.0395882483129999e+03 },
};

static const VSOP87Term VSOP87A_MERCURY_X1[792] = {
    { 3.1884803400000000e-03, 0.0000000000000000e+00, 0.0000000000000000e+00 },
    { 1.0528901900000000e-03, 5.9160047500599999e+00, 5.2175806283148399e+04 },
    { 3.2316000999999998e-04, 2.6824727334700000e+00, 7.8263709424722590e+04 },
    { 1.1992889000000000e-04, 5.8157511296299997e+00, 2.6087903141574199e+04 },
    { 8.7831999999999995e-05, 5.7328574742500003e+00, 1.0435161256629678e+05 },
    { 2.3290420000000001e-05, 2.5002379340699998e+00, 1.3043951570787099e+05 },
    { 6.1447300000000002e-06, 5.5508760284400003e+00, 1.5652741884944518e+05 },
    { 1.6219200000000000e-06, 2.3183652924799998e+00, 1.8261532199101939e+05 },
    { 6.9027999999999996e-07, 2.9149493805800000e+00, 2.4978524589480799e+04 },
    { 6.2918000000000003e-07, 2.7319685326199998e+00, 2.7197281693667599e+04 },
    { 4.2903999999999998e-07, 5.3690606391799998e+00, 2.0870322513259359e+05 },
    { 1.7519000000000000e-07, 5.9683086318500003e+00, 5.1066427731055002e+04 },
    { 1.3252000000000000e-07, 5.7838847444099999e+00, 5.3285184835241802e+04 },
    { 1.2470000000000000e-07, 1.4066828397800000e+00, 1.0593819301891999e+03 },
    { 1.1874000000000000e-07, 1.7869376191000000e-01, 1.1093785520934000e+03 },
    { 1.1379000000000000e-07, 2.1365978608099998e+00, 2.3479112827416777e+05 },
    { 1.1372000000000000e-07, 3.4951033395999997e-01, 2.0426571092422000e+04 },
    { 8.3640000000000000e-08, 5.3132684102200001e+00, 3.1749235190726398e+04 },
    { 6.2960000000000003e-08, 5.4757644761200002e+00, 4.5519534970588002e+03 },
    { 4.8429999999999998e-08, 2.7371928422699998e+00, 7.7154330872629187e+04 },
    { 4.8300000000000002e-08, 2.5724123783000002e-01, 2.1535949644515400e+04 },
    { 4.3789999999999997e-08, 5.0978824712600002e+00, 2.5132303399965600e+04 },
    { 4.3110000000000000e-08, 4.2636683478700004e+00, 5.1116424352959199e+04 },
    { 4.2470000000000002e-08, 3.3940786389299999e+00, 4.6514474233996203e+04 },
    { 4.0079999999999999e-08, 4.4160613994700002e+00, 2.7147285071763388e+04 },
    { 3.9190000000000002e-08, 2.3794765959099999e+00, 1.5890728952838001e+03 },
    { 3.8330000000000000e-08, 5.3521442894000004e-01, 2.7043502883182799e+04 },
    { 3.7550000000000002e-08, 4.1788409144900003e+00, 1.4765239043269799e+04 },
    { 3.1470000000000003e-08, 1.0983195901000000e+00, 2.6617594106668799e+04 },
    { 3.1340000000000001e-08, 4.6820132065399998e+00, 2.5558212176479599e+04 },
    { 3.0260000000000001e-08, 5.1873243030999996e+00, 2.6087903141574195e+05 },
    { 2.8920000000000000e-08, 5.4264057430000001e+00, 2.6107572902473989e+04 },
    { 2.8880000000000001e-08, 2.2490912553000000e-01, 2.6068233380674399e+04 },
    { 2.3720000000000000e-08, 2.0717380560600001e+00, 2.6084021806216198e+04 },
    { 2.3670000000000000e-08, 3.5795959645400002e+00, 2.6091784476932200e+04 },
    { 2.0599999999999999e-08, 1.4744477429000000e+00, 3.7410567239878597e+04 },
    { 1.9950000000000000e-08, 5.4370240539700001e+00, 3.0639856638632999e+04 },
    { 1.9930000000000000e-08, 2.4928339654100000e+00, 7.9373087976815994e+04 },
    { 1.8819999999999999e-08, 2.1679831472000000e-01, 4.7623852786089599e+04 },
    { 1.5819999999999999e-08, 4.3139271826599996e+00, 9.5559974160859997e+02 },
    { 1.5810000000000001e-08, 1.6926631926200000e+00, 9.1039069941176003e+03 },
    { 1.4999999999999999e-08, 4.3230658390999999e-01, 1.9317192540328600e+04 },
    { 1.4860000000000000e-08, 3.7763613869800001e+00, 1.2566151699982800e+04 },
    { 1.4680000000000000e-08, 2.9891271252100000e+00, 2.3869146037387400e+04 },
    { 1.4440000000000001e-08, 4.9054551733600000e+00, 5.2969096509459996e+02 },
    { 1.4210000000000000e-08, 2.0441019795100002e+00, 4.2659819087599999e+02 },
    { 1.3930000000000000e-08, 1.6139897514000001e-01, 7.2602377375570388e+04 },
    { 1.3720000000000000e-08, 5.7887712447600004e+00, 1.0324223401420339e+05 },
    { 1.3560000000000000e-08, 3.2817719859199999e+00, 5.0586733387864588e+04 },
    { 1.2909999999999999e-08, 2.4819361405299998e+00, 2.6095016688575000e+04 },
    { 1.2470000000000000e-08, 3.2253805184800002e+00, 2.6080789594573391e+04 },
    { 1.2380000000000001e-08, 1.8218326629799999e+00, 5.1220206541539788e+04 },
    { 1.2050000000000000e-08, 3.9356146896299999e+00, 5.2705497248242988e+04 },
    { 1.1880000000000000e-08, 2.6371656269399999e+00, 2.8306660245760992e+04 },
    { 1.1700000000000001e-08, 5.4921299748100001e+00, 3.4425749449653999e+03 },
    { 1.1010000000000001e-08, 1.2173059191200000e+00, 5.3235188213337591e+04 },
    { 9.8099999999999998e-09, 1.3945164139999999e-01, 2.2645328196608789e+04 },
    { 9.6600000000000001e-09, 3.5655836302799999e+00, 5.3131406024757001e+04 },
    { 9.2300000000000006e-09, 7.4286443513000000e-01, 6.3989728631399998e+02 },
    { 8.9100000000000000e-09, 5.1532559721300002e+00, 3.2858613742819791e+04 },
    { 8.6699999999999992e-09, 2.1319652799300002e+00, 8.3925041473874793e+04 },
    { 8.6499999999999997e-09, 6.1940053748999997e+00, 7.1135470007999997e+00 },
    { 8.5700000000000000e-09, 9.4331413205000003e-01, 4.0853142184844000e+04 },
    { 8.5299999999999993e-09, 5.3308559891699998e+00, 2.7676976036857999e+04 },
    { 8.2800000000000004e-09, 1.0470726028299999e+00, 2.5874604046136199e+04 },
    { 8.0800000000000002e-09, 4.7628939465400002e+00, 3.6301188687785187e+04 },
    { 8.0600000000000007e-09, 1.9547006186000000e+00, 2.8696693455731618e+05 },
    { 8.0600000000000007e-09, 4.3356760600499999e+00, 1.0664954771900000e+03 },
    { 7.9699999999999996e-09, 4.6325354228800002e+00, 2.6301202237012199e+04 },
    { 7.7200000000000006e-09, 5.2311166751600000e+00, 1.1322664098304400e+04 },
    { 7.6500000000000007e-09, 1.8838272888500001e+00, 3.9609654583165597e+04 },
    { 7.3200000000000004e-09, 2.1941787341299999e+00, 5.6727759780207198e+04 },
    { 7.2000000000000000e-09, 1.4982036193799999e+00, 1.8849227549974199e+04 },
    { 7.0299999999999999e-09, 2.6869104815300000e+00, 6.7707106012455997e+03 },
    { 6.7299999999999997e-09, 4.8137644201200001e+00, 1.0522683831884001e+03 },
    { 6.6599999999999997e-09, 8.5409428955000000e-01, 1.5874617595363199e+04 },
    { 6.6199999999999999e-09, 3.2782120327800000e+00, 5.2156136522248598e+04 },
    { 6.5100000000000001e-09, 5.4842494786099998e+00, 2.9530478086539599e+04 },
    { 6.2700000000000001e-09, 7.0425946007999995e-01, 5.1646115318053788e+04 },
    { 6.2099999999999999e-09, 1.4363238089799999e+00, 2.5028521211384999e+04 },
    { 6.1900000000000003e-09, 3.9232791683400001e+00, 4.3071899289030800e+04 },
    { 6.0699999999999999e-09, 2.0289871578800001e+00, 5.6613320491521999e+03 },
    { 5.5400000000000003e-09, 3.0141510303699999e+00, 2.1187638603783998e+03 },
    { 5.3300000000000004e-09, 3.5442356294999999e-01, 5.2179687618506403e+04 },
    { 5.2700000000000002e-09, 1.3511199028000001e-01, 2.2187571041868000e+03 },
    { 5.2499999999999999e-09, 4.9331623243799996e+00, 2.6514501332450200e+04 },
    { 5.1899999999999997e-09, 2.1887762698300000e+00, 5.2195476044048191e+04 },
    { 5.0700000000000001e-09, 4.5193225300500002e+00, 6.3498470381452789e+04 },
    { 5.0000000000000001e-09, 5.2252923933999995e-01, 3.8654054841556987e+04 },
    { 4.8600000000000002e-09, 3.6398502670699999e+00, 5.1749208092272391e+04 },
    { 4.6699999999999998e-09, 5.1548631219600001e+00, 1.1001294461544899e+05 },
    { 4.3899999999999999e-09, 3.2121733023900001e+00, 9.8690280517144580e+04 },
    { 4.3599999999999998e-09, 4.1193160912700000e+00, 3.3326578733174203e+04 },
    { 4.3299999999999997e-09, 5.1228749123500004e+00, 5.2171924947790401e+04 },
    { 4.2999999999999996e-09, 2.4946600576100000e+00, 2.1329909543799999e+02 },
    { 4.1899999999999998e-09, 1.6415078289900000e+00, 7.2386755916000002e+03 },
    { 4.0199999999999998e-09, 6.0079048895799998e+00, 5.2168692736147590e+04 },
    { 4.0100000000000000e-09, 5.7597278349299996e+00, 5.4394563387335191e+04 },
    { 3.9199999999999997e-09, 2.5571896217800001e+00, 1.2933013715577759e+05 },
    { 3.8799999999999998e-09, 6.1592624794999995e-01, 7.8793400389817180e+04 },
    { 3.8600000000000003e-09, 2.0345726555899999e+00, 5.8946516884393990e+04 },
    { 3.8199999999999996e-09, 5.5117908787100003e+00, 5.2182919830149200e+04 },
    { 3.7000000000000000e-09, 9.8752045979000003e-01, 1.0329223063610759e+05 },
    { 3.6600000000000002e-09, 4.8496083863299999e+00, 7.7308109683113988e+04 },
    { 3.5400000000000002e-09, 1.5171442916100000e+00, 6.2389091829359393e+04 },
    { 3.2599999999999999e-09, 4.2971552111599998e+00, 7.9323091354911783e+04 },
    { 3.2500000000000002e-09, 4.0481376650899996e+00, 6.8050423878511589e+04 },
    { 3.2200000000000000e-09, 6.0452614857000002e+00, 4.9957049178961599e+04 },
    { 3.1800000000000002e-09, 4.2365418922900000e+00, 1.3655860491176400e+04 },
    { 3.1700000000000000e-09, 6.1026267997700003e+00, 7.6674636529438780e+04 },
    { 3.1500000000000001e-09, 3.4894093859400002e+00, 4.5405095681902800e+04 },
    { 3.1200000000000000e-09, 4.9113789228300000e+00, 5.1535908996834391e+04 },
    { 2.7400000000000001e-09, 1.3171269593199999e+00, 5.1109310805958390e+04 },
    { 2.5899999999999999e-09, 1.4878875880900000e+00, 5.2389105378586399e+04 },
    { 2.3499999999999999e-09, 1.4479942195000001e-01, 4.8733231338182988e+04 },
    { 2.2999999999999999e-09, 5.2397039821000000e+00, 8.2815662921781390e+04 },
    { 2.2999999999999999e-09, 8.3436604790000002e-01, 5.1123537899959993e+04 },
    { 2.2900000000000002e-09, 1.7447221657400001e+00, 4.3371173787679999e+02 },
    { 2.2200000000000002e-09, 3.9929404398999999e+00, 6.6941045326418185e+04 },
    { 2.1799999999999999e-09, 3.3599071828399998e+00, 3.2370978991565600e+04 },
    { 2.1499999999999998e-09, 5.0057087490200001e+00, 3.1305483769889036e+05 },
    { 2.0799999999999998e-09, 1.1389773189400001e+00, 5.7837138332300601e+04 },
    { 2.0700000000000001e-09, 3.2441801559000000e-01, 2.4498830246290399e+04 },
    { 2.0099999999999999e-09, 4.0254553219300000e+00, 1.2938013377768178e+05 },
    { 2.0000000000000001e-09, 4.4657955583600000e+00, 4.4937130691548402e+04 },
    { 1.9700000000000000e-09, 2.0507471914100002e+00, 5.3764879178432202e+04 },
    { 1.9700000000000000e-09, 2.7021416319500000e+00, 1.0213285546211000e+04 },
    { 1.9500000000000001e-09, 7.3253707199999996e-01, 6.9159802430604992e+04 },
    { 1.9500000000000001e-09, 2.6791414966899998e+00, 5.0057042422769999e+04 },
    { 1.9000000000000001e-09, 3.8724692308000002e+00, 2.6727800427888200e+04 },
    { 1.8600000000000000e-09, 1.9176431260700000e+00, 1.3610084775702318e+05 },
    { 1.8300000000000001e-09, 2.2903996470400001e+00, 4.1948464387519999e+02 },
    { 1.8100000000000000e-09, 2.2515665277300001e+00, 5.5618381228113802e+04 },
    { 1.7900000000000000e-09, 5.2546037655599998e+00, 1.0546099111839019e+05 },
    { 1.7400000000000000e-09, 3.7589057539800002e+00, 5.1962507187710398e+04 },
    { 1.7200000000000001e-09, 7.8909849325000003e-01, 9.4138327020085781e+04 },
    { 1.6999999999999999e-09, 5.1200081680000002e-02, 7.8244039663822798e+04 },
    { 1.6200000000000000e-09, 2.4882330122299998e+00, 1.6983996147456601e+04 },
    { 1.5900000000000001e-09, 6.1693791548799997e+00, 8.5319638175199998e+02 },
    { 1.5700000000000000e-09, 3.5907880670900001e+00, 6.4741957983131193e+04 },
    { 1.5600000000000000e-09, 8.9517325451999996e-01, 5.9414481874748402e+04 },
    { 1.5500000000000000e-09, 4.6975752340500003e+00, 3.5191810135691798e+04 },
    { 1.5500000000000000e-09, 3.3929411095300002e+00, 9.9799659069237983e+04 },
    { 1.5400000000000001e-09, 2.2944133501600001e+00, 1.9804827291582798e+04 },
    { 1.4800000000000001e-09, 1.1428812759600000e+00, 2.7154398618764200e+04 },
    { 1.4700000000000001e-09, 1.7069355594800000e+00, 5.2602404474024399e+04 },
    { 1.4500000000000000e-09, 1.0269998474399999e+00, 2.5661304950698199e+04 },
    { 1.4200000000000001e-09, 4.9345769377700002e+00, 1.5961864422845999e+03 },
    { 1.3900000000000000e-09, 1.2526218171700001e+00, 7.7734018459627987e+04 },
    { 1.3800000000000000e-09, 6.2016354817000003e+00, 2.5973463852888959e+04 },
    { 1.3800000000000000e-09, 5.7417102826799997e+00, 2.6202342430259410e+04 },
    { 1.3600000000000001e-09, 7.6148701959999995e-02, 2.3754706748702189e+04 },
    { 1.3500000000000001e-09, 3.4142649983300002e+00, 7.8267590760080580e+04 },
    { 1.3500000000000001e-09, 6.2676349798800004e+00, 1.2477818365871879e+05 },
    { 1.3000000000000001e-09, 1.2493006863000000e+00, 3.8519945791972001e+04 },
    { 1.2799999999999999e-09, 4.5687041055300002e+00, 8.8476994970933592e+04 },
    { 1.2100000000000000e-09, 3.6163746592399999e+00, 1.0488130353139139e+05 },
    { 1.1800000000000001e-09, 5.2526669546100004e+00, 1.2432042650397800e+04 },
    { 1.1700000000000001e-09, 1.7609646978400000e+00, 7.9945284420241996e+03 },
    { 1.1599999999999999e-09, 4.7507912915099997e+00, 2.5934124331089399e+04 },
    { 1.1500000000000000e-09, 4.0282683909900001e+00, 1.0382192160120218e+05 },
    { 1.1300000000000000e-09, 8.9519838975999999e-01, 2.6241681952059000e+04 },
    { 1.1300000000000000e-09, 5.6118711968600001e+00, 1.5541804029735178e+05 },
    { 1.1300000000000000e-09, 2.4607126628499998e+00, 5.3274761083827998e+03 },
    { 1.0999999999999999e-09, 5.9951080845600000e+00, 2.8206667001952599e+04 },
    { 1.0999999999999999e-09, 6.0872569413699997e+00, 7.9219309166331193e+04 },
    { 1.0900000000000000e-09, 1.6067718475899999e+00, 1.0339601282468818e+05 },
    { 1.0800000000000000e-09, 1.6842056196699999e+00, 2.7140171524762591e+04 },
    { 1.0600000000000001e-09, 5.5650123856600002e+00, 2.8421099534446199e+04 },
    { 1.0399999999999999e-09, 2.2791122991599999e+00, 7.8270822971723392e+04 },
    { 1.0300000000000000e-09, 4.9464745947399997e+00, 9.1785460866313981e+04 },
    { 1.0300000000000000e-09, 5.2332687649500000e+00, 7.8283379185622383e+04 },
    { 1.0100000000000000e-09, 5.5634930064499999e+00, 2.3331963928720002e+03 },
    { 1.0000000000000001e-09, 3.5109518592500000e+00, 2.5619938151219802e+04 },
    { 9.6999999999999996e-10, 1.0868841287000000e+00, 1.0541099449648599e+05 },
    { 9.6999999999999996e-10, 5.3739819807900000e+00, 7.1980633574731182e+04 },
    { 9.5999999999999999e-10, 2.1421839832399998e+00, 2.6555868131928601e+04 },
    { 9.5000000000000003e-10, 9.9246934400000000e-02, 5.8458882133139792e+04 },
    { 9.4000000000000006e-10, 3.2739986675599999e+00, 7.4821134479757180e+04 },
    { 9.4000000000000006e-10, 2.7433645039700001e+00, 7.8256595877721789e+04 },
    { 9.4000000000000006e-10, 4.4196641816700000e+00, 1.5819593482830001e+03 },
    { 8.9000000000000003e-10, 1.0217527089000000e-01, 1.2588756221081219e+05 },
    { 8.7999999999999996e-10, 2.6130110786400000e+00, 7.3711755927663791e+04 },
    { 8.7999999999999996e-10, 1.8949149912100001e+00, 7.8259828089364586e+04 },
    { 8.3000000000000003e-10, 3.9165029105200002e+00, 7.7211441041534184e+04 },
    { 8.1999999999999996e-10, 7.9472961749000004e-01, 1.5546803691925600e+05 },
    { 8.1999999999999996e-10, 1.7746921984799999e+00, 7.7623812138408583e+04 },
    { 8.1999999999999996e-10, 4.3833642797400003e+00, 7.7197213947532582e+04 },
    { 8.0999999999999999e-10, 3.1286386182700001e+00, 4.1962520736937397e+04 },
    { 8.0999999999999999e-10, 2.8171191139600000e+00, 7.6044952320535798e+04 },
    { 8.0000000000000003e-10, 5.0422937431300001e+00, 2.6137899763478399e+04 },
    { 8.0000000000000003e-10, 6.1667373644000001e-01, 2.6037906519669988e+04 },
    { 8.0000000000000003e-10, 1.0465807661600000e+00, 5.3680451209540001e+02 },
    { 7.8999999999999996e-10, 2.4711326538299998e+00, 8.0482466528909383e+04 },
    { 7.7999999999999999e-10, 3.7893317219200000e+00, 2.5565325723480400e+04 },
    { 7.7999999999999999e-10, 2.6259075736999998e-01, 7.1492998823476984e+04 },
    { 7.7000000000000003e-10, 5.2151658040000004e-01, 1.8207813988235212e+04 },
    { 7.5999999999999996e-10, 4.5900276147100003e+00, 6.4607848933546193e+04 },
    { 7.5999999999999996e-10, 4.6030933986899996e+00, 7.8477008520160583e+04 },
    { 7.5999999999999996e-10, 3.9162121426100001e+00, 5.1742094545271590e+04 },
    { 7.5000000000000000e-10, 4.9309058476400001e+00, 1.4956319713459999e+02 },
    { 7.1000000000000003e-10, 2.0084008101399999e+00, 1.0890356606335558e+05 },
    { 6.8000000000000003e-10, 5.0216138648699999e+00, 8.5034420025968182e+04 },
    { 6.6999999999999996e-10, 3.8322109317100002e+00, 1.2022623016165999e+05 },
    { 6.6000000000000000e-10, 1.8347706545700000e+00, 2.6610480559667991e+04 },
    { 6.5000000000000003e-10, 4.9629209192700001e+00, 1.6218875089859738e+05 },
    { 6.3999999999999996e-10, 2.4396671377099999e+00, 1.0276253967101299e+05 },
    { 6.3999999999999996e-10, 5.3989015517699999e+00, 1.3521751441591399e+04 },
    { 6.3000000000000000e-10, 7.7267148825999998e-01, 9.3028948467992392e+04 },
    { 6.2000000000000003e-10, 3.2830578502400001e+00, 5.1756321639273199e+04 },
    { 5.9000000000000003e-10, 1.2140744253300000e+00, 7.1025033833122594e+04 },
    { 5.7999999999999996e-10, 1.3008714971899999e+00, 6.3278373931320004e+02 },
    { 5.7000000000000000e-10, 5.5425118148000001e+00, 7.6144945564344191e+04 },
    { 5.7000000000000000e-10, 1.7759624620200001e+00, 3.3914274084046460e+05 },
    { 5.7000000000000000e-10, 1.7064228356600000e+00, 1.1787336400788819e+05 },
    { 5.6000000000000003e-10, 3.5921351895500000e+00, 2.6484548254729998e+03 },
    { 5.6000000000000003e-10, 3.5577228313400000e+00, 2.6624707653669589e+04 },
    { 5.4000000000000000e-10, 5.7674359511700004e+00, 5.1322609901396390e+04 },
    { 5.3000000000000003e-10, 5.0704056991400002e+00, 7.9852782320006387e+04 },
    { 5.3000000000000003e-10, 2.1733504041000001e-01, 7.8050410329284583e+04 },
    { 5.1999999999999996e-10, 5.6412394027000001e-01, 1.0021837280099400e+04 },
    { 5.1999999999999996e-10, 2.1595750612400000e+00, 9.8068536716305389e+04 },
    { 5.1999999999999996e-10, 1.0060591655600000e+00, 3.9743763632750597e+04 },
    { 5.1000000000000000e-10, 6.0093922780100000e+00, 1.3154889425996438e+05 },
    { 5.1000000000000000e-10, 7.2899300866000005e-01, 1.2990982474277639e+05 },
    { 5.1000000000000000e-10, 8.6113541342000000e-01, 6.4701083331480004e+02 },
    { 5.0000000000000003e-10, 5.3029678948800001e+00, 8.1706284369687986e+04 },
    { 4.8999999999999996e-10, 3.7484483696999998e-01, 9.0829861124705378e+04 },
    { 4.8999999999999996e-10, 7.5521435692000005e-01, 5.0579619840863787e+04 },
    { 4.8999999999999996e-10, 2.2574309871199998e+00, 2.5551098629478791e+04 },
    { 4.5000000000000000e-10, 3.0983977878000002e+00, 1.0433194280539699e+05 },
    { 4.5000000000000000e-10, 4.5830930198499997e+00, 7.6266071275599998e+01 },
    { 4.3999999999999998e-10, 2.6298023247700000e+00, 7.8800891533389904e+03 },
    { 4.3000000000000001e-10, 4.7810669343200001e+00, 7.8690307615598591e+04 },
    { 4.3000000000000001e-10, 4.9578719152000000e-01, 1.0392501437542078e+05 },
    { 4.3000000000000001e-10, 4.8073081127400004e+00, 2.6521614879451001e+04 },
    { 4.3000000000000001e-10, 6.0976409004700001e+00, 6.6812248533995999e+03 },
    { 4.2000000000000000e-10, 1.3386254953600001e+00, 1.1456489811250778e+05 },
    { 4.2000000000000000e-10, 6.0146700595600002e+00, 2.6507387785449391e+04 },
    { 4.0999999999999998e-10, 3.0354882414400000e+00, 1.5086608680029298e+05 },
    { 4.0999999999999998e-10, 5.7259526223000001e-01, 2.5881717593137000e+04 },
    { 4.0999999999999998e-10, 4.1763899027800004e+00, 5.2329585093633192e+04 },
    { 4.0000000000000001e-10, 5.8487787277600001e+00, 2.3969139281195799e+04 },
    { 4.0000000000000001e-10, 7.0008569908999996e-01, 5.2815703569462399e+04 },
    { 4.0000000000000001e-10, 5.0066830225799999e+00, 2.6294088690011391e+04 },
    { 3.7999999999999998e-10, 1.2510850307999999e-01, 7.7204327494533383e+04 },
    { 3.7999999999999998e-10, 4.9613882489999996e+00, 3.3967992294913187e+04 },
    { 3.7000000000000001e-10, 3.1444662363200000e+00, 1.5197546535238638e+05 },
    { 3.7000000000000001e-10, 3.0806053558800000e+00, 2.2759767485294011e+04 },
    { 3.6000000000000000e-10, 3.6884188791299999e+00, 4.4181277841124189e+04 },
    { 3.6000000000000000e-10, 3.4367134725999998e-01, 1.3096920667296558e+05 },
    { 3.6000000000000000e-10, 1.8284823248000001e-01, 1.0435549390165479e+05 },
    { 3.6000000000000000e-10, 3.1187687757800000e+00, 4.6848330174765601e+04 },
    { 3.6000000000000000e-10, 2.1811442195800002e+00, 5.2257741809380002e+02 },
    { 3.4999999999999998e-10, 3.1848926330000000e+00, 8.4546785274713984e+04 },
    { 3.4999999999999998e-10, 2.5362863809600000e+00, 1.5763679740153858e+05 },
    { 3.4999999999999998e-10, 4.8164932276999997e+00, 6.2830758499914000e+03 },
    { 3.4000000000000001e-10, 9.3646404164999997e-01, 2.6190995915792791e+04 },
    { 3.4000000000000001e-10, 8.5062194570000005e-01, 8.9586373523026981e+04 },
    { 3.3000000000000000e-10, 4.7275489509300002e+00, 2.5984810367355600e+04 },
    { 3.3000000000000000e-10, 1.2787580461000001e+00, 5.0593846934865389e+04 },
    { 3.1999999999999998e-10, 2.3724307361000001e+00, 1.8150594343892598e+05 },
    { 3.1999999999999998e-10, 1.3024720448800000e+00, 1.0530721230790539e+05 },
    { 3.1999999999999998e-10, 4.6417811959500002e+00, 1.2948391596626239e+05 },
    { 3.1000000000000002e-10, 2.5118550750200002e+00, 5.2290245571833613e+04 },
    { 3.1000000000000002e-10, 4.7378219851800001e+00, 5.1639001771052986e+04 },
    { 3.0000000000000000e-10, 5.2032286523600000e+00, 5.2643771273502804e+04 },
    { 3.0000000000000000e-10, 2.9903961350900001e+00, 2.6941099523326200e+04 },
    { 2.8999999999999998e-10, 5.2457348718299999e+00, 2.5668418497699000e+04 },
    { 2.8999999999999998e-10, 2.5184353166500002e+00, 2.9416038797854391e+04 },
    { 2.8999999999999998e-10, 4.2142662325900000e+00, 5.3242301760338400e+04 },
    { 2.8999999999999998e-10, 4.1486639433699999e+00, 1.3149889763806018e+05 },
    { 2.8999999999999998e-10, 1.6040838910699999e+00, 2.5448005855260191e+04 },
    { 2.8000000000000002e-10, 3.8306718557099999e+00, 1.8155594006083018e+05 },
    { 2.8000000000000002e-10, 3.6409281878400002e+00, 9.5247705572179184e+04 },
    { 2.8000000000000002e-10, 5.3257802926500002e+00, 1.0435872611329758e+05 },
    { 2.7000000000000000e-10, 4.5276794017200004e+00, 2.5021407664384202e+04 },
    { 2.7000000000000000e-10, 6.8051286566000002e-01, 1.0329934418310839e+05 },
    { 2.5999999999999998e-10, 2.3266308338099999e+00, 5.4509002676020398e+04 },
    { 2.5999999999999998e-10, 2.9575384970599998e+00, 5.2061366994463169e+04 },
    { 2.5999999999999998e-10, 1.7251203016300001e+00, 2.7684089583858800e+04 },
    { 2.5999999999999998e-10, 2.0528527107899999e+00, 6.0055895436487393e+04 },
    { 2.5999999999999998e-10, 1.2627235625600000e+00, 1.3362449706799200e+04 },
    { 2.5000000000000002e-10, 3.8485630329999999e-02, 9.5628915597059995e+02 },
    { 2.5000000000000002e-10, 5.1372103179100002e+00, 4.2153969003049002e+04 },
    { 2.5000000000000002e-10, 4.2860125929999998e-01, 2.9428515568274001e+04 },
    { 2.5000000000000002e-10, 1.6777979621100001e+00, 2.6237466338708789e+04 },
    { 2.5000000000000002e-10, 7.2182905302000000e-01, 5.2026243086013797e+04 },
    { 2.5000000000000002e-10, 1.2352621190000000e-02, 2.4864085300795588e+04 },
    { 2.5000000000000002e-10, 5.6575086519999997e+00, 2.7311720982352810e+04 },
    { 2.4000000000000000e-10, 5.2880727945799997e+00, 2.2747290714874402e+04 },
    { 2.4000000000000000e-10, 2.7195042904200002e+00, 5.4294570143526791e+04 },
    { 2.3000000000000001e-10, 5.8725165783000000e-01, 1.4631413330323418e+05 },
    { 2.3000000000000001e-10, 3.7829937011900001e+00, 2.5035634758385801e+04 },
    { 2.3000000000000001e-10, 1.7361107850199999e+00, 1.8827665404017159e+05 },
    { 2.3000000000000001e-10, 1.3995071774800001e+00, 6.1279713277265997e+04 },
    { 2.3000000000000001e-10, 6.8702668183000004e-01, 7.7829997686845789e+04 },
    { 2.1999999999999999e-10, 4.7444214680099996e+00, 1.4396126714946239e+05 },
    { 2.1999999999999999e-10, 3.4921482949799998e+00, 1.3001291751699499e+05 },
    { 2.1999999999999999e-10, 1.0565363711000000e-01, 7.7844224780847391e+04 },
    { 2.1999999999999999e-10, 8.4681454740999995e-01, 7.0269180982698381e+04 },
    { 2.1999999999999999e-10, 1.4237432192599999e+00, 1.0456491166173479e+05 },
    { 2.1000000000000000e-10, 5.8683080713500004e+00, 1.0213285546210999e+05 },
    { 2.1000000000000000e-10, 5.7870995232000002e+00, 1.0434449901929598e+05 },
    { 2.1000000000000000e-10, 5.2107806823100002e+00, 1.2415643985787958e+05 },
    { 2.1000000000000000e-10, 2.3851667257100000e+00, 8.5502385016322587e+04 },
    { 2.1000000000000000e-10, 1.1421146331700001e+00, 1.0328511708910679e+05 },
    { 2.1000000000000000e-10, 2.1698673476399999e+00, 4.9527351457675388e+04 },
    { 2.1000000000000000e-10, 1.9827243883000001e+00, 1.0437128232719658e+05 },
    { 2.1000000000000000e-10, 5.0526071788599998e+00, 1.3499146920492980e+05 },
    { 2.0000000000000001e-10, 4.9666486690499996e+00, 1.0371171527998279e+05 },
    { 2.0000000000000001e-10, 2.2934750545000000e+00, 1.4788665740644001e+03 },
    { 2.0000000000000001e-10, 3.3034856429500001e+00, 9.7580901965051191e+04 },
    { 2.0000000000000001e-10, 3.9148232959699998e+00, 1.3139511544947958e+05 },
    { 2.0000000000000001e-10, 6.1306912300500001e+00, 1.1610551958374201e+04 },
    { 1.8999999999999999e-10, 3.6497225732700000e+00, 5.2125809661244188e+04 },
    { 1.8999999999999999e-10, 4.5876008119999999e-02, 3.3281356562801898e+03 },
    { 1.8999999999999999e-10, 5.6070067034299997e+00, 1.0657036967048359e+05 },
    { 1.8999999999999999e-10, 3.7438492033699999e+00, 1.5599772788435058e+05 },
    { 1.8000000000000000e-10, 4.7807519796299998e+00, 5.3228074666336790e+04 },
    { 1.8000000000000000e-10, 4.9127976711399999e+00, 5.2698383701242194e+04 },
    { 1.8000000000000000e-10, 4.5393112096200001e+00, 5.1543022543835199e+04 },
    { 1.8000000000000000e-10, 3.8208873067900000e+00, 1.1911685160956658e+05 },
    { 1.8000000000000000e-10, 5.9186367814000000e+00, 4.5494581429748789e+04 },
    { 1.8000000000000000e-10, 4.9395867456899998e+00, 1.0434773123093879e+05 },
    { 1.7000000000000001e-10, 2.5153150696400002e+00, 1.1624747044077999e+03 },
    { 1.7000000000000001e-10, 2.5929357120300001e+00, 7.7410513042970590e+04 },
    { 1.7000000000000001e-10, 3.1026067416599998e+00, 1.0413831347085879e+05 },
    { 1.7000000000000001e-10, 2.3949740820500001e+00, 1.8093374699549990e+04 },
    { 1.7000000000000001e-10, 1.5441801406600000e+00, 1.1567427666460119e+05 },
    { 1.7000000000000001e-10, 4.8184870313200001e+00, 1.2885044281258718e+05 },
    { 1.7000000000000001e-10, 4.8039313209600003e+00, 5.1528795449833589e+04 },
    { 1.7000000000000001e-10, 4.2344325677599999e+00, 9.7112936974696786e+04 },
    { 1.5999999999999999e-10, 6.2086725028799998e+00, 1.0090903762133139e+05 },
    { 1.5999999999999999e-10, 4.3269835951099997e+00, 1.2546481939083000e+04 },
    { 1.5999999999999999e-10, 8.4901493535000006e-01, 2.5867490499135391e+04 },
    { 1.5999999999999999e-10, 5.5593117527300002e+00, 1.8372470054311279e+05 },
    { 1.5000000000000000e-10, 2.3810265637899999e+00, 1.6066065861474801e+04 },
    { 1.5000000000000000e-10, 2.0774665167599999e+00, 1.0779418751126219e+05 },
    { 1.5000000000000000e-10, 4.9394005367500000e+00, 2.6308315784013001e+04 },
    { 1.5000000000000000e-10, 1.3365183843299999e+00, 2.6164169212849800e+04 },
    { 1.5000000000000000e-10, 2.2694340725900002e+00, 1.0223284870591838e+05 },
    { 1.5000000000000000e-10, 5.1783096077600002e+00, 2.6222012191159200e+04 },
    { 1.5000000000000000e-10, 1.7425869768800000e+00, 5.2225802905052602e+04 },
    { 1.5000000000000000e-10, 3.8278629188900002e+00, 7.6667522982437978e+04 },
    { 1.5000000000000000e-10, 4.8251014264299998e+00, 3.6523064398203877e+05 },
    { 1.5000000000000000e-10, 2.4698669297000000e-01, 5.2712610795243789e+04 },
    { 1.5000000000000000e-10, 1.0565847080799999e+00, 5.2099540211872802e+04 },
    { 1.5000000000000000e-10, 1.2834249922300001e+00, 2.7669862489857191e+04 },
    { 1.5000000000000000e-10, 5.4946945383600001e+00, 2.1258774073792001e+03 },
    { 1.5000000000000000e-10, 1.5525153801300000e+00, 2.0760427033191401e+04 },
    { 1.4000000000000001e-10, 3.5817338390900000e+00, 4.4295717129809404e+04 },
    { 1.4000000000000001e-10, 4.7655081868300000e+00, 2.1116503133776000e+03 },
    { 1.4000000000000001e-10, 4.7714765174900000e+00, 2.7999102624791401e+04 },
    { 1.4000000000000001e-10, 3.4303020430600002e+00, 1.1691776426627958e+05 },
    { 1.4000000000000001e-10, 4.9576253379000002e-01, 2.5953794091989192e+04 },
    { 1.4000000000000001e-10, 1.8222818408799999e+00, 1.0594068546158058e+05 },
    { 1.4000000000000001e-10, 1.0306317854500000e+00, 7.8417488235207391e+04 },
    { 1.4000000000000001e-10, 5.2990346792400000e+00, 7.7837111233846590e+04 },
    { 1.4000000000000001e-10, 2.6625368470100002e+00, 3.2769127994973802e+04 },
    { 1.2999999999999999e-10, 3.6904026392999998e+00, 3.2350541665740002e+02 },
    { 1.2999999999999999e-10, 9.2750711335000002e-01, 2.4176703658357001e+04 },
    { 1.2999999999999999e-10, 6.1820327591100002e+00, 1.7806336849396059e+05 },
    { 1.2999999999999999e-10, 3.3164462848500000e+00, 2.2041264243879999e+02 },
    { 1.2999999999999999e-10, 4.3061741100699997e+00, 7.6681750076439581e+04 },
    { 1.2999999999999999e-10, 5.7425435430400000e+00, 5.5503941939428587e+04 },
    { 1.2999999999999999e-10, 4.4908618767700004e+00, 1.0309277421860000e+02 },
    { 1.2999999999999999e-10, 1.0820742036000000e+00, 1.6921656695024001e+03 },
    { 1.2999999999999999e-10, 3.3120904153000001e+00, 1.4477351183200000e+04 },
    { 1.2999999999999999e-10, 4.3926752147599997e+00, 1.4065280125408198e+05 },
    { 1.2000000000000000e-10, 1.7984545666300000e+00, 1.1112232316754239e+05 },
    { 1.2000000000000000e-10, 1.5703639869799999e+00, 1.0477821075717278e+05 },
    { 1.2000000000000000e-10, 1.3352163858799999e+00, 5.2072713508929788e+04 },
    { 1.2000000000000000e-10, 4.6672519719699999e+00, 1.1159028815789679e+05 },
    { 1.2000000000000000e-10, 6.2195814627799999e+00, 7.2936233316339785e+04 },
    { 1.2000000000000000e-10, 6.2453305509400003e+00, 1.1063468841628819e+05 },
    { 1.2000000000000000e-10, 5.2166870055499999e+00, 3.1415379249957001e+04 },
    { 1.2000000000000000e-10, 6.1549112619299997e+00, 1.3041984594697120e+05 },
    { 1.2000000000000000e-10, 8.9596440764999996e-01, 2.2434479570190001e+02 },
    { 1.2000000000000000e-10, 4.5176766950599996e+00, 3.8813356576349201e+04 },
    { 1.2000000000000000e-10, 1.5041992044600001e+00, 1.2725453434775000e+04 },
    { 1.2000000000000000e-10, 2.7544235334000000e-01, 2.6395460762543789e+04 },
    { 1.2000000000000000e-10, 5.3881826778699997e+00, 2.5780345520604598e+04 },
    { 1.2000000000000000e-10, 4.1814817960999999e+00, 2.6724899413598399e+04 },
    { 1.2000000000000000e-10, 1.4821432351099999e+00, 2.5450906869549999e+04 },
    { 1.2000000000000000e-10, 2.2821394758600002e+00, 2.5446489579835201e+04 },
    { 1.2000000000000000e-10, 3.3814855553499998e+00, 2.6729316703313190e+04 },
    { 1.2000000000000000e-10, 6.0867597559500002e+00, 1.7695398994186718e+05 },
    { 1.0999999999999999e-10, 2.9836593327299998e+00, 2.5977696820354791e+04 },
    { 1.0999999999999999e-10, 3.5704741813999998e+00, 3.6109740421673603e+04 },
    { 1.0999999999999999e-10, 2.6761923970199999e+00, 2.6198109462793600e+04 },
    { 1.0999999999999999e-10, 4.1203863402099996e+00, 5.1969620734711192e+04 },
    { 1.0999999999999999e-10, 3.6205173728200002e+00, 6.5697557724739789e+04 },
    { 1.0999999999999999e-10, 2.3229639732599998e+00, 5.2022027472663598e+04 },
    { 1.0999999999999999e-10, 3.3665356066399998e+00, 1.5705710981453978e+05 },
    { 1.0000000000000000e-10, 3.0174606732999998e-01, 2.8736357967047199e+04 },
    { 1.0000000000000000e-10, 1.7861491580500000e+00, 6.8241872144623194e+04 },
    { 1.0000000000000000e-10, 1.5150262025500001e+00, 1.7004917029103660e+05 },
    { 1.0000000000000000e-10, 6.5896029584000004e-01, 1.5748301859105378e+05 },
    { 1.0000000000000000e-10, 6.0086822741000001e-01, 2.0764384320240439e+05 },
    { 1.0000000000000000e-10, 4.4395134714099997e+00, 1.4176217980617538e+05 },
    { 1.0000000000000000e-10, 1.0687658864800000e+00, 9.0695752075120385e+04 },
    { 1.0000000000000000e-10, 4.0448945555600000e+00, 6.5831666774324796e+04 },
    { 1.0000000000000000e-10, 1.9536557678299999e+00, 5.2381991831585590e+04 },
    { 8.9999999999999999e-11, 5.6276451864199997e+00, 5.1219517127177787e+04 },
    { 8.9999999999999999e-11, 1.4097008549100001e+00, 1.5557181910783658e+05 },
    { 8.9999999999999999e-11, 3.8072883585900001e+00, 7.8903606711036584e+04 },
    { 8.9999999999999999e-11, 1.9834087763499999e+00, 7.8731674415076996e+04 },
    { 8.9999999999999999e-11, 1.9798553735000000e+00, 1.6703062133498999e+04 },
    { 8.9999999999999999e-11, 1.9461343328000000e-01, 5.1707841292793993e+04 },
    { 8.9999999999999999e-11, 5.4281234095900004e+00, 2.0759384658050019e+05 },
    { 8.9999999999999999e-11, 2.5320629703000003e-01, 1.5610082065856917e+05 },
    { 8.9999999999999999e-11, 4.5024996640000002e-02, 4.9842609890276392e+04 },
    { 8.9999999999999999e-11, 3.4879033622300000e+00, 5.5516418709848200e+04 },
    { 8.9999999999999999e-11, 1.6065983026799999e+00, 5.2609518021025193e+04 },
    { 8.9999999999999999e-11, 2.1965770869000001e+00, 1.9111994832171999e+03 },
    { 8.9999999999999999e-11, 3.9120537646600000e+00, 2.4491716699289591e+04 },
    { 8.9999999999999999e-11, 3.2060226886100001e+00, 1.9669760899789999e+01 },
    { 8.9999999999999999e-11, 1.0781731132600001e+00, 2.0618554843720000e+02 },
    { 8.9999999999999999e-11, 3.2344143912700001e+00, 1.3044339704322898e+05 },
    { 8.9999999999999999e-11, 3.9305993871300000e+00, 2.6734913974889001e+04 },
    { 8.9999999999999999e-11, 3.4249730944699999e+00, 7.5930513031850584e+04 },
    { 7.9999999999999995e-11, 5.8999850186199998e+00, 5.2182434842048402e+04 },
    { 7.9999999999999995e-11, 7.9946525840000005e-02, 2.7177611932767799e+04 },
    { 7.9999999999999995e-11, 5.5836785053700000e+00, 2.4998194350380589e+04 },
    { 7.9999999999999995e-11, 4.0442315403200002e+00, 3.3406124266997999e+03 },
    { 7.9999999999999995e-11, 6.1331104888699999e+00, 4.8847670626868203e+04 },
    { 7.9999999999999995e-11, 6.2374882105699996e+00, 8.6030992875280003e+02 },
    { 7.9999999999999995e-11, 2.9533780543799999e+00, 5.2595290927023590e+04 },
    { 7.9999999999999995e-11, 2.0908510994200000e+00, 1.3044662925487179e+05 },
    { 7.9999999999999995e-11, 4.2372268047099997e+00, 2.6720686880887391e+04 },
    { 7.9999999999999995e-11, 5.6503915925000001e+00, 1.2238178407786099e+03 },
    { 7.9999999999999995e-11, 3.7399034266700002e+00, 1.2938724732468258e+05 },
    { 7.9999999999999995e-11, 2.7510626251699999e+00, 5.2169177724248388e+04 },
    { 7.9999999999999995e-11, 1.5034721620900000e+00, 7.7726904912627186e+04 },
    { 7.9999999999999995e-11, 9.3245740025000001e-01, 1.5758680077963436e+05 },
    { 7.0000000000000004e-11, 6.2849516745000000e-01, 9.3848410080752001e+03 },
    { 7.0000000000000004e-11, 3.3728386887899999e+00, 5.0696939709083992e+04 },
    { 7.0000000000000004e-11, 2.3047455678700000e+00, 2.0981260368468694e+05 },
    { 7.0000000000000004e-11, 3.6420050423500001e+00, 1.7240203644480839e+05 },
    { 7.0000000000000004e-11, 3.1569931227699999e+00, 5.1013331578740588e+04 },
    { 7.0000000000000004e-11, 7.5605454895000002e-01, 8.4608283475120004e+02 },
    { 7.0000000000000004e-11, 3.0494187245500002e+00, 2.7044192297544800e+04 },
    { 7.0000000000000004e-11, 1.9933732905999999e+00, 1.5024434299945380e+05 },
    { 7.0000000000000004e-11, 4.7571496014000001e+00, 5.2325369480282992e+04 },
    { 7.0000000000000004e-11, 5.1022697695600003e+00, 1.3541421202491190e+04 },
    { 7.0000000000000004e-11, 5.3842447091299999e+00, 8.0596905817594597e+04 },
    { 7.0000000000000004e-11, 2.5162092330000000e-02, 1.1020632121940000e+02 },
    { 7.0000000000000004e-11, 5.5652182448299996e+00, 7.8378148713407805e+04 },
    { 7.0000000000000004e-11, 1.6256997159300000e+00, 2.6162684740141500e+04 },
    { 7.0000000000000004e-11, 1.4369695039199999e+00, 7.7630925685409384e+04 },
    { 7.0000000000000004e-11, 5.8535276681499999e+00, 8.1942753332085995e+03 },
    { 7.0000000000000004e-11, 1.1574004156100000e+00, 1.5493834595416137e+05 },
    { 7.0000000000000004e-11, 4.7797020176200000e+00, 2.1436455718174577e+05 },
    { 7.0000000000000004e-11, 4.3658808026999996e+00, 2.5754047200804798e+04 },
    { 7.0000000000000004e-11, 1.2977442285100000e+00, 2.6421759082343600e+04 },
    { 7.0000000000000004e-11, 1.0104101397400000e+00, 6.4901259717923393e+04 },
    { 6.0000000000000000e-11, 3.7836002500100001e+00, 5.2278899057366987e+04 },
    { 6.0000000000000000e-11, 1.0054374092200000e+00, 7.9330204901912584e+04 },
    { 6.0000000000000000e-11, 4.2039727357599999e+00, 1.2937302023068098e+05 },
    { 6.0000000000000000e-11, 4.5091444535600003e+00, 1.3065281480330898e+05 },
    { 6.0000000000000000e-11, 2.5065144064000000e-01, 2.5508215554575399e+04 },
    { 6.0000000000000000e-11, 5.4129735905700000e+00, 2.6667590728572999e+04 },
    { 6.0000000000000000e-11, 1.0463553730199999e+00, 3.9629324344065390e+04 },
    { 6.0000000000000000e-11, 5.5212415178300001e+00, 4.0565254324774200e+04 },
    { 6.0000000000000000e-11, 3.7514799097600000e+00, 1.0391790082841998e+05 },
    { 6.0000000000000000e-11, 2.3651610335600002e+00, 5.1653228865054589e+04 },
    { 6.0000000000000000e-11, 4.0071663936500004e+00, 2.6013121543006899e+04 },
    { 6.0000000000000000e-11, 1.8488859607899999e+00, 6.8851498899308099e+03 },
    { 6.0000000000000000e-11, 6.0089369397099999e+00, 7.8149270136037361e+04 },
    { 6.0000000000000000e-11, 2.6321280125099999e+00, 1.2822075860368418e+05 },
    { 6.0000000000000000e-11, 2.8093423360399998e+00, 6.6653157466348392e+04 },
    { 6.0000000000000000e-11, 6.8307752730000004e-02, 1.2366880510662538e+05 },
    { 6.0000000000000000e-11, 1.8184006018500001e+00, 1.6107937234650398e+05 },
    { 6.0000000000000000e-11, 3.1789357935800000e+00, 1.0393212792242158e+05 },
    { 6.0000000000000000e-11, 3.5364062286100002e+00, 1.2655674786264001e+03 },
    { 6.0000000000000000e-11, 4.7609620784000001e-01, 1.8208563102592478e+05 },
    { 6.0000000000000000e-11, 1.8266035721000001e-01, 5.0049928875769190e+04 },
    { 5.0000000000000002e-11, 8.9617169225000004e-01, 5.0064155969770793e+04 },
    { 5.0000000000000002e-11, 6.0596564833900004e+00, 1.3022621661243298e+05 },
    { 5.0000000000000002e-11, 1.1767614724000000e+00, 1.0276965321801379e+05 },
    { 5.0000000000000002e-11, 3.8930063695500001e+00, 3.5472744149649399e+04 },
    { 5.0000000000000002e-11, 5.7446255983299999e+00, 8.0382473285100990e+04 },
    { 5.0000000000000002e-11, 4.0751082829000002e-01, 7.8213712802818380e+04 },
    { 5.0000000000000002e-11, 7.2763573757000000e-01, 1.0275542612401219e+05 },
    { 5.0000000000000002e-11, 5.0660647958600000e+00, 7.5615254599249587e+04 },
    { 5.0000000000000002e-11, 6.0998919138299996e+00, 6.0442285813753997e+03 },
    { 5.0000000000000002e-11, 3.6844971655599998e+00, 1.9406678288174600e+04 },
    { 5.0000000000000002e-11, 5.5347836801999994e-01, 2.5654191403697390e+04 },
    { 5.0000000000000002e-11, 1.2153395476800000e+00, 1.3767819129947099e+05 },
    { 5.0000000000000002e-11, 2.4211785721500001e+00, 5.3399624123927002e+04 },
    { 5.0000000000000002e-11, 3.9964717433499999e+00, 2.7780068811076591e+04 },
    { 5.0000000000000002e-11, 5.1237606894999999e-01, 2.6402089321443800e+04 },
    { 5.0000000000000002e-11, 5.1512489622600004e+00, 2.5773716961704591e+04 },
    { 5.0000000000000002e-11, 1.5854967299100000e+00, 7.7616698591407781e+04 },
    { 5.0000000000000002e-11, 3.5667067951300000e+00, 1.7893627808365600e+04 },
    { 5.0000000000000002e-11, 1.6143006864600000e+00, 5.4087005766365597e+04 },
    { 5.0000000000000002e-11, 5.9515032793000000e-01, 4.5892730433156990e+04 },
    { 5.0000000000000002e-11, 4.3594901333299996e+00, 2.4505943793291190e+04 },
    { 5.0000000000000002e-11, 2.9249449983800000e+00, 2.0415127163553477e+05 },
    { 5.0000000000000002e-11, 4.1408142243299997e+00, 2.5938339944439598e+04 },
    { 5.0000000000000002e-11, 4.3142842779199997e+00, 3.9450352848373397e+04 },
    { 5.0000000000000002e-11, 4.6408984722900000e+00, 6.5717227485639582e+04 },
    { 5.0000000000000002e-11, 1.6397490010900000e+00, 5.2396218925587200e+04 },
    { 5.0000000000000002e-11, 9.9790116576999999e-01, 1.2320084011627098e+05 },
    { 5.0000000000000002e-11, 6.0974870551900002e+00, 5.3029002664900399e+04 },
    { 5.0000000000000002e-11, 5.7466690775999996e-01, 1.4520475475114078e+05 },
    { 5.0000000000000002e-11, 3.4150602684800000e+00, 2.6709646942413401e+04 },
    { 5.0000000000000002e-11, 2.2485647627300001e+00, 2.5466159340735001e+04 },
    { 5.0000000000000002e-11, 2.4637161389000002e+00, 2.5234706759822191e+04 },
    { 5.0000000000000002e-11, 1.9822768851399999e+00, 5.2808590022461591e+04 },
    { 3.9999999999999998e-11, 2.5468219743199998e+00, 1.3043240216087017e+05 },
    { 3.9999999999999998e-11, 1.9782136363700000e+00, 5.1852300866490987e+04 },
    { 3.9999999999999998e-11, 2.4217157051800000e+00, 1.3265827281205778e+05 },
    { 3.9999999999999998e-11, 2.8024515920300002e+00, 5.1955393640709590e+04 },
    { 3.9999999999999998e-11, 5.6613747168600002e+00, 1.0349841618454478e+05 },
    { 3.9999999999999998e-11, 4.1343425646899998e+00, 1.0450539137678158e+05 },
    { 3.9999999999999998e-11, 5.0451225486300002e+00, 9.4917560896980001e+02 },
    { 3.9999999999999998e-11, 2.1653964739000000e+00, 2.8791519296249800e+04 },
    { 3.9999999999999998e-11, 3.4982285573100000e+00, 2.3384286986898602e+04 },
    { 3.9999999999999998e-11, 3.3834827064400002e+00, 2.6823779655105991e+04 },
    { 3.9999999999999998e-11, 2.2801423247699999e+00, 2.5352026628042389e+04 },
    { 3.9999999999999998e-11, 5.1105497694800004e+00, 1.3388209065283637e+05 },
    { 3.9999999999999998e-11, 4.7946406672300004e+00, 5.3771992725432989e+04 },
    { 3.9999999999999998e-11, 3.5807622885299999e+00, 1.8357092173262799e+05 },
    { 3.9999999999999998e-11, 5.0873993576799998e+00, 2.3439448316101189e+04 },
    { 3.9999999999999998e-11, 2.9194232820999999e+00, 2.7170983373867792e+04 },
    { 3.9999999999999998e-11, 2.7442017491100001e+00, 2.5004822909280600e+04 },
    { 3.9999999999999998e-11, 1.7129760263800000e+00, 7.8786286842816378e+04 },
    { 3.9999999999999998e-11, 5.1688944607099998e+00, 6.3699627202420004e+02 },
    { 3.9999999999999998e-11, 5.0921794612899998e+00, 1.7952584437210000e+03 },
    { 3.9999999999999998e-11, 5.8417274779000001e+00, 5.2065599961928987e+04 },
    { 3.9999999999999998e-11, 1.6812380760300001e+00, 2.2218566345969998e+03 },
    { 3.9999999999999998e-11, 4.5545805465600004e+00, 5.0483640613645999e+04 },
    { 3.9999999999999998e-11, 2.6851768600999999e+00, 7.8270337983622594e+04 },
    { 3.9999999999999998e-11, 1.7984786320099999e+00, 3.4282178474782799e+04 },
    { 3.9999999999999998e-11, 5.4205020736999998e-01, 1.9959931813847594e+05 },
    { 3.9999999999999998e-11, 1.9066423879000000e-01, 1.4300566740785379e+05 },
    { 3.9999999999999998e-11, 3.2775258200100001e+00, 7.8800513936817981e+04 },
    { 3.9999999999999998e-11, 5.0247606570499999e+00, 1.3045918546877075e+05 },
    { 3.9999999999999998e-11, 4.1490761070200000e+00, 3.1781457905676002e+03 },
    { 3.9999999999999998e-11, 4.9650038177700004e+00, 5.2250587881715699e+04 },
    { 3.9999999999999998e-11, 4.4618102699800000e+00, 8.7367616418840189e+04 },
    { 3.9999999999999998e-11, 5.8144057711099997e+00, 6.0370081616356991e+04 },
    { 3.9999999999999998e-11, 1.7285982169500000e+00, 1.3043563437251298e+05 },
    { 3.9999999999999998e-11, 1.6379093308799999e+00, 2.6575537892828390e+04 },
    { 3.9999999999999998e-11, 4.0257157003300001e+00, 2.5600268390320001e+04 },
    { 3.9999999999999998e-11, 5.3420938062000003e+00, 1.2832075184749259e+05 },
    { 3.9999999999999998e-11, 5.7499558302100002e+00, 5.8857031136547987e+04 },
    { 3.9999999999999998e-11, 4.7717168337300002e+00, 5.1951461487446490e+04 },
    { 3.9999999999999998e-11, 2.7436483090100001e+00, 2.0043674560198800e+04 },
    { 3.9999999999999998e-11, 3.0303124028099999e+00, 1.3672259155786238e+05 },
    { 3.9999999999999998e-11, 1.8978294181299999e+00, 1.2979961842155698e+05 },
    { 3.9999999999999998e-11, 2.8674663945400001e+00, 3.7698455099948398e+04 },
    { 3.9999999999999998e-11, 4.8495206316399999e+00, 1.3202858860315479e+05 },
    { 3.9999999999999998e-11, 1.8887751897000000e+00, 4.6796499035440002e+02 },
    { 3.9999999999999998e-11, 3.0064538938599998e+00, 3.2132131722949602e+04 },
    { 3.9999999999999998e-11, 1.1485401455399999e+00, 1.6674070439565618e+05 },
    { 3.9999999999999998e-11, 4.0583501169000002e-01, 7.8109930614237790e+04 },
    { 3.9999999999999998e-11, 6.0503859362699997e+00, 2.8286990484861199e+04 },
    { 3.9999999999999998e-11, 5.8964244021200001e+00, 2.3888815798287189e+04 },
    { 3.0000000000000000e-11, 4.3021183000600001e+00, 7.8160616650503987e+04 },
    { 3.0000000000000000e-11, 6.6817274433999996e-01, 1.4859801210651999e+03 },
    { 3.0000000000000000e-11, 1.9301424825100000e+00, 5.2309915332733399e+04 },
    { 3.0000000000000000e-11, 1.0380980906999999e+00, 1.6785008294774959e+05 },
    { 3.0000000000000000e-11, 3.4394173964000002e-01, 7.0383620271383596e+04 },
    { 3.0000000000000000e-11, 4.4176544235900002e+00, 5.2252072354423988e+04 },
    { 3.0000000000000000e-11, 3.9722369184800002e+00, 5.0264606799931200e+04 },
    { 3.0000000000000000e-11, 5.5362700715499997e+00, 2.5985940623308590e+04 },
    { 3.0000000000000000e-11, 1.2735495965999999e-01, 2.6189865659839801e+04 },
    { 3.0000000000000000e-11, 6.1004720971000004e-01, 7.3587651353180001e+02 },
    { 3.0000000000000000e-11, 2.5671438430300002e+00, 2.5131613985603592e+04 },
    { 3.0000000000000000e-11, 1.9576064708200001e+00, 4.8835193856448590e+04 },
    { 3.0000000000000000e-11, 4.1941094526000002e+00, 1.8102624909573558e+05 },
    { 3.0000000000000000e-11, 2.0996976206000001e-01, 2.6073676047572590e+04 },
    { 3.0000000000000000e-11, 5.4536552691500004e+00, 2.6102130235575802e+04 },
    { 3.0000000000000000e-11, 4.5324736047499998e+00, 8.6143798578061585e+04 },
    { 3.0000000000000000e-11, 3.4442107945800000e+00, 2.6247204876366399e+04 },
    { 3.0000000000000000e-11, 2.2194142366300000e+00, 2.5928601406782000e+04 },
    { 3.0000000000000000e-11, 2.5371986654900001e+00, 8.9894677054323893e+03 },
    { 3.0000000000000000e-11, 5.1650711820800002e+00, 2.7036161546755998e+03 },
    { 3.0000000000000000e-11, 1.5777272009100001e+00, 7.9315977807910982e+04 },
    { 3.0000000000000000e-11, 4.6286385512599999e+00, 1.3086611389874699e+05 },
    { 3.0000000000000000e-11, 2.2321629020899998e+00, 5.1868248662178798e+04 },
    { 3.0000000000000000e-11, 2.3462607951400001e+00, 1.2699694076290558e+05 },
    { 3.0000000000000000e-11, 1.9144773586000000e-01, 7.6784842850658184e+04 },
    { 3.0000000000000000e-11, 1.3129271637800000e+00, 1.9202753251643389e+04 },
    { 3.0000000000000000e-11, 1.9319971565999999e-01, 6.1560647291223591e+04 },
    { 3.0000000000000000e-11, 5.7216715568599996e+00, 5.2286012604367788e+04 },
    { 3.0000000000000000e-11, 6.0619733035900003e+00, 7.7101234720314780e+04 },
    { 3.0000000000000000e-11, 5.9775197941800000e+00, 7.7741132006628788e+04 },
    { 3.0000000000000000e-11, 3.6248677839000001e+00, 4.5290656393217592e+04 },
    { 3.0000000000000000e-11, 5.6856736975000004e+00, 2.4925428437166400e+04 },
    { 3.0000000000000000e-11, 5.5266661450400001e+00, 7.4781598567299994e+01 },
    { 3.0000000000000000e-11, 6.2479768744699999e+00, 2.7250377845981991e+04 },
    { 3.0000000000000000e-11, 5.7105145781799997e+00, 5.1315496354395589e+04 },
    { 3.0000000000000000e-11, 2.3735872794999999e-01, 8.1604321851422384e+04 },
    { 3.0000000000000000e-11, 9.5743283728999995e-01, 5.2812802555172602e+04 },
    { 3.0000000000000000e-11, 4.4760621428600000e+00, 3.4082431583598402e+04 },
    { 3.0000000000000000e-11, 1.5925305464199999e+00, 3.9131854712361295e+05 },
    { 3.0000000000000000e-11, 2.8525265092600001e+00, 1.5650774908854539e+05 },
    { 3.0000000000000000e-11, 2.5084740667599998e+00, 2.7726972658762199e+04 },
    { 3.0000000000000000e-11, 3.1551509644500002e+00, 2.4448833624386200e+04 },
    { 3.0000000000000000e-11, 4.7503012678200003e+00, 7.8313706046626787e+04 },
    { 3.0000000000000000e-11, 2.8385727020300000e+00, 2.0304189308344139e+05 },
    { 3.0000000000000000e-11, 5.8293072847199996e+00, 7.8257080865822587e+04 },
    { 3.0000000000000000e-11, 2.9514927421100001e+00, 1.2810631931499895e+05 },
    { 3.0000000000000000e-11, 3.5270775955200002e+00, 5.2041697233563391e+04 },
    { 3.0000000000000000e-11, 5.2321709565400001e+00, 5.1534392721409400e+04 },
    { 1.9999999999999999e-11, 3.2639264356700002e+00, 1.8218872380014337e+05 },
    { 1.9999999999999999e-11, 3.8378675986699999e+00, 1.7351141499690176e+05 },
    { 1.9999999999999999e-11, 2.7931764745700001e+00, 2.0085575391590000e+03 },
    { 1.9999999999999999e-11, 2.3207091758099998e+00, 8.1591845081002786e+04 },
    { 1.9999999999999999e-11, 4.9168674850200000e+00, 5.1329723448397199e+04 },
    { 1.9999999999999999e-11, 5.6820066797999997e+00, 1.4742351185532758e+05 },
    { 1.9999999999999999e-11, 4.5402920981500001e+00, 1.9613707343261078e+05 },
    { 1.9999999999999999e-11, 6.1206244923900002e+00, 1.2133560871375339e+05 },
    { 1.9999999999999999e-11, 4.4639432075799999e+00, 2.6011637070298599e+04 },
    { 1.9999999999999999e-11, 1.0618338600799999e+00, 7.8057523876285384e+04 },
    { 1.9999999999999999e-11, 1.1167030011000001e+00, 3.8634385080657201e+04 },
    { 1.9999999999999999e-11, 5.6204370169999997e-02, 2.0526065018762814e+05 },
    { 1.9999999999999999e-11, 5.6545081013600003e+00, 1.1443928868521000e+02 },
    { 1.9999999999999999e-11, 8.1911436141000005e-01, 2.4609036567509800e+04 },
    { 1.9999999999999999e-11, 5.0372813945899999e+00, 1.0481957755665119e+05 },
    { 1.9999999999999999e-11, 3.4244940933000001e-01, 6.2197643563247788e+04 },
    { 1.9999999999999999e-11, 1.0993975041499999e+00, 7.4923096998022782e+04 },
    { 1.9999999999999999e-11, 2.4833292216400000e+00, 1.0170315774082378e+05 },
    { 1.9999999999999999e-11, 6.0560213673999996e-01, 1.7098435436141812e+04 },
    { 1.9999999999999999e-11, 5.1194601522200003e+00, 7.8469894973159782e+04 },
    { 1.9999999999999999e-11, 2.2061370314700000e+00, 7.7307420268751986e+04 },
    { 1.9999999999999999e-11, 5.9446302071000001e-01, 1.0499150985261079e+05 },
    { 1.9999999999999999e-11, 4.0391846874199997e+00, 2.6312247937276101e+04 },
    { 1.9999999999999999e-11, 8.0941764272000005e-01, 9.1919569915898988e+04 },
    { 1.9999999999999999e-11, 4.0295826207600003e+00, 2.6183882368791990e+04 },
    { 1.9999999999999999e-11, 1.6340424104500000e+00, 2.5991923914356390e+04 },
    { 1.9999999999999999e-11, 3.6411868438799999e+00, 2.2568722128005017e+05 },
    { 1.9999999999999999e-11, 2.9080008254299998e+00, 9.6357084124272587e+04 },
    { 1.9999999999999999e-11, 3.3874844304799998e+00, 1.7728784588263658e+05 },
    { 1.9999999999999999e-11, 1.7228095499600000e+00, 2.1547393573383917e+05 },
    { 1.9999999999999999e-11, 1.7391059183199999e+00, 2.5455119402261000e+04 },
    { 1.9999999999999999e-11, 8.6558333439999996e-02, 1.8314501295611399e+05 },
    { 1.9999999999999999e-11, 4.9132050582900000e+00, 1.8938603259226496e+05 },
    { 1.9999999999999999e-11, 2.7396425909000000e+00, 2.6397181464229991e+04 },
    { 1.9999999999999999e-11, 2.9239824403100001e+00, 2.5778624818918390e+04 },
    { 1.9999999999999999e-11, 5.0502403844400003e+00, 4.2790965275073198e+04 },
    { 1.9999999999999999e-11, 5.1110114566500000e+00, 5.1538810011124187e+04 },
    { 1.9999999999999999e-11, 4.0270758674799998e+00, 2.4952226269680999e+04 },
    { 1.9999999999999999e-11, 1.6365491637300000e+00, 2.7223580013467399e+04 },
    { 1.9999999999999999e-11, 4.5698949468099999e+00, 1.0371882882698359e+05 },
    { 1.9999999999999999e-11, 3.2168630452100002e+00, 2.5984120952993599e+04 },
    { 1.9999999999999999e-11, 2.4467619859999998e+00, 2.6191685330154789e+04 },
    { 1.9999999999999999e-11, 7.3451437899000005e-01, 2.5927495772189392e+04 },
    { 1.9999999999999999e-11, 4.9291106522200003e+00, 2.6248310510959000e+04 },
    { 1.9999999999999999e-11, 3.1448670647500001e+00, 5.3265515074342002e+04 },
    { 1.9999999999999999e-11, 1.4662887750000000e+00, 5.1841950342379001e+04 },
    { 1.9999999999999999e-11, 2.0986389867200002e+00, 2.9550147847439392e+04 },
    { 1.9999999999999999e-11, 3.5649860444899999e+00, 2.2625658435709000e+04 },
    { 1.9999999999999999e-11, 1.4446320402599999e+00, 2.7037078750543991e+04 },
    { 1.9999999999999999e-11, 4.2189929909500004e+00, 2.5138727532604400e+04 },
    { 1.9999999999999999e-11, 1.2919974221000000e-01, 5.2817219844887390e+04 },
    { 1.9999999999999999e-11, 5.0253808312599997e+00, 1.3831960486120995e+05 },
    { 1.9999999999999999e-11, 3.7980542913600002e+00, 1.0420204936916218e+05 },
    { 1.9999999999999999e-11, 3.6503102511700001e+00, 2.3373174634397859e+05 },
    { 1.9999999999999999e-11, 3.2933844398900001e+00, 5.4824261108621402e+04 },
    { 1.9999999999999999e-11, 2.1289418782700000e+00, 5.0910238804522000e+04 },
    { 1.9999999999999999e-11, 3.0897447693900002e+00, 5.1859414413491788e+04 },
    { 1.9999999999999999e-11, 5.2645596993700003e+00, 9.2741060607922584e+04 },
    { 1.9999999999999999e-11, 5.2513982516400004e+00, 1.6895946149984296e+05 },
    { 1.9999999999999999e-11, 1.4684052993300001e+00, 2.5863558345872290e+04 },
    { 1.9999999999999999e-11, 4.3324574600799997e+00, 2.3976252828196601e+04 },
    { 1.9999999999999999e-11, 4.4067233650900004e+00, 5.3757765631431394e+04 },
    { 1.9999999999999999e-11, 5.6076823777399998e+00, 5.2492198152804987e+04 },
    { 1.9999999999999999e-11, 5.5088200932499998e+00, 2.6076857441310302e+04 },
    { 1.9999999999999999e-11, 1.5480493796000000e-01, 2.6098948841838101e+04 },
    { 1.9999999999999999e-11, 4.9507527645799998e+00, 1.7633224614102798e+05 },
    { 1.9999999999999999e-11, 5.1777492251700004e+00, 7.8896493164035783e+04 },
    { 1.9999999999999999e-11, 2.6673303771000001e-01, 2.6411408558231600e+04 },
    { 1.9999999999999999e-11, 4.4257912845899998e+00, 1.8165972224941078e+05 },
    { 1.9999999999999999e-11, 3.4467852711900000e+00, 5.2483363904117992e+04 },
    { 1.9999999999999999e-11, 5.2767241894800003e+00, 1.1334108027172917e+05 },
    { 1.9999999999999999e-11, 4.5366527551900004e+00, 1.0381480805420138e+05 },
    { 1.9999999999999999e-11, 4.5833132752100001e+00, 1.1678365521669458e+05 },
    { 1.9999999999999999e-11, 7.0194343477999999e-01, 5.1226630674178588e+04 },
    { 1.9999999999999999e-11, 1.8180872147999999e-01, 1.5119994274106238e+05 },
    { 1.9999999999999999e-11, 2.6784625008699998e+00, 5.0951988442369788e+04 },
    { 1.9999999999999999e-11, 1.5843579411900000e+00, 2.5440892308259390e+04 },
    { 1.9999999999999999e-11, 3.3559141310700000e+00, 8.6457984757931190e+04 },
    { 1.9999999999999999e-11, 4.9461578221900000e+00, 9.4329775286197386e+04 },
    { 1.9999999999999999e-11, 5.6492283760899999e+00, 2.6713573333886590e+04 },
    { 1.9999999999999999e-11, 1.4396655120000001e-02, 2.5462232949261801e+04 },
    { 1.9999999999999999e-11, 3.0892503803200002e+00, 1.1882896374949679e+05 },
    { 1.9999999999999999e-11, 1.9701217386300001e+00, 9.9024136457913992e+04 },
    { 1.9999999999999999e-11, 4.1167473043399996e+00, 2.3919142659291600e+04 },
    { 1.9999999999999999e-11, 1.5468777268700000e+00, 2.8256663623856792e+04 },
    { 1.9999999999999999e-11, 4.7968456901899996e+00, 3.5077370847006590e+04 },
    { 1.9999999999999999e-11, 5.6200277173700002e+00, 2.7566769715638591e+04 },
    { 9.9999999999999994e-12, 4.6397149488400000e+00, 2.7351060504152389e+04 },
    { 9.9999999999999994e-12, 1.0239100823700000e+00, 2.4824745778995999e+04 },
    { 9.9999999999999994e-12, 4.6688096835800001e+00, 7.8697421162599392e+04 },
    { 9.9999999999999994e-12, 3.2490506719100001e+00, 5.1596118696149599e+04 },
    { 9.9999999999999994e-12, 3.0338304094400002e+00, 2.3134855332920235e+05 },
    { 9.9999999999999994e-12, 1.2553753136900001e+00, 1.9393798608932379e+05 },
    { 9.9999999999999994e-12, 4.7170430755900004e+00, 5.4879422437824003e+04 },
    { 9.9999999999999994e-12, 4.1952512627400003e+00, 1.8985399758261937e+05 },
    { 9.9999999999999994e-12, 4.7919152872800002e+00, 5.2509662223917803e+04 },
    { 9.9999999999999994e-12, 1.5830059177000000e+00, 2.8199553454951791e+04 },
    { 9.9999999999999994e-12, 5.3708606274099999e+00, 8.7253177130154960e+04 },
    { 9.9999999999999994e-12, 3.6331366873999998e-01, 1.0312779472551816e+05 },
    { 9.9999999999999994e-12, 5.6287398103199999e+00, 1.1254588789950538e+05 },
    { 9.9999999999999994e-12, 2.5317348819499998e+00, 1.1223170171963578e+05 },
    { 9.9999999999999994e-12, 3.5002394372699999e+00, 2.6933985976325399e+04 },
    { 9.9999999999999994e-12, 5.7318968992999997e-01, 5.0380547839427389e+04 },
    { 9.9999999999999994e-12, 4.8719803132799999e+00, 2.4156183887541335e+05 },
    { 9.9999999999999994e-12, 4.3868636314099998e+00, 8.0174908907939782e+04 },
    { 9.9999999999999994e-12, 3.9805800940400000e+00, 4.9953949648551388e+04 },
    { 9.9999999999999994e-12, 7.0022281901000005e-01, 1.5546092337225520e+05 },
    { 9.9999999999999994e-12, 7.0726844016000001e-01, 1.7917274704605396e+05 },
    { 9.9999999999999994e-12, 4.3732148829400002e+00, 2.6926872429324590e+04 },
    { 9.9999999999999994e-12, 1.2904101482699999e+00, 2.5248933853823801e+04 },
    { 9.9999999999999994e-12, 5.3232486597099999e+00, 1.3942898341330336e+05 },
    { 9.9999999999999994e-12, 1.5165094424800001e+00, 4.8997660492580799e+04 },
    { 9.9999999999999994e-12, 4.6930727585999996e+00, 9.0989162859497592e+04 },
    { 9.9999999999999994e-12, 5.4761639709999999e-01, 2.3962025734194991e+04 },
    { 9.9999999999999994e-12, 5.2207065255699998e+00, 2.7005833427555990e+04 },
    { 9.9999999999999994e-12, 4.4291850564000002e-01, 2.5169972855592401e+04 },
    { 9.9999999999999994e-12, 4.4392645167899998e+00, 1.4398093691036216e+05 },
    { 9.9999999999999994e-12, 4.9971576103400004e+00, 5.0689826162083191e+04 },
    { 9.9999999999999994e-12, 2.6741554518999999e-01, 1.6329812945069079e+05 },
    { 9.9999999999999994e-12, 3.6796362721799998e+00, 1.3376765136415116e+05 },
    { 9.9999999999999994e-12, 1.2489039253900001e+00, 1.2244498726584678e+05 },
    { 9.9999999999999994e-12, 4.0747361834699998e+00, 7.6152059111344992e+04 },
    { 9.9999999999999994e-12, 1.4145385900000000e+00, 8.4307938006097989e+04 },
    { 9.9999999999999994e-12, 5.8204025979200003e+00, 2.4072921469776400e+04 },
    { 9.9999999999999994e-12, 6.1264077404700004e+00, 2.8102884813371998e+04 },
    { 9.9999999999999994e-12, 5.1650041444000006e-01, 2.4844415539895788e+04 },
    { 9.9999999999999994e-12, 5.1471246167700002e+00, 2.7331390743252599e+04 },
    { 9.9999999999999994e-12, 1.9597424653400000e+00, 1.5440865498906677e+05 },
    { 9.9999999999999994e-12, 1.8002802465300001e+00, 1.0626281204951399e+05 },
    { 9.9999999999999994e-12, 5.6263178559400000e+00, 5.8220034864523797e+04 },
    { 9.9999999999999994e-12, 2.3798810491600002e+00, 9.7670387712897180e+04 },
    { 9.9999999999999994e-12, 4.9900540597900003e+00, 1.1789303376878797e+05 },
    { 9.9999999999999994e-12, 4.6316989568800002e+00, 2.4356780788641601e+04 },
    { 9.9999999999999994e-12, 1.0319260743300001e+00, 2.7819025494506801e+04 },
    { 9.9999999999999994e-12, 2.0001845374400000e+00, 2.4395737472071800e+04 },
    { 9.9999999999999994e-12, 6.4266724685999999e-01, 1.4964226895951436e+05 },
    { 9.9999999999999994e-12, 1.8670838786399999e+00, 1.3235071519108818e+05 },
    { 9.9999999999999994e-12, 3.7230927290500002e+00, 1.4853289040742096e+05 },
    { 9.9999999999999994e-12, 3.3911349458000001e-01, 2.0965882487420217e+05 },
    { 9.9999999999999994e-12, 3.1724393274499998e+00, 1.2511203959948818e+05 },
    { 9.9999999999999994e-12, 5.0502462172199998e+00, 1.3721022630911658e+05 },
    { 9.9999999999999994e-12, 5.4143642272600001e+00, 2.3704710126797989e+04 },
    { 9.9999999999999994e-12, 2.4926080395000000e-01, 2.8471096156350399e+04 },
    { 9.9999999999999994e-12, 5.2198522397999998e+00, 5.1439929769616589e+04 },
    { 9.9999999999999994e-12, 4.0264946424800003e+00, 2.6177388889420188e+04 },
    { 9.9999999999999994e-12, 1.6371303887299999e+00, 2.5998417393728199e+04 },
    { 9.9999999999999994e-12, 1.9871349945000001e+00, 2.3368174972207437e+05 },
    { 9.9999999999999994e-12, 3.1731818157900000e+00, 2.1650388933200611e+04 },
    { 9.9999999999999994e-12, 2.4904432154200000e+00, 3.0525417349947791e+04 },
    { 9.9999999999999994e-12, 1.9731686066000001e-01, 2.6094531700474210e+04 },
    { 9.9999999999999994e-12, 4.1505555048899998e+00, 2.4203001978156801e+04 },
    { 9.9999999999999994e-12, 1.5130695263300000e+00, 2.7972804304991590e+04 },
    { 9.9999999999999994e-12, 3.8219801651899998e+00, 1.0419783375581198e+05 },
    { 9.9999999999999994e-12, 2.8602805205599999e+00, 5.0167248743989388e+04 },
    { 9.9999999999999994e-12, 1.5485424763700000e+00, 7.8413272621857192e+04 },
    { 9.9999999999999994e-12, 6.2816268996000000e+00, 1.5653130018480320e+05 },
    { 9.9999999999999994e-12, 6.0952533246399998e+00, 2.5227593212821390e+04 },
    { 9.9999999999999994e-12, 4.5133845814900004e+00, 2.5151973160865389e+04 },
    { 9.9999999999999994e-12, 1.1502404497200001e+00, 2.7023833122282998e+04 },
    { 9.9999999999999994e-12, 4.2611828788899997e+00, 2.4722317092456558e+05 },
    { 9.9999999999999994e-12, 4.5661189797099997e+00, 1.0201841617342478e+05 },
    { 9.9999999999999994e-12, 1.4582141841600000e+00, 1.6376609444104519e+05 },
    { 9.9999999999999994e-12, 3.9646181674599998e+00, 1.8367470392120857e+05 },
    { 9.9999999999999994e-12, 6.0692067902900000e+00, 2.6126036177212001e+04 },
    { 9.9999999999999994e-12, 5.8776035480999997e+00, 2.6049770105936401e+04 },
    { 9.9999999999999994e-12, 6.2654008135400003e+00, 1.0767974822257696e+05 },
    { 9.9999999999999994e-12, 5.3489236341600002e+00, 2.7388729959779190e+04 },
    { 9.9999999999999994e-12, 3.1470139705000000e-01, 2.4787076323369201e+04 },
    { 9.9999999999999994e-12, 5.1689218530000003e-01, 1.5547515046625677e+05 },
    { 9.9999999999999994e-12, 5.1733955627899997e+00, 2.4602612434870989e+04 },
    { 9.9999999999999994e-12, 4.9022946841999998e-01, 2.7573193848277399e+04 },
    { 9.9999999999999994e-12, 4.0614537067000001e-01, 1.9848993958638259e+05 },
    { 9.9999999999999994e-12, 5.1350910215700001e+00, 1.5653453239644598e+05 },
    { 9.9999999999999994e-12, 2.2325916594600002e+00, 6.1165273988580790e+04 },
    { 9.9999999999999994e-12, 5.2115552294000000e-01, 1.3000580396999417e+05 },
    { 9.9999999999999994e-12, 6.1669479944000005e-01, 7.7039891583943958e+04 },
    { 9.9999999999999994e-12, 6.2803198306499999e+00, 1.2921569786709236e+05 },
    { 9.9999999999999994e-12, 5.2547592058300001e+00, 2.5923467230984788e+04 },
    { 9.9999999999999994e-12, 4.0886582537999999e-01, 2.6252339052163610e+04 },
    { 9.9999999999999994e-12, 1.8247459416400000e+00, 7.3891833057948388e+04 },
    { 9.9999999999999994e-12, 1.2051856656700000e+00, 1.2606763934109679e+05 },
    { 9.9999999999999994e-12, 2.4016642711300000e+00, 3.0171891648278601e+04 },
    { 9.9999999999999994e-12, 3.2619607600800000e+00, 2.2003914634869801e+04 },
};

static const VSOP87Term VSOP87A_MERCURY_X2[299] = {
    { 1.4841849999999999e-05, 4.3540121026899996e+00, 5.2175806283148399e+04 },
    { 1.2149950000000000e-05, 0.0000000000000000e+00, 0.0000000000000000e+00 },
    { 9.0746699999999993e-06, 1.1321634301800001e+00, 7.8263709424722590e+04 },
    { 3.6880900000000000e-06, 4.1870594412599997e+00, 1.0435161256629678e+05 },
    { 2.5430600000000000e-06, 4.1281737714000002e+00, 2.6087903141574199e+04 },
    { 1.3014900000000001e-06, 9.5681684788999999e-01, 1.3043951570787099e+05 },
    { 4.2868000000000000e-07, 4.0089219672600001e+00, 1.5652741884944518e+05 },
    { 1.3566000000000001e-07, 7.7740665693000000e-01, 1.8261532199101939e+05 },
    { 4.1840000000000003e-08, 3.8288421051600001e+00, 2.0870322513259359e+05 },
    { 2.9060000000000000e-08, 1.3352965530500001e+00, 2.4978524589480799e+04 },
    { 2.6770000000000001e-08, 4.3151764727700002e+00, 2.7197281693667599e+04 },
    { 1.2680000000000000e-08, 5.9695956916000004e-01, 2.3479112827416777e+05 },
    { 1.0109999999999999e-08, 4.4049023041500002e+00, 5.1066427731055002e+04 },
    { 3.7900000000000004e-09, 3.6477276561699998e+00, 2.6087903141574195e+05 },
    { 3.5400000000000002e-09, 1.0891756234500001e+00, 5.3285184835241802e+04 },
    { 3.5199999999999998e-09, 1.1817031615300000e+00, 7.7154330872629187e+04 },
    { 3.4999999999999999e-09, 1.7728070279599999e+00, 1.1093785520934000e+03 },
    { 1.9300000000000002e-09, 4.1564810748100003e+00, 2.5132303399965600e+04 },
    { 1.7700000000000001e-09, 1.1261156069900000e+00, 1.5890728952838001e+03 },
    { 1.7100000000000001e-09, 1.4749013873800001e+00, 2.7043502883182799e+04 },
    { 1.6100000000000000e-09, 4.9022477180000003e-02, 1.0593819301891999e+03 },
    { 1.3999999999999999e-09, 1.8498391049600000e+00, 2.1535949644515400e+04 },
    { 1.3999999999999999e-09, 4.9940824671200001e+00, 2.0426571092422000e+04 },
    { 1.2400000000000001e-09, 1.4240566371700001e+00, 2.3869146037387400e+04 },
    { 1.2000000000000000e-09, 4.2379210915899996e+00, 1.0324223401420339e+05 },
    { 1.1700000000000001e-09, 5.8859277206699998e+00, 2.6617594106668799e+04 },
    { 1.1599999999999999e-09, 1.8284658595700001e+00, 4.6514474233996203e+04 },
    { 1.1500000000000000e-09, 6.0930613975199996e+00, 2.5558212176479599e+04 },
    { 1.1200000000000001e-09, 4.1587357893999999e-01, 2.8696693455731618e+05 },
    { 1.0700000000000000e-09, 2.5777243245600001e+00, 1.4765239043269799e+04 },
    { 1.0500000000000001e-09, 2.9429779427299998e+00, 2.7147285071763388e+04 },
    { 1.0300000000000000e-09, 3.7821677705700001e+00, 4.5519534970588002e+03 },
    { 1.0300000000000000e-09, 4.2129725789699997e+00, 2.8306660245760992e+04 },
    { 9.9000000000000010e-10, 6.6196730607999998e-01, 3.1749235190726398e+04 },
    { 8.4999999999999996e-10, 5.1449254656600001e+00, 1.9317192540328600e+04 },
    { 6.9999999999999996e-10, 1.7202133343199999e+00, 2.2645328196608789e+04 },
    { 6.8000000000000003e-10, 1.1271966893000000e-01, 9.1039069941176003e+03 },
    { 6.6000000000000000e-10, 3.9201650325899999e+00, 3.4425749449653999e+03 },
    { 6.5000000000000003e-10, 6.5463924387000005e-01, 5.1220206541539788e+04 },
    { 6.0999999999999996e-10, 4.5339775730199996e+00, 5.0586733387864588e+04 },
    { 5.9000000000000003e-10, 4.8886982661299996e+00, 7.2602377375570388e+04 },
    { 5.9000000000000003e-10, 3.1050716338000002e+00, 3.7410567239878597e+04 },
    { 5.9000000000000003e-10, 2.5612205311700000e+00, 5.2705497248242988e+04 },
    { 5.9000000000000003e-10, 2.6533851218300000e+00, 5.2969096509459996e+02 },
    { 5.7000000000000000e-10, 3.8064422388199999e+00, 3.0639856638632999e+04 },
    { 5.4999999999999996e-10, 5.6198241750899998e+00, 5.1116424352959199e+04 },
    { 5.3000000000000003e-10, 4.6679718569000000e-01, 3.2858613742819791e+04 },
    { 5.3000000000000003e-10, 5.1652690046999998e+00, 9.5559974160859997e+02 },
    { 5.1000000000000000e-10, 5.3470945718599996e+00, 6.3989728631399998e+02 },
    { 4.8999999999999996e-10, 4.0277699318700000e+00, 2.7676976036857999e+04 },
    { 4.5000000000000000e-10, 1.6002605373400001e+00, 2.5973463852888959e+04 },
    { 4.5000000000000000e-10, 5.9536199805900001e+00, 5.3235188213337591e+04 },
    { 4.3999999999999998e-10, 4.0581023257700002e+00, 2.6202342430259410e+04 },
    { 4.3999999999999998e-10, 3.9198717387599999e+00, 2.9530478086539599e+04 },
    { 4.0000000000000001e-10, 1.0119383818000001e+00, 1.2933013715577759e+05 },
    { 3.9000000000000000e-10, 2.6549311101800002e+00, 2.5874604046136199e+04 },
    { 3.9000000000000000e-10, 5.6491127240900001e+00, 4.0853142184844000e+04 },
    { 3.7999999999999998e-10, 2.9669423335800000e+00, 2.6301202237012199e+04 },
    { 3.7999999999999998e-10, 4.2789190512599999e+00, 5.3131406024757001e+04 },
    { 3.7999999999999998e-10, 1.6904270601900000e+00, 2.2187571041868000e+03 },
    { 3.6000000000000000e-10, 1.8095334688300000e+00, 2.1187638603783998e+03 },
    { 3.3000000000000000e-10, 6.2515314784999998e-01, 5.6727759780207198e+04 },
    { 3.3000000000000000e-10, 3.4694711313700002e+00, 3.1305483769889036e+05 },
    { 3.1999999999999998e-10, 4.4877441742800004e+00, 4.9957049178961599e+04 },
    { 3.1000000000000002e-10, 4.2671368814399999e+00, 6.7707106012455997e+03 },
    { 3.1000000000000002e-10, 1.9386603393599999e+00, 4.7623852786089599e+04 },
    { 2.7000000000000000e-10, 9.8357005483000004e-01, 5.4394563387335191e+04 },
    { 2.5000000000000002e-10, 5.5538455151599999e+00, 4.3071899289030800e+04 },
    { 2.5000000000000002e-10, 1.6642960536900000e+00, 9.8690280517144580e+04 },
    { 2.4000000000000000e-10, 5.5151260399900002e+00, 7.8793400389817180e+04 },
    { 2.4000000000000000e-10, 6.1193595038000004e-01, 1.8849227549974199e+04 },
    { 2.3000000000000001e-10, 3.5612011284700000e+00, 7.7308109683113988e+04 },
    { 2.1999999999999999e-10, 1.9275676523300000e+00, 4.5405095681902800e+04 },
    { 2.1000000000000000e-10, 2.6809386336299998e+00, 1.3655860491176400e+04 },
    { 1.8000000000000000e-10, 3.6102569446100001e+00, 5.8946516884393990e+04 },
    { 1.8000000000000000e-10, 6.1418759198900004e+00, 2.1329909543799999e+02 },
    { 1.8000000000000000e-10, 1.7399098118900000e+00, 5.2156136522248598e+04 },
    { 1.8000000000000000e-10, 3.1168509807000000e-01, 5.1535908996834391e+04 },
    { 1.7000000000000001e-10, 2.7291623415899999e+00, 7.9323091354911783e+04 },
    { 1.7000000000000001e-10, 2.4077379672200001e+00, 1.2566151699982800e+04 },
    { 1.7000000000000001e-10, 6.1062092735300002e+00, 5.2389105378586399e+04 },
    { 1.7000000000000001e-10, 3.2178718679000000e+00, 5.1646115318053788e+04 },
    { 1.7000000000000001e-10, 4.7473519960999999e-01, 4.2659819087599999e+02 },
    { 1.5999999999999999e-10, 1.6658333932500000e+00, 2.3754706748702189e+04 },
    { 1.5999999999999999e-10, 1.8566366544799999e+00, 4.8733231338182988e+04 },
    { 1.5999999999999999e-10, 4.1513335606000004e+00, 7.9373087976815994e+04 },
    { 1.5999999999999999e-10, 6.9576423226999995e-01, 5.5618381228113802e+04 },
    { 1.5000000000000000e-10, 5.3982266680200004e+00, 3.8654054841556987e+04 },
    { 1.5000000000000000e-10, 4.3760119840399998e+00, 8.5319638175199998e+02 },
    { 1.5000000000000000e-10, 6.6471022805000002e-01, 5.2195476044048191e+04 },
    { 1.5000000000000000e-10, 4.9596414534199997e+00, 3.3326578733174203e+04 },
    { 1.4000000000000001e-10, 5.0966470636099999e+00, 5.2179687618506403e+04 },
    { 1.4000000000000001e-10, 3.6899182348799999e+00, 8.2815662921781390e+04 },
    { 1.4000000000000001e-10, 2.4282872598399998e+00, 6.6941045326418185e+04 },
    { 1.4000000000000001e-10, 3.6353872535899998e+00, 1.1001294461544899e+05 },
    { 1.4000000000000001e-10, 6.9791035823000003e-01, 5.3764879178432202e+04 },
    { 1.4000000000000001e-10, 4.1724899668399997e+00, 5.6613320491521999e+03 },
    { 1.2999999999999999e-10, 1.0748470696100001e+00, 7.6674636529438780e+04 },
    { 1.2999999999999999e-10, 6.7881557905000001e-01, 8.3925041473874793e+04 },
    { 1.2999999999999999e-10, 4.0601257913600000e+00, 1.5541804029735178e+05 },
    { 1.2999999999999999e-10, 3.5972706913599999e+00, 5.2171924947790401e+04 },
    { 1.2999999999999999e-10, 3.1490831870200000e+00, 3.6301188687785187e+04 },
    { 1.2999999999999999e-10, 2.1349002954200000e+00, 2.6727800427888200e+04 },
    { 1.2999999999999999e-10, 6.3715146769999997e-01, 1.1322664098304400e+04 },
    { 1.2000000000000000e-10, 3.8607281065199999e+00, 5.0057042422769999e+04 },
    { 1.2000000000000000e-10, 4.0010717906600002e+00, 2.8421099534446199e+04 },
    { 1.2000000000000000e-10, 3.1429052360799998e+00, 2.6514501332450200e+04 },
    { 1.0999999999999999e-10, 1.6725069606400000e+00, 2.4498830246290399e+04 },
    { 1.0999999999999999e-10, 3.0072189002399998e+00, 3.8519945791972001e+04 },
    { 1.0000000000000000e-10, 6.1816962149999997e+00, 6.2389091829359393e+04 },
    { 1.0000000000000000e-10, 8.3598356131999996e-01, 5.2290245571833613e+04 },
    { 1.0000000000000000e-10, 3.1112628717300002e+00, 3.5191810135691798e+04 },
    { 1.0000000000000000e-10, 2.5227870566999999e+00, 1.5874617595363199e+04 },
    { 1.0000000000000000e-10, 2.4410880924000000e-01, 3.3914274084046460e+05 },
    { 1.0000000000000000e-10, 2.9353947452000000e+00, 2.5028521211384999e+04 },
    { 8.9999999999999999e-11, 1.2575781585900001e+00, 7.6044952320535798e+04 },
    { 8.9999999999999999e-11, 4.7098403805400002e+00, 1.2477818365871879e+05 },
    { 8.9999999999999999e-11, 4.7818660462500002e+00, 7.8244039663822798e+04 },
    { 8.9999999999999999e-11, 4.6645420711299996e+00, 5.2061366994463169e+04 },
    { 8.9999999999999999e-11, 3.2314302680400000e+00, 3.9609654583165597e+04 },
    { 8.9999999999999999e-11, 2.1912195251400002e+00, 1.0488130353139139e+05 },
    { 8.9999999999999999e-11, 4.9159516199000004e+00, 1.6983996147456601e+04 },
    { 8.9999999999999999e-11, 2.0735942028000001e-01, 7.9945284420241996e+03 },
    { 7.9999999999999995e-11, 4.0454991575500001e+00, 2.3331963928720002e+03 },
    { 7.9999999999999995e-11, 4.7448541233099997e+00, 2.8206667001952599e+04 },
    { 7.9999999999999995e-11, 2.3967004016999999e-01, 1.0339601282468818e+05 },
    { 7.9999999999999995e-11, 3.7987106631000001e-01, 1.3610084775702318e+05 },
    { 7.9999999999999995e-11, 6.0920060214000005e-01, 7.9219309166331193e+04 },
    { 7.9999999999999995e-11, 4.5328868577999998e-01, 1.2432042650397800e+04 },
    { 7.9999999999999995e-11, 2.0946147414400000e+00, 7.2386755916000002e+03 },
    { 7.9999999999999995e-11, 6.1963468763299998e+00, 6.3498470381452789e+04 },
    { 7.9999999999999995e-11, 4.0020768003800002e+00, 5.2182919830149200e+04 },
    { 7.9999999999999995e-11, 3.3691422036700001e+00, 4.4937130691548402e+04 },
    { 7.0000000000000004e-11, 2.3654783283200000e+00, 6.9159802430604992e+04 },
    { 7.0000000000000004e-11, 4.7153187114700001e+00, 5.2168692736147590e+04 },
    { 7.0000000000000004e-11, 2.1061002797000001e+00, 6.4741957983131193e+04 },
    { 7.0000000000000004e-11, 1.8591135483000001e+00, 7.8267590760080580e+04 },
    { 7.0000000000000004e-11, 5.2351217121599998e+00, 1.8207813988235212e+04 },
    { 7.0000000000000004e-11, 6.1599962247000004e+00, 5.1962507187710398e+04 },
    { 7.0000000000000004e-11, 4.9854555160100000e+00, 7.1492998823476984e+04 },
    { 7.0000000000000004e-11, 3.7089491258299998e+00, 7.8283379185622383e+04 },
    { 6.0000000000000000e-11, 5.8044351967800001e+00, 1.0541099449648599e+05 },
    { 6.0000000000000000e-11, 2.4028462570300002e+00, 1.2938013377768178e+05 },
    { 6.0000000000000000e-11, 6.2382539105600001e+00, 5.2602404474024399e+04 },
    { 6.0000000000000000e-11, 2.9304385946600000e+00, 7.8477008520160583e+04 },
    { 6.0000000000000000e-11, 4.6188986304000001e-01, 1.0890356606335558e+05 },
    { 6.0000000000000000e-11, 6.0834861577700003e+00, 1.0213285546211000e+04 },
    { 6.0000000000000000e-11, 5.4475797807899999e+00, 1.0329223063610759e+05 },
    { 5.0000000000000002e-11, 2.9615544773300000e+00, 8.8476994970933592e+04 },
    { 5.0000000000000002e-11, 3.5912786218999999e+00, 2.5661304950698199e+04 },
    { 5.0000000000000002e-11, 4.9574606924999998e-01, 3.3967992294913187e+04 },
    { 5.0000000000000002e-11, 3.7544167515700000e+00, 8.1706284369687986e+04 },
    { 5.0000000000000002e-11, 5.5367343393399997e+00, 9.4138327020085781e+04 },
    { 5.0000000000000002e-11, 5.8966456876100004e+00, 5.7837138332300601e+04 },
    { 5.0000000000000002e-11, 4.8302766751300004e+00, 7.4821134479757180e+04 },
    { 5.0000000000000002e-11, 1.2996411113699999e+00, 5.1322609901396390e+04 },
    { 5.0000000000000002e-11, 2.4907466999599999e+00, 6.8050423878511589e+04 },
    { 5.0000000000000002e-11, 3.8271939187699999e+00, 1.0546099111839019e+05 },
    { 5.0000000000000002e-11, 3.5178516619000000e-01, 7.8259828089364586e+04 },
    { 5.0000000000000002e-11, 5.2763659913399996e+00, 5.1749208092272391e+04 },
    { 5.0000000000000002e-11, 2.2804238075000000e-01, 7.7734018459627987e+04 },
    { 5.0000000000000002e-11, 1.4423922808500000e+00, 5.9414481874748402e+04 },
    { 5.0000000000000002e-11, 5.4894339123900000e+00, 9.3028948467992392e+04 },
    { 5.0000000000000002e-11, 5.6047614787100004e+00, 4.4181277841124189e+04 },
    { 3.9999999999999998e-11, 1.3028235846000000e+00, 7.8256595877721789e+04 },
    { 3.9999999999999998e-11, 6.0686919458100004e+00, 6.4607848933546193e+04 },
    { 3.9999999999999998e-11, 4.1269315359899998e+00, 2.9416038797854391e+04 },
    { 3.9999999999999998e-11, 1.5366934952200000e+00, 2.2759767485294011e+04 },
    { 3.9999999999999998e-11, 8.3479952169000005e-01, 1.8150594343892598e+05 },
    { 3.9999999999999998e-11, 3.6926449000399999e+00, 7.9852782320006387e+04 },
    { 3.9999999999999998e-11, 3.4140382700999999e-01, 7.6144945564344191e+04 },
    { 3.9999999999999998e-11, 7.6148268878000003e-01, 7.7204327494533383e+04 },
    { 3.9999999999999998e-11, 5.4914251637599998e+00, 1.5546803691925600e+05 },
    { 3.9999999999999998e-11, 2.4011060046599999e+00, 3.2370978991565600e+04 },
    { 3.9999999999999998e-11, 7.7482738791000005e-01, 5.4509002676020398e+04 },
    { 3.9999999999999998e-11, 7.2336948817000002e-01, 7.8270822971723392e+04 },
    { 3.9999999999999998e-11, 4.1262880495300003e+00, 7.8800891533389904e+03 },
    { 3.9999999999999998e-11, 3.9892264034500000e+00, 8.0482466528909383e+04 },
    { 3.9999999999999998e-11, 5.7484151802400003e+00, 3.9743763632750597e+04 },
    { 3.9999999999999998e-11, 5.6941747035199999e+00, 1.3521751441591399e+04 },
    { 3.9999999999999998e-11, 4.0993046738499999e+00, 2.7311720982352810e+04 },
    { 3.9999999999999998e-11, 1.5643203573600000e+00, 2.4864085300795588e+04 },
    { 3.0000000000000000e-11, 3.4232064416700001e+00, 1.6218875089859738e+05 },
    { 3.0000000000000000e-11, 1.5631779594199999e+00, 1.0433194280539699e+05 },
    { 3.0000000000000000e-11, 4.8879555402900001e+00, 1.2588756221081219e+05 },
    { 3.0000000000000000e-11, 3.3893547425800001e+00, 7.7623812138408583e+04 },
    { 3.0000000000000000e-11, 3.7797120904399999e+00, 2.5448005855260191e+04 },
    { 3.0000000000000000e-11, 2.2766481117400001e+00, 1.2022623016165999e+05 },
    { 3.0000000000000000e-11, 1.9264054357600000e+00, 9.9799659069237983e+04 },
    { 3.0000000000000000e-11, 5.3291276134899999e+00, 5.8458882133139792e+04 },
    { 3.0000000000000000e-11, 5.9916519697500004e+00, 7.3711755927663791e+04 },
    { 3.0000000000000000e-11, 4.3184314583000001e+00, 1.0213285546210999e+05 },
    { 3.0000000000000000e-11, 3.2494370000899999e+00, 1.2948391596626239e+05 },
    { 3.0000000000000000e-11, 1.4903556701700000e+00, 1.5086608680029298e+05 },
    { 3.0000000000000000e-11, 5.1347074178099996e+00, 9.0829861124705378e+04 },
    { 3.0000000000000000e-11, 2.7198686518100001e+00, 1.9804827291582798e+04 },
    { 3.0000000000000000e-11, 2.8166883192199998e+00, 1.0382192160120218e+05 },
    { 3.0000000000000000e-11, 5.2035179081200003e+00, 1.3096920667296558e+05 },
    { 3.0000000000000000e-11, 5.2414566535999996e-01, 6.2830758499914000e+03 },
    { 3.0000000000000000e-11, 1.1974212040100001e+00, 2.6941099523326200e+04 },
    { 3.0000000000000000e-11, 6.2224887667899997e+00, 7.1025033833122594e+04 },
    { 3.0000000000000000e-11, 4.9224999951299999e+00, 1.0435549390165479e+05 },
    { 3.0000000000000000e-11, 5.2427454830900002e+00, 5.2815703569462399e+04 },
    { 1.9999999999999999e-11, 1.6003417850099999e+00, 3.3281356562801898e+03 },
    { 1.9999999999999999e-11, 2.5229971829900002e+00, 2.6484548254729998e+03 },
    { 1.9999999999999999e-11, 6.0522856235500004e+00, 1.1456489811250778e+05 },
    { 1.9999999999999999e-11, 3.8985657433999998e+00, 7.8378148713407805e+04 },
    { 1.9999999999999999e-11, 3.2873060850700000e+00, 3.6523064398203877e+05 },
    { 1.9999999999999999e-11, 2.8442498113100001e+00, 7.7197213947532582e+04 },
    { 1.9999999999999999e-11, 3.0549673273100000e+00, 7.8690307615598591e+04 },
    { 1.9999999999999999e-11, 4.7881810383000001e-01, 1.0437128232719658e+05 },
    { 1.9999999999999999e-11, 1.0075531072899999e+00, 2.3969139281195799e+04 },
    { 1.9999999999999999e-11, 2.3594604845600000e+00, 7.7211441041534184e+04 },
    { 1.9999999999999999e-11, 4.2974993645000001e+00, 1.0434449901929598e+05 },
    { 1.9999999999999999e-11, 9.2718547499999998e-02, 4.1962520736937397e+04 },
    { 1.9999999999999999e-11, 3.7031071803700000e+00, 7.8050410329284583e+04 },
    { 1.9999999999999999e-11, 3.1811079756799998e+00, 2.5450906869549999e+04 },
    { 1.9999999999999999e-11, 2.4825170555299998e+00, 2.6724899413598399e+04 },
    { 1.9999999999999999e-11, 1.7558960816400000e+00, 9.7580901965051191e+04 },
    { 1.9999999999999999e-11, 6.1148265675499998e+00, 6.1279713277265997e+04 },
    { 1.9999999999999999e-11, 2.5696336888700002e+00, 1.3149889763806018e+05 },
    { 1.9999999999999999e-11, 3.4029690456799999e+00, 1.8093374699549990e+04 },
    { 1.9999999999999999e-11, 1.4334092672000001e+00, 7.8149270136037361e+04 },
    { 1.9999999999999999e-11, 5.0955931589000003e-01, 1.0779418751126219e+05 },
    { 1.9999999999999999e-11, 6.0164826554499999e+00, 1.0456491166173479e+05 },
    { 1.9999999999999999e-11, 2.0094971601100000e+00, 4.4295717129809404e+04 },
    { 1.9999999999999999e-11, 3.7885555911000002e+00, 1.0435872611329758e+05 },
    { 1.9999999999999999e-11, 3.4150470476799999e+00, 1.0434773123093879e+05 },
    { 1.9999999999999999e-11, 3.5006932310400001e+00, 1.3499146920492980e+05 },
    { 1.9999999999999999e-11, 1.4452317713200000e+00, 5.4294570143526791e+04 },
    { 1.9999999999999999e-11, 5.8556928870400000e+00, 2.7154398618764200e+04 },
    { 1.9999999999999999e-11, 5.6037955654799996e+00, 1.2990982474277639e+05 },
    { 9.9999999999999994e-12, 1.5761479745200000e+00, 1.5197546535238638e+05 },
    { 9.9999999999999994e-12, 1.5538967236000001e-01, 1.3154889425996438e+05 },
    { 9.9999999999999994e-12, 1.3924678529000001e-01, 1.1787336400788819e+05 },
    { 9.9999999999999994e-12, 2.2710741768000000e+00, 1.8155594006083018e+05 },
    { 9.9999999999999994e-12, 3.4653162003200002e+00, 9.1785460866313981e+04 },
    { 9.9999999999999994e-12, 5.1779294769999998e-02, 2.7140171524762591e+04 },
    { 9.9999999999999994e-12, 1.8945575369000001e+00, 8.4546785274713984e+04 },
    { 9.9999999999999994e-12, 9.3976716900000001e-01, 5.5503941939428587e+04 },
    { 9.9999999999999994e-12, 5.8976623027999997e+00, 1.0328511708910679e+05 },
    { 9.9999999999999994e-12, 5.8625817602999997e-01, 9.8068536716305389e+04 },
    { 9.9999999999999994e-12, 3.6734492234000000e+00, 1.0530721230790539e+05 },
    { 9.9999999999999994e-12, 1.7566823495899999e+00, 4.9842609890276392e+04 },
    { 9.9999999999999994e-12, 1.8883482364000001e-01, 1.8827665404017159e+05 },
    { 9.9999999999999994e-12, 5.3342143444000003e+00, 1.4631413330323418e+05 },
    { 9.9999999999999994e-12, 4.0536401965400000e+00, 6.5697557724739789e+04 },
    { 9.9999999999999994e-12, 2.8799656846600001e+00, 3.9629324344065390e+04 },
    { 9.9999999999999994e-12, 2.7836593465499999e+00, 1.2546481939083000e+04 },
    { 9.9999999999999994e-12, 8.4435710643999995e-01, 2.6080789594573391e+04 },
    { 9.9999999999999994e-12, 4.9622610538799998e+00, 2.9428515568274001e+04 },
    { 9.9999999999999994e-12, 7.0136397733000000e-01, 2.2747290714874402e+04 },
    { 9.9999999999999994e-12, 4.5885935049200004e+00, 4.8847670626868203e+04 },
    { 9.9999999999999994e-12, 2.1694136110800000e+00, 1.1911685160956658e+05 },
    { 9.9999999999999994e-12, 2.4585281392899998e+00, 2.5021407664384202e+04 },
    { 9.9999999999999994e-12, 3.4446508161000000e-01, 2.6107572902473989e+04 },
    { 9.9999999999999994e-12, 1.6919019737800001e+00, 2.6068233380674399e+04 },
    { 9.9999999999999994e-12, 1.1231008203299999e+00, 1.0657036967048359e+05 },
    { 9.9999999999999994e-12, 2.5164940914399998e+00, 6.5831666774324796e+04 },
    { 9.9999999999999994e-12, 4.0144081263999998e-01, 1.0594068546158058e+05 },
    { 9.9999999999999994e-12, 8.6957384492000001e-01, 5.3399624123927002e+04 },
    { 9.9999999999999994e-12, 1.1740948034800001e+00, 2.6095016688575000e+04 },
    { 9.9999999999999994e-12, 3.8263610629999998e+00, 8.0596905817594597e+04 },
    { 9.9999999999999994e-12, 6.2221363363100002e+00, 8.5034420025968182e+04 },
    { 9.9999999999999994e-12, 3.8109626329799999e+00, 7.1980633574731182e+04 },
    { 9.9999999999999994e-12, 3.5325856866300001e+00, 6.0055895436487393e+04 },
    { 9.9999999999999994e-12, 5.0398367952200003e+00, 2.6091784476932200e+04 },
    { 9.9999999999999994e-12, 1.7839099273500001e+00, 1.1691776426627958e+05 },
    { 9.9999999999999994e-12, 3.1374303723999999e+00, 4.9527351457675388e+04 },
    { 9.9999999999999994e-12, 5.1544717659400003e+00, 1.0021837280099400e+04 },
    { 9.9999999999999994e-12, 3.6613348683500000e+00, 5.2643771273502804e+04 },
    { 9.9999999999999994e-12, 3.4983918032200001e+00, 8.5502385016322587e+04 },
    { 9.9999999999999994e-12, 3.2755700703200001e+00, 2.6084021806216198e+04 },
    { 9.9999999999999994e-12, 4.4276014594399999e+00, 7.7410513042970590e+04 },
    { 9.9999999999999994e-12, 5.3402582675700003e+00, 3.5472744149649399e+04 },
    { 9.9999999999999994e-12, 3.2336676364000000e-01, 1.6703062133498999e+04 },
    { 9.9999999999999994e-12, 2.9031592817000001e-01, 8.6143798578061585e+04 },
    { 9.9999999999999994e-12, 5.1202513195999999e-01, 4.2153969003049002e+04 },
    { 9.9999999999999994e-12, 5.6479849654000001e+00, 5.1123537899959993e+04 },
    { 9.9999999999999994e-12, 3.4181660961699998e+00, 1.0276253967101299e+05 },
    { 9.9999999999999994e-12, 2.1480063315100000e+00, 9.5247705572179184e+04 },
    { 9.9999999999999994e-12, 1.8836723728500000e+00, 2.5035634758385801e+04 },
    { 9.9999999999999994e-12, 5.4243447991000000e+00, 1.0329934418310839e+05 },
    { 9.9999999999999994e-12, 6.0304194471299999e+00, 7.1135470007999997e+00 },
    { 0.0000000000000000e+00, 4.4590471485699998e+00, 4.5892730433156990e+04 },
    { 0.0000000000000000e+00, 3.2498533090800001e+00, 1.1112232316754239e+05 },
    { 0.0000000000000000e+00, 5.8924767193300003e+00, 1.1678365521669458e+05 },
    { 0.0000000000000000e+00, 8.0068790299999998e-02, 1.0664954771900000e+03 },
    { 0.0000000000000000e+00, 3.1487751176700001e+00, 1.0223284870591838e+05 },
    { 0.0000000000000000e+00, 5.4329168934499998e+00, 7.7829997686845789e+04 },
    { 0.0000000000000000e+00, 6.0524771694200004e+00, 2.5654191403697390e+04 },
    { 0.0000000000000000e+00, 2.6498142514100000e+00, 5.3242301760338400e+04 },
    { 0.0000000000000000e+00, 2.5537919124999999e+00, 5.2329585093633192e+04 },
    { 0.0000000000000000e+00, 9.2833282389999994e-02, 8.9586373523026981e+04 },
    { 0.0000000000000000e+00, 1.8478686542100000e+00, 2.5234706759822191e+04 },
    { 0.0000000000000000e+00, 2.9005218758900000e+00, 9.7112936974696786e+04 },
    { 0.0000000000000000e+00, 4.6135286414600003e+00, 1.3041984594697120e+05 },
    { 0.0000000000000000e+00, 3.8877173936400000e+00, 2.0759384658050019e+05 },
    { 0.0000000000000000e+00, 2.0482312484000000e+00, 7.8903606711036584e+04 },
};

static const VSOP87Term VSOP87A_MERCURY_X3[54] = {
    { 4.6251999999999999e-07, 3.1415926535900001e+00, 0.0000000000000000e+00 },
    { 1.7461000000000000e-07, 5.9057057522600003e+00, 7.8263709424722590e+04 },
    { 1.4854999999999999e-07, 2.9808127006100000e+00, 5.2175806283148399e+04 },
    { 1.0499000000000000e-07, 2.6555559735199998e+00, 1.0435161256629678e+05 },
    { 6.4739999999999996e-08, 3.1044261199199998e+00, 2.6087903141574199e+04 },
    { 4.9059999999999997e-08, 5.7023529781100004e+00, 1.3043951570787099e+05 },
    { 2.0120000000000001e-08, 2.4691787091799999e+00, 1.5652741884944518e+05 },
    { 7.6199999999999997e-09, 5.5201272364399996e+00, 1.8261532199101939e+05 },
    { 2.7400000000000001e-09, 2.2881510837599999e+00, 2.0870322513259359e+05 },
    { 9.5000000000000003e-10, 5.3393047546499997e+00, 2.3479112827416777e+05 },
    { 3.1999999999999998e-10, 2.1088155924900001e+00, 2.6087903141574195e+05 },
    { 2.1999999999999999e-10, 2.8595806616399999e+00, 5.1066427731055002e+04 },
    { 1.2999999999999999e-10, 5.9141310128900004e+00, 7.7154330872629187e+04 },
    { 1.0999999999999999e-10, 5.8393310301000003e+00, 5.3285184835241802e+04 },
    { 1.0000000000000000e-10, 5.1575461370999998e+00, 2.8696693455731618e+05 },
    { 6.0000000000000000e-11, 2.6853162360299998e+00, 1.0324223401420339e+05 },
    { 3.9999999999999998e-11, 2.6398430888200002e+00, 7.9373087976815994e+04 },
    { 3.0000000000000000e-11, 1.9288219696400000e+00, 3.1305483769889036e+05 },
    { 1.9999999999999999e-11, 5.7403818750699998e+00, 1.2933013715577759e+05 },
    { 1.9999999999999999e-11, 3.0227695594999998e-01, 4.6514474233996203e+04 },
    { 9.9999999999999994e-12, 3.3005535753899999e+00, 7.2602377375570388e+04 },
    { 9.9999999999999994e-12, 2.9853304465499999e+00, 1.1093785520934000e+03 },
    { 9.9999999999999994e-12, 5.4894917420500002e+00, 5.1220206541539788e+04 },
    { 9.9999999999999994e-12, 4.4166931460200001e+00, 5.3235188213337591e+04 },
    { 9.9999999999999994e-12, 9.8973080298000005e-01, 5.2705497248242988e+04 },
    { 9.9999999999999994e-12, 9.9963519350000002e-02, 9.8690280517144580e+04 },
    { 9.9999999999999994e-12, 1.3949975457499999e+00, 2.7147285071763388e+04 },
    { 9.9999999999999994e-12, 2.1540502635299998e+00, 7.7308109683113988e+04 },
    { 9.9999999999999994e-12, 5.6047847547599998e+00, 3.0639856638632999e+04 },
    { 9.9999999999999994e-12, 2.9271914916799999e+00, 4.9957049178961599e+04 },
    { 9.9999999999999994e-12, 2.8500990922399998e+00, 5.3131406024757001e+04 },
    { 9.9999999999999994e-12, 4.0998705684900001e+00, 4.0853142184844000e+04 },
    { 9.9999999999999994e-12, 4.9807884833499996e+00, 3.3914274084046460e+05 },
    { 9.9999999999999994e-12, 5.3623008995700001e+00, 5.6727759780207198e+04 },
    { 9.9999999999999994e-12, 5.7198982586299998e+00, 1.0546099111839019e+05 },
    { 9.9999999999999994e-12, 2.5147397797800002e+00, 1.5541804029735178e+05 },
    { 9.9999999999999994e-12, 1.0436927831700000e+00, 2.5028521211384999e+04 },
    { 9.9999999999999994e-12, 4.0122344507900003e+00, 7.8793400389817180e+04 },
    { 9.9999999999999994e-12, 1.0175115085999999e+00, 7.9323091354911783e+04 },
    { 9.9999999999999994e-12, 2.2454013136099999e+00, 2.4978524589480799e+04 },
    { 9.9999999999999994e-12, 5.7511550330499999e+00, 5.4394563387335191e+04 },
    { 0.0000000000000000e+00, 1.3098051046700001e+00, 5.1646115318053788e+04 },
    { 0.0000000000000000e+00, 2.5559680644800000e+00, 2.7676976036857999e+04 },
    { 0.0000000000000000e+00, 3.8100437824000000e-01, 4.5405095681902800e+04 },
    { 0.0000000000000000e+00, 2.1240129926900000e+00, 8.2815662921781390e+04 },
    { 0.0000000000000000e+00, 2.5568351371600002e+00, 2.7197281693667599e+04 },
    { 0.0000000000000000e+00, 2.6499211404200000e+00, 4.5519534970588002e+03 },
    { 0.0000000000000000e+00, 8.3114621691000001e-01, 2.0426571092422000e+04 },
    { 0.0000000000000000e+00, 5.5108866146100004e+00, 5.6613320491521999e+03 },
    { 0.0000000000000000e+00, 5.9290425821400001e+00, 7.6674636529438780e+04 },
    { 0.0000000000000000e+00, 3.0779797418999999e-01, 2.4498830246290399e+04 },
    { 0.0000000000000000e+00, 8.8049716681000001e-01, 6.6941045326418185e+04 },
    { 0.0000000000000000e+00, 3.4700031614300002e+00, 2.3869146037387400e+04 },
    { 0.0000000000000000e+00, 5.9920355390499997e+00, 7.6044952320535798e+04 },
};

static const VSOP87Term VSOP87A_MERCURY_X4[15] = {
    { 3.9600000000000004e-09, 3.1415926535900001e+00, 0.0000000000000000e+00 },
    { 2.7700000000000002e-09, 4.4691152179199998e+00, 7.8263709424722590e+04 },
    { 2.3300000000000000e-09, 1.1585925556700001e+00, 1.0435161256629678e+05 },
    { 1.7500000000000000e-09, 1.6969773154000001e+00, 5.2175806283148399e+04 },
    { 1.4200000000000001e-09, 4.1829483266100000e+00, 1.3043951570787099e+05 },
    { 7.2000000000000000e-10, 9.3996426088999996e-01, 1.5652741884944518e+05 },
    { 3.4000000000000001e-10, 2.1255529423600001e+00, 2.6087903141574199e+04 },
    { 3.1999999999999998e-10, 3.9862464307900001e+00, 1.8261532199101939e+05 },
    { 1.4000000000000001e-10, 7.5012775830000000e-01, 2.0870322513259359e+05 },
    { 5.0000000000000002e-11, 3.7972133010600002e+00, 2.3479112827416777e+05 },
    { 1.9999999999999999e-11, 5.6145150603000005e-01, 2.6087903141574195e+05 },
    { 9.9999999999999994e-12, 3.6122009319399999e+00, 2.8696693455731618e+05 },
    { 0.0000000000000000e+00, 4.3412496844800001e+00, 7.7154330872629187e+04 },
    { 0.0000000000000000e+00, 1.1733578941900000e+00, 5.1066427731055002e+04 },
    { 0.0000000000000000e+00, 4.2158235525699999e+00, 5.3285184835241802e+04 },
};

static const VSOP87Term VSOP87A_MERCURY_X5[10] = {
    { 1.0000000000000000e-10, 0.0000000000000000e+00, 0.0000000000000000e+00 },
    { 3.9999999999999998e-11, 5.9164920066600004e+00, 1.0435161256629678e+05 },
    { 3.9999999999999998e-11, 2.8375794700000001e+00, 7.8263709424722590e+04 },
    { 3.0000000000000000e-11, 2.6681700505100001e+00, 1.3043951570787099e+05 },
    { 3.0000000000000000e-11, 5.1547700853500000e+00, 5.2175806283148399e+04 },
    { 1.9999999999999999e-11, 5.6979472989100000e+00, 1.5652741884944518e+05 },
    { 1.9999999999999999e-11, 6.2034775990000002e+00, 2.6087903141574199e+04 },
    { 9.9999999999999994e-12, 2.4587478924599999e+00, 1.8261532199101939e+05 },
    { 9.9999999999999994e-12, 5.4897829635099997e+00, 2.0870322513259359e+05 },
    { 0.0000000000000000e+00, 2.2436002168900000e+00, 2.3479112827416777e+05 },
};

static const VSOP87Term VSOP87A_MERCURY_Y0[1438] = {
    { 3.7953642888000000e-01, 2.8378061782000001e+00, 2.6087903141574199e+04 },
    { 1.1626131831000000e-01, 3.1415926535900001e+00, 0.0000000000000000e+00 },
    { 3.8546682149999997e-02, 5.8878060896599997e+00, 5.2175806283148399e+04 },
    { 5.8771126800000002e-03, 2.6549889620100000e+00, 7.8263709424722590e+04 },
    { 1.0623549299999999e-03, 5.7055061673500003e+00, 1.0435161256629678e+05 },
    { 2.1100827999999999e-04, 2.4729131584899999e+00, 1.3043951570787099e+05 },
    { 4.4500560000000003e-05, 5.5235490707099997e+00, 1.5652741884944518e+05 },
    { 9.7828599999999997e-06, 2.2910264302600001e+00, 1.8261532199101939e+05 },
    { 7.0750000000000000e-06, 2.8951659153099998e+00, 2.4978524589480799e+04 },
    { 6.5474199999999999e-06, 5.9289212388100001e+00, 2.7197281693667599e+04 },
    { 4.4856099999999997e-06, 1.4059504221100001e+00, 1.0593819301891999e+03 },
    { 4.0216800000000003e-06, 3.0317998006000002e-01, 2.0426571092422000e+04 },
    { 2.9060400000000000e-06, 2.2364586839200000e+00, 3.1749235190726398e+04 },
    { 2.2171800000000000e-06, 5.3417067656999997e+00, 2.0870322513259359e+05 },
    { 1.9337199999999999e-06, 5.3869878199699999e+00, 4.5519534970588002e+03 },
    { 1.9135800000000001e-06, 2.7079284254700000e+00, 5.3285184835241802e+04 },
    { 1.8111900000000001e-06, 2.3941291054000000e-01, 1.1093785520934000e+03 },
    { 1.7724200000000001e-06, 2.7885581342900001e+00, 5.6613320491521999e+03 },
    { 1.5492400000000000e-06, 1.1220186576100000e+00, 5.1116424352959199e+04 },
    { 1.3951400000000000e-06, 5.9469866231899999e+00, 5.1066427731055002e+04 },
    { 1.1607200000000001e-06, 5.2860817011599996e+00, 5.7837138332300601e+04 },
    { 8.6595000000000005e-07, 5.0699984325400003e+00, 5.2969096509459996e+02 },
    { 8.3249999999999997e-07, 1.6012788581799999e+00, 1.0213285546211000e+04 },
    { 8.1399000000000000e-07, 3.5086279795799999e+00, 2.1535949644515400e+04 },
    { 7.5456999999999996e-07, 4.5039681444499999e+00, 2.7147285071763388e+04 },
    { 7.5077999999999995e-07, 3.3629117097500001e+00, 4.6514474233996203e+04 },
    { 6.9044999999999999e-07, 4.9216042833499998e+00, 2.5132303399965600e+04 },
    { 6.3460000000000003e-07, 3.9123907533400000e+00, 2.7043502883182799e+04 },
    { 6.1404999999999997e-07, 3.4374737243200002e+00, 4.7623852786089599e+04 },
    { 5.8159999999999997e-07, 4.1367760221000003e+00, 1.4765239043269799e+04 },
    { 5.5112999999999999e-07, 3.7429861112700000e+00, 1.2566151699982800e+04 },
    { 5.4049999999999995e-07, 7.2631991357000003e-01, 2.6617594106668799e+04 },
    { 5.2555999999999995e-07, 1.9863670542600000e+00, 2.5558212176479599e+04 },
    { 5.1435000000000000e-07, 2.1092138113000001e+00, 2.3479112827416777e+05 },
    { 5.1302999999999998e-07, 1.9678229250799999e+00, 4.2659819087599999e+02 },
    { 4.9166000000000005e-07, 4.1709455134800004e+00, 7.7204327494533383e+04 },
    { 4.8520999999999999e-07, 1.9501689701200000e+00, 1.5890728952838001e+03 },
    { 4.7932000000000003e-07, 5.7619536402599998e+00, 7.9373087976815994e+04 },
    { 4.5434999999999999e-07, 9.1835136157999997e-01, 4.1962520736937397e+04 },
    { 3.5814000000000000e-07, 4.7396651207899998e+00, 3.7410567239878597e+04 },
    { 3.5132000000000000e-07, 4.1198767987300000e+00, 5.1646115318053788e+04 },
    { 3.2753999999999998e-07, 4.5735269411499999e+00, 9.5559974160859997e+02 },
    { 3.2250999999999999e-07, 2.0535722578000000e+00, 8.3925041473874793e+04 },
    { 3.1572000000000002e-07, 2.7147027436900002e+00, 7.7154330872629187e+04 },
    { 2.8901999999999998e-07, 5.4392297599699999e+00, 3.0639856638632999e+04 },
    { 2.8206000000000003e-07, 5.0733860349400004e+00, 3.9609654583165597e+04 },
    { 2.7076999999999998e-07, 4.6929813386500001e+00, 3.6301188687785187e+04 },
    { 2.6847999999999998e-07, 5.2532067029700000e+00, 1.1322664098304400e+04 },
    { 2.6742000000000000e-07, 1.7305422749999999e-01, 6.2830758499914000e+03 },
    { 2.2896000000000001e-07, 7.1333836962999997e-01, 5.3131406024757001e+04 },
    { 2.1301000000000000e-07, 2.0534560768999999e-01, 7.3711755927663791e+04 },
    { 2.1010999999999999e-07, 5.6511909288000002e+00, 1.3521751441591399e+04 },
    { 2.1009000000000000e-07, 4.0868458022100000e+00, 1.5874617595363199e+04 },
    { 1.7483000000000001e-07, 5.6186392438999999e-01, 5.1749208092272391e+04 },
    { 1.7112000000000001e-07, 5.5137156526999997e-01, 5.0586733387864588e+04 },
    { 1.6502999999999999e-07, 1.3257325259000000e-01, 7.2602377375570388e+04 },
    { 1.6124999999999999e-07, 1.6566777192000000e+00, 9.1039069941176003e+03 },
    { 1.5173000000000000e-07, 1.2481099529599999e+00, 1.8849227549974199e+04 },
    { 1.5078999999999999e-07, 1.9787911855100000e+00, 7.2386755916000002e+03 },
    { 1.4545000000000000e-07, 1.4838530087599999e+00, 6.3498470381452789e+04 },
    { 1.3890999999999999e-07, 1.2966574879199999e+00, 5.3235188213337591e+04 },
    { 1.3657000000000001e-07, 1.6927128072700000e+00, 5.1220206541539788e+04 },
    { 1.3631000000000001e-07, 3.6521468465400000e+00, 5.2705497248242988e+04 },
    { 1.3598000000000000e-07, 9.3759628331000000e-01, 1.0329223063610759e+05 },
    { 1.3152000000000000e-07, 2.1408490668999999e-01, 2.6068233380674399e+04 },
    { 1.3131000000000000e-07, 2.3199419347000001e+00, 2.6107572902473989e+04 },
    { 1.2963000000000000e-07, 4.1891192995199997e+00, 2.5028521211384999e+04 },
    { 1.2617000000000000e-07, 2.3290397263100000e+00, 4.5892730433156990e+04 },
    { 1.2253000000000000e-07, 1.8426034790600001e+00, 6.5697557724739789e+04 },
    { 1.2153999999999999e-07, 5.1599027638399999e+00, 2.6087903141574195e+05 },
    { 1.2108000000000000e-07, 2.5318221998000001e+00, 1.0546099111839019e+05 },
    { 1.2081000000000001e-07, 3.9694877935499999e+00, 6.8050423878511589e+04 },
    { 1.1751000000000000e-07, 1.4313287672799999e+00, 3.3326578733174203e+04 },
    { 1.1477000000000000e-07, 4.1407124478000001e-01, 1.9317192540328600e+04 },
    { 1.0236000000000000e-07, 8.2838354702000006e-01, 7.7734018459627987e+04 },
    { 9.5770000000000003e-08, 5.4300980489999995e-01, 3.8654054841556987e+04 },
    { 9.2430000000000001e-08, 9.1470178389000001e-01, 4.0853142184844000e+04 },
    { 9.0260000000000002e-08, 5.1945641359000003e+00, 2.6084021806216198e+04 },
    { 9.0240000000000006e-08, 3.6226546286599999e+00, 2.6091784476932200e+04 },
    { 8.9210000000000003e-08, 2.2127912379499999e+00, 3.2858613742819791e+04 },
    { 8.9039999999999995e-08, 5.4608657405200001e+00, 3.4425749449653999e+03 },
    { 8.9029999999999997e-08, 5.0527830949899997e+00, 2.6514501332450200e+04 },
    { 8.8650000000000003e-08, 3.3008547517000002e+00, 3.2370978991565600e+04 },
    { 8.5319999999999999e-08, 5.1042307628600003e+00, 1.1001294461544899e+05 },
    { 8.2770000000000006e-08, 9.7930362795000003e-01, 4.3071899289030800e+04 },
    { 8.1820000000000001e-08, 5.0427972108899999e+00, 2.7676976036857999e+04 },
    { 8.0210000000000002e-08, 5.7638419303499999e+00, 2.1329909543799999e+02 },
    { 7.7000000000000001e-08, 2.1715746944199998e+00, 5.6727759780207198e+04 },
    { 7.5709999999999994e-08, 5.7655589429100003e+00, 1.0324223401420339e+05 },
    { 7.4429999999999999e-08, 2.9822405364700000e+00, 2.3869146037387400e+04 },
    { 7.3879999999999997e-08, 2.7131194189799999e+00, 6.7707106012455997e+03 },
    { 6.9520000000000003e-08, 3.4228449978200000e+00, 2.2645328196608789e+04 },
    { 6.6069999999999995e-08, 5.4898663107800001e+00, 1.9804827291582798e+04 },
    { 6.5620000000000001e-08, 5.8624831651799996e+00, 2.8306660245760992e+04 },
    { 6.4469999999999994e-08, 1.3021953439900000e+00, 1.6983996147456601e+04 },
    { 6.3090000000000004e-08, 4.4433403161999996e+00, 5.9414481874748402e+04 },
    { 6.2849999999999997e-08, 1.1781328984199999e+00, 6.3989728631399998e+02 },
    { 6.0920000000000006e-08, 3.2570573150400000e+00, 9.9799659069237983e+04 },
    { 6.0150000000000006e-08, 1.4769834114600000e+00, 6.2389091829359393e+04 },
    { 5.9249999999999998e-08, 3.7745432594000001e+00, 7.9219309166331193e+04 },
    { 5.8950000000000002e-08, 3.6687378738500001e+00, 2.5874604046136199e+04 },
    { 5.5169999999999997e-08, 3.6062086631999999e+00, 7.6674636529438780e+04 },
    { 5.3850000000000002e-08, 3.6108367985099998e+00, 7.7837111233846590e+04 },
    { 5.1679999999999997e-08, 5.0580499590199999e+00, 2.6301202237012199e+04 },
    { 4.7139999999999997e-08, 2.5442904507500002e+00, 2.6095016688575000e+04 },
    { 4.5769999999999999e-08, 2.1298790690000000e-02, 2.6080789594573391e+04 },
    { 4.0280000000000001e-08, 2.4939632775199998e+00, 2.1187638603783998e+03 },
    { 3.9620000000000000e-08, 5.4836321336899996e+00, 2.9530478086539599e+04 },
    { 3.9080000000000003e-08, 3.1851953735100000e+00, 9.8690280517144580e+04 },
    { 3.7970000000000002e-08, 4.5246968848800000e+00, 8.9586373523026981e+04 },
    { 3.6440000000000001e-08, 3.9878371423000001e+00, 1.2938013377768178e+05 },
    { 3.6290000000000003e-08, 4.8938531946400001e+00, 9.1785460866313981e+04 },
    { 3.4989999999999998e-08, 5.3804644862600002e+00, 7.1980633574731182e+04 },
    { 3.3379999999999998e-08, 3.8724993765999999e-01, 7.8793400389817180e+04 },
    { 3.3160000000000000e-08, 5.1218345754800003e+00, 5.8946516884393990e+04 },
    { 3.2749999999999999e-08, 1.1479920532400001e+00, 2.5934124331089399e+04 },
    { 3.2460000000000001e-08, 1.3866100897600000e+00, 2.6241681952059000e+04 },
    { 3.2339999999999997e-08, 1.5700821776000001e-01, 2.2187571041868000e+03 },
    { 3.1049999999999997e-08, 7.3732898185999995e-01, 9.4138327020085781e+04 },
    { 3.1030000000000001e-08, 5.5848129036199996e+00, 1.3154889425996438e+05 },
    { 3.0950000000000003e-08, 4.7444035979599999e+00, 7.7308109683113988e+04 },
    { 3.0680000000000001e-08, 4.3577554504200000e+00, 7.9323091354911783e+04 },
    { 2.9460000000000000e-08, 3.9141378799800002e+00, 2.5661304950698199e+04 },
    { 2.9150000000000000e-08, 1.9274596430599999e+00, 2.8696693455731618e+05 },
    { 2.7370000000000000e-08, 4.3198761671700003e+00, 4.4937130691548402e+04 },
    { 2.7360000000000001e-08, 3.8584136501200001e+00, 1.0382192160120218e+05 },
    { 2.7360000000000001e-08, 4.9041574043300002e+00, 1.4956319713459999e+02 },
    { 2.6919999999999999e-08, 3.8966447212099999e+00, 6.9159802430604992e+04 },
    { 2.6899999999999999e-08, 5.3706406629699996e+00, 5.2195476044048191e+04 },
    { 2.6700000000000001e-08, 3.2641179627099999e+00, 5.2156136522248598e+04 },
    { 2.5590000000000000e-08, 3.4998792772599998e+00, 4.8733231338182988e+04 },
    { 2.5370000000000002e-08, 4.4500762790000001e+00, 1.0664954771900000e+03 },
    { 2.4870000000000000e-08, 1.5487970601600001e+00, 1.0522683831884001e+03 },
    { 2.2370000000000001e-08, 1.8718783570899999e+00, 1.3610084775702318e+05 },
    { 2.2020000000000001e-08, 2.5964439066599998e+00, 5.4394563387335191e+04 },
    { 2.1380000000000000e-08, 1.3510590477900000e+00, 5.1535908996834391e+04 },
    { 1.9790000000000001e-08, 3.6025764690700002e+00, 6.4741957983131193e+04 },
    { 1.9779999999999999e-08, 3.4717285410300001e+00, 4.5405095681902800e+04 },
    { 1.9550000000000000e-08, 1.1915602691000000e-01, 5.8458882133139792e+04 },
    { 1.9350000000000001e-08, 4.2195912235400002e+00, 1.3655860491176400e+04 },
    { 1.8819999999999999e-08, 5.2111395579700002e+00, 8.2815662921781390e+04 },
    { 1.8740000000000001e-08, 2.5331905879100001e+00, 1.2933013715577759e+05 },
    { 1.8419999999999999e-08, 3.9659480680499999e+00, 6.6941045326418185e+04 },
    { 1.8380000000000000e-08, 3.8908718330999997e-01, 5.2179687618506403e+04 },
    { 1.8329999999999999e-08, 1.9613635362999999e+00, 5.2171924947790401e+04 },
    { 1.8170000000000000e-08, 2.8147419436999999e-01, 7.1135470007999997e+00 },
    { 1.7879999999999999e-08, 3.7792450541000000e-01, 2.5619938151219802e+04 },
    { 1.7229999999999999e-08, 2.1558626915399999e+00, 2.6555868131928601e+04 },
    { 1.6919999999999999e-08, 2.7583341526099998e+00, 5.2168692736147590e+04 },
    { 1.6650000000000000e-08, 2.2597126211199998e+00, 5.3274761083827998e+03 },
    { 1.6600000000000000e-08, 2.5021331689999999e-02, 1.2588756221081219e+05 },
    { 1.6569999999999999e-08, 4.5576252874699996e+00, 7.6266071275599998e+01 },
    { 1.6470000000000001e-08, 1.8353170324200001e+00, 5.2602404474024399e+04 },
    { 1.5650000000000001e-08, 6.2493701332700002e+00, 6.6812248533995999e+03 },
    { 1.5609999999999999e-08, 1.1845452447300000e+00, 8.5502385016322587e+04 },
    { 1.5559999999999998e-08, 1.8710299618699999e+00, 5.2389105378586399e+04 },
    { 1.5489999999999998e-08, 3.9789620728000002e-01, 1.0276253967101299e+05 },
    { 1.5390000000000001e-08, 5.5256735057999995e-01, 1.0530721230790539e+05 },
    { 1.5209999999999999e-08, 1.8310407350900000e+00, 5.3764879178432202e+04 },
    { 1.4690000000000000e-08, 3.7777284264000000e-01, 1.0392501437542078e+05 },
    { 1.4540000000000000e-08, 4.5375426966100001e+00, 8.8476994970933592e+04 },
    { 1.4410000000000000e-08, 6.2458110136400000e+00, 5.0057042422769999e+04 },
    { 1.4200000000000000e-08, 6.0349595659100004e+00, 4.9957049178961599e+04 },
    { 1.4079999999999999e-08, 2.8368738778600000e+00, 2.6094531700474210e+04 },
    { 1.4079999999999999e-08, 5.9801589883500004e+00, 2.6081274582674188e+04 },
    { 1.3420000000000000e-08, 4.7575044524400001e+00, 3.8519945791972001e+04 },
    { 1.2720000000000001e-08, 2.5153849715100001e+00, 5.1962507187710398e+04 },
    { 1.2639999999999999e-08, 4.6622216497500002e+00, 3.5191810135691798e+04 },
    { 1.2429999999999999e-08, 5.5600452202100001e+00, 5.2182919830149200e+04 },
    { 1.1059999999999999e-08, 5.1522238427299998e+00, 1.2432042650397800e+04 },
    { 1.0719999999999999e-08, 4.2594254256799999e+00, 2.6727800427888200e+04 },
    { 9.9699999999999993e-09, 1.6615790196300000e+00, 1.1787336400788819e+05 },
    { 9.6699999999999999e-09, 7.5487021033000001e-01, 1.5546803691925600e+05 },
    { 9.5999999999999999e-09, 6.2366372227599998e+00, 1.2477818365871879e+05 },
    { 9.3800000000000003e-09, 1.2828533268300000e+00, 1.1567427666460119e+05 },
    { 9.3700000000000005e-09, 2.0005458370999998e+00, 8.5034420025968182e+04 },
    { 9.3600000000000008e-09, 2.2402377268100002e+00, 5.5618381228113802e+04 },
    { 9.2300000000000006e-09, 3.9058774034299999e+00, 5.2026243086013797e+04 },
    { 9.1299999999999997e-09, 2.1475914188599998e+00, 9.8068536716305389e+04 },
    { 8.7500000000000006e-09, 4.3619228690899998e+00, 5.1109310805958390e+04 },
    { 8.6399999999999999e-09, 9.7123767040999998e-01, 5.1123537899959993e+04 },
    { 8.3699999999999998e-09, 4.8875422450000000e-01, 4.6848330174765601e+04 },
    { 8.3199999999999994e-09, 3.4220273590499999e+00, 1.0488130353139139e+05 },
    { 8.0399999999999995e-09, 2.3541275082899999e+00, 1.5763679740153858e+05 },
    { 7.9699999999999996e-09, 3.7873028531099999e+00, 1.2022623016165999e+05 },
    { 7.4600000000000003e-09, 2.7653286570000001e-02, 7.4821134479757180e+04 },
    { 7.4300000000000002e-09, 1.5124100121999999e+00, 1.0339601282468818e+05 },
    { 7.4199999999999996e-09, 4.4395404262899998e+00, 3.3406124266997999e+03 },
    { 7.3099999999999998e-09, 1.8504474303000000e+00, 4.3371173787679999e+02 },
    { 7.2600000000000002e-09, 1.1307286445700000e+00, 1.0541099449648599e+05 },
    { 7.1900000000000002e-09, 6.1492825169999998e-01, 1.2990982474277639e+05 },
    { 7.0800000000000004e-09, 4.9783505807499999e+00, 3.1305483769889036e+05 },
    { 7.0299999999999999e-09, 5.2498029305599996e+00, 4.1948464387519999e+02 },
    { 6.7700000000000004e-09, 4.4341032750600000e+00, 5.2329585093633192e+04 },
    { 6.7500000000000001e-09, 5.5841386344000004e+00, 2.8206667001952599e+04 },
    { 6.6599999999999997e-09, 4.4015183721900000e+00, 7.7623812138408583e+04 },
    { 6.6299999999999996e-09, 4.1979432290199998e+00, 5.2022027472663598e+04 },
    { 6.5000000000000003e-09, 6.3237258720000000e-01, 9.5247705572179184e+04 },
    { 6.4400000000000001e-09, 2.5870808176100000e+00, 4.5494581429748789e+04 },
    { 6.3899999999999996e-09, 5.7618772403999996e+00, 8.0482466528909383e+04 },
    { 6.1700000000000000e-09, 2.1381339773599999e+00, 7.8283379185622383e+04 },
    { 6.1000000000000000e-09, 3.1465510619999998e-02, 7.8244039663822798e+04 },
    { 6.0200000000000003e-09, 7.1110689228000001e-01, 2.0760427033191401e+04 },
    { 5.9600000000000001e-09, 1.7432261017599999e+00, 7.9945284420241996e+03 },
    { 5.9200000000000002e-09, 1.0921453152600000e+00, 7.1025033833122594e+04 },
    { 5.9200000000000002e-09, 3.6143126121000002e-01, 8.5319638175199998e+02 },
    { 5.8600000000000000e-09, 4.9220752497099998e+00, 1.6218875089859738e+05 },
    { 5.5800000000000002e-09, 4.2519833730999999e+00, 5.2099540211872802e+04 },
    { 5.5500000000000001e-09, 8.2693925421000003e-01, 1.6921656695024001e+03 },
    { 5.5100000000000002e-09, 3.2939518397000001e+00, 2.3754706748702189e+04 },
    { 5.3300000000000004e-09, 3.3641046554399998e+00, 2.5455119402261000e+04 },
    { 5.2599999999999996e-09, 6.5987913796999997e-01, 1.0021837280099400e+04 },
    { 5.1499999999999998e-09, 5.5370951149999996e+00, 2.6720686880887391e+04 },
    { 4.9000000000000000e-09, 3.0263154174400002e+00, 7.6144945564344191e+04 },
    { 4.7300000000000000e-09, 5.5838296896899999e+00, 1.5541804029735178e+05 },
    { 4.7099999999999997e-09, 1.7251078619000000e+00, 2.6237466338708789e+04 },
    { 4.6999999999999999e-09, 1.9750978509899999e+00, 1.0890356606335558e+05 },
    { 4.6699999999999998e-09, 3.1889951261700000e+00, 8.4546785274713984e+04 },
    { 4.6399999999999997e-09, 3.4844826615500000e+00, 2.4498830246290399e+04 },
    { 4.5999999999999998e-09, 4.6045568028200003e+00, 2.6190995915792791e+04 },
    { 4.5900000000000001e-09, 4.4821212177999997e-01, 2.6037906519669988e+04 },
    { 4.5900000000000001e-09, 2.0860274139000001e+00, 2.6137899763478399e+04 },
    { 4.5699999999999997e-09, 4.2587025729499999e+00, 2.5984810367355600e+04 },
    { 4.5600000000000000e-09, 3.7602833212999998e-01, 9.0829861124705378e+04 },
    { 4.4900000000000000e-09, 5.7600939750000002e+00, 7.8256595877721789e+04 },
    { 4.4500000000000001e-09, 3.0752326760900002e+00, 1.5197546535238638e+05 },
    { 4.4200000000000000e-09, 4.7012575180900003e+00, 2.7140171524762591e+04 },
    { 4.4100000000000003e-09, 5.5500240050500000e+00, 2.3331963928720002e+03 },
    { 4.4100000000000003e-09, 5.7476971936499996e+00, 7.8050410329284583e+04 },
    { 4.3899999999999999e-09, 7.4557141997999998e-01, 2.9428515568274001e+04 },
    { 4.2800000000000001e-09, 1.2656173401799999e+00, 2.7154398618764200e+04 },
    { 4.2199999999999999e-09, 7.3875718619999997e-01, 9.3028948467992392e+04 },
    { 4.2000000000000004e-09, 3.4687583200800001e+00, 1.2885044281258718e+05 },
    { 4.2000000000000004e-09, 3.4388014647300000e+00, 7.8267590760080580e+04 },
    { 4.1899999999999998e-09, 5.0112089008299998e+00, 7.8259828089364586e+04 },
    { 4.1800000000000000e-09, 2.3893690129000000e-01, 7.1492998823476984e+04 },
    { 4.1199999999999998e-09, 4.9702378515300003e+00, 1.5961864422845999e+03 },
    { 4.0899999999999997e-09, 1.8923597906600000e+00, 2.2747290714874402e+04 },
    { 4.0599999999999996e-09, 9.9733773104000001e-01, 1.5819593482830001e+03 },
    { 4.0599999999999996e-09, 5.5643873635599999e+00, 2.8421099534446199e+04 },
    { 4.0199999999999998e-09, 3.6121911952699999e+00, 1.3139511544947958e+05 },
    { 3.9499999999999998e-09, 4.9303307383400004e+00, 7.8477008520160583e+04 },
    { 3.9000000000000002e-09, 3.4268867161899998e+00, 1.3001291751699499e+05 },
    { 3.7900000000000004e-09, 2.0123089155899998e+00, 2.6162684740141500e+04 },
    { 3.6899999999999999e-09, 4.2110670240900001e+00, 1.1159028815789679e+05 },
    { 3.6399999999999998e-09, 4.8910680828900004e+00, 7.8690307615598591e+04 },
    { 3.6399999999999998e-09, 4.5154624397000003e+00, 4.8835193856448590e+04 },
    { 3.6199999999999999e-09, 4.0032851110900003e+00, 2.5565325723480400e+04 },
    { 3.6100000000000001e-09, 1.3024609091799999e+00, 1.1456489811250778e+05 },
    { 3.6100000000000001e-09, 5.0112315849000000e-01, 2.6013121543006899e+04 },
    { 3.5600000000000001e-09, 5.2013734814800001e+00, 5.2643771273502804e+04 },
    { 3.5600000000000001e-09, 1.0867810075400000e+00, 6.4607848933546193e+04 },
    { 3.3700000000000001e-09, 4.8896906971499998e+00, 7.9852782320006387e+04 },
    { 3.3600000000000000e-09, 5.0294839756999998e-01, 1.8207813988235212e+04 },
    { 3.3099999999999999e-09, 4.7817630537699998e+00, 2.6610480559667991e+04 },
    { 3.3000000000000002e-09, 7.9870708800000006e-03, 2.5771511271917600e+04 },
    { 3.2900000000000000e-09, 2.5277426029500001e+00, 2.6404295011230799e+04 },
    { 3.2599999999999999e-09, 9.6777570630999998e-01, 4.4181277841124189e+04 },
    { 3.1700000000000000e-09, 2.8019560146400000e+00, 7.6044952320535798e+04 },
    { 3.1000000000000000e-09, 4.0661833024599998e+00, 6.3278373931320004e+02 },
    { 3.0800000000000001e-09, 2.3170124748599998e+00, 7.8270822971723392e+04 },
    { 3.0399999999999998e-09, 3.4146141624099999e+00, 5.1707841292793993e+04 },
    { 3.0399999999999998e-09, 4.6700565525000002e+00, 2.5448005855260191e+04 },
    { 3.0300000000000001e-09, 1.0190128842599999e+00, 2.2434479570190001e+02 },
    { 3.0199999999999999e-09, 3.0667726469700001e+00, 2.5973463852888959e+04 },
    { 3.0100000000000002e-09, 3.0320539298800000e+00, 2.6484548254729998e+03 },
    { 3.0000000000000000e-09, 5.7386913281299998e+00, 2.6202342430259410e+04 },
    { 2.9199999999999998e-09, 6.1560179268999997e+00, 1.1624747044077999e+03 },
    { 2.8900000000000002e-09, 1.3825942858000000e+00, 2.6164169212849800e+04 },
    { 2.8699999999999998e-09, 5.8829384117199996e+00, 5.2182434842048402e+04 },
    { 2.8699999999999998e-09, 2.7422700489300000e+00, 5.2169177724248388e+04 },
    { 2.8499999999999999e-09, 1.1664548111399999e+00, 1.8093374699549990e+04 },
    { 2.8200000000000002e-09, 3.1061838192000000e+00, 2.6507387785449391e+04 },
    { 2.7999999999999998e-09, 1.1265611814000001e+00, 7.7197213947532582e+04 },
    { 2.7999999999999998e-09, 6.7080969267000001e-01, 7.8114146227587989e+04 },
    { 2.7600000000000000e-09, 4.0209366746499997e+00, 7.7211441041534184e+04 },
    { 2.7499999999999998e-09, 5.8201692029799998e+00, 9.5628915597059995e+02 },
    { 2.7499999999999998e-09, 2.9219496437000001e+00, 1.9406678288174600e+04 },
    { 2.7299999999999999e-09, 5.0964021354500000e+00, 1.1112232316754239e+05 },
    { 2.7000000000000002e-09, 2.1747804250699998e+00, 3.3967992294913187e+04 },
    { 2.6900000000000000e-09, 4.7096757133400002e+00, 1.4396126714946239e+05 },
    { 2.6799999999999998e-09, 1.8171202802599999e+00, 4.2153969003049002e+04 },
    { 2.6599999999999999e-09, 9.9085054169999998e-01, 3.9743763632750597e+04 },
    { 2.6099999999999999e-09, 2.9171830568599999e+00, 3.2769127994973802e+04 },
    { 2.5599999999999998e-09, 3.8000358322899999e+00, 1.8155594006083018e+05 },
    { 2.5200000000000000e-09, 5.4320601320200002e+00, 2.5668418497699000e+04 },
    { 2.4899999999999999e-09, 5.6023139990199997e+00, 7.1582484571322988e+04 },
    { 2.4899999999999999e-09, 5.3658834860100004e+00, 5.2257741809380002e+02 },
    { 2.4800000000000001e-09, 6.7786738916000000e-01, 5.1742094545271590e+04 },
    { 2.4600000000000002e-09, 3.2737522345299999e+00, 5.1756321639273199e+04 },
    { 2.4399999999999998e-09, 5.9481563583700003e+00, 1.1610551958374201e+04 },
    { 2.4100000000000002e-09, 3.0004146594200001e+00, 1.5086608680029298e+05 },
    { 2.3699999999999999e-09, 5.1982525737799996e+00, 1.2415643985787958e+05 },
    { 2.3300000000000000e-09, 2.6253376230500001e+00, 7.8800891533389904e+03 },
    { 2.3199999999999998e-09, 1.1624305753099999e+00, 5.3680451209540001e+02 },
    { 2.3100000000000001e-09, 4.3250195345100000e+00, 1.4176217980617538e+05 },
    { 2.2600000000000001e-09, 3.2143741634400000e+00, 1.0090903762133139e+05 },
    { 2.2300000000000000e-09, 5.2815972319400002e+00, 8.1706284369687986e+04 },
    { 2.1499999999999998e-09, 3.0719460787799999e+00, 7.2936233316339785e+04 },
    { 2.1400000000000001e-09, 5.2486835730500001e+00, 1.4788665740644001e+03 },
    { 2.1100000000000000e-09, 1.7681817187000001e-01, 1.3096920667296558e+05 },
    { 2.1000000000000002e-09, 4.6479702893599999e+00, 7.4781598567299994e+01 },
    { 2.1000000000000002e-09, 5.4018083184899996e+00, 1.8372470054311279e+05 },
    { 2.1000000000000002e-09, 5.5524838499299998e+00, 1.1020632121940000e+02 },
    { 2.0799999999999998e-09, 5.3974604249000002e-01, 1.4631413330323418e+05 },
    { 2.0799999999999998e-09, 4.2417863890499996e+00, 1.3757737998458001e+03 },
    { 2.0400000000000000e-09, 3.1984637477100000e+00, 4.0565254324774200e+04 },
    { 2.0400000000000000e-09, 7.8597046713999996e-01, 2.5881717593137000e+04 },
    { 2.0099999999999999e-09, 1.7151052407200000e+00, 2.6294088690011391e+04 },
    { 2.0099999999999999e-09, 2.1605257301799998e+00, 5.1322609901396390e+04 },
    { 1.9700000000000000e-09, 1.0427763989600001e+00, 5.2815703569462399e+04 },
    { 1.9500000000000001e-09, 1.6940152366300001e+00, 5.0483640613645999e+04 },
    { 1.9099999999999998e-09, 3.6537589282799998e+00, 1.5599772788435058e+05 },
    { 1.8600000000000000e-09, 2.1439349308600000e+00, 1.6066065861474801e+04 },
    { 1.8400000000000001e-09, 4.5550053938500001e+00, 1.2948391596626239e+05 },
    { 1.8400000000000001e-09, 1.1633503607200000e+00, 1.0371171527998279e+05 },
    { 1.8100000000000000e-09, 3.5297148681100001e+00, 1.4477351183200000e+04 },
    { 1.7800000000000001e-09, 4.1780584589199998e+00, 1.3149889763806018e+05 },
    { 1.7700000000000001e-09, 3.6225607719599999e+00, 2.6624707653669589e+04 },
    { 1.7500000000000000e-09, 5.6887091896699999e+00, 5.1543022543835199e+04 },
    { 1.7400000000000000e-09, 1.7381871218800000e+00, 3.3914274084046460e+05 },
    { 1.7400000000000000e-09, 9.9979811068000002e-01, 2.5938339944439598e+04 },
    { 1.7300000000000000e-09, 1.0154075599600001e+00, 7.8187443353446986e+04 },
    { 1.6900000000000000e-09, 3.6876559450299999e+00, 1.2133560871375339e+05 },
    { 1.6800000000000000e-09, 3.6335840293600001e+00, 1.0309277421860000e+02 },
    { 1.6500000000000001e-09, 2.1759214956299999e+00, 4.6796499035440002e+02 },
    { 1.6300000000000000e-09, 5.6328494864599996e+00, 2.5780345520604598e+04 },
    { 1.6100000000000000e-09, 3.1679606300200001e+00, 2.6395460762543789e+04 },
    { 1.5799999999999999e-09, 1.1920252710899999e+00, 7.8417488235207391e+04 },
    { 1.5700000000000000e-09, 1.3695958305100000e+00, 6.1279713277265997e+04 },
    { 1.5700000000000000e-09, 5.2873540813600002e+00, 2.5551098629478791e+04 },
    { 1.5600000000000000e-09, 4.6713022884599997e+00, 9.0695752075120385e+04 },
    { 1.5500000000000000e-09, 1.6864542884000000e+00, 1.8827665404017159e+05 },
    { 1.5400000000000001e-09, 3.7117243606199999e+00, 5.2101024684581091e+04 },
    { 1.5300000000000001e-09, 9.5690434145000003e-01, 7.8109930614237790e+04 },
    { 1.5100000000000000e-09, 1.3729940144499999e+00, 1.3362449706799200e+04 },
    { 1.5000000000000000e-09, 5.1841076627499998e+00, 1.0437128232719658e+05 },
    { 1.4800000000000001e-09, 3.0772612547499998e+00, 1.0433194280539699e+05 },
    { 1.4700000000000001e-09, 2.5178901001899998e+00, 2.3888815798287189e+04 },
    { 1.4500000000000000e-09, 3.8199592265800000e+00, 5.0579619840863787e+04 },
    { 1.4400000000000000e-09, 2.3639061742199998e+00, 1.0657036967048359e+05 },
    { 1.4100000000000001e-09, 1.5087015084399999e+00, 5.0593846934865389e+04 },
    { 1.4100000000000001e-09, 8.9841562129999997e-01, 5.2278899057366987e+04 },
    { 1.4100000000000001e-09, 2.2559741847899999e+00, 5.2808590022461591e+04 },
    { 1.3999999999999999e-09, 6.1315252519300003e+00, 1.0223284870591838e+05 },
    { 1.3900000000000000e-09, 4.1421688769099996e+00, 9.7112936974696786e+04 },
    { 1.3399999999999999e-09, 3.0748238074700001e+00, 2.2759767485294011e+04 },
    { 1.3100000000000000e-09, 1.0672682783600000e+00, 6.4701083331480004e+02 },
    { 1.2900000000000001e-09, 2.5573703712000002e+00, 1.0413831347085879e+05 },
    { 1.2799999999999999e-09, 4.9346180853200003e+00, 2.6521614879451001e+04 },
    { 1.2700000000000000e-09, 2.3601132603799999e+00, 5.4294570143526791e+04 },
    { 1.2300000000000001e-09, 9.6970788408999997e-01, 5.2072713508929788e+04 },
    { 1.2199999999999999e-09, 2.3467641983399998e+00, 1.8150594343892598e+05 },
    { 1.2199999999999999e-09, 6.2784619600499996e+00, 2.8286990484861199e+04 },
    { 1.1900000000000000e-09, 6.1239919951699999e+00, 1.7806336849396059e+05 },
    { 1.1900000000000000e-09, 5.0190287766799999e+00, 1.3499146920492980e+05 },
    { 1.1599999999999999e-09, 2.4969055270600000e+00, 1.0434449901929598e+05 },
    { 1.1500000000000000e-09, 5.7981080442300001e+00, 2.9416038797854391e+04 },
    { 1.1500000000000000e-09, 6.2451458964700004e+00, 1.1063468841628819e+05 },
    { 1.1500000000000000e-09, 5.5273739332099998e+00, 2.5754047200804798e+04 },
    { 1.1500000000000000e-09, 1.3407265470300000e+00, 2.6011637070298599e+04 },
    { 1.1400000000000000e-09, 2.4802201170999999e-01, 1.5493834595416137e+05 },
    { 1.1300000000000000e-09, 3.2778155315599999e+00, 2.6421759082343600e+04 },
    { 1.1200000000000001e-09, 7.9767917932999999e-01, 2.4176703658357001e+04 },
    { 1.1200000000000001e-09, 1.5796425437499999e+00, 5.1639001771052986e+04 },
    { 1.1100000000000001e-09, 3.4252500939599999e+00, 1.1691776426627958e+05 },
    { 1.0999999999999999e-09, 4.8254692543299997e+00, 6.8241872144623194e+04 },
    { 1.0999999999999999e-09, 4.8750045576999996e+00, 2.4356780788641601e+04 },
    { 1.0800000000000000e-09, 5.6435888474300002e+00, 4.9527351457675388e+04 },
    { 1.0700000000000000e-09, 3.7251624379299999e+00, 5.5516418709848200e+04 },
    { 1.0700000000000000e-09, 3.8517739981999999e-01, 1.5748301859105378e+05 },
    { 1.0500000000000001e-09, 1.8392017095999999e-01, 1.5610082065856917e+05 },
    { 1.0399999999999999e-09, 2.5902352604300001e+00, 2.6198109462793600e+04 },
    { 1.0300000000000000e-09, 1.2647636392699999e+00, 7.4923096998022782e+04 },
    { 1.0300000000000000e-09, 1.5077453109300001e+00, 5.1951461487446490e+04 },
    { 1.0300000000000000e-09, 3.4376518923999999e+00, 2.6941099523326200e+04 },
    { 1.0200000000000000e-09, 2.0220739054000000e-01, 1.0435549390165479e+05 },
    { 1.0200000000000000e-09, 5.0920618573700001e+00, 5.2250587881715699e+04 },
    { 1.0200000000000000e-09, 5.0865850779599997e+00, 5.2225802905052602e+04 },
    { 1.0200000000000000e-09, 1.7746576936700000e+00, 1.0434773123093879e+05 },
    { 1.0100000000000000e-09, 1.8063045173200001e+00, 2.7999102624791401e+04 },
    { 1.0000000000000001e-09, 3.8157397133400002e+00, 2.5035634758385801e+04 },
    { 1.0000000000000001e-09, 1.6991225472600000e+00, 1.0456491166173479e+05 },
    { 1.0000000000000001e-09, 3.9390140741200002e+00, 2.7819025494506801e+04 },
    { 1.0000000000000001e-09, 3.7853823749900002e+00, 1.1911685160956658e+05 },
    { 9.9000000000000010e-10, 5.1152017698199996e+00, 3.1415379249957001e+04 },
    { 9.9000000000000010e-10, 3.2899059570999998e+00, 9.7580901965051191e+04 },
    { 9.6999999999999996e-10, 3.5686011306799998e+00, 3.2350541665740002e+02 },
    { 9.6999999999999996e-10, 2.9825882904899998e+00, 5.1219517127177787e+04 },
    { 9.5999999999999999e-10, 1.0322459495500000e+00, 3.8813356576349201e+04 },
    { 9.5000000000000003e-10, 1.9153850211300001e+00, 7.4299006053259995e+02 },
    { 9.5000000000000003e-10, 1.5660492849600001e+00, 5.2698383701242194e+04 },
    { 9.4000000000000006e-10, 2.3232088899200001e+00, 5.4509002676020398e+04 },
    { 9.4000000000000006e-10, 3.4925934984100002e+00, 5.2125809661244188e+04 },
    { 9.4000000000000006e-10, 3.9151861530600001e+00, 2.7780068811076591e+04 },
    { 9.4000000000000006e-10, 3.0199229192199999e+00, 5.1859414413491788e+04 },
    { 9.4000000000000006e-10, 1.3957997384700001e+00, 2.2218566345969998e+03 },
    { 9.2000000000000003e-10, 4.3584148485899998e+00, 1.4065280125408198e+05 },
    { 9.0999999999999996e-10, 2.6654859529000001e+00, 5.1013331578740588e+04 },
    { 8.9999999999999999e-10, 3.6739419694000000e+00, 3.6109740421673603e+04 },
    { 8.9999999999999999e-10, 5.5685278912999996e+00, 5.2492198152804987e+04 },
    { 8.9000000000000003e-10, 9.5977221727999995e-01, 1.3767819129947099e+05 },
    { 8.9000000000000003e-10, 2.4356216709699998e+00, 1.9111994832171999e+03 },
    { 8.7999999999999996e-10, 3.1492583002500001e+00, 5.2065599961928987e+04 },
    { 8.6999999999999999e-10, 4.8022795508199998e+00, 5.2325369480282992e+04 },
    { 8.6999999999999999e-10, 1.6591639266200000e+00, 1.0477821075717278e+05 },
    { 8.6000000000000003e-10, 1.5084715121100001e+00, 5.3228074666336790e+04 },
    { 8.4999999999999996e-10, 1.2975702964500000e+00, 1.2725453434775000e+04 },
    { 8.3000000000000003e-10, 1.9723041975800000e+00, 7.8731674415076996e+04 },
    { 8.1999999999999996e-10, 4.8756228026099997e+00, 2.6089387614282488e+04 },
    { 8.0999999999999999e-10, 6.2021502564200004e+00, 2.5977696820354791e+04 },
    { 8.0999999999999999e-10, 4.3378864994699997e+00, 5.3242301760338400e+04 },
    { 8.0999999999999999e-10, 1.6599813105200001e+00, 1.0594068546158058e+05 },
    { 8.0999999999999999e-10, 3.5796603033799999e+00, 4.9842609890276392e+04 },
    { 8.0999999999999999e-10, 3.9512390143300000e+00, 2.6086418668865899e+04 },
    { 8.0000000000000003e-10, 6.1393665889199998e+00, 8.1942753332085995e+03 },
    { 7.8999999999999996e-10, 4.1636679964000001e+00, 1.0328511708910679e+05 },
    { 7.8999999999999996e-10, 3.7178475791599999e+00, 1.0420204936916218e+05 },
    { 7.8999999999999996e-10, 2.2287694810800001e+00, 2.5446489579835201e+04 },
    { 7.7999999999999999e-10, 4.8002200097600003e+00, 6.0055895436487393e+04 },
    { 7.7999999999999999e-10, 5.3647275286099996e+00, 1.0435872611329758e+05 },
    { 7.7999999999999999e-10, 3.7144244734999998e+00, 7.7829997686845789e+04 },
    { 7.7000000000000003e-10, 5.8542058012000000e+00, 1.0213285546210999e+05 },
    { 7.7000000000000003e-10, 2.9329825953999999e-01, 2.6729316703313190e+04 },
    { 7.7000000000000003e-10, 7.7427822658000001e-01, 1.0329934418310839e+05 },
    { 7.5999999999999996e-10, 7.2423080439999996e-02, 3.3281356562801898e+03 },
    { 7.5999999999999996e-10, 6.2773142704199998e+00, 5.2595290927023590e+04 },
    { 7.5999999999999996e-10, 8.6881407999999993e-02, 7.7844224780847391e+04 },
    { 7.5999999999999996e-10, 1.8731601175899999e+00, 1.3721022630911658e+05 },
    { 7.4000000000000003e-10, 5.0889180779399998e+00, 2.6222012191159200e+04 },
    { 7.4000000000000003e-10, 2.3458298969500002e+00, 9.7670387712897180e+04 },
    { 7.2999999999999996e-10, 3.5759823307600000e+00, 5.0696939709083992e+04 },
    { 7.2999999999999996e-10, 1.1193297019299999e+00, 2.5021407664384202e+04 },
    { 7.2999999999999996e-10, 3.7070573617100000e+00, 2.5953794091989192e+04 },
    { 7.2999999999999996e-10, 3.1818002268900001e+00, 2.4864085300795588e+04 },
    { 7.2000000000000000e-10, 1.4741633561100000e+00, 1.7004917029103660e+05 },
    { 7.2000000000000000e-10, 2.4080620106200001e+00, 2.4998194350380589e+04 },
    { 7.2000000000000000e-10, 4.5732399367000003e+00, 5.0800032483302588e+04 },
    { 7.1000000000000003e-10, 4.1129372914099998e+00, 2.7669862489857191e+04 },
    { 7.1000000000000003e-10, 7.2381505769999993e-02, 1.2699694076290558e+05 },
    { 7.1000000000000003e-10, 1.7687846616200000e+00, 2.7684089583858800e+04 },
    { 7.1000000000000003e-10, 3.3618482135700001e+00, 1.7893627808365600e+04 },
    { 7.1000000000000003e-10, 5.7555824034600001e+00, 3.4282178474782799e+04 },
    { 6.9000000000000000e-10, 5.7091762081999997e-01, 2.0764384320240439e+05 },
    { 6.8000000000000003e-10, 3.1146103034100001e+00, 2.0618554843720000e+02 },
    { 6.8000000000000003e-10, 2.9337521842999998e+00, 1.9669760899789999e+01 },
    { 6.8000000000000003e-10, 6.5742763600000004e-02, 2.1990873432869998e+03 },
    { 6.8000000000000003e-10, 5.7968733104599997e+00, 7.8257080865822587e+04 },
    { 6.6999999999999996e-10, 2.6545784245299999e+00, 7.8270337983622594e+04 },
    { 6.6000000000000000e-10, 3.2539324058100001e+00, 7.0269180982698381e+04 },
    { 6.6000000000000000e-10, 1.2436731145000000e+00, 2.1116503133776000e+03 },
    { 6.6000000000000000e-10, 9.4934372069999995e-02, 2.0085575391590000e+03 },
    { 6.6000000000000000e-10, 1.8130668165000000e-01, 7.7795744434368185e+04 },
    { 6.5000000000000003e-10, 4.3021987590300004e+00, 1.2546481939083000e+04 },
    { 6.5000000000000003e-10, 1.0862838700999999e-01, 2.7177611932767799e+04 },
    { 6.5000000000000003e-10, 5.2222898513100002e+00, 7.7410513042970590e+04 },
    { 6.5000000000000003e-10, 5.6324748494000003e+00, 2.7311720982352810e+04 },
    { 6.3999999999999996e-10, 4.7309016615699999e+00, 7.6571543755220191e+04 },
    { 6.3000000000000000e-10, 1.9429943924799999e+00, 1.5024434299945380e+05 },
    { 6.3000000000000000e-10, 2.5060430775799998e+00, 5.2290245571833613e+04 },
    { 6.3000000000000000e-10, 6.0552591970499998e+00, 1.7695398994186718e+05 },
    { 6.2000000000000003e-10, 2.8268884135799999e+00, 8.4608283475120004e+02 },
    { 6.2000000000000003e-10, 3.6419522502700001e+00, 3.8133035637799999e+01 },
    { 6.0000000000000000e-10, 6.1167383476700001e+00, 5.2061366994463169e+04 },
    { 5.7999999999999996e-10, 1.0751030248900000e+00, 1.6785008294774959e+05 },
    { 5.7999999999999996e-10, 6.2543264682100004e+00, 2.5508215554575399e+04 },
    { 5.6000000000000003e-10, 3.5549434457600002e+00, 4.4295717129809404e+04 },
    { 5.6000000000000003e-10, 3.8416576898699999e+00, 2.2625658435709000e+04 },
    { 5.6000000000000003e-10, 3.4232837003099998e+00, 2.4448833624386200e+04 },
    { 5.6000000000000003e-10, 4.9574218554099998e+00, 2.7037078750543991e+04 },
    { 5.4999999999999996e-10, 5.9685130302299996e+00, 2.3969139281195799e+04 },
    { 5.4999999999999996e-10, 2.1633088479000002e+00, 2.0981260368468694e+05 },
    { 5.4999999999999996e-10, 2.0517247425099998e+00, 1.0779418751126219e+05 },
    { 5.4999999999999996e-10, 3.5933692819999998e+00, 1.7240203644480839e+05 },
    { 5.4999999999999996e-10, 3.2196009980300002e+00, 1.5705710981453978e+05 },
    { 5.4999999999999996e-10, 4.2152409889299998e+00, 2.3919142659291600e+04 },
    { 5.4000000000000000e-10, 4.4480815782700001e+00, 5.2252072354423988e+04 },
    { 5.4000000000000000e-10, 2.5705083838200000e+00, 2.6667590728572999e+04 },
    { 5.4000000000000000e-10, 4.8479481260500004e+00, 5.2381991831585590e+04 },
    { 5.3000000000000003e-10, 4.1084385559500003e+00, 2.6312247937276101e+04 },
    { 5.1999999999999996e-10, 4.9127322047200002e+00, 6.6653157466348392e+04 },
    { 5.1999999999999996e-10, 6.1110227925700000e+00, 2.8736357967047199e+04 },
    { 5.1999999999999996e-10, 3.8119198755400001e+00, 2.2041264243879999e+02 },
    { 5.1999999999999996e-10, 3.9055663258500002e+00, 2.5138727532604400e+04 },
    { 5.1000000000000000e-10, 5.9453457263000002e+00, 5.8857031136547987e+04 },
    { 5.1000000000000000e-10, 5.3613855530899999e+00, 2.7726972658762199e+04 },
    { 5.1000000000000000e-10, 4.2322753563999999e-01, 1.8208563102592478e+05 },
    { 5.0000000000000003e-10, 4.2040954722099997e+00, 1.2979961842155698e+05 },
    { 4.8999999999999996e-10, 1.0360623516000000e+00, 9.6357084124272587e+04 },
    { 4.8999999999999996e-10, 2.6134279529600000e+00, 2.7044192297544800e+04 },
    { 4.8999999999999996e-10, 2.3000452673899998e+00, 7.7630925685409384e+04 },
    { 4.8999999999999996e-10, 2.2636679091700000e+00, 8.6143798578061585e+04 },
    { 4.8000000000000000e-10, 6.0361129036800003e+00, 9.9024136457913992e+04 },
    { 4.8000000000000000e-10, 4.6852406671300004e+00, 3.9629324344065390e+04 },
    { 4.8000000000000000e-10, 4.0336414060500001e+00, 6.5831666774324796e+04 },
    { 4.8000000000000000e-10, 5.2221656419100002e+00, 2.5466159340735001e+04 },
    { 4.8000000000000000e-10, 5.5204010547000004e-01, 7.6667522982437978e+04 },
    { 4.7000000000000003e-10, 4.1047029967400004e+00, 5.1969620734711192e+04 },
    { 4.7000000000000003e-10, 4.9623378819099999e+00, 2.9550147847439392e+04 },
    { 4.7000000000000003e-10, 1.3304875162200001e+00, 1.5557181910783658e+05 },
    { 4.7000000000000003e-10, 4.0632510288299999e+00, 1.0427534649502118e+05 },
    { 4.7000000000000003e-10, 3.5737752769000002e+00, 2.6709646942413401e+04 },
    { 4.7000000000000003e-10, 4.6836410864000000e-01, 1.4742351185532758e+05 },
    { 4.6000000000000001e-10, 4.5283649936100003e+00, 7.6681750076439581e+04 },
    { 4.6000000000000001e-10, 2.0905451039599998e+00, 9.4917560896980001e+02 },
    { 4.6000000000000001e-10, 4.5896695237899996e+00, 2.8256663623856792e+04 },
    { 4.5000000000000000e-10, 3.6905674935000001e+00, 2.5867490499135391e+04 },
    { 4.5000000000000000e-10, 5.5090903908800000e+00, 2.1258774073792001e+03 },
    { 4.5000000000000000e-10, 1.8662983743899999e+00, 1.5377881048480000e+02 },
    { 4.3999999999999998e-10, 2.1182620800400001e+00, 1.6703062133498999e+04 },
    { 4.3999999999999998e-10, 4.0937360578900002e+00, 7.8903606711036584e+04 },
    { 4.3999999999999998e-10, 1.8464522171000000e-01, 1.2655674786264001e+03 },
    { 4.3999999999999998e-10, 2.8326275745000001e-01, 5.2712610795243789e+04 },
    { 4.3999999999999998e-10, 5.2329138144399998e+00, 2.6308315784013001e+04 },
    { 4.3999999999999998e-10, 1.4387327102100000e+00, 5.1528795449833589e+04 },
    { 4.3000000000000001e-10, 2.7831427428700000e+00, 1.2726810256271999e+03 },
    { 4.3000000000000001e-10, 4.7835978073599996e+00, 3.6523064398203877e+05 },
    { 4.3000000000000001e-10, 9.4126409896999996e-01, 1.5758680077963436e+05 },
    { 4.2000000000000000e-10, 4.6089983609000001e-01, 1.4859801210651999e+03 },
    { 4.2000000000000000e-10, 5.1072992827999997e-01, 7.8188927826155283e+04 },
    { 4.2000000000000000e-10, 1.7652671701200000e+00, 1.7952584437210000e+03 },
    { 4.2000000000000000e-10, 4.7364588250299997e+00, 2.1436455718174577e+05 },
    { 4.2000000000000000e-10, 3.6265281733500001e+00, 2.5169972855592401e+04 },
    { 4.2000000000000000e-10, 2.4422190317000001e+00, 7.5615254599249587e+04 },
    { 4.0999999999999998e-10, 4.6864573618299996e+00, 5.4087005766365597e+04 },
    { 4.0999999999999998e-10, 4.1089314908499999e+00, 6.4901259717923393e+04 },
    { 4.0000000000000001e-10, 2.4359373510400002e+00, 2.5352026628042389e+04 },
    { 4.0000000000000001e-10, 6.4266071719999998e-02, 2.6823779655105991e+04 },
    { 4.0000000000000001e-10, 4.0238264738599998e+00, 2.5654191403697390e+04 },
    { 4.0000000000000001e-10, 2.8352631561099999e+00, 1.4807872442630000e+02 },
    { 3.9000000000000000e-10, 2.4225285815499999e+00, 5.5503941939428587e+04 },
    { 3.9000000000000000e-10, 3.8295290652000000e+00, 1.5104766984290001e+02 },
    { 3.9000000000000000e-10, 5.1659501855499999e+00, 2.7005833427555990e+04 },
    { 3.7999999999999998e-10, 2.9526515212300000e+00, 1.2832075184749259e+05 },
    { 3.7999999999999998e-10, 4.2496407940700003e+00, 1.0450539137678158e+05 },
    { 3.7999999999999998e-10, 5.0541454200800002e+00, 1.3541421202491190e+04 },
    { 3.7999999999999998e-10, 2.7407746634999999e+00, 3.7698455099948398e+04 },
    { 3.7000000000000001e-10, 5.4587010376100000e+00, 1.3265827281205778e+05 },
    { 3.7000000000000001e-10, 1.5651744362000000e-01, 2.6073676047572590e+04 },
    { 3.7000000000000001e-10, 1.9542951069600001e+00, 1.3045918546877075e+05 },
    { 3.7000000000000001e-10, 1.0010579661000001e-01, 2.4203001978156801e+04 },
    { 3.7000000000000001e-10, 3.8648817061499998e+00, 7.8366802198941179e+04 },
    { 3.7000000000000001e-10, 2.4373196166700000e+00, 2.5773716961704591e+04 },
    { 3.7000000000000001e-10, 7.5602096450000003e-02, 2.6402089321443800e+04 },
    { 3.6000000000000000e-10, 4.0150505676300003e+00, 1.0419783375581198e+05 },
    { 3.6000000000000000e-10, 1.2530415996499999e+00, 1.1678365521669458e+05 },
    { 3.6000000000000000e-10, 1.0796154075600000e+00, 2.6057576280569789e+04 },
    { 3.6000000000000000e-10, 2.3535154614999998e+00, 2.6102130235575802e+04 },
    { 3.6000000000000000e-10, 5.2845791866800003e+00, 7.8896493164035783e+04 },
    { 3.6000000000000000e-10, 1.6123286222300000e+00, 5.1653228865054589e+04 },
    { 3.6000000000000000e-10, 6.1242853673499997e+00, 1.3041984594697120e+05 },
    { 3.4999999999999998e-10, 1.4333063055499999e+00, 2.6118230002578599e+04 },
    { 3.4999999999999998e-10, 1.0532011063400000e+00, 2.7463676941419999e+04 },
    { 3.4999999999999998e-10, 4.0523188780000002e-01, 9.3848410080752001e+03 },
    { 3.4999999999999998e-10, 5.6332374337299997e+00, 1.3022621661243298e+05 },
    { 3.4999999999999998e-10, 6.2190016918200000e+00, 5.2483363904117992e+04 },
    { 3.4000000000000001e-10, 1.9958682343100000e+00, 2.7566769715638591e+04 },
    { 3.4000000000000001e-10, 2.4076765011500001e+00, 5.1868248662178798e+04 },
    { 3.4000000000000001e-10, 5.9661487909300002e+00, 2.2003914634869801e+04 },
    { 3.4000000000000001e-10, 4.7148789572499998e+00, 1.9054647649404001e+03 },
    { 3.3000000000000000e-10, 4.2458452244000000e-01, 2.6189865659839801e+04 },
    { 3.3000000000000000e-10, 2.8893001358200001e+00, 2.0415127163553477e+05 },
    { 3.3000000000000000e-10, 2.3168510760000000e-01, 3.5472744149649399e+04 },
    { 3.3000000000000000e-10, 2.4091600474299999e+00, 2.7972804304991590e+04 },
    { 3.3000000000000000e-10, 4.6418311064399997e+00, 7.7726904912627186e+04 },
    { 3.3000000000000000e-10, 4.5490464587900004e+00, 7.8039364629020682e+04 },
    { 3.3000000000000000e-10, 9.1422316489000000e-01, 1.2320084011627098e+05 },
    { 3.1999999999999998e-10, 5.2328398346300000e+00, 5.1852300866490987e+04 },
    { 3.1999999999999998e-10, 2.0883371906699999e+00, 2.5985940623308590e+04 },
    { 3.1999999999999998e-10, 1.0369299589200001e+00, 4.9953949648551388e+04 },
    { 3.1999999999999998e-10, 2.6228153298999999e+00, 2.4925428437166400e+04 },
    { 3.1000000000000002e-10, 4.0186236623299996e+00, 7.8160616650503987e+04 },
    { 3.1000000000000002e-10, 5.3938168664499999e+00, 2.0759384658050019e+05 },
    { 3.1000000000000002e-10, 1.6356504355699999e+00, 9.4329775286197386e+04 },
    { 3.1000000000000002e-10, 5.8465350275000005e-01, 5.1432816222615787e+04 },
    { 3.1000000000000002e-10, 4.4397058845600004e+00, 8.7367616418840189e+04 },
    { 3.0000000000000000e-10, 5.5392057325700002e+00, 1.3043240216087017e+05 },
    { 3.0000000000000000e-10, 3.3034856773699999e+00, 1.8102624909573558e+05 },
    { 3.0000000000000000e-10, 5.6975359730799999e+00, 7.7101234720314780e+04 },
    { 3.0000000000000000e-10, 6.0257805353500000e+00, 7.7307420268751986e+04 },
    { 3.0000000000000000e-10, 1.7758015057400001e+00, 1.6107937234650398e+05 },
    { 3.0000000000000000e-10, 7.0937754307000001e-01, 5.3906928636080993e+04 },
    { 2.8999999999999998e-10, 2.8445439790200000e+00, 1.1439910691340000e+02 },
    { 2.8999999999999998e-10, 2.6068945651000002e+00, 2.6083970988311092e+04 },
    { 2.8999999999999998e-10, 6.1929106927499999e+00, 2.6091835294837289e+04 },
    { 2.8999999999999998e-10, 3.0258248858200001e+00, 1.3672259155786238e+05 },
    { 2.8999999999999998e-10, 4.0143107474499997e+00, 2.5600268390320001e+04 },
    { 2.8000000000000002e-10, 5.4222646207600000e+00, 8.0382473285100990e+04 },
    { 2.8000000000000002e-10, 3.2353034339200000e+00, 1.8218872380014337e+05 },
    { 2.8000000000000002e-10, 4.8396088623000000e-01, 5.1226630674178588e+04 },
    { 2.8000000000000002e-10, 4.7790891309300001e+00, 2.6575537892828390e+04 },
    { 2.8000000000000002e-10, 2.8276354952700000e+00, 3.0171891648278601e+04 },
    { 2.8000000000000002e-10, 4.6718713922999999e+00, 6.1292970395066004e+03 },
    { 2.8000000000000002e-10, 4.5042392461200000e+00, 1.7311223529326001e+03 },
    { 2.7000000000000000e-10, 4.3495889237099998e+00, 1.0101100013959699e+05 },
    { 2.7000000000000000e-10, 5.5491266960000001e-02, 5.2509662223917803e+04 },
    { 2.7000000000000000e-10, 3.4078961964299999e+00, 1.8357092173262799e+05 },
    { 2.7000000000000000e-10, 5.6352029092600002e+00, 1.2238178407786099e+03 },
    { 2.7000000000000000e-10, 5.4501526303399999e+00, 2.7170983373867792e+04 },
    { 2.7000000000000000e-10, 3.3550650544599998e+00, 2.5004822909280600e+04 },
    { 2.7000000000000000e-10, 1.9476264673999999e-01, 1.4300566740785379e+05 },
    { 2.7000000000000000e-10, 6.0107687912500003e+00, 2.4072921469776400e+04 },
    { 2.5999999999999998e-10, 4.2754418905699998e+00, 2.6049770105936401e+04 },
    { 2.5999999999999998e-10, 3.0461113788900001e+00, 5.4374893626435398e+04 },
    { 2.5999999999999998e-10, 4.4049774718099997e+00, 3.9450352848373397e+04 },
    { 2.5999999999999998e-10, 6.2185379079400001e+00, 7.8153503103503186e+04 },
    { 2.5999999999999998e-10, 5.4164299512699996e+00, 2.5234706759822191e+04 },
    { 2.5000000000000002e-10, 4.7501018530800003e+00, 1.3065281480330898e+05 },
    { 2.5000000000000002e-10, 1.8691841655899999e+00, 7.8338491023289884e+04 },
    { 2.5000000000000002e-10, 3.4270389497800000e+00, 2.9912639426919998e+02 },
    { 2.5000000000000002e-10, 3.2851131540499998e+00, 9.2741060607922584e+04 },
    { 2.5000000000000002e-10, 6.1237209805699999e+00, 4.8847670626868203e+04 },
    { 2.5000000000000002e-10, 4.3679186924600000e+00, 2.6724899413598399e+04 },
    { 2.5000000000000002e-10, 4.4372989923399997e+00, 2.5450906869549999e+04 },
    { 2.5000000000000002e-10, 4.8252160600199998e+00, 1.3043563437251298e+05 },
    { 2.5000000000000002e-10, 7.2627176074999999e-01, 5.2174321810440088e+04 },
    { 2.5000000000000002e-10, 4.4167057897000001e-01, 8.1604321851422384e+04 },
    { 2.5000000000000002e-10, 3.2528702465600001e+00, 1.3044339704322898e+05 },
    { 2.5000000000000002e-10, 1.7902803960200000e+00, 7.8313706046626787e+04 },
    { 2.4000000000000000e-10, 1.0987928441299999e+00, 2.4824745778995999e+04 },
    { 2.4000000000000000e-10, 1.6716074970299999e+00, 1.0451548361876000e+03 },
    { 2.4000000000000000e-10, 1.7057028062099999e+00, 5.2609518021025193e+04 },
    { 2.4000000000000000e-10, 9.2190112786000000e-01, 1.2937302023068098e+05 },
    { 2.4000000000000000e-10, 5.6839536695500001e+00, 5.2286012604367788e+04 },
    { 2.4000000000000000e-10, 4.6206594872600002e+00, 7.8786286842816378e+04 },
    { 2.4000000000000000e-10, 2.2946657685699998e+00, 5.1841950342379001e+04 },
    { 2.4000000000000000e-10, 2.7793844821499998e+00, 2.8102884813371998e+04 },
    { 2.4000000000000000e-10, 5.5755098688000004e-01, 1.4520475475114078e+05 },
    { 2.4000000000000000e-10, 1.4759612186600000e+00, 5.2177290755856688e+04 },
    { 2.3000000000000001e-10, 1.2892315226300000e+00, 6.0370081616356991e+04 },
    { 2.3000000000000001e-10, 3.0769835394000000e-01, 7.6784842850658184e+04 },
    { 2.3000000000000001e-10, 6.0398663687100003e+00, 7.7947317555065980e+04 },
    { 2.3000000000000001e-10, 6.0053909570000001e-02, 1.2366880510662538e+05 },
    { 2.3000000000000001e-10, 4.8029876686000001e+00, 3.1639186965660002e+02 },
    { 2.3000000000000001e-10, 4.8284788905999998e-01, 1.0391790082841998e+05 },
    { 2.3000000000000001e-10, 3.8348394009900000e+00, 1.2938724732468258e+05 },
    { 2.3000000000000001e-10, 1.1269081674400001e+00, 1.6674070439565618e+05 },
    { 2.3000000000000001e-10, 2.3118397081700000e+00, 5.0167248743989388e+04 },
    { 2.1999999999999999e-10, 2.6166099168000001e-01, 7.8213712802818380e+04 },
    { 2.1999999999999999e-10, 1.4148360441200001e+00, 2.7351060504152389e+04 },
    { 2.1999999999999999e-10, 3.8482436623299998e+00, 5.0264606799931200e+04 },
    { 2.1999999999999999e-10, 1.8321109589800000e+00, 6.8851498899308099e+03 },
    { 2.1999999999999999e-10, 1.1988183394100000e+00, 5.0064155969770793e+04 },
    { 2.1999999999999999e-10, 1.6426216390599999e+00, 5.0444683930215797e+04 },
    { 2.1999999999999999e-10, 2.3181594237900001e+00, 7.8580101294379187e+04 },
    { 2.1999999999999999e-10, 5.3707223792800001e+00, 8.0596905817594597e+04 },
    { 2.1999999999999999e-10, 3.9856311745299999e+00, 1.6376609444104519e+05 },
    { 2.1999999999999999e-10, 3.3996515677799999e+00, 1.6342582585717601e+04 },
    { 2.1999999999999999e-10, 5.5741232303299997e+00, 3.4622447058652001e+03 },
    { 2.1999999999999999e-10, 4.7614591329899998e+00, 2.6126036177212001e+04 },
    { 2.1999999999999999e-10, 4.1442582553599996e+00, 2.6734913974889001e+04 },
    { 2.1999999999999999e-10, 5.5488878699799997e+00, 2.4952226269680999e+04 },
    { 2.1999999999999999e-10, 3.1797351217499998e+00, 1.0393212792242158e+05 },
    { 2.1999999999999999e-10, 1.3091790041400000e+00, 7.6887935624876787e+04 },
    { 2.1999999999999999e-10, 4.5105409187900003e+00, 1.0554497769261000e+03 },
    { 2.1000000000000000e-10, 1.4399095625300000e+00, 1.0633140834523001e+03 },
    { 2.1000000000000000e-10, 4.8382399190000003e-01, 1.3028995251073639e+05 },
    { 2.1000000000000000e-10, 3.2421993296600000e+00, 2.7223580013467399e+04 },
    { 2.1000000000000000e-10, 3.1550395135799998e+00, 1.5308484390447979e+05 },
    { 2.1000000000000000e-10, 3.5572830385000000e+00, 3.1781457905676002e+03 },
    { 2.0000000000000001e-10, 5.6376644040299997e+00, 5.1955393640709590e+04 },
    { 2.0000000000000001e-10, 6.0626362838799999e+00, 8.6457984757931190e+04 },
    { 2.0000000000000001e-10, 5.3739248347900004e+00, 1.2375829085447139e+05 },
    { 2.0000000000000001e-10, 4.5282350610500002e+00, 1.9613707343261078e+05 },
    { 2.0000000000000001e-10, 5.0179156258999997e+00, 1.0481957755665119e+05 },
    { 2.0000000000000001e-10, 4.3652103027900004e+00, 2.5907826011289599e+04 },
    { 2.0000000000000001e-10, 4.4400073820100001e+00, 2.6267980271858800e+04 },
    { 2.0000000000000001e-10, 5.0959169307299996e+00, 1.2412372889497999e+04 },
    { 2.0000000000000001e-10, 3.6126538990100001e+00, 2.3373174634397859e+05 },
    { 1.8999999999999999e-10, 1.4811705167000000e-01, 1.0897087911936001e+03 },
    { 1.8999999999999999e-10, 5.9989646448400000e+00, 5.1329723448397199e+04 },
    { 1.8999999999999999e-10, 3.0980526438800000e+00, 7.8683194068597775e+04 },
    { 1.8999999999999999e-10, 1.5775421269900001e+00, 7.8413272621857192e+04 },
    { 1.8999999999999999e-10, 3.5601360574399998e+00, 1.0201841617342478e+05 },
    { 1.8999999999999999e-10, 2.1224418674600001e+00, 1.3044662925487179e+05 },
    { 1.8999999999999999e-10, 2.2114816361000000e-01, 5.3029002664900399e+04 },
    { 1.8999999999999999e-10, 4.6623545938399999e+00, 2.6248310510959000e+04 },
    { 1.8999999999999999e-10, 4.1428630909599997e+00, 2.5927495772189392e+04 },
    { 1.8999999999999999e-10, 3.3635247628000000e+00, 5.2817219844887390e+04 },
    { 1.8999999999999999e-10, 4.8618933810799998e+00, 2.5863558345872290e+04 },
    { 1.8999999999999999e-10, 4.7148761539299997e+00, 1.3086611389874699e+05 },
    { 1.8999999999999999e-10, 4.9719558142600002e+00, 1.6329812945069079e+05 },
    { 1.8000000000000000e-10, 2.6185373208399998e+00, 1.2822075860368418e+05 },
    { 1.8000000000000000e-10, 3.0828612349000001e-01, 2.6411408558231600e+04 },
    { 1.8000000000000000e-10, 5.7868584248500001e+00, 7.5930513031850584e+04 },
    { 1.8000000000000000e-10, 4.5873805064199997e+00, 7.9315977807910982e+04 },
    { 1.8000000000000000e-10, 4.3707176640000001e-01, 6.2197643563247788e+04 },
    { 1.8000000000000000e-10, 4.7185778009800003e+00, 1.3202858860315479e+05 },
    { 1.7000000000000001e-10, 7.2171071986000002e-01, 5.3867971952650791e+04 },
    { 1.7000000000000001e-10, 5.4827795794499998e+00, 2.4602612434870989e+04 },
    { 1.7000000000000001e-10, 3.6844604423999999e-01, 1.1045700263900001e+01 },
    { 1.7000000000000001e-10, 5.5044523925700002e+00, 5.2137673247510589e+04 },
    { 1.7000000000000001e-10, 1.7578182580199999e+00, 5.5638050989013587e+04 },
    { 1.7000000000000001e-10, 2.0076222452899999e+00, 8.0174908907939782e+04 },
    { 1.7000000000000001e-10, 1.4830573241000000e+00, 1.0265944689679438e+05 },
    { 1.7000000000000001e-10, 1.4097228090000000e-01, 1.4227094001599999e+01 },
    { 1.5999999999999999e-10, 4.0128463582200000e+00, 3.8813353579999998e+00 },
    { 1.5999999999999999e-10, 1.1221337218100000e+00, 7.9330204901912584e+04 },
    { 1.5999999999999999e-10, 3.3077769896799998e+00, 2.7573193848277399e+04 },
    { 1.5999999999999999e-10, 4.5313509619200003e+00, 2.8309759776171191e+04 },
    { 1.5999999999999999e-10, 1.9585084407400000e+00, 1.0349841618454478e+05 },
    { 1.5999999999999999e-10, 5.7022597637899999e+00, 1.0435824112519680e+05 },
    { 1.5999999999999999e-10, 5.0013042714899996e+00, 1.7633224614102798e+05 },
    { 1.5999999999999999e-10, 3.4142596191000002e-01, 7.7750543983900002e+01 },
    { 1.5999999999999999e-10, 5.8332265620500001e+00, 8.1591845081002786e+04 },
    { 1.5999999999999999e-10, 2.5601216669100002e+00, 1.0434498400739678e+05 },
    { 1.5999999999999999e-10, 2.2954009331999998e+00, 5.0910238804522000e+04 },
    { 1.5999999999999999e-10, 2.1444378766300001e+00, 5.3814875800336391e+04 },
    { 1.5999999999999999e-10, 2.8198240192599999e+00, 2.0304189308344139e+05 },
    { 1.5999999999999999e-10, 5.2789342323100001e+00, 5.1534392721409400e+04 },
    { 1.5999999999999999e-10, 4.4292493527700003e+00, 3.4082431583598402e+04 },
    { 1.5000000000000000e-10, 6.0154457145500002e+00, 5.0903125257521198e+04 },
    { 1.5000000000000000e-10, 9.4947396087000002e-01, 1.5588752156313116e+05 },
    { 1.5000000000000000e-10, 3.3656080356700002e+00, 6.1560647291223591e+04 },
    { 1.5000000000000000e-10, 1.8565118049200000e+00, 5.2309915332733399e+04 },
    { 1.5000000000000000e-10, 5.8551952775500000e+00, 6.0442285813753997e+03 },
    { 1.5000000000000000e-10, 2.0522311762499998e+00, 5.0689826162083191e+04 },
    { 1.5000000000000000e-10, 5.2056256729600001e+00, 2.3590050682626115e+05 },
    { 1.5000000000000000e-10, 1.0457829860900001e+00, 5.0007045800865802e+04 },
    { 1.5000000000000000e-10, 4.7760569011099996e+00, 2.6830893202106799e+04 },
    { 1.5000000000000000e-10, 4.5085431389600004e+00, 7.7616698591407781e+04 },
    { 1.5000000000000000e-10, 5.0250137064000002e+00, 2.1687604822826002e+03 },
    { 1.5000000000000000e-10, 3.1802518163500002e+00, 5.0049928875769190e+04 },
    { 1.5000000000000000e-10, 3.2239261298700002e+00, 1.0388364757594238e+05 },
    { 1.5000000000000000e-10, 6.0337346223599999e+00, 1.6390695171880000e+03 },
    { 1.4000000000000001e-10, 4.1332546224200000e+00, 1.9393798608932379e+05 },
    { 1.4000000000000001e-10, 5.0944170324900000e+00, 1.3388209065283637e+05 },
    { 1.4000000000000001e-10, 7.3264409159999999e-01, 5.0380547839427389e+04 },
    { 1.4000000000000001e-10, 3.8043611029000002e-01, 2.6247204876366399e+04 },
    { 1.4000000000000001e-10, 2.1415962673300002e+00, 2.5928601406782000e+04 },
    { 1.4000000000000001e-10, 4.2151696282400000e+00, 1.1695882514085999e+03 },
    { 1.4000000000000001e-10, 2.8825791901900000e+00, 7.8149270136037361e+04 },
    { 1.4000000000000001e-10, 2.8207090763400000e+00, 5.2161579189146789e+04 },
    { 1.4000000000000001e-10, 5.5545023020000004e+00, 7.8378148713407805e+04 },
    { 1.4000000000000001e-10, 1.6469233897200000e+00, 2.6076857441310302e+04 },
    { 1.4000000000000001e-10, 8.7510898790000002e-01, 2.6098948841838101e+04 },
    { 1.4000000000000001e-10, 2.8798157315299999e+00, 2.0043674560198800e+04 },
    { 1.4000000000000001e-10, 3.6434746970999998e-01, 1.9848993958638259e+05 },
    { 1.4000000000000001e-10, 3.1693162770400001e+00, 5.3265515074342002e+04 },
    { 1.4000000000000001e-10, 2.4104476958399998e+00, 5.3399624123927002e+04 },
    { 1.4000000000000001e-10, 5.5122452418799996e+00, 2.5241820306823000e+04 },
    { 1.4000000000000001e-10, 5.7209534406999998e+00, 5.2755493870147191e+04 },
    { 1.4000000000000001e-10, 4.8193051968000000e-01, 5.2041697233563391e+04 },
    { 1.4000000000000001e-10, 2.8303105850999999e-01, 8.6030992875280003e+02 },
    { 1.4000000000000001e-10, 6.2461369264900002e+00, 1.8314501295611399e+05 },
    { 1.4000000000000001e-10, 1.3687096205200000e+00, 5.4344566765430987e+04 },
    { 1.4000000000000001e-10, 1.6986591061500000e+00, 5.3124981892118187e+04 },
    { 1.2999999999999999e-10, 2.9550264105399999e+00, 1.2511203959948818e+05 },
    { 1.2999999999999999e-10, 5.2547247292100003e+00, 1.0371882882698359e+05 },
    { 1.2999999999999999e-10, 5.6846815097099999e+00, 3.2132131722949602e+04 },
    { 1.2999999999999999e-10, 5.3886561489100000e+00, 3.5833223697430803e+04 },
    { 1.2999999999999999e-10, 3.5362982826799998e+00, 1.0275542612401219e+05 },
    { 1.2999999999999999e-10, 4.9711914118999996e+00, 5.2024758613305487e+04 },
    { 1.2999999999999999e-10, 3.5353020397599999e+00, 1.7351141499690176e+05 },
    { 1.2999999999999999e-10, 3.4461114002300000e+00, 2.0817353416749899e+05 },
    { 1.2999999999999999e-10, 5.9676906073899998e+00, 5.2027727558722087e+04 },
    { 1.2999999999999999e-10, 1.6378401672700000e+00, 7.8469894973159782e+04 },
    { 1.2999999999999999e-10, 2.7129106428399998e+00, 2.7250377845981991e+04 },
    { 1.2999999999999999e-10, 4.0838967606700001e+00, 5.0270341518207999e+04 },
    { 1.2999999999999999e-10, 5.6162244063399998e+00, 2.6014606015715191e+04 },
    { 1.2999999999999999e-10, 3.1889932784599999e+00, 2.6161200267433189e+04 },
    { 1.2999999999999999e-10, 4.3832059845700000e+00, 1.0736090241908000e+03 },
    { 1.2999999999999999e-10, 4.0838512423499997e+00, 7.6041852790125587e+04 },
    { 1.2000000000000000e-10, 1.0978452006499999e+00, 1.9202753251643389e+04 },
    { 1.2000000000000000e-10, 8.4912804767000005e-01, 4.5290656393217592e+04 },
    { 1.2000000000000000e-10, 5.3562575136500001e+00, 2.3439448316101189e+04 },
    { 1.2000000000000000e-10, 5.9748323266499996e+00, 2.5984120952993599e+04 },
    { 1.2000000000000000e-10, 2.8303853581500000e+00, 2.6191685330154789e+04 },
    { 1.2000000000000000e-10, 4.3790254739699996e+00, 1.8165972224941078e+05 },
    { 1.2000000000000000e-10, 4.8523224955100002e+00, 5.3771992725432989e+04 },
    { 1.2000000000000000e-10, 1.5450821256000000e-01, 4.3981530949939799e+04 },
    { 1.2000000000000000e-10, 4.3540662699199997e+00, 2.8199553454951791e+04 },
    { 1.2000000000000000e-10, 3.0203043689800002e+00, 5.1596118696149599e+04 },
    { 1.2000000000000000e-10, 8.3151252252999996e-01, 1.3036324963659538e+05 },
    { 1.2000000000000000e-10, 5.6383636557100001e+00, 1.0170315774082378e+05 },
    { 1.2000000000000000e-10, 1.0618565424599999e+00, 9.0989162859497592e+04 },
    { 1.2000000000000000e-10, 5.4272845053800003e+00, 5.1439929769616589e+04 },
    { 1.2000000000000000e-10, 2.6532175537399998e+00, 5.2179738436411491e+04 },
    { 1.2000000000000000e-10, 9.4217302732999997e-01, 5.3757765631431394e+04 },
    { 1.2000000000000000e-10, 6.0370134181099999e+00, 2.6149629116314391e+04 },
    { 1.2000000000000000e-10, 2.7682042666900002e+00, 2.6026177166834001e+04 },
    { 1.2000000000000000e-10, 1.9007735119500000e+00, 5.2396218925587200e+04 },
    { 1.2000000000000000e-10, 3.7982169491200000e+00, 1.6992792165031999e+03 },
    { 1.0999999999999999e-10, 3.0769028375000000e+00, 5.2911682796680187e+04 },
    { 1.0999999999999999e-10, 3.2804624047999997e-01, 7.0383620271383596e+04 },
    { 1.0999999999999999e-10, 6.0906260335999995e-01, 4.8713561577283202e+04 },
    { 1.0999999999999999e-10, 3.5726852930600002e+00, 1.0427683096772949e+05 },
    { 1.0999999999999999e-10, 1.2516502766399999e+00, 1.0276965321801379e+05 },
    { 1.0999999999999999e-10, 1.6102048812100000e+00, 5.7503282391531196e+04 },
    { 1.0999999999999999e-10, 5.9633922030299997e+00, 2.0894536082776402e+04 },
    { 1.0999999999999999e-10, 1.3380048704000000e-01, 5.0536736765960391e+04 },
    { 1.0999999999999999e-10, 3.3132635640700001e+00, 7.8800513936817981e+04 },
    { 1.0999999999999999e-10, 5.4481283332899997e+00, 6.3699627202420004e+02 },
    { 1.0999999999999999e-10, 9.6041649145999997e-01, 7.8057523876285384e+04 },
    { 1.0999999999999999e-10, 2.4316868243899998e+00, 1.5874617595363196e+05 },
    { 1.0999999999999999e-10, 4.8363679337700001e+00, 9.1805130627213788e+04 },
    { 1.0999999999999999e-10, 3.6804422015800000e+00, 7.7520719364189979e+04 },
    { 1.0999999999999999e-10, 3.7078144335799998e+00, 3.9763433393650390e+04 },
    { 1.0999999999999999e-10, 4.0064410743799996e+00, 1.8367470392120857e+05 },
    { 1.0999999999999999e-10, 5.9867133729099997e+00, 4.0222625319642002e+03 },
    { 1.0999999999999999e-10, 1.8102064784200000e+00, 1.9367189162232800e+04 },
    { 1.0999999999999999e-10, 5.9578258716699999e+00, 3.9321532631000000e+00 },
    { 1.0999999999999999e-10, 4.1212653836900000e+00, 4.6046509243641798e+04 },
    { 1.0999999999999999e-10, 1.0139837719600000e+00, 2.5382485042536000e+03 },
    { 1.0999999999999999e-10, 3.2175271996200001e+00, 2.8096460680733191e+04 },
    { 1.0000000000000000e-10, 1.2262117190999999e+00, 7.8339975495998180e+04 },
    { 1.0000000000000000e-10, 2.6878942140800000e+00, 8.4944934278122193e+04 },
    { 1.0000000000000000e-10, 6.1887134617499999e+00, 1.5440865498906677e+05 },
    { 1.0000000000000000e-10, 1.9885134044799999e+00, 5.3093736569130189e+04 },
    { 1.0000000000000000e-10, 3.9084038572900002e+00, 3.3385594076939998e+02 },
    { 1.0000000000000000e-10, 9.4586399547000000e-01, 6.4141356173898998e+02 },
    { 1.0000000000000000e-10, 2.6101187208100001e+00, 6.7510408403457996e+03 },
    { 1.0000000000000000e-10, 7.9218036562000005e-01, 9.1919569915898988e+04 },
    { 1.0000000000000000e-10, 1.9920658507800000e+00, 7.7940204008065179e+04 },
    { 1.0000000000000000e-10, 5.7527432986000004e+00, 1.1443928868521000e+02 },
    { 1.0000000000000000e-10, 8.6888264917000002e-01, 1.0499150985261079e+05 },
    { 1.0000000000000000e-10, 5.9558119033799999e+00, 5.2061407176234992e+04 },
    { 1.0000000000000000e-10, 4.4203536250199997e+00, 4.3718763667741996e+03 },
    { 1.0000000000000000e-10, 3.5963730578100002e+00, 2.3384286986898602e+04 },
    { 1.0000000000000000e-10, 4.2445878508200003e+00, 1.9958606102067599e+04 },
    { 1.0000000000000000e-10, 5.9714085358100002e+00, 2.3023917477710897e+05 },
    { 1.0000000000000000e-10, 1.5370751224500001e+00, 3.9131854712361295e+05 },
    { 1.0000000000000000e-10, 5.4782245989399998e+00, 5.4060707446565793e+04 },
    { 1.0000000000000000e-10, 2.6436235855399999e+00, 4.9996621904199998e+01 },
    { 1.0000000000000000e-10, 1.9551080096700000e+00, 2.7515475996916002e+03 },
    { 1.0000000000000000e-10, 3.8039815769999999e-02, 1.1882896374949679e+05 },
    { 1.0000000000000000e-10, 1.5053863251299999e+00, 2.4045246032331997e+05 },
    { 1.0000000000000000e-10, 2.9267315275299999e+00, 2.2065640609610000e+04 },
    { 1.0000000000000000e-10, 1.7047873516600001e+00, 4.4082528487759998e+02 },
    { 8.9999999999999999e-11, 1.0121663795600000e+00, 1.3059329451835579e+05 },
    { 8.9999999999999999e-11, 3.6490625518000003e-01, 2.6513811918088199e+04 },
    { 8.9999999999999999e-11, 2.1571261224399998e+00, 2.5661994365060200e+04 },
    { 8.9999999999999999e-11, 2.3990277191199998e+00, 1.0318913786188899e+05 },
    { 8.9999999999999999e-11, 4.9303575942400002e+00, 1.5654708861034497e+05 },
    { 8.9999999999999999e-11, 5.1942576488699999e+00, 2.8791519296249800e+04 },
    { 8.9999999999999999e-11, 7.7711516174999995e-01, 1.3028573689738619e+05 },
    { 8.9999999999999999e-11, 6.1390015821199997e+00, 2.5131613985603592e+04 },
    { 8.9999999999999999e-11, 2.0403872510799999e+00, 1.0498439630560997e+05 },
    { 8.9999999999999999e-11, 5.4561476549599996e+00, 7.6255151885563580e+04 },
    { 8.9999999999999999e-11, 2.8452104223700001e+00, 3.1281270200372001e+04 },
    { 8.9999999999999999e-11, 8.4632922145000000e-01, 5.1130651446960786e+04 },
    { 8.9999999999999999e-11, 3.9890242427000000e-01, 5.2797550083987589e+04 },
    { 8.9999999999999999e-11, 5.8179332919300002e+00, 2.7360584167201388e+04 },
    { 8.9999999999999999e-11, 3.8150577137699999e+00, 2.4844415539895788e+04 },
    { 8.9999999999999999e-11, 4.9901599710299998e+00, 2.7331390743252599e+04 },
    { 8.9999999999999999e-11, 2.4689345920400001e+00, 1.5631411975400720e+05 },
    { 8.9999999999999999e-11, 5.4552789215099997e+00, 5.2190033377149994e+04 },
    { 8.9999999999999999e-11, 9.6943014828999996e-01, 1.6850521225016000e+03 },
    { 8.9999999999999999e-11, 3.9954966212999998e-01, 6.5717227485639582e+04 },
    { 8.9999999999999999e-11, 2.2657564911699999e+00, 2.2595331574704589e+04 },
    { 8.9999999999999999e-11, 4.0188189828500001e+00, 1.4852907067031999e+03 },
    { 8.9999999999999999e-11, 5.6114743673799996e+00, 5.2171874129885291e+04 },
    { 8.9999999999999999e-11, 5.7156355892999999e-01, 1.0445470534051539e+05 },
    { 8.9999999999999999e-11, 2.8236537491600000e+00, 1.5650774908854539e+05 },
    { 8.9999999999999999e-11, 4.4173702130899999e+00, 7.6152059111344992e+04 },
    { 8.9999999999999999e-11, 5.0012003607000004e+00, 2.7036161546755998e+03 },
    { 8.9999999999999999e-11, 3.6032324419399999e+00, 2.4395737472071800e+04 },
    { 8.9999999999999999e-11, 4.1530729712100003e+00, 2.4505943793291190e+04 },
    { 8.9999999999999999e-11, 5.8665797134200002e+00, 5.6259794789852793e+04 },
    { 8.9999999999999999e-11, 1.3133062267400000e+00, 1.8073704938650200e+04 },
    { 7.9999999999999995e-11, 1.0871974962700000e+00, 3.8634385080657201e+04 },
    { 7.9999999999999995e-11, 3.8183454662299998e+00, 2.6933985976325399e+04 },
    { 7.9999999999999995e-11, 5.6632406614899997e+00, 5.3132095439118988e+04 },
    { 7.9999999999999995e-11, 1.1729505378700000e+00, 2.3249494088155998e+03 },
    { 7.9999999999999995e-11, 8.9598602982999997e-01, 1.0681250536565400e+04 },
    { 7.9999999999999995e-11, 5.9346140744999998e-01, 1.5748458012822000e+03 },
    { 7.9999999999999995e-11, 5.3738923575099999e+00, 5.1876679888879189e+04 },
    { 7.9999999999999995e-11, 5.4533850466200002e+00, 7.7956151803752990e+04 },
    { 7.9999999999999995e-11, 8.9135797231000002e-01, 5.2400151078850300e+04 },
    { 7.9999999999999995e-11, 2.8918190165100000e+00, 5.4824261108621402e+04 },
    { 7.9999999999999995e-11, 5.2700943944700001e+00, 7.7741132006628788e+04 },
    { 7.9999999999999995e-11, 5.1376341867299997e+00, 4.2790965275073198e+04 },
    { 7.9999999999999995e-11, 3.1354573671499999e+00, 5.2489992463018003e+04 },
    { 7.9999999999999995e-11, 2.2966453241900000e+00, 1.5652030530244438e+05 },
    { 7.9999999999999995e-11, 5.8781443842000003e-01, 1.7098435436141812e+04 },
    { 7.9999999999999995e-11, 1.0794867429699999e+00, 5.1112492199696091e+04 },
    { 7.9999999999999995e-11, 2.0947934430099999e+00, 3.5077370847006590e+04 },
    { 7.9999999999999995e-11, 4.7070672226300001e+00, 1.2041767842777158e+05 },
    { 7.9999999999999995e-11, 4.8306296891500002e+00, 1.8716727548807819e+05 },
    { 7.9999999999999995e-11, 5.1536393920099997e+00, 4.3053034413910001e+02 },
    { 7.9999999999999995e-11, 3.9653888774600001e+00, 1.4928874325784517e+05 },
    { 7.9999999999999995e-11, 3.3220268434100002e+00, 1.1223170171963578e+05 },
    { 7.9999999999999995e-11, 3.4652076664900000e+00, 7.7314533815752788e+04 },
    { 7.9999999999999995e-11, 2.9854035862699999e+00, 7.8571267045692191e+04 },
    { 7.9999999999999995e-11, 3.4622724911399998e+00, 2.0149816717977999e+03 },
    { 7.9999999999999995e-11, 3.3429193232499999e+00, 1.0287274599223239e+05 },
    { 7.9999999999999995e-11, 1.6231496773000001e-01, 2.0711415223730978e+05 },
    { 7.9999999999999995e-11, 7.1218709318999995e-01, 3.2808617120915602e+04 },
    { 7.9999999999999995e-11, 2.8038530280799998e+00, 1.0339532341032618e+05 },
    { 7.9999999999999995e-11, 1.3908819625800000e+00, 1.0381480805420138e+05 },
    { 7.9999999999999995e-11, 3.1946978271200002e+00, 5.7968758699880004e+02 },
    { 7.9999999999999995e-11, 3.9455403552000001e-01, 5.1257875997166600e+04 },
    { 7.9999999999999995e-11, 5.4880642228700003e+00, 5.1861620103278787e+04 },
    { 7.9999999999999995e-11, 2.0147293932700001e+00, 5.1554062482309193e+04 },
    { 7.9999999999999995e-11, 5.8349605701799998e+00, 5.4190787954946187e+04 },
    { 7.9999999999999995e-11, 4.2920359796299996e+00, 5.1120356506222292e+04 },
    { 7.9999999999999995e-11, 2.2339925563200000e+00, 5.3258886515441991e+04 },
    { 7.9999999999999995e-11, 1.9192692505600000e+00, 4.2266603761290003e+02 },
    { 7.0000000000000004e-11, 4.4931615762600003e+00, 5.2206133144152802e+04 },
    { 7.0000000000000004e-11, 3.1508571506099998e+00, 5.0290905119730996e+04 },
    { 7.0000000000000004e-11, 5.4546577013700004e+00, 5.1086097491954788e+04 },
    { 7.0000000000000004e-11, 2.1336184499100002e+00, 2.3368174972207437e+05 },
    { 7.0000000000000004e-11, 7.6923088177999999e-01, 1.0424851979207818e+05 },
    { 7.0000000000000004e-11, 2.6499960610000001e-02, 2.9396369036954598e+04 },
    { 7.0000000000000004e-11, 2.4955324170100002e+00, 2.2779437246193800e+04 },
    { 7.0000000000000004e-11, 4.3853466292299998e+00, 2.6290156536748291e+04 },
    { 7.0000000000000004e-11, 4.4198710555699998e+00, 2.5885649746400089e+04 },
    { 7.0000000000000004e-11, 1.4671250316200000e+00, 1.5674071794488319e+05 },
    { 7.0000000000000004e-11, 1.5729913032200000e+00, 5.2663441034402589e+04 },
    { 7.0000000000000004e-11, 5.1481924613799999e+00, 5.2073843764882789e+04 },
    { 7.0000000000000004e-11, 2.1064452897999999e+00, 1.4984619399604559e+05 },
    { 7.0000000000000004e-11, 2.7381565372200001e+00, 7.7417626589971391e+04 },
    { 7.0000000000000004e-11, 3.9904320248400000e+00, 1.1356768718932001e+03 },
    { 7.0000000000000004e-11, 1.3457898252700000e+00, 2.3267071997448002e+04 },
    { 7.0000000000000004e-11, 1.1762425523500000e+00, 2.8908734285700401e+04 },
    { 7.0000000000000004e-11, 5.9282807903199997e+00, 9.1793028598180001e+02 },
    { 7.0000000000000004e-11, 4.1303600137699998e+00, 5.2145479422143988e+04 },
    { 7.0000000000000004e-11, 7.6139044700000002e-03, 2.0827662694171758e+05 },
    { 7.0000000000000004e-11, 5.7952196711499999e+00, 6.3786358241522597e+04 },
    { 7.0000000000000004e-11, 1.0744821974300001e+00, 1.2709890328117118e+05 },
    { 7.0000000000000004e-11, 4.3420430817300000e+00, 1.0297583876645098e+05 },
    { 7.0000000000000004e-11, 3.9591094313899999e+00, 1.8023719907218001e+03 },
    { 7.0000000000000004e-11, 4.7575701989999997e-02, 5.3311483155041598e+04 },
    { 7.0000000000000004e-11, 3.7577208175800001e+00, 7.9994831777655185e+04 },
    { 7.0000000000000004e-11, 3.4753291286499999e+00, 5.2277768801414000e+04 },
    { 7.0000000000000004e-11, 2.3424532162000000e-01, 2.9580474708443799e+04 },
    { 7.0000000000000004e-11, 2.1083489506499999e+00, 2.7388729959779190e+04 },
    { 7.0000000000000004e-11, 4.1368342696999999e-01, 2.4787076323369201e+04 },
    { 7.0000000000000004e-11, 1.3179012128500001e+00, 1.0412726777059489e+05 },
    { 7.0000000000000004e-11, 5.0533088963599999e+00, 2.7623879884543599e+04 },
    { 7.0000000000000004e-11, 3.7519087884400002e+00, 2.4551926398604792e+04 },
    { 7.0000000000000004e-11, 1.2757231640500000e+00, 5.2213939318786201e+04 },
    { 7.0000000000000004e-11, 2.7316705015200000e+00, 2.9746099546109999e+03 },
    { 7.0000000000000004e-11, 4.0171788358700002e+00, 2.5572439270481191e+04 },
    { 7.0000000000000004e-11, 4.7880388489300003e+00, 2.6603367012667200e+04 },
    { 7.0000000000000004e-11, 3.8107500043000000e+00, 4.1555249061209997e+02 },
    { 7.0000000000000004e-11, 6.1906274091900002e+00, 1.0830802322935999e+03 },
    { 7.0000000000000004e-11, 6.3875539048999996e-01, 3.3084658953804001e+03 },
    { 7.0000000000000004e-11, 2.7328897657000000e+00, 4.8091817776443997e+04 },
    { 7.0000000000000004e-11, 5.1200578191000004e+00, 4.8997660492580799e+04 },
    { 7.0000000000000004e-11, 6.0809577203899998e+00, 1.6281049469943656e+05 },
    { 7.0000000000000004e-11, 5.8745742804800001e+00, 3.0110165673538399e+04 },
    { 7.0000000000000004e-11, 1.2091478517600001e+00, 3.4102101344498187e+04 },
    { 7.0000000000000004e-11, 6.0831494509999997e+00, 9.7453205558566005e+03 },
    { 7.0000000000000004e-11, 3.2475236320500001e+00, 1.6909357054942797e+05 },
    { 7.0000000000000004e-11, 1.3998069900200001e+00, 2.6413412784722000e+03 },
    { 6.0000000000000000e-11, 3.5071750217800002e+00, 2.7573883262639389e+04 },
    { 6.0000000000000000e-11, 2.5652267994800000e+00, 8.9894677054323893e+03 },
    { 6.0000000000000000e-11, 2.2622742655999999e+00, 2.8774625236115400e+04 },
    { 6.0000000000000000e-11, 2.5975811201999999e-01, 2.3401181047032998e+04 },
    { 6.0000000000000000e-11, 6.1016389363599997e+00, 8.0462796768009590e+04 },
    { 6.0000000000000000e-11, 2.0479101510100000e+00, 2.3704710126797989e+04 },
    { 6.0000000000000000e-11, 4.7412222660999997e-01, 2.8471096156350399e+04 },
    { 6.0000000000000000e-11, 4.4977614696000003e+00, 5.3438963645726588e+04 },
    { 6.0000000000000000e-11, 2.2007061426500001e+00, 1.0647037642667518e+05 },
    { 6.0000000000000000e-11, 2.2720735257700002e+00, 7.8043296782283782e+04 },
    { 6.0000000000000000e-11, 3.5934549593799998e+00, 2.4822335662947800e+04 },
    { 6.0000000000000000e-11, 1.2929656544000001e-01, 5.3661096989851598e+04 },
    { 6.0000000000000000e-11, 3.5086113339000002e-01, 2.0965882487420217e+05 },
    { 6.0000000000000000e-11, 4.9264419142899998e+00, 1.0442639416486409e+05 },
    { 6.0000000000000000e-11, 2.3971511093000002e+00, 2.5764397724916791e+04 },
    { 6.0000000000000000e-11, 4.9732042379500001e+00, 2.7883161585295191e+04 },
    { 6.0000000000000000e-11, 3.6983934174200002e+00, 1.4287155835826878e+05 },
    { 6.0000000000000000e-11, 1.5222757660499999e+00, 1.5652353751408719e+05 },
    { 6.0000000000000000e-11, 5.1694822224200001e+00, 1.1789303376878797e+05 },
    { 6.0000000000000000e-11, 6.2336926599100000e+00, 2.2582926841020000e+02 },
    { 6.0000000000000000e-11, 3.5666390738799998e+00, 5.1933670587978004e+03 },
    { 6.0000000000000000e-11, 3.1103291799599999e+00, 1.4975670824819960e+05 },
    { 6.0000000000000000e-11, 3.4996105608099999e+00, 1.3000580396999417e+05 },
    { 6.0000000000000000e-11, 5.0746564227900004e+00, 7.6777729303657383e+04 },
    { 6.0000000000000000e-11, 4.6679820710000000e+00, 4.7803929916374203e+04 },
    { 6.0000000000000000e-11, 6.4154839978000000e-01, 2.7441651886591000e+04 },
    { 6.0000000000000000e-11, 1.8804839778400000e+00, 2.4734154396557398e+04 },
    { 6.0000000000000000e-11, 3.9528778596500000e+00, 1.5546092337225520e+05 },
    { 6.0000000000000000e-11, 2.8155848358900002e+00, 5.0160824611350588e+04 },
    { 6.0000000000000000e-11, 2.1031892671299999e+00, 2.2286032299359999e+02 },
    { 6.0000000000000000e-11, 3.7796193011599999e+00, 7.8262224952014280e+04 },
    { 6.0000000000000000e-11, 3.7960016411400002e+00, 7.6468450981001588e+04 },
    { 6.0000000000000000e-11, 2.1924181144700001e+00, 7.8225576389084788e+04 },
    { 6.0000000000000000e-11, 3.1062815054800001e+00, 7.8597565365491988e+04 },
    { 6.0000000000000000e-11, 3.6330037050600001e+00, 4.0839885067044002e+03 },
    { 6.0000000000000000e-11, 3.6078693285300001e+00, 1.7129265789271498e+05 },
    { 6.0000000000000000e-11, 2.6302312451000001e+00, 2.9864334027309000e+04 },
    { 6.0000000000000000e-11, 6.1749864396999996e+00, 2.2311472255839399e+04 },
    { 6.0000000000000000e-11, 2.8169928665300001e+00, 7.6991028399095390e+04 },
    { 6.0000000000000000e-11, 4.9789229915100002e+00, 2.1716026774800001e+04 },
    { 6.0000000000000000e-11, 1.2287107982999999e-01, 5.1092726050854799e+04 },
    { 6.0000000000000000e-11, 6.2639645091900000e+00, 1.7917274704605396e+05 },
    { 6.0000000000000000e-11, 3.9865825647599999e+00, 3.1903014001211191e+04 },
    { 6.0000000000000000e-11, 4.8186351200399997e+00, 2.0272792281937200e+04 },
    { 6.0000000000000000e-11, 2.9944849231199999e+00, 1.8849011634174001e+03 },
    { 6.0000000000000000e-11, 1.2900421419500001e+00, 1.0487418998439058e+05 },
    { 6.0000000000000000e-11, 3.2789858106100001e+00, 2.5462232949261801e+04 },
    { 6.0000000000000000e-11, 5.5262318741899996e+00, 2.6713573333886590e+04 },
    { 6.0000000000000000e-11, 4.6985232287900001e+00, 2.5440892308259390e+04 },
    { 6.0000000000000000e-11, 6.2331592292800000e+00, 1.5653130018480320e+05 },
    { 6.0000000000000000e-11, 2.8280295122800001e+00, 1.2244498726584678e+05 },
    { 6.0000000000000000e-11, 6.4362124464000003e-01, 3.4925715668695998e+03 },
    { 6.0000000000000000e-11, 4.4186389072400001e+00, 1.3675530252076200e+04 },
    { 6.0000000000000000e-11, 2.3006953625000000e+00, 2.8213780548953389e+04 },
    { 6.0000000000000000e-11, 3.2618055911399999e+00, 4.3764389113990001e+02 },
    { 6.0000000000000000e-11, 2.9906402213700001e+00, 1.0424140624507738e+05 },
    { 6.0000000000000000e-11, 7.8206719979999995e-01, 5.1688171531894201e+04 },
    { 5.0000000000000002e-11, 5.4018816164399999e+00, 2.7410580789105599e+04 },
    { 5.0000000000000002e-11, 3.4033360683599998e+00, 2.4765225494042799e+04 },
    { 5.0000000000000002e-11, 5.9743867876000001e-01, 1.5547515046625677e+05 },
    { 5.0000000000000002e-11, 4.5717651041499998e+00, 1.2874735003836859e+05 },
    { 5.0000000000000002e-11, 3.4802041223000000e+00, 2.6500274238448590e+04 },
    { 5.0000000000000002e-11, 5.3250135624999997e+00, 2.5675532044699801e+04 },
    { 5.0000000000000002e-11, 6.7567879748000004e-01, 1.8985399758261937e+05 },
    { 5.0000000000000002e-11, 5.6258682387399999e+00, 4.8763475125420001e+02 },
    { 5.0000000000000002e-11, 4.1211750589199996e+00, 5.3551580082994202e+04 },
    { 5.0000000000000002e-11, 5.0584782545299998e+00, 6.0170334725172601e+04 },
    { 5.0000000000000002e-11, 4.4857598138799997e+00, 2.7353470620200591e+04 },
    { 5.0000000000000002e-11, 4.5209405317000001e-01, 6.2885158605009997e+02 },
    { 5.0000000000000002e-11, 2.0404729345899999e+00, 1.8938603259226496e+05 },
    { 5.0000000000000002e-11, 1.1301777840300000e+00, 5.2812802555172602e+04 },
    { 5.0000000000000002e-11, 5.9115094950499998e+00, 2.6235981866000500e+04 },
    { 5.0000000000000002e-11, 3.1667556993899999e+00, 2.6034806989259800e+04 },
    { 5.0000000000000002e-11, 5.6384619854100002e+00, 2.6140999293888592e+04 },
    { 5.0000000000000002e-11, 2.7268336981700001e+00, 1.2779106088239799e+05 },
    { 5.0000000000000002e-11, 2.7985531355500002e+00, 1.0403522069664019e+05 },
    { 5.0000000000000002e-11, 2.1661272819300001e+00, 1.2631573625781900e+03 },
    { 5.0000000000000002e-11, 2.1773062421899998e+00, 5.2098055739164491e+04 },
    { 5.0000000000000002e-11, 1.1706495471200000e+00, 6.5538255989947589e+04 },
    { 5.0000000000000002e-11, 3.3124047651900002e+00, 1.0430161594439259e+05 },
    { 5.0000000000000002e-11, 4.7612219208499997e+00, 7.8697421162599392e+04 },
    { 5.0000000000000002e-11, 5.2065746881799999e+00, 1.2434876016784001e+03 },
    { 5.0000000000000002e-11, 2.8517340754500000e+00, 2.6867220945412000e+03 },
    { 5.0000000000000002e-11, 4.8251650600699998e+00, 1.0440160918820098e+05 },
    { 5.0000000000000002e-11, 5.0375850448200001e+00, 5.3654672857212790e+04 },
    { 5.0000000000000002e-11, 2.3041519252899998e+00, 2.3866046506977189e+04 },
    { 5.0000000000000002e-11, 4.5725336874400000e+00, 5.1006218031739787e+04 },
    { 5.0000000000000002e-11, 5.3673431378999998e+00, 1.0466800443595338e+05 },
    { 5.0000000000000002e-11, 5.0200123281999998e-01, 2.4491716699289591e+04 },
    { 5.0000000000000002e-11, 3.5991409511699999e+00, 2.3389451694197000e+04 },
    { 5.0000000000000002e-11, 5.2060767336299998e+00, 2.8786354588951399e+04 },
    { 5.0000000000000002e-11, 4.5254130108000004e+00, 7.8265193897430887e+04 },
    { 5.0000000000000002e-11, 2.5393498927199998e+00, 5.8220034864523797e+04 },
    { 5.0000000000000002e-11, 6.2773659926200001e+00, 7.6137832017343389e+04 },
    { 5.0000000000000002e-11, 3.0287959707400001e+00, 2.5778624818918390e+04 },
    { 5.0000000000000002e-11, 5.7764217140599996e+00, 2.6397181464229991e+04 },
    { 5.0000000000000002e-11, 2.4520384617600000e+00, 7.8373915745941980e+04 },
    { 5.0000000000000002e-11, 5.3110349159999997e+00, 7.0363318461740005e+02 },
    { 5.0000000000000002e-11, 7.5362139039999998e-01, 2.0156710861598001e+03 },
    { 5.0000000000000002e-11, 3.5536530975399998e+00, 2.6090872086990799e+04 },
    { 5.0000000000000002e-11, 5.2515645872599999e+00, 2.6084934196157588e+04 },
    { 5.0000000000000002e-11, 5.5546196882899999e+00, 1.3705857113080599e+04 },
    { 5.0000000000000002e-11, 3.2505979965099998e+00, 3.8469949170067790e+04 },
    { 5.0000000000000002e-11, 3.1281653861000001e+00, 1.8227483749135001e+04 },
    { 5.0000000000000002e-11, 5.6770522986999996e+00, 3.3948322534013401e+04 },
    { 5.0000000000000002e-11, 4.4355558393900001e+00, 5.1102197258957589e+04 },
    { 5.0000000000000002e-11, 2.2366506942000002e+00, 5.1315496354395589e+04 },
    { 5.0000000000000002e-11, 3.4547458814300001e+00, 1.0769222499299659e+05 },
    { 5.0000000000000002e-11, 1.4798965769700001e+00, 4.9637557778894792e+04 },
    { 5.0000000000000002e-11, 5.3457192824600002e+00, 7.7929853483953193e+04 },
    { 5.0000000000000002e-11, 6.2590504766399997e+00, 1.3002003106399579e+05 },
    { 5.0000000000000002e-11, 4.8637443775499998e+00, 4.5455092303807003e+04 },
    { 5.0000000000000002e-11, 6.0689093425299996e+00, 1.0059506735726200e+04 },
    { 3.9999999999999998e-11, 4.6935783039499999e+00, 7.6532587071789982e+04 },
    { 3.9999999999999998e-11, 5.8475689631699996e+00, 3.1775533510526198e+04 },
    { 3.9999999999999998e-11, 2.9576487216300000e+00, 2.0400272772622200e+04 },
    { 3.9999999999999998e-11, 1.1673360774999999e+00, 3.5211479896591591e+04 },
    { 3.9999999999999998e-11, 1.3546963001200001e+00, 1.6964326386556801e+04 },
    { 3.9999999999999998e-11, 5.7972475814899997e+00, 2.7039979764833792e+04 },
    { 3.9999999999999998e-11, 3.0079701033099999e+00, 2.5135826518314600e+04 },
    { 3.9999999999999998e-11, 1.0837463046000000e-01, 3.3480357543659004e+04 },
    { 3.9999999999999998e-11, 2.4136577471599998e+00, 1.8695448739489399e+04 },
    { 3.9999999999999998e-11, 3.5361877268100002e+00, 1.5637785565231059e+05 },
    { 3.9999999999999998e-11, 4.8853685556700004e+00, 2.4182438376633800e+04 },
    { 3.9999999999999998e-11, 4.8440263097500003e+00, 1.2958631932611899e+05 },
    { 3.9999999999999998e-11, 4.1914706804000001e-01, 1.9959931813847594e+05 },
    { 3.9999999999999998e-11, 5.3321985168800001e+00, 7.6998141946096192e+04 },
    { 3.9999999999999998e-11, 1.2448752514000001e+00, 1.3686602528450001e+03 },
    { 3.9999999999999998e-11, 6.2034407624999997e+00, 4.5424765442802593e+04 },
    { 3.9999999999999998e-11, 3.8279428731500000e+00, 2.8928404046600190e+04 },
    { 3.9999999999999998e-11, 4.9772748116500001e+00, 2.3247402236548201e+04 },
    { 3.9999999999999998e-11, 7.8108419172999999e-01, 2.0452869412221800e+04 },
    { 3.9999999999999998e-11, 1.7409481858900000e+00, 3.1722936870926598e+04 },
    { 3.9999999999999998e-11, 7.3252151870000004e-01, 2.6238950811417089e+04 },
    { 3.9999999999999998e-11, 1.2075666810300001e+00, 5.2355883413433003e+04 },
    { 3.9999999999999998e-11, 1.8926274945399999e+00, 5.1538810011124187e+04 },
    { 3.9999999999999998e-11, 2.1286495812599999e+00, 3.2278926155820998e+04 },
    { 3.9999999999999998e-11, 3.9338279636000001e-01, 1.9896880127327400e+04 },
    { 3.9999999999999998e-11, 1.3108394310999999e-01, 7.8905122986461574e+04 },
    { 3.9999999999999998e-11, 1.4299138928599999e+00, 5.2336213652533203e+04 },
    { 3.9999999999999998e-11, 4.1159638165899999e+00, 5.0912648920570202e+04 },
    { 3.9999999999999998e-11, 1.7763220164500000e+00, 2.5923467230984788e+04 },
    { 3.9999999999999998e-11, 7.4571036118000000e-01, 2.6252339052163610e+04 },
    { 3.9999999999999998e-11, 3.4596291277000000e+00, 1.0626281204951399e+05 },
    { 3.9999999999999998e-11, 2.7874902052200001e+00, 7.4935573768442395e+04 },
    { 3.9999999999999998e-11, 5.0882800874700003e+00, 2.6130786216477602e+04 },
    { 3.9999999999999998e-11, 3.7169375973299998e+00, 2.6045020066670801e+04 },
    { 3.9999999999999998e-11, 6.1971837599199997e+00, 1.0477109721017198e+05 },
    { 3.9999999999999998e-11, 2.6322316370999999e-01, 1.9336862301228390e+04 },
    { 3.9999999999999998e-11, 1.7694582588900001e+00, 1.8319536584879599e+04 },
    { 3.9999999999999998e-11, 7.5257411874000002e-01, 3.3856269698268799e+04 },
    { 3.9999999999999998e-11, 1.6935905827900000e+00, 1.5996999379441058e+05 },
    { 3.9999999999999998e-11, 2.8930164615799998e+00, 2.5939824417147891e+04 },
    { 3.9999999999999998e-11, 2.1395852187400002e+00, 2.4292644697853189e+04 },
    { 3.9999999999999998e-11, 2.1353738933100002e+00, 1.0668480895916879e+05 },
    { 3.9999999999999998e-11, 1.3024137469999999e+00, 2.5248933853823801e+04 },
    { 3.9999999999999998e-11, 1.2196186306200001e+00, 2.6926872429324590e+04 },
    { 3.9999999999999998e-11, 1.6908897765000000e-01, 4.2430485727291802e+04 },
    { 3.9999999999999998e-11, 6.5996952307000001e-01, 2.4712129341728400e+04 },
    { 3.9999999999999998e-11, 1.6793838241700001e+00, 1.3090748069822539e+05 },
    { 3.9999999999999998e-11, 4.9273474013000002e-01, 7.6352509941505385e+04 },
    { 3.9999999999999998e-11, 3.0381884069399998e+00, 8.0445250639284004e+03 },
    { 3.9999999999999998e-11, 2.2495947918599999e+00, 5.0690515576445192e+04 },
    { 3.9999999999999998e-11, 5.1151715665799999e+00, 2.7047026001531802e+04 },
    { 3.9999999999999998e-11, 3.6900461182200002e+00, 2.5128780281616589e+04 },
    { 3.9999999999999998e-11, 3.0648216132799999e+00, 5.2164760582884490e+04 },
    { 3.9999999999999998e-11, 9.8879727435999998e-01, 2.1594190072419358e+05 },
    { 3.9999999999999998e-11, 6.0926487539699998e+00, 1.5119994274106238e+05 },
    { 3.9999999999999998e-11, 3.8490725195300000e+00, 2.5991923914356390e+04 },
    { 3.9999999999999998e-11, 4.9561451652699997e+00, 2.6183882368791990e+04 },
    { 3.9999999999999998e-11, 2.7831224620400001e+00, 4.8153543751184188e+04 },
    { 3.9999999999999998e-11, 5.1605854769099997e+00, 1.6496361396202399e+04 },
    { 3.9999999999999998e-11, 3.6446322078900000e+00, 3.5679444886945988e+04 },
    { 3.9999999999999998e-11, 2.3168239234299999e+00, 5.1040129411255199e+04 },
    { 3.9999999999999998e-11, 8.9612134983000002e-01, 1.2884332926558638e+05 },
    { 3.9999999999999998e-11, 9.1042238998000002e-01, 5.2015398913763587e+04 },
    { 3.9999999999999998e-11, 1.2782700408400001e+00, 1.1345551956041438e+05 },
    { 3.9999999999999998e-11, 5.4997773998200001e+00, 2.3543230504681789e+04 },
    { 3.9999999999999998e-11, 3.3054402849800000e+00, 2.8632575778466598e+04 },
    { 3.9999999999999998e-11, 2.6060694027900002e+00, 2.5998417393728199e+04 },
    { 3.9999999999999998e-11, 6.1991482820100003e+00, 2.6177388889420188e+04 },
    { 3.9999999999999998e-11, 5.7435110624799997e+00, 5.0951988442369788e+04 },
    { 3.9999999999999998e-11, 4.7557336695199997e+00, 1.4235548078175199e+04 },
    { 3.9999999999999998e-11, 4.0494840152800000e+00, 3.7940258204973186e+04 },
    { 3.9999999999999998e-11, 2.1545740379599998e+00, 2.6198840996783535e+05 },
    { 3.9999999999999998e-11, 4.5756220311200000e+00, 3.2217200181080800e+04 },
    { 3.9999999999999998e-11, 1.1327696018100000e+00, 5.1995729152863802e+04 },
    { 3.9999999999999998e-11, 7.4705899090000005e-01, 4.9424258683456792e+04 },
    { 3.9999999999999998e-11, 1.4835961756200000e+00, 1.5695401704032117e+05 },
    { 3.9999999999999998e-11, 4.1613254951899998e+00, 1.9282860753723036e+05 },
    { 3.9999999999999998e-11, 5.3056923315400004e+00, 1.1731986822019999e+02 },
    { 3.9999999999999998e-11, 5.0037030107899998e+00, 5.0476527066645191e+04 },
    { 3.9999999999999998e-11, 3.7085762071800001e+00, 2.7511467873537200e+04 },
    { 3.9999999999999998e-11, 5.0966414776200004e+00, 2.4664338409611199e+04 },
    { 3.9999999999999998e-11, 2.1943519000100000e+00, 6.1921126839004988e+04 },
    { 3.9999999999999998e-11, 1.4890208158500000e+00, 1.5811649174472899e+05 },
    { 3.9999999999999998e-11, 5.7701837785099999e+00, 2.5810868147832789e+04 },
    { 3.9999999999999998e-11, 3.0350339062899998e+00, 2.6364938135315599e+04 },
    { 3.9999999999999998e-11, 4.3343269871999999e+00, 1.2810631931499895e+05 },
    { 3.0000000000000000e-11, 4.1186071351300004e+00, 7.6094948942439994e+04 },
    { 3.0000000000000000e-11, 2.8742329415599999e+00, 2.5980597834644592e+04 },
    { 3.0000000000000000e-11, 5.9309847432399998e+00, 2.6195208448503799e+04 },
    { 3.0000000000000000e-11, 6.0625684128800001e+00, 2.3113293186963201e+04 },
    { 3.0000000000000000e-11, 2.7426492719200000e+00, 2.9062513096185201e+04 },
    { 3.0000000000000000e-11, 4.8523119871400002e+00, 2.5794492357196999e+04 },
    { 3.0000000000000000e-11, 3.9529056976599999e+00, 2.6381313925951399e+04 },
    { 3.0000000000000000e-11, 4.5366666205900001e+00, 1.1707706600107178e+05 },
    { 3.0000000000000000e-11, 1.7436927390000001e+00, 9.1235767550174005e+03 },
    { 3.0000000000000000e-11, 7.7597702447000005e-01, 2.6279351407685790e+04 },
    { 3.0000000000000000e-11, 1.7460553531500000e+00, 2.5896454875462601e+04 },
    { 3.0000000000000000e-11, 4.8440127391100001e+00, 1.4398093691036216e+05 },
    { 3.0000000000000000e-11, 1.2664244678300001e+00, 4.9850856874332792e+04 },
    { 3.0000000000000000e-11, 1.6252952716700000e+00, 4.1494555746582992e+04 },
    { 3.0000000000000000e-11, 5.6030650724699997e+00, 1.5430866174525840e+05 },
    { 3.0000000000000000e-11, 4.9072565263000003e+00, 7.8397818474307584e+04 },
    { 3.0000000000000000e-11, 1.6513880572799999e+00, 2.0242014928260216e+05 },
    { 3.0000000000000000e-11, 3.2825714796200001e+00, 7.9116905806474591e+04 },
    { 3.0000000000000000e-11, 2.5268995906000002e-01, 2.6114201461373999e+04 },
    { 3.0000000000000000e-11, 2.2693424185600000e+00, 2.6061604821774388e+04 },
    { 3.0000000000000000e-11, 7.6377303551999998e-01, 1.1254588789950538e+05 },
    { 3.0000000000000000e-11, 1.5399828363300001e+00, 2.7993367906514599e+04 },
    { 3.0000000000000000e-11, 4.9613087414199999e+00, 2.5344913081041599e+04 },
    { 3.0000000000000000e-11, 1.4139715563899999e+00, 1.9144826611159999e+02 },
    { 3.0000000000000000e-11, 2.3599815850799999e+00, 1.3044614426677099e+05 },
    { 3.0000000000000000e-11, 5.8877390821999998e-01, 2.6298319799800002e+01 },
    { 3.0000000000000000e-11, 6.1490924102899998e+00, 2.3420850807932791e+04 },
    { 3.0000000000000000e-11, 2.6561252745099999e+00, 2.8754955475215600e+04 },
    { 3.0000000000000000e-11, 5.9140201922599998e+00, 1.8483407909520617e+05 },
    { 3.0000000000000000e-11, 2.6437825133700001e+00, 5.2072024094567802e+04 },
    { 3.0000000000000000e-11, 2.0471269194400001e+00, 7.7622295862983592e+04 },
    { 3.0000000000000000e-11, 5.1923408735300001e+00, 5.2016504548356199e+04 },
    { 3.0000000000000000e-11, 1.1683553157000000e-01, 2.5970583273353990e+04 },
    { 3.0000000000000000e-11, 2.4051968460499999e+00, 2.6205223009794390e+04 },
    { 3.0000000000000000e-11, 3.6063580800200001e+00, 2.4079345602415189e+04 },
    { 3.0000000000000000e-11, 7.2953121469000004e-01, 3.3917995673008991e+04 },
    { 3.0000000000000000e-11, 1.7925011629300001e+00, 1.8257810610139401e+04 },
    { 3.0000000000000000e-11, 9.0477621291999999e-01, 2.2002588923089797e+05 },
    { 3.0000000000000000e-11, 3.6705091171299999e+00, 8.4821631932784003e+03 },
    { 3.0000000000000000e-11, 3.8762554650399998e+00, 2.2679659983214355e+05 },
    { 3.0000000000000000e-11, 1.3131971176599999e+00, 2.2222497657418498e+05 },
    { 3.0000000000000000e-11, 5.8713080905700004e+00, 7.8249482330720988e+04 },
    { 3.0000000000000000e-11, 3.5052613743000000e-01, 1.0767974822257696e+05 },
    { 3.0000000000000000e-11, 3.1258950206000002e-01, 2.6387029535843401e+04 },
    { 3.0000000000000000e-11, 1.7372816678100000e+00, 2.5936855471731291e+04 },
    { 3.0000000000000000e-11, 3.4771639206299998e+00, 2.6983982598229599e+04 },
    { 3.0000000000000000e-11, 5.3280537641699999e+00, 2.5191823684918800e+04 },
    { 3.0000000000000000e-11, 5.4607395875300000e+00, 7.9487527265501194e+04 },
    { 3.0000000000000000e-11, 1.5488031718299999e+00, 5.0490754160646793e+04 },
    { 3.0000000000000000e-11, 3.3667155775000002e-01, 2.5177512442162435e+05 },
    { 3.0000000000000000e-11, 3.1258790608500000e+00, 5.5484272178528787e+04 },
    { 3.0000000000000000e-11, 3.3566031629799999e+00, 2.0526065018762814e+05 },
    { 3.0000000000000000e-11, 6.1051613097399997e+00, 1.3828873468465999e+03 },
    { 3.0000000000000000e-11, 3.1573481704400002e+00, 1.4650558156934578e+05 },
    { 3.0000000000000000e-11, 4.7064678310300003e+00, 5.0199184874131997e+03 },
    { 3.0000000000000000e-11, 5.4152117993899997e+00, 1.4717530270636000e+03 },
    { 3.0000000000000000e-11, 1.1964784851600001e+00, 2.8208311441261999e+03 },
    { 3.0000000000000000e-11, 3.4741374610800002e+00, 2.4510559599126402e+04 },
    { 3.0000000000000000e-11, 5.3310802237199999e+00, 2.7665246684022000e+04 },
    { 3.0000000000000000e-11, 5.0148100927500003e+00, 1.7996686716554637e+05 },
    { 3.0000000000000000e-11, 1.9265698588500000e+00, 5.0600960481866197e+04 },
    { 3.0000000000000000e-11, 6.0909341679300004e+00, 1.1039584114767218e+05 },
    { 3.0000000000000000e-11, 6.0286362166899998e+00, 2.6202302248487598e+04 },
    { 3.0000000000000000e-11, 5.7713510571400004e+00, 1.5773435424478000e+03 },
    { 3.0000000000000000e-11, 2.4897457815099999e+00, 1.8452652147423659e+05 },
    { 3.0000000000000000e-11, 6.1391207158299999e+00, 2.9000787121444999e+04 },
    { 3.0000000000000000e-11, 2.6660969689699998e+00, 2.3175019161703389e+04 },
    { 3.0000000000000000e-11, 6.0714702683099997e+00, 4.9201196328537400e+04 },
    { 3.0000000000000000e-11, 4.2549422539400004e+00, 1.4316496914264598e+05 },
    { 3.0000000000000000e-11, 8.1593599477000001e-01, 5.1734980998270788e+04 },
    { 3.0000000000000000e-11, 5.3020694841499996e+00, 9.6340270297140000e+02 },
    { 3.0000000000000000e-11, 1.2745250002299999e+00, 2.5914454570189610e+04 },
    { 3.0000000000000000e-11, 1.2475073773900001e+00, 2.6261351712958789e+04 },
    { 3.0000000000000000e-11, 3.6496205432800002e+00, 5.1745275939009291e+04 },
    { 3.0000000000000000e-11, 1.3315056882700000e+00, 5.4879422437824003e+04 },
    { 3.0000000000000000e-11, 2.2458193574299998e+00, 1.5406652605008790e+04 },
    { 3.0000000000000000e-11, 3.6367384596400001e+00, 2.5530452403373802e+04 },
    { 3.0000000000000000e-11, 5.1684792251599996e+00, 2.6645353879774601e+04 },
    { 3.0000000000000000e-11, 3.4648617591700002e+00, 2.5626566710119801e+04 },
    { 3.0000000000000000e-11, 5.3403559256299999e+00, 2.6549239573028590e+04 },
    { 3.0000000000000000e-11, 3.7728194504900001e+00, 2.0956262057516600e+04 },
    { 3.0000000000000000e-11, 5.0323982343100004e+00, 3.1219544225631798e+04 },
    { 3.0000000000000000e-11, 2.8746904787899998e+00, 2.8352435961235070e+05 },
    { 3.0000000000000000e-11, 9.1363958260000000e-01, 7.6358244659782184e+04 },
    { 3.0000000000000000e-11, 1.1615408862500001e+00, 9.7466462676365962e+04 },
    { 3.0000000000000000e-11, 5.5452857462200003e+00, 2.5361421121703999e+04 },
    { 3.0000000000000000e-11, 3.2599319385799999e+00, 2.6814385161444388e+04 },
    { 3.0000000000000000e-11, 3.8444589769999999e+00, 5.2186851983412300e+04 },
    { 3.0000000000000000e-11, 4.3529423091500004e+00, 2.2224464633508475e+05 },
    { 3.0000000000000000e-11, 1.5245249429400001e+00, 6.7608757207956980e+04 },
    { 3.0000000000000000e-11, 4.5560385270000001e+00, 5.8151108596369904e+03 },
    { 3.0000000000000000e-11, 4.5006894938300004e+00, 1.2885755635958798e+05 },
    { 3.0000000000000000e-11, 3.1604851653199999e+00, 2.1650388933200611e+04 },
    { 3.0000000000000000e-11, 5.6447325194799998e+00, 3.0525417349947791e+04 },
    { 3.0000000000000000e-11, 1.8885508193700000e+00, 2.3528661537717999e+03 },
    { 3.0000000000000000e-11, 2.7846086687399998e+00, 7.3587651353180001e+02 },
    { 3.0000000000000000e-11, 2.7732302769300001e+00, 5.3338280987556187e+04 },
    { 3.0000000000000000e-11, 4.7236557038700004e+00, 2.3549654637320600e+04 },
    { 3.0000000000000000e-11, 1.2896830372600001e+00, 7.7683665566946001e+03 },
    { 3.0000000000000000e-11, 1.8259679740200001e+00, 2.4971411042479991e+04 },
    { 3.0000000000000000e-11, 6.9606440359999999e-01, 2.7204395240668400e+04 },
    { 3.0000000000000000e-11, 5.1724578256599996e+00, 1.3410904958500001e+02 },
    { 3.0000000000000000e-11, 1.5613341436799999e+00, 2.0658446127221518e+05 },
    { 3.0000000000000000e-11, 5.5800535537000000e+00, 4.9976718939861392e+04 },
    { 3.0000000000000000e-11, 1.1174437048899999e+00, 2.7161512165765002e+04 },
    { 3.0000000000000000e-11, 4.1497175960000000e-02, 2.9050783743349199e+04 },
    { 3.0000000000000000e-11, 2.4805352016600000e+00, 2.3125022539799200e+04 },
    { 3.0000000000000000e-11, 1.0162457862300001e+00, 2.4611379237247215e+05 },
    { 3.0000000000000000e-11, 1.7061296417500000e+00, 7.5085563634154983e+04 },
    { 3.0000000000000000e-11, 4.9034011660900001e+00, 1.9615674319351057e+05 },
    { 3.0000000000000000e-11, 3.1411955786800001e+00, 1.8049655813064100e+05 },
    { 3.0000000000000000e-11, 2.7357468955700002e+00, 7.8115630700296286e+04 },
    { 3.0000000000000000e-11, 1.7408982423499999e+00, 7.8112661754879678e+04 },
    { 3.0000000000000000e-11, 2.7364456035600000e+00, 4.2116299547422190e+04 },
    { 3.0000000000000000e-11, 4.3112747750800002e+00, 2.5949385644703500e+04 },
    { 3.0000000000000000e-11, 4.4939429097200003e+00, 2.6226420638444899e+04 },
    { 3.0000000000000000e-11, 5.0196484661999996e-01, 6.5851336535224589e+04 },
    { 3.0000000000000000e-11, 3.3246489326000001e-01, 2.6375791001644000e+04 },
    { 3.0000000000000000e-11, 2.1895674843699999e+00, 2.5800015281504398e+04 },
    { 3.0000000000000000e-11, 2.2271638315300000e+00, 2.3581258177317599e+04 },
    { 3.0000000000000000e-11, 2.9486854609000002e-01, 2.8594548105830800e+04 },
    { 3.0000000000000000e-11, 2.6587902775300001e+00, 2.6051254578644690e+04 },
    { 3.0000000000000000e-11, 6.1464274072700000e+00, 2.6124551704503690e+04 },
    { 3.0000000000000000e-11, 6.0080693478000002e-01, 5.1753140245535491e+04 },
    { 3.0000000000000000e-11, 4.5811979024499996e+00, 2.5632993775046889e+04 },
    { 3.0000000000000000e-11, 4.2240197823500001e+00, 2.6542812508101491e+04 },
    { 1.9999999999999999e-11, 3.7323516522500002e+00, 2.4601923020508992e+04 },
    { 1.9999999999999999e-11, 3.2116073475300002e+00, 2.2946365216578494e+05 },
    { 1.9999999999999999e-11, 1.4648653114500001e+00, 2.5554280023216488e+04 },
    { 1.9999999999999999e-11, 1.0571670661699999e+00, 2.6621526259931888e+04 },
    { 1.9999999999999999e-11, 2.9192853693099998e+00, 2.1092198223678037e+05 },
    { 1.9999999999999999e-11, 3.0185219468100000e+00, 2.1202093703745999e+04 },
    { 1.9999999999999999e-11, 5.7866957379899997e+00, 3.0973712579402400e+04 },
    { 1.9999999999999999e-11, 1.4839840679100000e+00, 1.9049541114435834e+05 },
    { 1.9999999999999999e-11, 2.2775432550499999e+00, 5.6350337293523999e+03 },
    { 1.9999999999999999e-11, 5.9573661574700001e+00, 5.7369173341946189e+04 },
    { 1.9999999999999999e-11, 1.0098432708000001e-01, 5.6876303689520000e+03 },
    { 1.9999999999999999e-11, 1.0654133612100001e+00, 3.1127491389887189e+04 },
    { 1.9999999999999999e-11, 1.4566190164099999e+00, 2.1048314893261198e+04 },
    { 1.9999999999999999e-11, 4.8807620349800001e+00, 2.7133057977761790e+04 },
    { 1.9999999999999999e-11, 6.1090203417000000e-01, 2.9266048932141799e+04 },
    { 1.9999999999999999e-11, 1.6864348852500000e+00, 1.0212975593169978e+05 },
    { 1.9999999999999999e-11, 5.4326577526900000e+00, 2.4298540360737636e+05 },
    { 1.9999999999999999e-11, 3.3645890712300002e+00, 1.2726136991730338e+05 },
    { 1.9999999999999999e-11, 5.3455761128599999e+00, 5.4862528377689603e+04 },
    { 1.9999999999999999e-11, 3.4473818032799999e+00, 2.6165653685558089e+04 },
    { 1.9999999999999999e-11, 1.2070357111700001e+00, 2.4815222115946999e+04 },
    { 1.9999999999999999e-11, 1.6171094914199999e+00, 2.6764974201698753e+05 },
    { 1.9999999999999999e-11, 3.2094591803400001e+00, 2.3976252828196601e+04 },
    { 1.9999999999999999e-11, 1.9549464955300000e+00, 3.0927832265580003e+02 },
    { 1.9999999999999999e-11, 8.8420623495999995e-01, 1.1553611574070001e+03 },
    { 1.9999999999999999e-11, 4.5537785108800000e+00, 2.7151217225026488e+04 },
    { 1.9999999999999999e-11, 4.9287409813999999e-01, 8.7648550432797783e+04 },
    { 1.9999999999999999e-11, 1.9902454271100001e+00, 2.5788776747305001e+04 },
    { 1.9999999999999999e-11, 2.2140031795500001e+00, 2.8918569166150293e+05 },
    { 1.9999999999999999e-11, 1.8476431377700000e+00, 3.2973053031505013e+04 },
    { 1.9999999999999999e-11, 5.3535925116699996e+00, 7.8484122067161385e+04 },
    { 1.9999999999999999e-11, 4.8445949329999998e+00, 5.0373434292426587e+04 },
    { 1.9999999999999999e-11, 3.9721293102999999e-01, 5.4391805909619995e+02 },
    { 1.9999999999999999e-11, 3.4426655140700002e+00, 2.2568722128005017e+05 },
    { 1.9999999999999999e-11, 5.1728182235800002e+00, 1.5653453239644598e+05 },
    { 1.9999999999999999e-11, 4.9925116226700004e+00, 5.1760253792536292e+04 },
    { 1.9999999999999999e-11, 3.5834227270299999e+00, 4.3981746865740097e+03 },
    { 1.9999999999999999e-11, 3.5552397148099999e+00, 3.0486077828148202e+04 },
    { 1.9999999999999999e-11, 5.2499779699899998e+00, 2.1689728455000190e+04 },
    { 1.9999999999999999e-11, 3.7087813445200002e+00, 9.5979227217800002e+01 },
    { 1.9999999999999999e-11, 2.7895869446799999e+00, 1.8983512179396000e+03 },
    { 1.9999999999999999e-11, 3.6457027592400002e+00, 2.3574672801577637e+05 },
    { 1.9999999999999999e-11, 3.5085293683300001e+00, 6.2174380083920005e+02 },
    { 1.9999999999999999e-11, 3.2034677683099999e+00, 1.4491686689107097e+05 },
    { 1.9999999999999999e-11, 5.6564502734900000e+00, 2.6024167243270800e+04 },
    { 1.9999999999999999e-11, 3.1487674113100002e+00, 2.6151639039877598e+04 },
    { 1.9999999999999999e-11, 5.8873232348900002e+00, 2.4926847945736776e+05 },
    { 1.9999999999999999e-11, 1.7680918177700000e+00, 4.5256551772590001e+03 },
    { 1.9999999999999999e-11, 1.4797875657099999e+00, 4.1520854066382803e+04 },
    { 1.9999999999999999e-11, 5.7508177700200003e+00, 7.3891833057948388e+04 },
    { 1.9999999999999999e-11, 1.9818436018200001e+00, 1.5930050485469001e+03 },
    { 1.9999999999999999e-11, 2.6093292755099999e+00, 1.3831960486120995e+05 },
    { 1.9999999999999999e-11, 5.1263259122099996e+00, 1.2797945726280000e+03 },
    { 1.9999999999999999e-11, 1.3633074361399999e+00, 1.9726612174560397e+05 },
    { 1.9999999999999999e-11, 2.0885779542599998e+00, 7.1378559534791784e+04 },
    { 1.9999999999999999e-11, 3.9149724972800000e+00, 5.2367254549259989e+04 },
    { 1.9999999999999999e-11, 2.5697696463300002e+00, 5.1949977014738201e+04 },
    { 1.9999999999999999e-11, 9.8913727678999996e-01, 1.7573017210108857e+05 },
    { 1.9999999999999999e-11, 9.8860870621999997e-01, 2.8133211674376402e+04 },
    { 1.9999999999999999e-11, 1.5334236714000000e+00, 2.4042594608771989e+04 },
    { 1.9999999999999999e-11, 3.1514623700700000e+00, 1.7728784588263658e+05 },
    { 1.9999999999999999e-11, 2.4043312515399999e+00, 2.4999299984973200e+04 },
    { 1.9999999999999999e-11, 1.1770112608000000e-01, 2.7176506298175191e+04 },
    { 1.9999999999999999e-11, 3.7354894100200000e+00, 1.3863379104107959e+05 },
    { 1.9999999999999999e-11, 1.7018827891999999e-01, 1.5387896402397219e+05 },
    { 1.9999999999999999e-11, 5.9497860416500004e+00, 1.3376765136415116e+05 },
    { 1.9999999999999999e-11, 1.3412271445999999e+00, 2.7143352918500292e+04 },
    { 1.9999999999999999e-11, 6.2315612170900003e+00, 2.0618019358533598e+04 },
    { 1.9999999999999999e-11, 2.5736564677099998e+00, 3.1557786924614789e+04 },
    { 1.9999999999999999e-11, 4.1751652574999998e-01, 5.1952945960154793e+04 },
    { 1.9999999999999999e-11, 7.6607594408000002e-01, 2.8546403739102002e+03 },
    { 1.9999999999999999e-11, 1.0341199399999999e-03, 5.2249103409007388e+04 },
    { 1.9999999999999999e-11, 5.3883969916599996e+00, 9.9979736199522595e+04 },
    { 1.9999999999999999e-11, 6.8315072404999999e-01, 7.3924544020847998e+03 },
    { 1.9999999999999999e-11, 3.1435698984300000e+00, 5.2202104602948202e+04 },
    { 1.9999999999999999e-11, 4.9901819584800000e+00, 6.6826606037732956e+04 },
    { 1.9999999999999999e-11, 5.6667602095299996e+00, 1.7824344562424519e+05 },
    { 1.9999999999999999e-11, 1.1272501505400001e+00, 4.6019501189630000e+03 },
    { 1.9999999999999999e-11, 3.7141554435500002e+00, 2.5446726368924001e+03 },
    { 1.9999999999999999e-11, 2.0963100372499999e+00, 3.3061896365820002e+02 },
    { 1.9999999999999999e-11, 5.0132198681700002e+00, 4.0738702896158793e+04 },
    { 1.9999999999999999e-11, 2.5469661915800001e+00, 2.5973504034660789e+04 },
    { 1.9999999999999999e-11, 2.4923780990500002e+00, 2.0637002873972157e+05 },
    { 1.9999999999999999e-11, 9.2029934107000000e-01, 5.2822817116463193e+04 },
    { 1.9999999999999999e-11, 1.4324778559200000e+00, 1.3537487450168001e+03 },
    { 1.9999999999999999e-11, 1.3067835936500001e+00, 7.8300925314347996e+03 },
    { 1.9999999999999999e-11, 2.8549030883299999e+00, 1.9080959732422797e+05 },
    { 1.9999999999999999e-11, 8.3915501029999995e-01, 2.7331107406613970e+05 },
    { 1.9999999999999999e-11, 6.0424233597599999e+00, 2.6555683724738001e+03 },
    { 1.9999999999999999e-11, 5.5026406303900002e+00, 2.4705705209089589e+04 },
    { 1.9999999999999999e-11, 3.3025770544099999e+00, 2.7470101074058799e+04 },
    { 1.9999999999999999e-11, 5.8747494497600004e+00, 4.5782518168586003e+03 },
    { 1.9999999999999999e-11, 4.8288867572600003e+00, 1.6032999892854000e+03 },
    { 1.9999999999999999e-11, 1.4307423530000001e+00, 2.0453085328022000e+03 },
    { 1.9999999999999999e-11, 4.1442516659299997e+00, 3.0459779508348402e+04 },
    { 1.9999999999999999e-11, 2.6281789992800002e+00, 3.2838943981919998e+04 },
    { 1.9999999999999999e-11, 4.8461323869199999e+00, 1.4297274052915400e+04 },
    { 1.9999999999999999e-11, 3.9590852978800002e+00, 3.7878532230233002e+04 },
    { 1.9999999999999999e-11, 4.6346325408500002e+00, 1.0450117576343138e+05 },
    { 1.9999999999999999e-11, 5.7763424973099999e+00, 4.4131281219219993e+04 },
    { 1.9999999999999999e-11, 2.0687903562800001e+00, 5.1546954697098292e+04 },
    { 1.9999999999999999e-11, 6.0721640081099997e+00, 2.3700988537835455e+05 },
    { 1.9999999999999999e-11, 4.6869541404700001e+00, 3.1107821628987400e+04 },
    { 1.9999999999999999e-11, 4.1182635443300004e+00, 2.1067984654160991e+04 },
    { 1.9999999999999999e-11, 4.1491214788000003e-01, 2.7299244211933201e+04 },
    { 1.9999999999999999e-11, 2.1071202297400000e+00, 2.4876562071215201e+04 },
    { 1.9999999999999999e-11, 4.1721907050300002e+00, 1.0541810804348679e+05 },
    { 1.9999999999999999e-11, 1.1201998471900001e+00, 4.9534465004676204e+04 },
    { 1.9999999999999999e-11, 4.1648223665600002e+00, 1.7006884005193636e+05 },
    { 1.9999999999999999e-11, 2.3628622398999999e-01, 2.4957749193988398e+04 },
    { 1.9999999999999999e-11, 2.2857461536299999e+00, 2.7218057089160000e+04 },
    { 1.9999999999999999e-11, 1.5459756801700000e+00, 2.7257491392982800e+04 },
    { 1.9999999999999999e-11, 5.0187724388900001e+00, 2.3670232775738495e+05 },
    { 1.9999999999999999e-11, 4.4517961879900003e+00, 1.2921569786709236e+05 },
    { 1.9999999999999999e-11, 5.0524629213100001e+00, 1.5851407420206999e+03 },
    { 1.9999999999999999e-11, 3.9953048523999999e+00, 2.0235122826310399e+04 },
    { 1.9999999999999999e-11, 4.8099128324000002e+00, 3.1940683456837989e+04 },
    { 1.9999999999999999e-11, 5.6108663184200003e+00, 1.3043288714897096e+05 },
    { 1.9999999999999999e-11, 3.7821541468800000e+00, 7.9955875094224975e+04 },
    { 1.9999999999999999e-11, 1.2773730430000000e-02, 1.6472169418265377e+05 },
    { 1.9999999999999999e-11, 4.1048191573899997e+00, 2.9939897720771394e+05 },
    { 1.9999999999999999e-11, 1.2324084337700001e+00, 1.2355436581794015e+05 },
    { 1.9999999999999999e-11, 5.5421040597599998e+00, 5.1738162392008489e+04 },
    { 1.9999999999999999e-11, 2.8380722450100002e+00, 2.7953134544091801e+04 },
    { 1.9999999999999999e-11, 5.9671454397900003e+00, 2.4222671739056601e+04 },
    { 1.9999999999999999e-11, 4.7420289233100004e+00, 2.8440769295345999e+04 },
    { 1.9999999999999999e-11, 4.0631887614900002e+00, 2.3735036987802388e+04 },
    { 1.9999999999999999e-11, 2.7790064926800002e+00, 3.7764308857348001e+03 },
    { 1.9999999999999999e-11, 3.4866866214500001e+00, 3.0755762096960001e+02 },
    { 1.9999999999999999e-11, 3.4875712541900001e+00, 4.2680941781935999e+03 },
    { 1.9999999999999999e-11, 3.3697315428699999e+00, 2.6948213070327001e+04 },
    { 1.9999999999999999e-11, 5.4354861419300002e+00, 2.5227593212821390e+04 },
    { 1.9999999999999999e-11, 6.4413971421000005e-01, 2.4918314890165591e+04 },
    { 1.9999999999999999e-11, 5.5778477985999997e-01, 2.5888831140137801e+04 },
    { 1.9999999999999999e-11, 1.9642475977600000e+00, 2.6286975143010590e+04 },
    { 1.9999999999999999e-11, 4.1459239597900002e+00, 8.0432469907005187e+04 },
    { 1.9999999999999999e-11, 1.3454980567499999e+00, 1.3989716431945801e+04 },
    { 1.9999999999999999e-11, 1.1765343208700001e+00, 3.8186089851202603e+04 },
    { 1.9999999999999999e-11, 5.6497706309000000e+00, 5.1546387109299997e+02 },
    { 1.9999999999999999e-11, 5.5064781421299998e+00, 7.6064622081435577e+04 },
    { 1.9999999999999999e-11, 4.1829918262600003e+00, 9.5915417453718001e+03 },
    { 1.9999999999999999e-11, 2.5011895585500001e+00, 8.7253177130154960e+04 },
    { 1.9999999999999999e-11, 6.1869620976800004e+00, 3.0831304904744600e+04 },
    { 1.9999999999999999e-11, 2.6182555871200002e+00, 2.1344501378403791e+04 },
    { 1.9999999999999999e-11, 1.3547398224000000e+00, 1.0540388094948517e+05 },
    { 1.9999999999999999e-11, 5.0146496796999998e+00, 1.4650799754584590e+04 },
    { 1.9999999999999999e-11, 1.1557744218699999e+00, 1.2466374437003357e+05 },
    { 1.9999999999999999e-11, 5.1375700641500002e+00, 4.3693643089869991e+04 },
    { 1.9999999999999999e-11, 2.5327229024800002e+00, 2.2147430875962000e+03 },
    { 1.9999999999999999e-11, 3.9209458764299998e+00, 2.1516279883615600e+04 },
    { 1.9999999999999999e-11, 4.8842718083700003e+00, 3.0659526399532791e+04 },
    { 1.9999999999999999e-11, 3.9351209719600000e+00, 1.0222573515891758e+05 },
    { 1.9999999999999999e-11, 5.0468522753600000e+00, 2.8839450741265791e+04 },
    { 1.9999999999999999e-11, 6.2729879085100002e+00, 1.2997155071751658e+05 },
    { 1.9999999999999999e-11, 3.4929236073599998e+00, 5.1472173098530991e+04 },
    { 1.9999999999999999e-11, 1.9610603006999999e-01, 9.5207662325959996e+02 },
    { 9.9999999999999994e-12, 5.7971982007999996e+00, 9.5912285995759999e+02 },
    { 9.9999999999999994e-12, 5.1726925104900001e+00, 2.6010152597590299e+04 },
    { 9.9999999999999994e-12, 5.2904850878799996e+00, 2.4919004304527600e+04 },
    { 9.9999999999999994e-12, 3.5147325969200001e+00, 2.7256801978620799e+04 },
    { 9.9999999999999994e-12, 1.6493913961900000e+00, 5.0395882483129999e+03 },
    { 9.9999999999999994e-12, 5.7146312681000000e-01, 2.3831930147762000e+03 },
    { 9.9999999999999994e-12, 3.3513506019800001e+00, 5.2499311699805789e+04 },
    { 9.9999999999999994e-12, 2.4779218629300002e+00, 2.5042748305386602e+04 },
    { 9.9999999999999994e-12, 4.1209078299500002e+00, 2.3320205537888399e+05 },
    { 9.9999999999999994e-12, 1.3950903633900000e+00, 1.5432950924808600e+04 },
    { 9.9999999999999994e-12, 1.0538901061499999e+00, 5.0807146030303389e+04 },
};

static const VSOP87Term VSOP87A_MERCURY_Y1[782] = {
    { 1.0780385200000000e-03, 4.3496479388299996e+00, 5.2175806283148399e+04 },
    { 8.0651544000000003e-04, 3.1415926535900001e+00, 0.0000000000000000e+00 },
    { 3.2715354000000001e-04, 1.1176373442500001e+00, 7.8263709424722590e+04 },
    { 1.1914709000000000e-04, 1.2213998634000001e+00, 2.6087903141574199e+04 },
    { 8.8581579999999998e-05, 4.1685240186700003e+00, 1.0435161256629678e+05 },
    { 2.3444689999999999e-05, 9.3615372641000005e-01, 1.3043951570787099e+05 },
    { 6.1783800000000003e-06, 3.9869399228400000e+00, 1.5652741884944518e+05 },
    { 1.6295499999999999e-06, 7.5452718043000000e-01, 1.8261532199101939e+05 },
    { 7.0134999999999996e-07, 1.3544766402399999e+00, 2.4978524589480799e+04 },
    { 6.3990999999999996e-07, 1.1800029406999999e+00, 2.7197281693667599e+04 },
    { 4.3081999999999999e-07, 3.8052884438399999e+00, 2.0870322513259359e+05 },
    { 1.7706000000000000e-07, 4.4063412741699999e+00, 5.1066427731055002e+04 },
    { 1.3279000000000001e-07, 4.2257756672099998e+00, 5.3285184835241802e+04 },
    { 1.2618999999999999e-07, 6.1222149799899999e+00, 1.0593819301891999e+03 },
    { 1.1682000000000000e-07, 1.7640449792399999e+00, 1.1093785520934000e+03 },
    { 1.1650000000000000e-07, 5.0672084700999998e+00, 2.0426571092422000e+04 },
    { 1.1421000000000000e-07, 5.7284937520000001e-01, 2.3479112827416777e+05 },
    { 8.0179999999999994e-08, 3.7352803161499999e+00, 3.1749235190726398e+04 },
    { 6.0990000000000005e-08, 3.8785602978000000e+00, 4.5519534970588002e+03 },
    { 4.8809999999999998e-08, 1.1745407487599999e+00, 7.7154330872629187e+04 },
    { 4.7379999999999998e-08, 4.9780092932300004e+00, 2.1535949644515400e+04 },
    { 4.4600000000000002e-08, 3.5357193780200000e+00, 2.5132303399965600e+04 },
    { 4.3070000000000001e-08, 1.8301764037199999e+00, 4.6514474233996203e+04 },
    { 4.2179999999999998e-08, 2.7028906274800000e+00, 5.1116424352959199e+04 },
    { 4.0369999999999997e-08, 2.8487973512100000e+00, 2.7147285071763388e+04 },
    { 3.9519999999999999e-08, 5.2713982460000004e+00, 2.7043502883182799e+04 },
    { 3.9360000000000003e-08, 8.0973598550000003e-01, 1.5890728952838001e+03 },
    { 3.7840000000000000e-08, 2.6102111475199998e+00, 1.4765239043269799e+04 },
    { 3.2990000000000000e-08, 5.8094577194700001e+00, 2.6617594106668799e+04 },
    { 3.0360000000000002e-08, 3.6236222055499998e+00, 2.6087903141574195e+05 },
    { 3.0220000000000002e-08, 3.1138976699100001e+00, 2.5558212176479599e+04 },
    { 2.9280000000000001e-08, 4.9490097592100000e+00, 2.6068233380674399e+04 },
    { 2.9160000000000001e-08, 3.8681357765700000e+00, 2.6107572902473989e+04 },
    { 2.3980000000000000e-08, 2.0206941911600000e+00, 2.6091784476932200e+04 },
    { 2.3919999999999998e-08, 5.1342544633999998e-01, 2.6084021806216198e+04 },
    { 2.0700000000000000e-08, 3.8863920762099999e+00, 3.0639856638632999e+04 },
    { 2.0700000000000000e-08, 6.1989961094300003e+00, 3.7410567239878597e+04 },
    { 1.9930000000000000e-08, 9.3804934862000000e-01, 7.9373087976815994e+04 },
    { 1.8640000000000000e-08, 4.9426793092499999e+00, 4.7623852786089599e+04 },
    { 1.5830000000000000e-08, 1.2078429237000000e-01, 9.1039069941176003e+03 },
    { 1.5740000000000001e-08, 5.8883916635600002e+00, 9.5559974160859997e+02 },
    { 1.5170000000000000e-08, 5.1521476813499998e+00, 1.9317192540328600e+04 },
    { 1.5130000000000000e-08, 2.2135235838299998e+00, 1.2566151699982800e+04 },
    { 1.4850000000000001e-08, 1.4281228520500000e+00, 2.3869146037387400e+04 },
    { 1.4060000000000000e-08, 4.8802595958900001e+00, 7.2602377375570388e+04 },
    { 1.3920000000000000e-08, 4.8634980708999997e-01, 4.2659819087599999e+02 },
    { 1.3809999999999999e-08, 4.2259348792600004e+00, 1.0324223401420339e+05 },
    { 1.3660000000000000e-08, 1.7184045475100000e+00, 5.0586733387864588e+04 },
    { 1.3290000000000000e-08, 9.1947275141999996e-01, 2.6095016688575000e+04 },
    { 1.2970000000000000e-08, 1.6479315392000000e+00, 2.6080789594573391e+04 },
    { 1.2880000000000000e-08, 1.1222680531900000e+00, 2.8306660245760992e+04 },
    { 1.2520000000000001e-08, 2.5878403717999998e-01, 5.1220206541539788e+04 },
    { 1.2280000000000000e-08, 2.3696613470600001e+00, 5.2705497248242988e+04 },
    { 1.1700000000000001e-08, 3.9201594265300002e+00, 3.4425749449653999e+03 },
    { 1.1090000000000000e-08, 5.9349642564999998e+00, 5.3235188213337591e+04 },
    { 1.1059999999999999e-08, 4.9461653705900002e+00, 2.2645328196608789e+04 },
    { 1.0840000000000000e-08, 3.7094067714899999e+00, 3.2858613742819791e+04 },
    { 1.0589999999999999e-08, 4.9317828059700002e+00, 5.2969096509459996e+02 },
    { 9.6400000000000006e-09, 2.0017493226799998e+00, 5.3131406024757001e+04 },
    { 9.2599999999999999e-09, 5.4553500094700000e+00, 6.3989728631399998e+02 },
    { 8.9000000000000003e-09, 5.6532726320000004e-01, 8.3925041473874793e+04 },
    { 8.8300000000000003e-09, 4.5084463736000002e-01, 1.1322664098304400e+04 },
    { 8.6399999999999999e-09, 5.6606063930900001e+00, 4.0853142184844000e+04 },
    { 8.5799999999999997e-09, 3.7638593236400002e+00, 2.7676976036857999e+04 },
    { 8.5799999999999997e-09, 5.7380622466900002e+00, 2.5874604046136199e+04 },
    { 8.3799999999999996e-09, 3.2068539436300001e+00, 3.6301188687785187e+04 },
    { 8.2399999999999997e-09, 3.0804927011599998e+00, 2.6301202237012199e+04 },
    { 8.0999999999999997e-09, 2.7669178098799998e+00, 1.0664954771900000e+03 },
    { 8.0900000000000000e-09, 3.9136440746000001e-01, 2.8696693455731618e+05 },
    { 7.4499999999999997e-09, 6.3676880132000002e-01, 5.6727759780207198e+04 },
    { 7.3300000000000001e-09, 3.1521674192999999e-01, 3.9609654583165597e+04 },
    { 7.3099999999999998e-09, 6.2169525075100003e+00, 1.8849227549974199e+04 },
    { 7.2600000000000002e-09, 4.2403582053299997e+00, 6.7707106012455997e+03 },
    { 7.1500000000000003e-09, 2.4457009040100002e+00, 4.3071899289030800e+04 },
    { 7.0200000000000002e-09, 5.5842390641900002e+00, 1.5874617595363199e+04 },
    { 6.7299999999999997e-09, 3.2413641455200000e+00, 1.0522683831884001e+03 },
    { 6.6800000000000001e-09, 1.7170462752100000e+00, 5.2156136522248598e+04 },
    { 6.6500000000000000e-09, 3.9310919700500002e+00, 2.9530478086539599e+04 },
    { 6.1900000000000003e-09, 6.1385896887399998e+00, 2.5028521211384999e+04 },
    { 5.9799999999999996e-09, 5.4071023249500003e+00, 5.1646115318053788e+04 },
    { 5.7600000000000000e-09, 6.8273915306999999e-01, 5.6613320491521999e+03 },
    { 5.5500000000000001e-09, 1.4448086567800000e+00, 2.1187638603783998e+03 },
    { 5.4000000000000004e-09, 2.7325167828299999e+00, 1.6983996147456601e+04 },
    { 5.3800000000000000e-09, 5.0769078702500003e+00, 5.2179687618506403e+04 },
    { 5.3400000000000002e-09, 1.7005496340799999e+00, 2.2187571041868000e+03 },
    { 5.3000000000000003e-09, 3.3642251932199998e+00, 2.6514501332450200e+04 },
    { 5.2199999999999998e-09, 6.2912353695000001e-01, 5.2195476044048191e+04 },
    { 5.0600000000000003e-09, 5.2415913530799996e+00, 3.8654054841556987e+04 },
    { 4.8699999999999999e-09, 2.9350668630899999e+00, 6.3498470381452789e+04 },
    { 4.7600000000000001e-09, 2.0791870257000000e+00, 5.1749208092272391e+04 },
    { 4.7300000000000000e-09, 3.5905517544899999e+00, 1.1001294461544899e+05 },
    { 4.4100000000000003e-09, 1.6479087454300001e+00, 9.8690280517144580e+04 },
    { 4.3500000000000001e-09, 2.5454842500899999e+00, 3.3326578733174203e+04 },
    { 4.3500000000000001e-09, 3.5626486971400002e+00, 5.2171924947790401e+04 },
    { 4.0700000000000002e-09, 4.4447628763699996e+00, 5.2168692736147590e+04 },
    { 4.0100000000000000e-09, 3.2346452284899998e+00, 7.2386755916000002e+03 },
    { 3.9400000000000000e-09, 9.9378340811999999e-01, 1.2933013715577759e+05 },
    { 3.9199999999999997e-09, 5.3336519074000002e+00, 7.8793400389817180e+04 },
    { 3.8700000000000001e-09, 3.9484676600799999e+00, 5.2182919830149200e+04 },
    { 3.7900000000000004e-09, 5.7018748429899997e+00, 1.0329223063610759e+05 },
    { 3.7900000000000004e-09, 4.1572488377200001e+00, 5.4394563387335191e+04 },
    { 3.6899999999999999e-09, 3.2858515327600002e+00, 7.7308109683113988e+04 },
    { 3.6199999999999999e-09, 4.2873204405999998e-01, 5.8946516884393990e+04 },
    { 3.5899999999999998e-09, 6.2330067052000002e+00, 6.2389091829359393e+04 },
    { 3.3400000000000000e-09, 2.4826079768499998e+00, 6.8050423878511589e+04 },
    { 3.2700000000000001e-09, 2.7315321985600001e+00, 7.9323091354911783e+04 },
    { 3.2400000000000000e-09, 4.4825422632000000e+00, 4.9957049178961599e+04 },
    { 3.2099999999999999e-09, 2.6730420748800001e+00, 1.3655860491176400e+04 },
    { 3.1700000000000000e-09, 1.9246340235399999e+00, 4.5405095681902800e+04 },
    { 3.1500000000000001e-09, 4.5449767331900004e+00, 7.6674636529438780e+04 },
    { 3.1500000000000001e-09, 3.3577886047100001e+00, 5.1535908996834391e+04 },
    { 3.0500000000000000e-09, 4.8924991441400003e+00, 7.1135470007999997e+00 },
    { 2.9800000000000000e-09, 4.9963184099499998e+00, 4.8733231338182988e+04 },
    { 2.7799999999999999e-09, 6.0451079666699998e+00, 5.1109310805958390e+04 },
    { 2.6299999999999998e-09, 6.2097540826600000e+00, 5.2389105378586399e+04 },
    { 2.3499999999999999e-09, 5.5625028666100000e+00, 5.1123537899959993e+04 },
    { 2.3199999999999998e-09, 3.6774729283199998e+00, 8.2815662921781390e+04 },
    { 2.2800000000000000e-09, 1.7150693290999999e-01, 4.3371173787679999e+02 },
    { 2.2600000000000001e-09, 1.8063131642300001e+00, 3.2370978991565600e+04 },
    { 2.2300000000000000e-09, 2.4246683601300001e+00, 6.6941045326418185e+04 },
    { 2.1600000000000000e-09, 3.4422347196500001e+00, 3.1305483769889036e+05 },
    { 2.0400000000000000e-09, 2.4584864338000001e+00, 1.2938013377768178e+05 },
    { 2.0200000000000001e-09, 2.9000895778200002e+00, 4.4937130691548402e+04 },
    { 1.9800000000000002e-09, 4.8439148557000000e-01, 5.3764879178432202e+04 },
    { 1.9800000000000002e-09, 5.4586842193000003e+00, 6.9159802430604992e+04 },
    { 1.9800000000000002e-09, 1.0508851743400001e+00, 5.0057042422769999e+04 },
    { 1.9800000000000002e-09, 6.2454878850700002e+00, 3.8519945791972001e+04 },
    { 1.9099999999999998e-09, 2.3064150235900001e+00, 2.6727800427888200e+04 },
    { 1.8600000000000000e-09, 3.4501313168999997e-01, 1.3610084775702318e+05 },
    { 1.8400000000000001e-09, 6.6520680194000004e-01, 4.1948464387519999e+02 },
    { 1.8199999999999999e-09, 6.9098340435000005e-01, 5.5618381228113802e+04 },
    { 1.8100000000000000e-09, 5.3809122786900003e+00, 5.7837138332300601e+04 },
    { 1.7900000000000000e-09, 2.1813544987000002e+00, 5.1962507187710398e+04 },
    { 1.7500000000000000e-09, 3.7018689040399999e+00, 1.0546099111839019e+05 },
    { 1.7400000000000000e-09, 5.5062851459499997e+00, 9.4138327020085781e+04 },
    { 1.6999999999999999e-09, 4.7648082075399998e+00, 7.8244039663822798e+04 },
    { 1.5900000000000001e-09, 4.5978120356099996e+00, 8.5319638175199998e+02 },
    { 1.5799999999999999e-09, 2.0220075296800002e+00, 6.4741957983131193e+04 },
    { 1.5400000000000001e-09, 3.1110693868200001e+00, 3.5191810135691798e+04 },
    { 1.5400000000000001e-09, 1.8226039409000001e+00, 9.9799659069237983e+04 },
    { 1.5000000000000000e-09, 3.8751874779000001e-01, 1.2432042650397800e+04 },
    { 1.4900000000000000e-09, 5.5576808610399997e+00, 5.9414481874748402e+04 },
    { 1.4900000000000000e-09, 5.8618649380800001e+00, 2.7154398618764200e+04 },
    { 1.4700000000000001e-09, 1.3708317947000001e-01, 5.2602404474024399e+04 },
    { 1.4700000000000001e-09, 5.7811859854999996e+00, 2.5661304950698199e+04 },
    { 1.4599999999999999e-09, 5.9476999652800000e+00, 7.7734018459627987e+04 },
    { 1.4599999999999999e-09, 2.3301964457600000e+00, 2.1329909543799999e+02 },
    { 1.4400000000000000e-09, 4.8432869931500004e+00, 2.3754706748702189e+04 },
    { 1.4300000000000000e-09, 3.3641372223800001e+00, 1.5961864422845999e+03 },
    { 1.4100000000000001e-09, 4.3737121378999999e-01, 1.9804827291582798e+04 },
    { 1.3900000000000000e-09, 4.6295445617900004e+00, 2.5973463852888959e+04 },
    { 1.3900000000000000e-09, 4.1764851177600004e+00, 2.6202342430259410e+04 },
    { 1.3500000000000001e-09, 1.8457923776600000e+00, 7.8267590760080580e+04 },
    { 1.3500000000000001e-09, 4.6955170958999997e+00, 1.2477818365871879e+05 },
    { 1.2900000000000001e-09, 3.0002336922400001e+00, 8.8476994970933592e+04 },
    { 1.2100000000000000e-09, 2.0467340118399999e+00, 1.0488130353139139e+05 },
    { 1.1800000000000001e-09, 1.8946350402000001e-01, 7.9945284420241996e+03 },
    { 1.1700000000000001e-09, 5.6234009401699998e+00, 2.6241681952059000e+04 },
    { 1.1500000000000000e-09, 2.4406996601299999e+00, 1.0382192160120218e+05 },
    { 1.1400000000000000e-09, 3.2022948990900000e+00, 2.5934124331089399e+04 },
    { 1.1300000000000000e-09, 8.9323533783999998e-01, 5.3274761083827998e+03 },
    { 1.1200000000000001e-09, 4.0362825289900002e+00, 1.5541804029735178e+05 },
    { 1.1100000000000001e-09, 4.4334153306999999e+00, 2.8206667001952599e+04 },
    { 1.0900000000000000e-09, 3.5468008560000000e-02, 1.0339601282468818e+05 },
    { 1.0800000000000000e-09, 1.2055562317000000e-01, 2.7140171524762591e+04 },
    { 1.0800000000000000e-09, 4.0030090344999998e+00, 2.8421099534446199e+04 },
    { 1.0800000000000000e-09, 4.5525576725999999e+00, 7.9219309166331193e+04 },
    { 1.0500000000000001e-09, 7.1334145620000000e-01, 7.8270822971723392e+04 },
    { 1.0399999999999999e-09, 3.3837792061299998e+00, 9.1785460866313981e+04 },
    { 1.0399999999999999e-09, 3.6634201528300001e+00, 7.8283379185622383e+04 },
    { 1.0300000000000000e-09, 3.9953304568800001e+00, 2.3331963928720002e+03 },
    { 1.0100000000000000e-09, 1.9462418693000001e+00, 2.5619938151219802e+04 },
    { 9.7999999999999992e-10, 5.8109085915000003e-01, 2.6555868131928601e+04 },
    { 9.7999999999999992e-10, 3.7944365565500000e+00, 7.1980633574731182e+04 },
    { 9.6999999999999996e-10, 5.7979901898200001e+00, 1.0541099449648599e+05 },
    { 9.5999999999999999e-10, 4.8241290318500001e+00, 5.8458882133139792e+04 },
    { 9.5000000000000003e-10, 1.1732245801400001e+00, 7.8256595877721789e+04 },
    { 9.4000000000000006e-10, 2.8473571973000000e+00, 1.5819593482830001e+03 },
    { 8.9999999999999999e-10, 4.8161524220600000e+00, 1.2588756221081219e+05 },
    { 8.9000000000000003e-10, 3.2104107901000001e-01, 7.8259828089364586e+04 },
    { 8.6999999999999999e-10, 9.6769986531999996e-01, 8.0482466528909383e+04 },
    { 8.3999999999999999e-10, 1.6397819074100000e+00, 7.4821134479757180e+04 },
    { 8.3000000000000003e-10, 5.1054026695700001e+00, 2.4498830246290399e+04 },
    { 8.3000000000000003e-10, 2.1041749661000000e-01, 7.7623812138408583e+04 },
    { 8.1999999999999996e-10, 2.1007939596399998e+00, 1.0213285546211000e+04 },
    { 8.1999999999999996e-10, 2.8111908778900001e+00, 7.7197213947532582e+04 },
    { 8.0999999999999999e-10, 1.2447150173899999e+00, 7.6044952320535798e+04 },
    { 8.0999999999999999e-10, 3.4727372985599998e+00, 2.6137899763478399e+04 },
    { 8.0999999999999999e-10, 5.3294721547100004e+00, 2.6037906519669988e+04 },
    { 8.0999999999999999e-10, 2.3369438876199999e+00, 7.7211441041534184e+04 },
    { 8.0000000000000003e-10, 5.5012391537800003e+00, 1.5546803691925600e+05 },
    { 8.0000000000000003e-10, 9.5243616638999995e-01, 4.1962520736937397e+04 },
    { 7.7999999999999999e-10, 9.8498478740999995e-01, 7.3711755927663791e+04 },
    { 7.7999999999999999e-10, 4.9721505207899996e+00, 7.1492998823476984e+04 },
    { 7.7000000000000003e-10, 5.2288508688300004e+00, 1.8207813988235212e+04 },
    { 7.7000000000000003e-10, 2.3494073958600001e+00, 5.1742094545271590e+04 },
    { 7.7000000000000003e-10, 2.2303938841500002e+00, 2.5565325723480400e+04 },
    { 7.5999999999999996e-10, 3.3694641846300000e+00, 1.4956319713459999e+02 },
    { 7.5999999999999996e-10, 3.0291058518599998e+00, 7.8477008520160583e+04 },
    { 7.2999999999999996e-10, 7.1807111235999999e-01, 5.2257741809380002e+02 },
    { 7.1000000000000003e-10, 4.3788427177000000e-01, 1.0890356606335558e+05 },
    { 6.8000000000000003e-10, 2.8167911101999998e-01, 2.6610480559667991e+04 },
    { 6.6999999999999996e-10, 3.4561192025100000e+00, 8.5034420025968182e+04 },
    { 6.6000000000000000e-10, 2.2534131752400000e+00, 1.2022623016165999e+05 },
    { 6.6000000000000000e-10, 3.3833574092699998e+00, 1.6218875089859738e+05 },
    { 6.3999999999999996e-10, 1.7266833956500001e+00, 5.1756321639273199e+04 },
    { 6.3000000000000000e-10, 5.4815187329899997e+00, 9.3028948467992392e+04 },
    { 6.0999999999999996e-10, 8.5156750769000000e-01, 1.0276253967101299e+05 },
    { 6.0999999999999996e-10, 5.8723289422800002e+00, 5.3680451209540001e+02 },
    { 6.0000000000000000e-10, 3.6887878380700001e+00, 3.3967992294913187e+04 },
    { 6.0000000000000000e-10, 5.9187371387900001e+00, 7.1025033833122594e+04 },
    { 6.0000000000000000e-10, 2.4678665730299998e+00, 4.4181277841124189e+04 },
    { 5.7999999999999996e-10, 2.0188459958999999e-01, 3.3914274084046460e+05 },
    { 5.7999999999999996e-10, 2.8115433779900001e+00, 6.4607848933546193e+04 },
    { 5.7000000000000000e-10, 2.0216428762100001e+00, 2.6484548254729998e+03 },
    { 5.7000000000000000e-10, 5.8976592792600000e+00, 6.3278373931320004e+02 },
    { 5.7000000000000000e-10, 1.9984565542799999e+00, 2.6624707653669589e+04 },
    { 5.6000000000000003e-10, 1.2827895263000000e-01, 1.1787336400788819e+05 },
    { 5.4999999999999996e-10, 3.3993162886500001e+00, 1.3521751441591399e+04 },
    { 5.4999999999999996e-10, 3.9524845891300000e+00, 7.6144945564344191e+04 },
    { 5.4999999999999996e-10, 4.2194613092499997e+00, 5.1322609901396390e+04 },
    { 5.4999999999999996e-10, 2.6703745784300001e+00, 1.8093374699549990e+04 },
    { 5.4000000000000000e-10, 4.9279231408999999e+00, 7.8050410329284583e+04 },
    { 5.3000000000000003e-10, 5.2834624793699998e+00, 1.0021837280099400e+04 },
    { 5.3000000000000003e-10, 3.5027007762400002e+00, 7.9852782320006387e+04 },
    { 5.3000000000000003e-10, 5.9571673904000000e-01, 9.8068536716305389e+04 },
    { 5.1999999999999996e-10, 5.7195438785899997e+00, 3.9743763632750597e+04 },
    { 5.1000000000000000e-10, 5.4478945396000000e+00, 1.2990982474277639e+05 },
    { 5.1000000000000000e-10, 6.4336380499000001e-01, 2.5551098629478791e+04 },
    { 5.0000000000000003e-10, 4.4349970551400002e+00, 1.3154889425996438e+05 },
    { 5.0000000000000003e-10, 3.7369679837400001e+00, 8.1706284369687986e+04 },
    { 5.0000000000000003e-10, 5.5595599380699996e+00, 6.4701083331480004e+02 },
    { 5.0000000000000003e-10, 3.5287690461999999e-01, 2.5448005855260191e+04 },
    { 5.0000000000000003e-10, 5.4124927224199997e+00, 5.0579619840863787e+04 },
    { 4.8999999999999996e-10, 5.0857047938799997e+00, 9.0829861124705378e+04 },
    { 4.8000000000000000e-10, 1.7216528946800000e+00, 4.6848330174765601e+04 },
    { 4.7000000000000003e-10, 4.1629492240400001e+00, 7.8800891533389904e+03 },
    { 4.6000000000000001e-10, 3.0159313395699998e+00, 7.6266071275599998e+01 },
    { 4.5000000000000000e-10, 1.5325657032500000e+00, 1.0433194280539699e+05 },
    { 4.5000000000000000e-10, 4.5632953317300000e+00, 6.6812248533995999e+03 },
    { 4.3999999999999998e-10, 3.2425308677300002e+00, 2.6521614879451001e+04 },
    { 4.3000000000000001e-10, 3.2137548601999999e+00, 7.8690307615598591e+04 },
    { 4.3000000000000001e-10, 5.2090752310899999e+00, 1.0392501437542078e+05 },
    { 4.3000000000000001e-10, 4.4596043923800002e+00, 2.6507387785449391e+04 },
    { 4.2000000000000000e-10, 6.0503268600000002e+00, 1.1456489811250778e+05 },
    { 4.2000000000000000e-10, 5.3133073489899996e+00, 2.5881717593137000e+04 },
    { 4.0999999999999998e-10, 5.4206592575699997e+00, 5.2815703569462399e+04 },
    { 4.0999999999999998e-10, 1.4637306887599999e+00, 1.5086608680029298e+05 },
    { 4.0999999999999998e-10, 2.5992930107399999e+00, 5.2329585093633192e+04 },
    { 4.0000000000000001e-10, 3.4520756368500001e+00, 2.6294088690011391e+04 },
    { 3.9000000000000000e-10, 1.5201591818400000e+00, 2.2759767485294011e+04 },
    { 3.7999999999999998e-10, 1.9446115980100001e+00, 2.0760427033191401e+04 },
    { 3.7000000000000001e-10, 1.5430606593200000e+00, 1.5197546535238638e+05 },
    { 3.7000000000000001e-10, 5.0561201246999996e+00, 1.3096920667296558e+05 },
    { 3.6000000000000000e-10, 1.6082325734800000e+00, 8.4546785274713984e+04 },
    { 3.6000000000000000e-10, 4.9039698092700004e+00, 1.0435549390165479e+05 },
    { 3.4999999999999998e-10, 3.1635292595400002e+00, 2.5984810367355600e+04 },
    { 3.4999999999999998e-10, 9.4458988494999996e-01, 1.5763679740153858e+05 },
    { 3.3000000000000000e-10, 5.6524225582499996e+00, 2.6190995915792791e+04 },
    { 3.3000000000000000e-10, 5.4971217421200000e+00, 8.9586373523026981e+04 },
    { 3.3000000000000000e-10, 1.0255711966500001e+00, 2.9416038797854391e+04 },
    { 3.3000000000000000e-10, 2.2090730004400001e+00, 9.5247705572179184e+04 },
    { 3.1999999999999998e-10, 8.1223636393999998e-01, 1.8150594343892598e+05 },
    { 3.1999999999999998e-10, 3.0783471800000002e+00, 1.2948391596626239e+05 },
    { 3.1999999999999998e-10, 5.9236069220300003e+00, 5.0593846934865389e+04 },
    { 3.1999999999999998e-10, 3.1758748606400000e+00, 5.1639001771052986e+04 },
    { 3.1000000000000002e-10, 9.4422467729000004e-01, 5.2290245571833613e+04 },
    { 3.1000000000000002e-10, 3.7907924519599998e+00, 7.7204327494533383e+04 },
    { 3.1000000000000002e-10, 1.4251202803800000e+00, 2.6941099523326200e+04 },
    { 3.0000000000000000e-10, 3.6403443024700000e+00, 5.2643771273502804e+04 },
    { 3.0000000000000000e-10, 2.2677433390999999e+00, 1.8155594006083018e+05 },
    { 3.0000000000000000e-10, 6.0210276835299998e+00, 1.0530721230790539e+05 },
    { 3.0000000000000000e-10, 2.6530757865200001e+00, 5.3242301760338400e+04 },
    { 3.0000000000000000e-10, 3.9664941476200002e+00, 2.5668418497699000e+04 },
    { 2.8999999999999998e-10, 3.7644578894400000e+00, 1.0435872611329758e+05 },
    { 2.8999999999999998e-10, 2.5775974989899999e+00, 1.3149889763806018e+05 },
    { 2.8000000000000002e-10, 7.6476883745000002e-01, 5.4509002676020398e+04 },
    { 2.8000000000000002e-10, 5.3982733326799996e+00, 1.0329934418310839e+05 },
    { 2.5999999999999998e-10, 1.3917488586600000e+00, 5.2061366994463169e+04 },
    { 2.5999999999999998e-10, 5.1758434897100001e+00, 2.9428515568274001e+04 },
    { 2.5999999999999998e-10, 1.2161666654000000e-01, 2.6237466338708789e+04 },
    { 2.5999999999999998e-10, 1.6015479279000000e-01, 2.7684089583858800e+04 },
    { 2.5999999999999998e-10, 5.9691429848000004e+00, 1.3362449706799200e+04 },
    { 2.5000000000000002e-10, 4.7131834783700004e+00, 9.5628915597059995e+02 },
    { 2.5000000000000002e-10, 3.8248268773300000e+00, 2.2747290714874402e+04 },
    { 2.5000000000000002e-10, 2.8129516154699998e+00, 2.5021407664384202e+04 },
    { 2.5000000000000002e-10, 4.7247416015799999e+00, 2.4864085300795588e+04 },
    { 2.4000000000000000e-10, 1.1537763681700000e+00, 5.4294570143526791e+04 },
    { 2.4000000000000000e-10, 2.1482835141800001e+00, 2.5035634758385801e+04 },
    { 2.4000000000000000e-10, 5.4330194330400001e+00, 5.2026243086013797e+04 },
    { 2.4000000000000000e-10, 4.0701715024900000e+00, 2.7311720982352810e+04 },
    { 2.3000000000000001e-10, 5.3124840512800002e+00, 1.4631413330323418e+05 },
    { 2.3000000000000001e-10, 3.5152240266499999e+00, 4.2153969003049002e+04 },
    { 2.3000000000000001e-10, 3.1618518667400002e+00, 1.4396126714946239e+05 },
    { 2.3000000000000001e-10, 7.6234430511999995e-01, 8.5502385016322587e+04 },
    { 2.3000000000000001e-10, 1.9321471535300001e+00, 1.3001291751699499e+05 },
    { 2.3000000000000001e-10, 4.8209524098300003e+00, 7.7844224780847391e+04 },
    { 2.3000000000000001e-10, 6.0984808095299998e+00, 6.1279713277265997e+04 },
    { 2.3000000000000001e-10, 5.4055000182899997e+00, 7.7829997686845789e+04 },
    { 2.1999999999999999e-10, 6.1334971241699998e+00, 1.0456491166173479e+05 },
    { 2.1999999999999999e-10, 3.4807273999100001e+00, 1.3499146920492980e+05 },
    { 2.1000000000000000e-10, 4.2260677311099997e+00, 1.0434449901929598e+05 },
    { 2.1000000000000000e-10, 4.3040308875199997e+00, 1.0213285546210999e+05 },
    { 2.1000000000000000e-10, 4.8258261456799998e+00, 2.0618554843720000e+02 },
    { 2.1000000000000000e-10, 3.6580876648600000e+00, 1.2415643985787958e+05 },
    { 2.1000000000000000e-10, 1.5049062298999999e-01, 1.8827665404017159e+05 },
    { 2.1000000000000000e-10, 1.7508940933199999e+00, 9.7580901965051191e+04 },
    { 2.1000000000000000e-10, 4.2232709112000000e-01, 1.0437128232719658e+05 },
    { 2.1000000000000000e-10, 4.5637212143200001e+00, 1.1610551958374201e+04 },
    { 2.0000000000000001e-10, 7.1942657444000002e-01, 1.4788665740644001e+03 },
    { 2.0000000000000001e-10, 5.8653982935800002e+00, 1.0328511708910679e+05 },
    { 2.0000000000000001e-10, 3.0180264178299998e+00, 5.1543022543835199e+04 },
    { 2.0000000000000001e-10, 4.2824727357099999e+00, 4.5494581429748789e+04 },
    { 1.8999999999999999e-10, 2.0832770778400000e+00, 5.2125809661244188e+04 },
    { 1.8999999999999999e-10, 3.9572623459000000e-01, 4.9527351457675388e+04 },
    { 1.8999999999999999e-10, 3.3804949788999998e+00, 1.0371171527998279e+05 },
    { 1.8999999999999999e-10, 1.6234318595399999e+00, 3.3281356562801898e+03 },
    { 1.8999999999999999e-10, 3.2161718860900002e+00, 5.3228074666336790e+04 },
    { 1.8999999999999999e-10, 1.4096729710000000e-02, 1.1567427666460119e+05 },
    { 1.8999999999999999e-10, 3.3528762601300000e+00, 5.2698383701242194e+04 },
    { 1.8999999999999999e-10, 2.1813620318900000e+00, 1.5599772788435058e+05 },
    { 1.8999999999999999e-10, 3.3799877555900002e+00, 1.0434773123093879e+05 },
    { 1.8000000000000000e-10, 3.9544482117699999e+00, 1.0657036967048359e+05 },
    { 1.8000000000000000e-10, 2.3620779537200001e+00, 1.3139511544947958e+05 },
    { 1.8000000000000000e-10, 2.2502074518900002e+00, 1.1911685160956658e+05 },
    { 1.8000000000000000e-10, 2.6812612930499999e+00, 9.7112936974696786e+04 },
    { 1.7000000000000001e-10, 8.5708896301000004e-01, 1.1624747044077999e+03 },
    { 1.7000000000000001e-10, 1.5300386947400000e+00, 1.0413831347085879e+05 },
    { 1.7000000000000001e-10, 3.2177648083800001e+00, 5.1528795449833589e+04 },
    { 1.7000000000000001e-10, 2.1341784726099999e+00, 7.6667522982437978e+04 },
    { 1.7000000000000001e-10, 5.0653230640600002e+00, 4.9842609890276392e+04 },
    { 1.5999999999999999e-10, 1.0145247680500000e+00, 7.7410513042970590e+04 },
    { 1.5999999999999999e-10, 2.7629951229700001e+00, 1.2546481939083000e+04 },
    { 1.5999999999999999e-10, 5.5152551321800001e+00, 2.5867490499135391e+04 },
    { 1.5999999999999999e-10, 1.4257422389999999e-01, 6.0055895436487393e+04 },
    { 1.5999999999999999e-10, 3.3106717927800000e+00, 2.6308315784013001e+04 },
    { 1.5999999999999999e-10, 4.9793433089699999e+00, 5.2712610795243789e+04 },
    { 1.5999999999999999e-10, 5.8139598832399999e+00, 5.2099540211872802e+04 },
    { 1.5000000000000000e-10, 4.6733316281799997e+00, 1.0090903762133139e+05 },
    { 1.5000000000000000e-10, 1.2287044845999999e+00, 6.5697557724739789e+04 },
    { 1.5000000000000000e-10, 6.0602537995399999e+00, 2.6164169212849800e+04 },
    { 1.5000000000000000e-10, 1.7179065008000000e-01, 5.2225802905052602e+04 },
    { 1.5000000000000000e-10, 3.2035972632899998e+00, 1.2885044281258718e+05 },
    { 1.5000000000000000e-10, 3.2543050996299998e+00, 3.6523064398203877e+05 },
    { 1.5000000000000000e-10, 2.4966144265000001e-01, 1.0594068546158058e+05 },
    { 1.5000000000000000e-10, 4.4753265309800003e+00, 4.0565254324774200e+04 },
    { 1.5000000000000000e-10, 6.0020037717200001e+00, 2.7669862489857191e+04 },
    { 1.5000000000000000e-10, 3.4533721451499999e+00, 7.7837111233846590e+04 },
    { 1.5000000000000000e-10, 3.9208203534899999e+00, 2.1258774073792001e+03 },
    { 1.5000000000000000e-10, 1.0893122941900000e+00, 3.2769127994973802e+04 },
    { 1.4000000000000001e-10, 6.0484795170299996e+00, 9.0695752075120385e+04 },
    { 1.4000000000000001e-10, 2.0056601651400001e+00, 4.4295717129809404e+04 },
    { 1.4000000000000001e-10, 3.8973718352600000e+00, 1.8372470054311279e+05 },
    { 1.4000000000000001e-10, 3.1866599963700000e+00, 2.7999102624791401e+04 },
    { 1.4000000000000001e-10, 3.1914167939999998e+00, 2.1116503133776000e+03 },
    { 1.4000000000000001e-10, 1.8595638631500000e+00, 1.1691776426627958e+05 },
    { 1.4000000000000001e-10, 3.1453604780100002e+00, 1.1159028815789679e+05 },
    { 1.4000000000000001e-10, 5.2081515141699999e+00, 2.5953794091989192e+04 },
    { 1.4000000000000001e-10, 4.9361194238000000e-01, 1.0779418751126219e+05 },
    { 1.4000000000000001e-10, 3.5861211487700002e+00, 2.6222012191159200e+04 },
    { 1.4000000000000001e-10, 2.7252655802499999e+00, 3.8813356576349201e+04 },
    { 1.4000000000000001e-10, 5.7525050120500003e+00, 7.8417488235207391e+04 },
    { 1.2999999999999999e-10, 5.6398960937300000e+00, 2.4176703658357001e+04 },
    { 1.2999999999999999e-10, 4.5871309992600002e+00, 1.7806336849396059e+05 },
    { 1.2999999999999999e-10, 2.1180670791799998e+00, 3.2350541665740002e+02 },
    { 1.2999999999999999e-10, 4.8885232832899996e+00, 1.4477351183200000e+04 },
    { 1.2999999999999999e-10, 2.8218921610500001e+00, 1.4065280125408198e+05 },
    { 1.2999999999999999e-10, 5.1672044652600002e+00, 7.0269180982698381e+04 },
    { 1.2999999999999999e-10, 5.7040607324100003e+00, 2.5654191403697390e+04 },
    { 1.2000000000000000e-10, 2.6751314742100001e+00, 7.6681750076439581e+04 },
    { 1.2000000000000000e-10, 4.6275944653899996e+00, 7.2936233316339785e+04 },
    { 1.2000000000000000e-10, 3.9222320241599999e+00, 1.6066065861474801e+04 },
    { 1.2000000000000000e-10, 6.0625088836899996e+00, 5.2072713508929788e+04 },
    { 1.2000000000000000e-10, 5.7700710308599996e+00, 1.6921656695024001e+03 },
    { 1.2000000000000000e-10, 4.6745857099099997e+00, 1.1063468841628819e+05 },
    { 1.2000000000000000e-10, 5.5371302917999998e-01, 1.0223284870591838e+05 },
    { 1.2000000000000000e-10, 4.6344500005000000e-01, 4.5892730433156990e+04 },
    { 1.2000000000000000e-10, 3.6273333924300002e+00, 3.1415379249957001e+04 },
    { 1.2000000000000000e-10, 4.5841149351399997e+00, 1.3041984594697120e+05 },
    { 1.2000000000000000e-10, 2.3485004645499998e+00, 2.2041264243879999e+02 },
    { 1.2000000000000000e-10, 5.6029028906400002e+00, 2.2434479570190001e+02 },
    { 1.2000000000000000e-10, 4.9878313337300000e+00, 2.6395460762543789e+04 },
    { 1.2000000000000000e-10, 3.8173863510700001e+00, 2.5780345520604598e+04 },
    { 1.2000000000000000e-10, 2.6106854693099999e+00, 2.6724899413598399e+04 },
    { 1.2000000000000000e-10, 6.1945322154899998e+00, 2.5450906869549999e+04 },
    { 1.2000000000000000e-10, 7.1134314906999996e-01, 2.5446489579835201e+04 },
    { 1.2000000000000000e-10, 1.8106892285500000e+00, 2.6729316703313190e+04 },
    { 1.2000000000000000e-10, 4.5159835673100002e+00, 1.7695398994186718e+05 },
    { 1.0999999999999999e-10, 2.0003323738300001e+00, 3.6109740421673603e+04 },
    { 1.0999999999999999e-10, 1.1053960702200001e+00, 2.6198109462793600e+04 },
    { 1.0999999999999999e-10, 1.4540231257000000e-01, 1.1112232316754239e+05 },
    { 1.0999999999999999e-10, 6.2746346672800000e+00, 1.0477821075717278e+05 },
    { 1.0999999999999999e-10, 1.4206951551600000e+00, 2.5977696820354791e+04 },
    { 1.0999999999999999e-10, 6.1730576660200001e+00, 3.9629324344065390e+04 },
    { 1.0999999999999999e-10, 2.5495900134100000e+00, 5.1969620734711192e+04 },
    { 1.0999999999999999e-10, 2.9504562684700000e+00, 1.4176217980617538e+05 },
    { 1.0999999999999999e-10, 3.0374895653100000e+00, 1.2725453434775000e+04 },
    { 1.0999999999999999e-10, 1.7957392798400000e+00, 1.5705710981453978e+05 },
    { 1.0000000000000000e-10, 4.0045208553200000e+00, 5.5503941939428587e+04 },
    { 1.0000000000000000e-10, 5.0149248088800000e+00, 2.8736357967047199e+04 },
    { 1.0000000000000000e-10, 2.2346664936999998e+00, 7.8903606711036584e+04 },
    { 1.0000000000000000e-10, 6.2284360651500004e+00, 1.7004917029103660e+05 },
    { 1.0000000000000000e-10, 5.3133462849699997e+00, 2.0764384320240439e+05 },
    { 1.0000000000000000e-10, 2.4746811068299999e+00, 6.5831666774324796e+04 },
    { 1.0000000000000000e-10, 8.1972980159999997e-01, 5.2022027472663598e+04 },
    { 1.0000000000000000e-10, 3.8285944104000003e-01, 5.2381991831585590e+04 },
    { 8.9999999999999999e-11, 4.0568488596299996e+00, 5.1219517127177787e+04 },
    { 8.9999999999999999e-11, 6.1220898352899997e+00, 1.5557181910783658e+05 },
    { 8.9999999999999999e-11, 4.1261244955999998e-01, 7.8731674415076996e+04 },
    { 8.9999999999999999e-11, 4.0952403441000002e-01, 1.6703062133498999e+04 },
    { 8.9999999999999999e-11, 4.9069756830499998e+00, 5.1707841292793993e+04 },
    { 8.9999999999999999e-11, 3.8573270827899999e+00, 2.0759384658050019e+05 },
    { 8.9999999999999999e-11, 4.9656180722399998e+00, 1.5610082065856917e+05 },
    { 8.9999999999999999e-11, 3.7577674563899999e+00, 1.9111994832171999e+03 },
    { 8.9999999999999999e-11, 4.0730044290000003e-02, 5.2609518021025193e+04 },
    { 8.9999999999999999e-11, 1.9171070354399999e+00, 5.5516418709848200e+04 },
    { 8.9999999999999999e-11, 4.8386414935199999e+00, 1.9669760899789999e+01 },
    { 8.9999999999999999e-11, 1.6636180644800000e+00, 1.3044339704322898e+05 },
    { 8.9999999999999999e-11, 1.2313460087000001e+00, 5.2169177724248388e+04 },
    { 8.9999999999999999e-11, 2.3604875232600002e+00, 2.6734913974889001e+04 },
    { 7.9999999999999995e-11, 4.3645168051100001e+00, 5.2182434842048402e+04 },
    { 7.9999999999999995e-11, 4.7923355062299997e+00, 2.7177611932767799e+04 },
    { 7.9999999999999995e-11, 4.0128821785700000e+00, 2.4998194350380589e+04 },
    { 7.9999999999999995e-11, 2.8580166264600000e+00, 2.6720686880887391e+04 },
    { 7.9999999999999995e-11, 4.5624045749600004e+00, 4.8847670626868203e+04 },
    { 7.9999999999999995e-11, 4.6642941835200000e+00, 8.6030992875280003e+02 },
    { 7.9999999999999995e-11, 3.8701534981000002e-01, 1.3541421202491190e+04 },
    { 7.9999999999999995e-11, 4.1051181648300004e+00, 1.2238178407786099e+03 },
    { 7.9999999999999995e-11, 1.3825817275900001e+00, 5.2595290927023590e+04 },
    { 7.9999999999999995e-11, 5.2005477263000000e-01, 1.3044662925487179e+05 },
    { 7.9999999999999995e-11, 2.1693267166300001e+00, 1.2938724732468258e+05 },
    { 7.9999999999999995e-11, 1.1396500548899999e+00, 2.5234706759822191e+04 },
    { 7.9999999999999995e-11, 6.2158611424699997e+00, 7.7726904912627186e+04 },
    { 7.9999999999999995e-11, 5.6448463806299998e+00, 1.5758680077963436e+05 },
    { 7.0000000000000004e-11, 3.6409349148999998e-01, 6.8241872144623194e+04 },
    { 7.0000000000000004e-11, 1.7734920652899999e+00, 5.0696939709083992e+04 },
    { 7.0000000000000004e-11, 7.3615049600000004e-01, 2.0981260368468694e+05 },
    { 7.0000000000000004e-11, 2.0711822206899999e+00, 1.7240203644480839e+05 },
    { 7.0000000000000004e-11, 5.4644308349399999e+00, 8.4608283475120004e+02 },
    { 7.0000000000000004e-11, 1.9376320489900001e+00, 3.5472744149649399e+04 },
    { 7.0000000000000004e-11, 1.4824906851199999e+00, 2.7044192297544800e+04 },
    { 7.0000000000000004e-11, 4.2254434486999998e-01, 1.5024434299945380e+05 },
    { 7.0000000000000004e-11, 3.1864882710900000e+00, 5.2325369480282992e+04 },
    { 7.0000000000000004e-11, 2.2404606562399998e+00, 9.3848410080752001e+03 },
    { 7.0000000000000004e-11, 3.8134064648999999e+00, 8.0596905817594597e+04 },
    { 7.0000000000000004e-11, 3.9944219180300000e+00, 7.8378148713407805e+04 },
    { 7.0000000000000004e-11, 6.5296081909999998e-02, 2.6162684740141500e+04 },
    { 7.0000000000000004e-11, 5.2314073285299996e+00, 1.5748301859105378e+05 },
    { 7.0000000000000004e-11, 6.1493584843000004e+00, 7.7630925685409384e+04 },
    { 7.0000000000000004e-11, 3.2089246972400001e+00, 2.1436455718174577e+05 },
    { 7.0000000000000004e-11, 2.7950844759000000e+00, 2.5754047200804798e+04 },
    { 7.0000000000000004e-11, 6.0101332089000001e+00, 2.6421759082343600e+04 },
    { 7.0000000000000004e-11, 5.6867672025299996e+00, 6.4901259717923393e+04 },
    { 6.0000000000000000e-11, 2.2128039232200001e+00, 5.2278899057366987e+04 },
    { 6.0000000000000000e-11, 5.7178263895999999e+00, 7.9330204901912584e+04 },
    { 6.0000000000000000e-11, 2.6338586881800001e+00, 1.2937302023068098e+05 },
    { 6.0000000000000000e-11, 1.5539009025699999e+00, 5.1013331578740588e+04 },
    { 6.0000000000000000e-11, 3.2921930899000000e+00, 7.5615254599249587e+04 },
    { 6.0000000000000000e-11, 2.4676397487499999e+00, 2.6013121543006899e+04 },
    { 6.0000000000000000e-11, 6.0305949588000001e+00, 1.3767819129947099e+05 },
    { 6.0000000000000000e-11, 2.9383481267599998e+00, 1.3065281480330898e+05 },
    { 6.0000000000000000e-11, 4.9630404210299996e+00, 2.5508215554575399e+04 },
    { 6.0000000000000000e-11, 3.8421772637700000e+00, 2.6667590728572999e+04 },
    { 6.0000000000000000e-11, 2.9225742955000000e-01, 6.8851498899308099e+03 },
    { 6.0000000000000000e-11, 2.1807916213000000e+00, 1.0391790082841998e+05 },
    { 6.0000000000000000e-11, 4.1687992097400004e+00, 8.0382473285100990e+04 },
    { 6.0000000000000000e-11, 6.6117422106000001e-01, 3.4282178474782799e+04 },
    { 6.0000000000000000e-11, 4.4381406129099998e+00, 7.8149270136037361e+04 },
    { 6.0000000000000000e-11, 1.0613316857199999e+00, 1.2822075860368418e+05 },
    { 6.0000000000000000e-11, 4.7807504739500004e+00, 1.2366880510662538e+05 },
    { 6.0000000000000000e-11, 1.6082578307199999e+00, 1.0393212792242158e+05 },
    { 6.0000000000000000e-11, 2.4754368045000000e-01, 1.6107937234650398e+05 },
    { 6.0000000000000000e-11, 3.5484891863499999e+00, 8.6143798578061585e+04 },
    { 6.0000000000000000e-11, 1.9621462218400001e+00, 1.2655674786264001e+03 },
    { 6.0000000000000000e-11, 5.1884851882299996e+00, 1.8208563102592478e+05 },
    { 6.0000000000000000e-11, 5.9952467664999998e-01, 5.1653228865054589e+04 },
    { 6.0000000000000000e-11, 1.1577208934000001e+00, 8.1942753332085995e+03 },
    { 6.0000000000000000e-11, 3.3976411747499999e+00, 6.2830758499914000e+03 },
    { 6.0000000000000000e-11, 4.8950493375999997e+00, 5.0049928875769190e+04 },
    { 6.0000000000000000e-11, 5.7770762142800001e+00, 1.5493834595416137e+05 },
    { 5.0000000000000002e-11, 5.6085606726300004e+00, 5.0064155969770793e+04 },
    { 5.0000000000000002e-11, 4.4888601566000004e+00, 1.3022621661243298e+05 },
    { 5.0000000000000002e-11, 4.5055999919099996e+00, 1.1020632121940000e+02 },
    { 5.0000000000000002e-11, 5.8914040733100004e+00, 1.0276965321801379e+05 },
    { 5.0000000000000002e-11, 2.5021308618900000e+00, 1.9406678288174600e+04 },
    { 5.0000000000000002e-11, 5.1198998086799996e+00, 7.8213712802818380e+04 },
    { 5.0000000000000002e-11, 5.4440184392299997e+00, 1.0275542612401219e+05 },
    { 5.0000000000000002e-11, 8.5038224535999996e-01, 5.3399624123927002e+04 },
    { 5.0000000000000002e-11, 2.4267172453599999e+00, 2.7780068811076591e+04 },
    { 5.0000000000000002e-11, 5.2247650493400002e+00, 2.6402089321443800e+04 },
    { 5.0000000000000002e-11, 3.5804526354599999e+00, 2.5773716961704591e+04 },
    { 5.0000000000000002e-11, 2.4299630193700001e+00, 2.5938339944439598e+04 },
    { 5.0000000000000002e-11, 1.4700403120000000e-02, 7.7616698591407781e+04 },
    { 5.0000000000000002e-11, 1.9958621471000000e+00, 1.7893627808365600e+04 },
    { 5.0000000000000002e-11, 4.7895404182699997e+00, 3.3406124266997999e+03 },
    { 5.0000000000000002e-11, 1.3548848125599999e+00, 2.0415127163553477e+05 },
    { 5.0000000000000002e-11, 2.7436660542700002e+00, 3.9450352848373397e+04 },
    { 5.0000000000000002e-11, 6.8952674300000003e-02, 5.2396218925587200e+04 },
    { 5.0000000000000002e-11, 5.7103245210000004e+00, 1.2320084011627098e+05 },
    { 5.0000000000000002e-11, 4.5266907284000002e+00, 5.3029002664900399e+04 },
    { 5.0000000000000002e-11, 5.2871113926300000e+00, 1.4520475475114078e+05 },
    { 5.0000000000000002e-11, 1.8442639416900000e+00, 2.6709646942413401e+04 },
    { 5.0000000000000002e-11, 6.7776843593000002e-01, 2.5466159340735001e+04 },
    { 5.0000000000000002e-11, 1.2565709933600000e+00, 3.2132131722949602e+04 },
    { 5.0000000000000002e-11, 4.1148055834000002e-01, 5.2808590022461591e+04 },
    { 3.9999999999999998e-11, 6.0056613449600000e+00, 1.6785008294774959e+05 },
    { 3.9999999999999998e-11, 9.7602564751999998e-01, 1.3043240216087017e+05 },
    { 3.9999999999999998e-11, 4.0741730958000000e-01, 5.1852300866490987e+04 },
    { 3.9999999999999998e-11, 3.6734561497099998e+00, 6.3699627202420004e+02 },
    { 3.9999999999999998e-11, 2.7422307846099998e+00, 1.1678365521669458e+05 },
    { 3.9999999999999998e-11, 1.2316552652299999e+00, 5.1955393640709590e+04 },
    { 3.9999999999999998e-11, 4.0912790708099998e+00, 1.0349841618454478e+05 },
    { 3.9999999999999998e-11, 2.3704532490600001e+00, 4.5290656393217592e+04 },
    { 3.9999999999999998e-11, 2.5635462378999998e+00, 1.0450539137678158e+05 },
    { 3.9999999999999998e-11, 5.9460014710999998e-01, 2.8791519296249800e+04 },
    { 3.9999999999999998e-11, 1.9274322305100000e+00, 2.3384286986898602e+04 },
    { 3.9999999999999998e-11, 1.8126863796499999e+00, 2.6823779655105991e+04 },
    { 3.9999999999999998e-11, 7.0934599796999998e-01, 2.5352026628042389e+04 },
    { 3.9999999999999998e-11, 3.5397534426799999e+00, 1.3388209065283637e+05 },
    { 3.9999999999999998e-11, 3.0193549877399999e+00, 5.0483640613645999e+04 },
    { 3.9999999999999998e-11, 2.0133471745099998e+00, 1.8357092173262799e+05 },
    { 3.9999999999999998e-11, 3.2238443404399999e+00, 5.3771992725432989e+04 },
    { 3.9999999999999998e-11, 1.3486269553000001e+00, 2.7170983373867792e+04 },
    { 3.9999999999999998e-11, 1.1734054223199999e+00, 2.5004822909280600e+04 },
    { 3.9999999999999998e-11, 1.4217969958000001e-01, 7.8786286842816378e+04 },
    { 3.9999999999999998e-11, 3.5178769836899999e+00, 1.7952584437210000e+03 },
    { 3.9999999999999998e-11, 4.2709311511000001e+00, 5.2065599961928987e+04 },
    { 3.9999999999999998e-11, 1.0710186974999999e-01, 2.2218566345969998e+03 },
    { 3.9999999999999998e-11, 1.1143805333100001e+00, 7.8270337983622594e+04 },
    { 3.9999999999999998e-11, 2.2173605480099998e+00, 9.6357084124272587e+04 },
    { 3.9999999999999998e-11, 3.3468510669399998e+00, 9.4917560896980001e+02 },
    { 3.9999999999999998e-11, 1.3402509489300001e+00, 6.0442285813753997e+03 },
    { 3.9999999999999998e-11, 4.9030978070000000e+00, 1.4300566740785379e+05 },
    { 3.9999999999999998e-11, 1.7067294932099999e+00, 7.8800513936817981e+04 },
    { 3.9999999999999998e-11, 9.0136038469000002e-01, 1.3265827281205778e+05 },
    { 3.9999999999999998e-11, 6.0476540446499998e+00, 5.4087005766365597e+04 },
    { 3.9999999999999998e-11, 3.4539643302499998e+00, 1.3045918546877075e+05 },
    { 3.9999999999999998e-11, 2.5743395544899998e+00, 3.1781457905676002e+03 },
    { 3.9999999999999998e-11, 3.3942074909800000e+00, 5.2250587881715699e+04 },
    { 3.9999999999999998e-11, 2.8928002362599998e+00, 8.7367616418840189e+04 },
    { 3.9999999999999998e-11, 1.5780189015000001e-01, 1.3043563437251298e+05 },
    { 3.9999999999999998e-11, 6.7113004079999997e-02, 2.6575537892828390e+04 },
    { 3.9999999999999998e-11, 2.4549193735400001e+00, 2.5600268390320001e+04 },
    { 3.9999999999999998e-11, 4.1793892225500002e+00, 5.8857031136547987e+04 },
    { 3.9999999999999998e-11, 3.2009205069300002e+00, 5.1951461487446490e+04 },
    { 3.9999999999999998e-11, 1.1728519822200001e+00, 2.0043674560198800e+04 },
    { 3.9999999999999998e-11, 1.4595160760100001e+00, 1.3672259155786238e+05 },
    { 3.9999999999999998e-11, 3.7955915191999998e+00, 1.2832075184749259e+05 },
    { 3.9999999999999998e-11, 3.2749351115000003e-01, 1.2979961842155698e+05 },
    { 3.9999999999999998e-11, 3.0171000275999998e-01, 4.6796499035440002e+02 },
    { 3.9999999999999998e-11, 1.2968752323299999e+00, 3.7698455099948398e+04 },
    { 3.9999999999999998e-11, 3.2787243048399999e+00, 1.3202858860315479e+05 },
    { 3.9999999999999998e-11, 5.8609291259200003e+00, 1.6674070439565618e+05 },
    { 3.9999999999999998e-11, 5.1182239920699999e+00, 7.8109930614237790e+04 },
    { 3.9999999999999998e-11, 4.4795896094699996e+00, 2.8286990484861199e+04 },
    { 3.9999999999999998e-11, 4.3256280753300000e+00, 2.3888815798287189e+04 },
    { 3.0000000000000000e-11, 2.7313219732700000e+00, 7.8160616650503987e+04 },
    { 3.0000000000000000e-11, 4.1441699816500002e+00, 1.1443928868521000e+02 },
    { 3.0000000000000000e-11, 5.3771340446100000e+00, 1.4859801210651999e+03 },
    { 3.0000000000000000e-11, 3.5934615572000000e-01, 5.2309915332733399e+04 },
    { 3.0000000000000000e-11, 5.0564968326599997e+00, 7.0383620271383596e+04 },
    { 3.0000000000000000e-11, 2.8468580967900001e+00, 5.2252072354423988e+04 },
    { 3.0000000000000000e-11, 2.4014405916900001e+00, 5.0264606799931200e+04 },
    { 3.0000000000000000e-11, 9.3320061996000003e-01, 8.1591845081002786e+04 },
    { 3.0000000000000000e-11, 3.9654737447600001e+00, 2.5985940623308590e+04 },
    { 3.0000000000000000e-11, 4.8397439400400000e+00, 2.6189865659839801e+04 },
    { 3.0000000000000000e-11, 3.4037768900000001e-02, 2.3969139281195799e+04 },
    { 3.0000000000000000e-11, 4.7556756333300001e+00, 6.1560647291223591e+04 },
    { 3.0000000000000000e-11, 2.6267158764000000e+00, 1.8102624909573558e+05 },
    { 3.0000000000000000e-11, 4.9223587424400002e+00, 2.6073676047572590e+04 },
    { 3.0000000000000000e-11, 3.8828589423600000e+00, 2.6102130235575802e+04 },
    { 3.0000000000000000e-11, 4.9521348869999998e-01, 2.7036161546755998e+03 },
    { 3.0000000000000000e-11, 2.8360415114499999e+00, 1.9202753251643389e+04 },
    { 3.0000000000000000e-11, 1.8734144677900000e+00, 2.6247204876366399e+04 },
    { 3.0000000000000000e-11, 6.4861790983000001e-01, 2.5928601406782000e+04 },
    { 3.0000000000000000e-11, 2.6860999362800002e+00, 1.0309277421860000e+02 },
    { 3.0000000000000000e-11, 1.4610241716900001e+00, 7.5930513031850584e+04 },
    { 3.0000000000000000e-11, 6.9308741099999996e-03, 7.9315977807910982e+04 },
    { 3.0000000000000000e-11, 3.0578422244699999e+00, 1.3086611389874699e+05 },
    { 3.0000000000000000e-11, 3.6499703996899999e+00, 3.5077370847006590e+04 },
    { 3.0000000000000000e-11, 6.6136657529999998e-01, 5.1868248662178798e+04 },
    { 3.0000000000000000e-11, 4.9038367162399998e+00, 7.6784842850658184e+04 },
    { 3.0000000000000000e-11, 5.0964492226300004e+00, 1.2133560871375339e+05 },
    { 3.0000000000000000e-11, 4.1508752300599996e+00, 5.2286012604367788e+04 },
    { 3.0000000000000000e-11, 4.4911769767900003e+00, 7.7101234720314780e+04 },
    { 3.0000000000000000e-11, 4.4067234673800000e+00, 7.7741132006628788e+04 },
    { 3.0000000000000000e-11, 1.1703311109500001e+00, 7.4781598567299994e+01 },
    { 3.0000000000000000e-11, 1.7206891226200001e+00, 2.4505943793291190e+04 },
    { 3.0000000000000000e-11, 4.6771805476699999e+00, 2.7250377845981991e+04 },
    { 3.0000000000000000e-11, 4.1397182513799997e+00, 5.1315496354395589e+04 },
    { 3.0000000000000000e-11, 2.7349912234299998e+00, 2.6011637070298599e+04 },
    { 3.0000000000000000e-11, 4.9497477083300003e+00, 8.1604321851422384e+04 },
    { 3.0000000000000000e-11, 5.6698218176699999e+00, 5.2812802555172602e+04 },
    { 3.0000000000000000e-11, 2.1734219629999999e-02, 3.9131854712361295e+05 },
    { 3.0000000000000000e-11, 1.2817301824700000e+00, 1.5650774908854539e+05 },
    { 3.0000000000000000e-11, 9.3767773997000003e-01, 2.7726972658762199e+04 },
    { 3.0000000000000000e-11, 1.5843546376500000e+00, 2.4448833624386200e+04 },
    { 3.0000000000000000e-11, 3.1795049410299998e+00, 7.8313706046626787e+04 },
    { 3.0000000000000000e-11, 1.2677763752300000e+00, 2.0304189308344139e+05 },
    { 3.0000000000000000e-11, 2.7748831613200000e+00, 3.4082431583598402e+04 },
    { 3.0000000000000000e-11, 4.2585109579299996e+00, 7.8257080865822587e+04 },
    { 3.0000000000000000e-11, 1.9562812687300000e+00, 5.2041697233563391e+04 },
    { 3.0000000000000000e-11, 3.6739870703499999e+00, 1.1334108027172917e+05 },
    { 3.0000000000000000e-11, 3.6613746297400001e+00, 5.1534392721409400e+04 },
    { 1.9999999999999999e-11, 1.5001821236999999e-01, 4.8835193856448590e+04 },
    { 1.9999999999999999e-11, 1.6931301088699999e+00, 1.8218872380014337e+05 },
    { 1.9999999999999999e-11, 1.2189653346500000e+00, 2.0085575391590000e+03 },
    { 1.9999999999999999e-11, 1.1318242968600001e+00, 1.2699694076290558e+05 },
    { 1.9999999999999999e-11, 3.3383239729900001e+00, 2.4925428437166400e+04 },
    { 1.9999999999999999e-11, 3.3460711582200000e+00, 5.1329723448397199e+04 },
    { 1.9999999999999999e-11, 2.9697862896700000e+00, 1.9613707343261078e+05 },
    { 1.9999999999999999e-11, 5.7742228404700002e+00, 7.8057523876285384e+04 },
    { 1.9999999999999999e-11, 4.0781170672300000e+00, 8.9894677054323893e+03 },
    { 1.9999999999999999e-11, 5.8293363756599996e+00, 3.8634385080657201e+04 },
    { 1.9999999999999999e-11, 2.7169828985600000e+00, 1.6376609444104519e+05 },
    { 1.9999999999999999e-11, 3.4664850677999999e+00, 1.0481957755665119e+05 },
    { 1.9999999999999999e-11, 5.0549326912500003e+00, 6.2197643563247788e+04 },
    { 1.9999999999999999e-11, 2.3139186556100002e+00, 6.5717227485639582e+04 },
    { 1.9999999999999999e-11, 5.7819709496000005e-01, 6.6653157466348392e+04 },
    { 1.9999999999999999e-11, 5.8113453850700001e+00, 7.4923096998022782e+04 },
    { 1.9999999999999999e-11, 5.3179911171200001e+00, 1.7098435436141812e+04 },
    { 1.9999999999999999e-11, 3.5486638254199998e+00, 7.8469894973159782e+04 },
    { 1.9999999999999999e-11, 6.3534070468000003e-01, 7.7307420268751986e+04 },
    { 1.9999999999999999e-11, 5.3068520011000002e+00, 1.0499150985261079e+05 },
    { 1.9999999999999999e-11, 1.0105293707800000e+00, 1.0170315774082378e+05 },
    { 1.9999999999999999e-11, 2.4683883606300001e+00, 2.6312247937276101e+04 },
    { 1.9999999999999999e-11, 1.4886162229700000e+00, 5.0951988442369788e+04 },
    { 1.9999999999999999e-11, 5.5220582473800004e+00, 9.1919569915898988e+04 },
    { 1.9999999999999999e-11, 1.4377674704200001e+00, 9.9024136457913992e+04 },
    { 1.9999999999999999e-11, 2.0749131367799998e+00, 2.2568722128005017e+05 },
    { 1.9999999999999999e-11, 2.4587862939699998e+00, 2.6183882368791990e+04 },
    { 1.9999999999999999e-11, 6.3246083650000007e-02, 2.5991923914356390e+04 },
    { 1.9999999999999999e-11, 4.7989473138200003e+00, 1.8314501295611399e+05 },
    { 1.9999999999999999e-11, 1.1688462640999999e+00, 2.6397181464229991e+04 },
    { 1.9999999999999999e-11, 1.3531861135200001e+00, 2.5778624818918390e+04 },
    { 1.9999999999999999e-11, 3.4796110167099998e+00, 4.2790965275073198e+04 },
    { 1.9999999999999999e-11, 3.5402151298600000e+00, 5.1538810011124187e+04 },
    { 1.9999999999999999e-11, 2.4562795406800002e+00, 2.4952226269680999e+04 },
    { 1.9999999999999999e-11, 6.5752836940000003e-02, 2.7223580013467399e+04 },
    { 1.9999999999999999e-11, 2.9990986200099998e+00, 1.0371882882698359e+05 },
    { 1.9999999999999999e-11, 1.6460667184200000e+00, 2.5984120952993599e+04 },
    { 1.9999999999999999e-11, 8.7596565920000002e-01, 2.6191685330154789e+04 },
    { 1.9999999999999999e-11, 5.4469033593800003e+00, 2.5927495772189392e+04 },
    { 1.9999999999999999e-11, 3.3583143254199999e+00, 2.6248310510959000e+04 },
    { 1.9999999999999999e-11, 1.8031751612500000e+00, 1.7728784588263658e+05 },
    { 1.9999999999999999e-11, 5.0595959267900001e+00, 1.1882896374949679e+05 },
    { 1.9999999999999999e-11, 1.5740707379600001e+00, 5.3265515074342002e+04 },
    { 1.9999999999999999e-11, 6.1786777553799999e+00, 5.1841950342379001e+04 },
    { 1.9999999999999999e-11, 5.2784265993000001e-01, 2.9550147847439392e+04 },
    { 1.9999999999999999e-11, 1.9941897176900001e+00, 2.2625658435709000e+04 },
    { 1.9999999999999999e-11, 6.1570210206400002e+00, 2.7037078750543991e+04 },
    { 1.9999999999999999e-11, 2.6481966641599999e+00, 2.5138727532604400e+04 },
    { 1.9999999999999999e-11, 4.8415887225900001e+00, 5.2817219844887390e+04 },
    { 1.9999999999999999e-11, 4.0042998444800002e+00, 8.7253177130154960e+04 },
    { 1.9999999999999999e-11, 1.4115672130000000e-02, 1.6895946149984296e+05 },
    { 1.9999999999999999e-11, 2.2272579645700001e+00, 1.0420204936916218e+05 },
    { 1.9999999999999999e-11, 2.0795139243800000e+00, 2.3373174634397859e+05 },
    { 1.9999999999999999e-11, 4.9016106129599999e+00, 1.0201841617342478e+05 },
    { 1.9999999999999999e-11, 1.7225881130900000e+00, 5.4824261108621402e+04 },
    { 1.9999999999999999e-11, 5.5814555148000000e-01, 5.0910238804522000e+04 },
    { 1.9999999999999999e-11, 1.5189484425999999e+00, 5.1859414413491788e+04 },
    { 1.9999999999999999e-11, 5.7437116939999999e-01, 2.5131613985603592e+04 },
    { 1.9999999999999999e-11, 2.8279411525599998e+00, 2.4609036567509800e+04 },
    { 1.9999999999999999e-11, 6.2807814904399999e+00, 2.5863558345872290e+04 },
    { 1.9999999999999999e-11, 2.8359270382999999e+00, 5.3757765631431394e+04 },
    { 1.9999999999999999e-11, 4.0368860509499997e+00, 5.2492198152804987e+04 },
    { 1.9999999999999999e-11, 3.9380237664600002e+00, 2.6076857441310302e+04 },
    { 1.9999999999999999e-11, 4.8671939183399999e+00, 2.6098948841838101e+04 },
    { 1.9999999999999999e-11, 3.3799564377900002e+00, 1.7633224614102798e+05 },
    { 1.9999999999999999e-11, 3.6069528983799999e+00, 7.8896493164035783e+04 },
    { 1.9999999999999999e-11, 4.9791220180900000e+00, 2.6411408558231600e+04 },
    { 1.9999999999999999e-11, 2.8549949577999998e+00, 1.8165972224941078e+05 },
    { 1.9999999999999999e-11, 1.8759889443900000e+00, 5.2483363904117992e+04 },
    { 1.9999999999999999e-11, 2.9658564283900000e+00, 1.0381480805420138e+05 },
    { 1.9999999999999999e-11, 5.4143324151599996e+00, 5.1226630674178588e+04 },
    { 1.9999999999999999e-11, 8.7777120540000003e-02, 2.3976252828196601e+04 },
    { 1.9999999999999999e-11, 5.4548139520300003e+00, 1.4964226895951436e+05 },
    { 1.9999999999999999e-11, 5.1488848648200003e+00, 2.3962025734194991e+04 },
    { 1.9999999999999999e-11, 3.3754880640499998e+00, 9.4329775286197386e+04 },
    { 1.9999999999999999e-11, 4.0784320492899999e+00, 2.6713573333886590e+04 },
    { 1.9999999999999999e-11, 4.7267856355099998e+00, 2.5462232949261801e+04 },
    { 1.9999999999999999e-11, 2.5459509775400000e+00, 2.3919142659291600e+04 },
    { 1.9999999999999999e-11, 6.2592667072600001e+00, 2.8256663623856792e+04 },
    { 1.9999999999999999e-11, 3.2047528054700001e+00, 8.0174908907939782e+04 },
    { 1.9999999999999999e-11, 1.8280343788100000e+00, 1.2511203959948818e+05 },
    { 1.9999999999999999e-11, 4.0492313905700001e+00, 2.7566769715638591e+04 },
    { 1.9999999999999999e-11, 2.0694684105000000e-01, 2.5440892308259390e+04 },
    { 1.9999999999999999e-11, 1.2452737797000001e-01, 1.4287155835826878e+05 },
    { 9.9999999999999994e-12, 3.0689186220500000e+00, 2.7351060504152389e+04 },
    { 9.9999999999999994e-12, 5.7362990627499997e+00, 2.4824745778995999e+04 },
    { 9.9999999999999994e-12, 3.0980133567800001e+00, 7.8697421162599392e+04 },
    { 9.9999999999999994e-12, 1.6782543451200000e+00, 5.1596118696149599e+04 },
    { 9.9999999999999994e-12, 2.5049193586900000e+00, 1.5308484390447979e+05 },
    { 9.9999999999999994e-12, 1.4677910499500000e+00, 2.3134855332920235e+05 },
    { 9.9999999999999994e-12, 3.8795127201100001e+00, 1.1254588789950538e+05 },
    { 9.9999999999999994e-12, 2.6287409255599998e+00, 1.8985399758261937e+05 },
    { 9.9999999999999994e-12, 3.1462467487899999e+00, 5.4879422437824003e+04 },
    { 9.9999999999999994e-12, 3.2211189604800001e+00, 5.2509662223917803e+04 },
    { 9.9999999999999994e-12, 1.2209590899999999e-02, 2.8199553454951791e+04 },
    { 9.9999999999999994e-12, 1.9294431104800001e+00, 2.6933985976325399e+04 },
    { 9.9999999999999994e-12, 5.9258926714000004e-01, 2.5241820306823000e+04 },
    { 9.9999999999999994e-12, 3.3057623864100001e+00, 2.4156183887541335e+05 },
    { 9.9999999999999994e-12, 5.2855786703199996e+00, 5.0380547839427389e+04 },
    { 9.9999999999999994e-12, 6.2761403210899998e+00, 2.1547393573383917e+05 },
    { 9.9999999999999994e-12, 2.0613619393999999e-01, 1.0626281204951399e+05 },
    { 9.9999999999999994e-12, 2.4097837672400000e+00, 4.9953949648551388e+04 },
    { 9.9999999999999994e-12, 5.4128689029199997e+00, 1.5546092337225520e+05 },
    { 9.9999999999999994e-12, 4.1253746799200002e+00, 1.6329812945069079e+05 },
    { 9.9999999999999994e-12, 2.8024185561500001e+00, 2.6926872429324590e+04 },
    { 9.9999999999999994e-12, 6.0027991286500004e+00, 2.5248933853823801e+04 },
    { 9.9999999999999994e-12, 4.5378660448400003e+00, 2.0526065018762814e+05 },
    { 9.9999999999999994e-12, 5.3598204209900002e+00, 1.2810631931499895e+05 },
    { 9.9999999999999994e-12, 6.2288984228700004e+00, 4.8997660492580799e+04 },
    { 9.9999999999999994e-12, 3.6499101987699998e+00, 2.7005833427555990e+04 },
    { 9.9999999999999994e-12, 5.1553074860299999e+00, 2.5169972855592401e+04 },
    { 9.9999999999999994e-12, 3.4263612835499999e+00, 5.0689826162083191e+04 },
    { 9.9999999999999994e-12, 4.0249468992899997e+00, 1.1223170171963578e+05 },
    { 9.9999999999999994e-12, 3.1787939754700001e+00, 9.0989162859497592e+04 },
    { 9.9999999999999994e-12, 1.2352160841000000e-01, 1.4398093691036216e+05 },
    { 9.9999999999999994e-12, 2.5039398566700002e+00, 7.6152059111344992e+04 },
    { 9.9999999999999994e-12, 6.2592290550899996e+00, 9.1805130627213788e+04 },
    { 9.9999999999999994e-12, 6.1269275703900004e+00, 8.4307938006097989e+04 },
    { 9.9999999999999994e-12, 3.4679274654199999e+00, 1.3831960486120995e+05 },
    { 9.9999999999999994e-12, 5.6984414891500004e+00, 9.2741060607922584e+04 },
    { 9.9999999999999994e-12, 4.2496062711200002e+00, 2.4072921469776400e+04 },
    { 9.9999999999999994e-12, 4.5556114136800003e+00, 2.8102884813371998e+04 },
    { 9.9999999999999994e-12, 6.1649339335000004e+00, 2.4395737472071800e+04 },
    { 9.9999999999999994e-12, 5.2288893948300004e+00, 2.4844415539895788e+04 },
    { 9.9999999999999994e-12, 3.5763282899700002e+00, 2.7331390743252599e+04 },
    { 9.9999999999999994e-12, 3.8942177437999997e-01, 1.5440865498906677e+05 },
    { 9.9999999999999994e-12, 2.7835710711799999e+00, 1.9393798608932379e+05 },
    { 9.9999999999999994e-12, 4.0555215291500000e+00, 5.8220034864523797e+04 },
    { 9.9999999999999994e-12, 8.1202957379999996e-01, 9.7670387712897180e+04 },
    { 9.9999999999999994e-12, 2.5590805700699999e+00, 6.0370081616356991e+04 },
    { 9.9999999999999994e-12, 3.0609026300800002e+00, 2.4356780788641601e+04 },
    { 9.9999999999999994e-12, 5.7443150547200004e+00, 2.7819025494506801e+04 },
    { 9.9999999999999994e-12, 3.7740051064200002e+00, 2.5764397724916791e+04 },
    { 9.9999999999999994e-12, 4.1418022337000000e+00, 1.5119994274106238e+05 },
    { 9.9999999999999994e-12, 5.0525160050800002e+00, 2.0965882487420217e+05 },
    { 9.9999999999999994e-12, 3.8435679004600001e+00, 2.3704710126797989e+04 },
    { 9.9999999999999994e-12, 4.9616497843399996e+00, 2.8471096156350399e+04 },
    { 9.9999999999999994e-12, 2.6444294835999999e-01, 2.3439448316101189e+04 },
    { 9.9999999999999994e-12, 3.6490559130100002e+00, 5.1439929769616589e+04 },
    { 9.9999999999999994e-12, 2.4556983156799999e+00, 2.6177388889420188e+04 },
    { 9.9999999999999994e-12, 6.6334061939999994e-02, 2.5998417393728199e+04 },
    { 9.9999999999999994e-12, 4.1633866770000000e-01, 2.3368174972207437e+05 },
    { 9.9999999999999994e-12, 1.6023854890000000e+00, 2.1650388933200611e+04 },
    { 9.9999999999999994e-12, 9.1964688861999999e-01, 3.0525417349947791e+04 },
    { 9.9999999999999994e-12, 4.3279415227299998e+00, 1.5985555450572536e+05 },
    { 9.9999999999999994e-12, 2.5797591780900002e+00, 2.4203001978156801e+04 },
    { 9.9999999999999994e-12, 6.2254585067099999e+00, 2.7972804304991590e+04 },
    { 9.9999999999999994e-12, 1.6990934238299999e+00, 2.6094531700474210e+04 },
    { 9.9999999999999994e-12, 2.2511838383999998e+00, 1.0419783375581198e+05 },
    { 9.9999999999999994e-12, 1.2894841937699999e+00, 5.0167248743989388e+04 },
    { 9.9999999999999994e-12, 3.1695316445900001e+00, 1.9959931813847594e+05 },
    { 9.9999999999999994e-12, 6.2609314567499998e+00, 7.8413272621857192e+04 },
    { 9.9999999999999994e-12, 4.7108305727999999e+00, 1.5653130018480320e+05 },
    { 9.9999999999999994e-12, 5.3455300707399998e+00, 1.4742351185532758e+05 },
    { 9.9999999999999994e-12, 2.6948650077999998e+00, 2.4722317092456558e+05 },
    { 9.9999999999999994e-12, 2.9425882546900000e+00, 2.5151973160865389e+04 },
    { 9.9999999999999994e-12, 5.8626294301100002e+00, 2.7023833122282998e+04 },
    { 9.9999999999999994e-12, 2.3938218406599998e+00, 1.8367470392120857e+05 },
    { 9.9999999999999994e-12, 3.4991867234499998e+00, 9.7466462676365962e+04 },
    { 9.9999999999999994e-12, 4.4984104634900000e+00, 2.6126036177212001e+04 },
    { 9.9999999999999994e-12, 4.3068072213099997e+00, 2.6049770105936401e+04 },
    { 9.9999999999999994e-12, 3.7781273073600001e+00, 2.7388729959779190e+04 },
    { 9.9999999999999994e-12, 5.0270903774400004e+00, 2.4787076323369201e+04 },
    { 9.9999999999999994e-12, 1.6003886949000001e-01, 1.3235071519108818e+05 },
    { 9.9999999999999994e-12, 5.2292811656899998e+00, 1.5547515046625677e+05 },
    { 9.9999999999999994e-12, 3.6025992359900001e+00, 2.4602612434870989e+04 },
    { 9.9999999999999994e-12, 5.2026184488100000e+00, 2.7573193848277399e+04 },
    { 9.9999999999999994e-12, 5.1185343510500001e+00, 1.9848993958638259e+05 },
    { 9.9999999999999994e-12, 3.5642946947700000e+00, 1.5653453239644598e+05 },
    { 9.9999999999999994e-12, 5.2335445033200001e+00, 1.3000580396999417e+05 },
    { 9.9999999999999994e-12, 2.1874911262299999e+00, 7.7039891583943958e+04 },
    { 9.9999999999999994e-12, 4.7095235038499998e+00, 1.2921569786709236e+05 },
    { 9.9999999999999994e-12, 3.6839628790400001e+00, 2.5923467230984788e+04 },
    { 9.9999999999999994e-12, 5.1212548057599996e+00, 2.6252339052163610e+04 },
    { 9.9999999999999994e-12, 3.3955422684299998e+00, 7.3891833057948388e+04 },
    { 9.9999999999999994e-12, 5.9175746460500003e+00, 1.2606763934109679e+05 },
    { 9.9999999999999994e-12, 8.3086794433000000e-01, 3.0171891648278601e+04 },
    { 9.9999999999999994e-12, 1.6911644332900000e+00, 2.2003914634869801e+04 },
};

static const VSOP87Term VSOP87A_MERCURY_Y2[299] = {
    { 4.6121570000000000e-05, 0.0000000000000000e+00, 0.0000000000000000e+00 },
    { 1.5756700000000001e-05, 2.8117273334899999e+00, 5.2175806283148399e+04 },
    { 9.2789599999999993e-06, 5.8536876912200002e+00, 7.8263709424722590e+04 },
    { 6.7025499999999999e-06, 9.0964509090000001e-01, 2.6087903141574199e+04 },
    { 3.7374399999999998e-06, 2.6227927569900000e+00, 1.0435161256629678e+05 },
    { 1.3138900000000000e-06, 5.6751905220800003e+00, 1.3043951570787099e+05 },
    { 4.3187999999999997e-07, 2.4440263183000002e+00, 1.5652741884944518e+05 },
    { 1.3650000000000001e-07, 5.4957356935900004e+00, 1.8261532199101939e+05 },
    { 4.2060000000000001e-08, 2.2640343792600000e+00, 2.0870322513259359e+05 },
    { 2.9840000000000001e-08, 6.0606783712799999e+00, 2.4978524589480799e+04 },
    { 2.6860000000000000e-08, 2.7636556225000000e+00, 2.7197281693667599e+04 },
    { 1.2740000000000000e-08, 5.3153985252300000e+00, 2.3479112827416777e+05 },
    { 1.0270000000000000e-08, 2.8430061478600002e+00, 5.1066427731055002e+04 },
    { 3.8099999999999999e-09, 2.0833371981100002e+00, 2.6087903141574195e+05 },
    { 3.5600000000000001e-09, 5.9014451585599996e+00, 7.7154330872629187e+04 },
    { 3.5199999999999998e-09, 5.8250781234100000e+00, 5.3285184835241802e+04 },
    { 3.3499999999999998e-09, 3.3753731938899998e+00, 1.1093785520934000e+03 },
    { 1.9700000000000000e-09, 2.5890144792099998e+00, 2.5132303399965600e+04 },
    { 1.7800000000000001e-09, 5.8387963476999998e+00, 1.5890728952838001e+03 },
    { 1.7400000000000000e-09, 6.1997051879400003e+00, 2.7043502883182799e+04 },
    { 1.6500000000000001e-09, 4.7796854860800000e+00, 1.0593819301891999e+03 },
    { 1.5199999999999999e-09, 3.4676629939199999e+00, 2.0426571092422000e+04 },
    { 1.3800000000000000e-09, 3.2050955419999999e-01, 2.1535949644515400e+04 },
    { 1.2600000000000000e-09, 6.1468204679799996e+00, 2.3869146037387400e+04 },
    { 1.2500000000000000e-09, 4.3248048874700000e+00, 2.6617594106668799e+04 },
    { 1.2100000000000000e-09, 2.6734268902500000e+00, 1.0324223401420339e+05 },
    { 1.1900000000000000e-09, 2.6940576275000000e-01, 4.6514474233996203e+04 },
    { 1.1700000000000001e-09, 4.5160858187799997e+00, 2.5558212176479599e+04 },
    { 1.1300000000000000e-09, 5.1347950609900002e+00, 2.8696693455731618e+05 },
    { 1.0800000000000000e-09, 1.0033754720700001e+00, 1.4765239043269799e+04 },
    { 1.0700000000000000e-09, 2.6750176851499998e+00, 2.8306660245760992e+04 },
    { 1.0600000000000001e-09, 1.3701953611800000e+00, 2.7147285071763388e+04 },
    { 1.0300000000000000e-09, 5.4571954264600002e+00, 3.1749235190726398e+04 },
    { 9.5000000000000003e-10, 2.1704782721800000e+00, 4.5519534970588002e+03 },
    { 8.6000000000000003e-10, 3.5819201266400000e+00, 1.9317192540328600e+04 },
    { 7.7000000000000003e-10, 2.2294716973000001e-01, 2.2645328196608789e+04 },
    { 6.6999999999999996e-10, 4.8163605817699997e+00, 9.1039069941176003e+03 },
    { 6.6000000000000000e-10, 2.3465480141900001e+00, 3.4425749449653999e+03 },
    { 6.6000000000000000e-10, 5.3696239593100001e+00, 5.1220206541539788e+04 },
    { 6.0000000000000000e-10, 2.2663191166000001e+00, 3.0639856638632999e+04 },
    { 6.0000000000000000e-10, 9.9475775846000003e-01, 5.2705497248242988e+04 },
    { 5.9000000000000003e-10, 3.3246329917700002e+00, 7.2602377375570388e+04 },
    { 5.9000000000000003e-10, 2.9842590747000002e+00, 5.0586733387864588e+04 },
    { 5.9000000000000003e-10, 1.5538131192600000e+00, 3.7410567239878597e+04 },
    { 5.7999999999999996e-10, 5.2631530241000002e+00, 3.2858613742819791e+04 },
    { 5.1999999999999996e-10, 3.7839357337099999e+00, 6.3989728631399998e+02 },
    { 5.1999999999999996e-10, 4.6087243356000002e-01, 9.5559974160859997e+02 },
    { 5.1000000000000000e-10, 4.0871547868500002e+00, 5.1116424352959199e+04 },
    { 4.8999999999999996e-10, 2.4593627785000001e+00, 2.7676976036857999e+04 },
    { 4.6000000000000001e-10, 2.3622661187900000e+00, 2.9530478086539599e+04 },
    { 4.5000000000000000e-10, 2.4885062109999998e+00, 2.6202342430259410e+04 },
    { 4.5000000000000000e-10, 2.9189986490000000e-02, 2.5973463852888959e+04 },
    { 4.5000000000000000e-10, 4.3836772154699997e+00, 5.3235188213337591e+04 },
    { 4.0000000000000001e-10, 1.4285145391300000e+00, 2.6301202237012199e+04 },
    { 4.0000000000000001e-10, 5.7253862289399997e+00, 1.2933013715577759e+05 },
    { 3.9000000000000000e-10, 1.1014721602299999e+00, 2.5874604046136199e+04 },
    { 3.9000000000000000e-10, 4.0844997664599996e+00, 4.0853142184844000e+04 },
    { 3.9000000000000000e-10, 2.7151028645699999e+00, 5.3131406024757001e+04 },
    { 3.7999999999999998e-10, 3.2618575720899998e+00, 2.2187571041868000e+03 },
    { 3.7000000000000001e-10, 2.4047079065999999e-01, 2.1187638603783998e+03 },
    { 3.6000000000000000e-10, 5.4055675234800002e+00, 5.2969096509459996e+02 },
    { 3.4000000000000001e-10, 5.3477094353999997e+00, 5.6727759780207198e+04 },
    { 3.3000000000000000e-10, 1.8931733564300000e+00, 3.1305483769889036e+05 },
    { 3.3000000000000000e-10, 2.9200418206399998e+00, 4.9957049178961599e+04 },
    { 3.1999999999999998e-10, 5.8142816574699996e+00, 6.7707106012455997e+03 },
    { 3.0000000000000000e-10, 4.0543761564100000e+00, 4.3071899289030800e+04 },
    { 3.0000000000000000e-10, 4.4155155116000000e-01, 4.7623852786089599e+04 },
    { 2.8000000000000002e-10, 5.7291484464200000e+00, 5.4394563387335191e+04 },
    { 2.5000000000000002e-10, 9.7055997160000004e-02, 9.8690280517144580e+04 },
    { 2.4000000000000000e-10, 3.9466405189399998e+00, 7.8793400389817180e+04 },
    { 2.4000000000000000e-10, 1.9916564680500000e+00, 7.7308109683113988e+04 },
    { 2.4000000000000000e-10, 5.3285863599700001e+00, 1.8849227549974199e+04 },
    { 2.3000000000000001e-10, 4.2292285153800000e+00, 1.6983996147456601e+04 },
    { 2.1999999999999999e-10, 3.5963548929000000e-01, 4.5405095681902800e+04 },
    { 2.1999999999999999e-10, 1.1193372753699999e+00, 1.3655860491176400e+04 },
    { 1.8000000000000000e-10, 1.7787797763999999e-01, 5.2156136522248598e+04 },
    { 1.8000000000000000e-10, 1.1610585721300000e+00, 7.9323091354911783e+04 },
    { 1.8000000000000000e-10, 8.3099200036999998e-01, 1.2566151699982800e+04 },
    { 1.8000000000000000e-10, 4.5463331729599998e+00, 5.2389105378586399e+04 },
    { 1.8000000000000000e-10, 5.0267254145800004e+00, 5.1535908996834391e+04 },
    { 1.8000000000000000e-10, 1.6559579870800001e+00, 5.1646115318053788e+04 },
    { 1.5999999999999999e-10, 9.5131923110000002e-02, 2.3754706748702189e+04 },
    { 1.5999999999999999e-10, 1.9943825023099999e+00, 5.8946516884393990e+04 },
    { 1.5999999999999999e-10, 2.8176557350200002e+00, 2.1329909543799999e+02 },
    { 1.5999999999999999e-10, 2.0896395045200000e+00, 1.1322664098304400e+04 },
    { 1.5999999999999999e-10, 3.4110037631500001e+00, 3.3326578733174203e+04 },
    { 1.5999999999999999e-10, 5.3999108154800002e+00, 5.5618381228113802e+04 },
    { 1.5000000000000000e-10, 3.8366084204800002e+00, 3.8654054841556987e+04 },
    { 1.5000000000000000e-10, 2.0309619508400001e+00, 5.6613320491521999e+03 },
    { 1.5000000000000000e-10, 2.2694790546999999e-01, 4.8733231338182988e+04 },
    { 1.5000000000000000e-10, 5.3650092022200004e+00, 4.2659819087599999e+02 },
    { 1.5000000000000000e-10, 2.1090210123899999e+00, 8.2815662921781390e+04 },
    { 1.5000000000000000e-10, 2.6122650165699999e+00, 7.9373087976815994e+04 },
    { 1.5000000000000000e-10, 5.3699049019200000e+00, 5.2195476044048191e+04 },
    { 1.4000000000000001e-10, 2.7861381497000002e+00, 8.5319638175199998e+02 },
    { 1.4000000000000001e-10, 3.5226130272099998e+00, 5.2179687618506403e+04 },
    { 1.4000000000000001e-10, 8.7917385568999995e-01, 6.6941045326418185e+04 },
    { 1.4000000000000001e-10, 5.4178756074500001e+00, 5.3764879178432202e+04 },
    { 1.4000000000000001e-10, 2.0737760463999999e+00, 1.1001294461544899e+05 },
    { 1.4000000000000001e-10, 1.6128844869400001e+00, 3.6301188687785187e+04 },
    { 1.4000000000000001e-10, 5.7769767887000001e-01, 2.6727800427888200e+04 },
    { 1.2999999999999999e-10, 2.4216932787199998e+00, 2.8421099534446199e+04 },
    { 1.2999999999999999e-10, 5.7738784433800001e+00, 7.6674636529438780e+04 },
    { 1.2999999999999999e-10, 5.4016718691100003e+00, 8.3925041473874793e+04 },
    { 1.2999999999999999e-10, 2.4978656814700000e+00, 1.5541804029735178e+05 },
    { 1.2999999999999999e-10, 2.0170700079500001e+00, 5.2171924947790401e+04 },
    { 1.2000000000000000e-10, 2.2871335175700001e+00, 5.0057042422769999e+04 },
    { 1.2000000000000000e-10, 1.5543831808199999e+00, 2.6514501332450200e+04 },
    { 1.0999999999999999e-10, 4.6244976657599999e+00, 6.2389091829359393e+04 },
    { 1.0999999999999999e-10, 1.3701222008000000e+00, 3.8519945791972001e+04 },
    { 1.0000000000000000e-10, 5.5571872887399998e+00, 5.2290245571833613e+04 },
    { 1.0000000000000000e-10, 3.1543443704200000e+00, 1.2477818365871879e+05 },
    { 1.0000000000000000e-10, 1.7800756005300000e+00, 2.5028521211384999e+04 },
    { 8.9999999999999999e-11, 5.9773107639200003e+00, 7.6044952320535798e+04 },
    { 8.9999999999999999e-11, 1.5260873066800000e+00, 3.5191810135691798e+04 },
    { 8.9999999999999999e-11, 4.9439976337199996e+00, 3.3914274084046460e+05 },
    { 8.9999999999999999e-11, 3.2329781475099999e+00, 7.8244039663822798e+04 },
    { 8.9999999999999999e-11, 1.6517889303500000e+00, 3.9609654583165597e+04 },
    { 8.9999999999999999e-11, 1.9937037569900000e+00, 1.2432042650397800e+04 },
    { 8.9999999999999999e-11, 6.4203429746999996e-01, 1.0488130353139139e+05 },
    { 8.9999999999999999e-11, 4.9317485298900001e+00, 7.9945284420241996e+03 },
    { 8.9999999999999999e-11, 4.6982631854500001e+00, 6.3498470381452789e+04 },
    { 8.9999999999999999e-11, 1.0697398508600000e+00, 1.5874617595363199e+04 },
    { 8.9999999999999999e-11, 1.7814226293900000e+00, 4.4937130691548402e+04 },
    { 7.9999999999999995e-11, 3.0840678177899998e+00, 5.2061366994463169e+04 },
    { 7.9999999999999995e-11, 2.4690933581499999e+00, 2.3331963928720002e+03 },
    { 7.9999999999999995e-11, 5.0916059328100003e+00, 1.3610084775702318e+05 },
    { 7.9999999999999995e-11, 4.9597780639200000e+00, 1.0339601282468818e+05 },
    { 7.9999999999999995e-11, 3.1751122278300001e+00, 2.8206667001952599e+04 },
    { 7.9999999999999995e-11, 5.3079775970299998e+00, 7.9219309166331193e+04 },
    { 7.9999999999999995e-11, 2.4325315491600001e+00, 5.2182919830149200e+04 },
    { 7.0000000000000004e-11, 3.1495467734000000e+00, 5.2168692736147590e+04 },
    { 7.0000000000000004e-11, 5.5559122997999999e-01, 6.4741957983131193e+04 },
    { 7.0000000000000004e-11, 7.6613463584999997e-01, 6.9159802430604992e+04 },
    { 7.0000000000000004e-11, 3.1047071847000002e-01, 7.8267590760080580e+04 },
    { 7.0000000000000004e-11, 3.6648871505999998e+00, 1.8207813988235212e+04 },
    { 7.0000000000000004e-11, 4.5583638614200002e+00, 5.1962507187710398e+04 },
    { 7.0000000000000004e-11, 3.9425611488499999e+00, 7.2386755916000002e+03 },
    { 7.0000000000000004e-11, 2.1526298564499999e+00, 7.8283379185622383e+04 },
    { 7.0000000000000004e-11, 3.4148353450400002e+00, 7.1492998823476984e+04 },
    { 6.0000000000000000e-11, 4.2018621214899996e+00, 1.0541099449648599e+05 },
    { 6.0000000000000000e-11, 1.4335227528400001e+00, 8.8476994970933592e+04 },
    { 6.0000000000000000e-11, 8.2384473986999995e-01, 1.2938013377768178e+05 },
    { 6.0000000000000000e-11, 4.6910766394800003e+00, 5.2602404474024399e+04 },
    { 6.0000000000000000e-11, 9.5309123619000002e-01, 6.8050423878511589e+04 },
    { 6.0000000000000000e-11, 1.3596422678600000e+00, 7.8477008520160583e+04 },
    { 6.0000000000000000e-11, 4.9083223705999997e+00, 7.7734018459627987e+04 },
    { 6.0000000000000000e-11, 5.0932345187900001e+00, 7.8259828089364586e+04 },
    { 6.0000000000000000e-11, 3.8727326279200001e+00, 1.0329223063610759e+05 },
    { 5.0000000000000002e-11, 4.2253819726700002e+00, 5.7837138332300601e+04 },
    { 5.0000000000000002e-11, 3.2774845300600002e+00, 7.4821134479757180e+04 },
    { 5.0000000000000002e-11, 4.7992850848400002e+00, 1.3521751441591399e+04 },
    { 5.0000000000000002e-11, 2.0025280408300001e+00, 2.5661304950698199e+04 },
    { 5.0000000000000002e-11, 2.3042309663100000e+00, 1.0546099111839019e+05 },
    { 5.0000000000000002e-11, 3.9572579419900000e+00, 9.4138327020085781e+04 },
    { 5.0000000000000002e-11, 6.0120300917599998e+00, 5.1322609901396390e+04 },
    { 5.0000000000000002e-11, 5.1575480067499999e+00, 1.0890356606335558e+05 },
    { 5.0000000000000002e-11, 2.1696221945400000e+00, 8.1706284369687986e+04 },
    { 5.0000000000000002e-11, 6.0316057690799996e+00, 7.8256595877721789e+04 },
    { 5.0000000000000002e-11, 3.9188910333600000e+00, 9.3028948467992392e+04 },
    { 5.0000000000000002e-11, 6.2445016508400002e+00, 5.9414481874748402e+04 },
    { 5.0000000000000002e-11, 5.4724047213200002e+00, 7.8270822971723392e+04 },
    { 3.9999999999999998e-11, 8.3457275877000003e-01, 3.2370978991565600e+04 },
    { 3.9999999999999998e-11, 5.0176136198599997e+00, 3.3967992294913187e+04 },
    { 3.9999999999999998e-11, 2.5561352091899998e+00, 2.9416038797854391e+04 },
    { 3.9999999999999998e-11, 6.2490824756099999e+00, 2.2759767485294011e+04 },
    { 3.9999999999999998e-11, 5.0931430785899998e+00, 7.6144945564344191e+04 },
    { 3.9999999999999998e-11, 5.5471885020700000e+00, 1.8150594343892598e+05 },
    { 3.9999999999999998e-11, 3.6659155516899999e+00, 5.1749208092272391e+04 },
    { 3.9999999999999998e-11, 5.4353583230800000e+00, 7.7204327494533383e+04 },
    { 3.9999999999999998e-11, 3.8991602512300001e+00, 4.4181277841124189e+04 },
    { 3.9999999999999998e-11, 6.2290255425499996e+00, 2.4498830246290399e+04 },
    { 3.9999999999999998e-11, 3.9211678357799999e+00, 1.5546803691925600e+05 },
    { 3.9999999999999998e-11, 5.4871284214199996e+00, 5.4509002676020398e+04 },
    { 3.9999999999999998e-11, 4.2746359877900000e+00, 1.8093374699549990e+04 },
    { 3.9999999999999998e-11, 2.1233541059399998e+00, 7.9852782320006387e+04 },
    { 3.9999999999999998e-11, 4.1780137509099999e+00, 3.9743763632750597e+04 },
    { 3.9999999999999998e-11, 5.7084030962699996e+00, 7.8800891533389904e+03 },
    { 3.9999999999999998e-11, 2.3856445950900000e+00, 8.0482466528909383e+04 },
    { 3.9999999999999998e-11, 2.5285083470599998e+00, 2.7311720982352810e+04 },
    { 3.9999999999999998e-11, 6.2767093377399998e+00, 2.4864085300795588e+04 },
    { 3.9999999999999998e-11, 1.2689387419300000e+00, 1.0382192160120218e+05 },
    { 3.0000000000000000e-11, 1.8524446536100001e+00, 1.6218875089859738e+05 },
    { 3.0000000000000000e-11, 4.7783642265599999e+00, 4.1962520736937397e+04 },
    { 3.0000000000000000e-11, 3.9806541191000000e-01, 9.9799659069237983e+04 },
    { 3.0000000000000000e-11, 6.2755669398000000e+00, 1.0433194280539699e+05 },
    { 3.0000000000000000e-11, 1.3506967835799999e+00, 1.9804827291582798e+04 },
    { 3.0000000000000000e-11, 1.8185584157900001e+00, 7.7623812138408583e+04 },
    { 3.0000000000000000e-11, 7.0573582046000005e-01, 1.2022623016165999e+05 },
    { 3.0000000000000000e-11, 3.3251499326700000e+00, 1.2588756221081219e+05 },
    { 3.0000000000000000e-11, 3.7583021840500002e+00, 5.8458882133139792e+04 },
    { 3.0000000000000000e-11, 2.7476351315100001e+00, 1.0213285546210999e+05 },
    { 3.0000000000000000e-11, 1.6786406733000001e+00, 1.2948391596626239e+05 },
    { 3.0000000000000000e-11, 6.2027775046000002e+00, 1.5086608680029298e+05 },
    { 3.0000000000000000e-11, 3.5640679126600001e+00, 9.0829861124705378e+04 },
    { 3.0000000000000000e-11, 3.6327215813199998e+00, 1.3096920667296558e+05 },
    { 3.0000000000000000e-11, 5.9104772837899997e+00, 2.6941099523326200e+04 },
    { 3.0000000000000000e-11, 4.6518484318100004e+00, 7.1025033833122594e+04 },
    { 3.0000000000000000e-11, 3.3517036683299999e+00, 1.0435549390165479e+05 },
    { 3.0000000000000000e-11, 4.2269103860200001e+00, 6.4607848933546193e+04 },
    { 3.0000000000000000e-11, 3.6719491563000002e+00, 5.2815703569462399e+04 },
    { 1.9999999999999999e-11, 3.2361033653900000e+00, 3.3281356562801898e+03 },
    { 1.9999999999999999e-11, 9.5061008775000000e-01, 2.6484548254729998e+03 },
    { 1.9999999999999999e-11, 4.4814892967500004e+00, 1.1456489811250778e+05 },
    { 1.9999999999999999e-11, 2.3277694165999998e+00, 7.8378148713407805e+04 },
    { 1.9999999999999999e-11, 5.1492344372199996e+00, 1.0213285546211000e+04 },
    { 1.9999999999999999e-11, 1.7165097582800000e+00, 3.6523064398203877e+05 },
    { 1.9999999999999999e-11, 1.2734534845100001e+00, 7.7197213947532582e+04 },
    { 1.9999999999999999e-11, 1.4841710005099999e+00, 7.8690307615598591e+04 },
    { 1.9999999999999999e-11, 2.3618795643800001e+00, 6.2830758499914000e+03 },
    { 1.9999999999999999e-11, 6.1230382658600000e+00, 2.6080789594573391e+04 },
    { 1.9999999999999999e-11, 5.1912070842100002e+00, 1.0437128232719658e+05 },
    { 1.9999999999999999e-11, 7.8853413649000004e-01, 7.7211441041534184e+04 },
    { 1.9999999999999999e-11, 2.7267030377100001e+00, 1.0434449901929598e+05 },
    { 1.9999999999999999e-11, 4.6773937414400004e+00, 7.3711755927663791e+04 },
    { 1.9999999999999999e-11, 1.5323787274800000e+00, 2.5021407664384202e+04 },
    { 1.9999999999999999e-11, 2.1323108535799999e+00, 7.8050410329284583e+04 },
    { 1.9999999999999999e-11, 9.1172072872999999e-01, 2.6724899413598399e+04 },
    { 1.9999999999999999e-11, 1.6103116488900000e+00, 2.5450906869549999e+04 },
    { 1.9999999999999999e-11, 1.8515603002000000e-01, 9.7580901965051191e+04 },
    { 1.9999999999999999e-11, 1.2446730774600001e+00, 2.5035634758385801e+04 },
    { 1.9999999999999999e-11, 4.5471804611900000e+00, 6.1279713277265997e+04 },
    { 1.9999999999999999e-11, 9.9883736207999996e-01, 1.3149889763806018e+05 },
    { 1.9999999999999999e-11, 6.1457982475800002e+00, 7.8149270136037361e+04 },
    { 1.9999999999999999e-11, 5.2219482962799999e+00, 1.0779418751126219e+05 },
    { 1.9999999999999999e-11, 4.4456863286499999e+00, 1.0456491166173479e+05 },
    { 1.9999999999999999e-11, 4.3885889697000002e-01, 4.4295717129809404e+04 },
    { 1.9999999999999999e-11, 2.2177592643000001e+00, 1.0435872611329758e+05 },
    { 1.9999999999999999e-11, 6.0124412607000002e+00, 2.6095016688575000e+04 },
    { 1.9999999999999999e-11, 1.9724711944500000e+00, 2.5448005855260191e+04 },
    { 1.9999999999999999e-11, 1.8442507208900001e+00, 1.0434773123093879e+05 },
    { 1.9999999999999999e-11, 1.9297878337500001e+00, 1.3499146920492980e+05 },
    { 1.9999999999999999e-11, 6.1576207517099997e+00, 5.4294570143526791e+04 },
    { 1.9999999999999999e-11, 4.2865190092300001e+00, 2.7154398618764200e+04 },
    { 1.9999999999999999e-11, 4.0329992386800004e+00, 1.2990982474277639e+05 },
    { 9.9999999999999994e-12, 6.4625947699999998e-03, 1.5197546535238638e+05 },
    { 9.9999999999999994e-12, 4.9649327891799997e+00, 1.3154889425996438e+05 },
    { 9.9999999999999994e-12, 4.8541117107999998e+00, 1.1787336400788819e+05 },
    { 9.9999999999999994e-12, 7.0037814448000002e-01, 1.8155594006083018e+05 },
    { 9.9999999999999994e-12, 4.7688755382499997e+00, 2.7140171524762591e+04 },
    { 9.9999999999999994e-12, 2.2066569273400001e+00, 7.1980633574731182e+04 },
    { 9.9999999999999994e-12, 3.2376121010999998e-01, 8.4546785274713984e+04 },
    { 9.9999999999999994e-12, 2.0226995436099999e+00, 1.0530721230790539e+05 },
    { 9.9999999999999994e-12, 5.6521561493799997e+00, 5.5503941939428587e+04 },
    { 9.9999999999999994e-12, 4.3272482621200004e+00, 1.0328511708910679e+05 },
    { 9.9999999999999994e-12, 5.2983300825199997e+00, 9.8068536716305389e+04 },
    { 9.9999999999999994e-12, 1.8572568831000000e-01, 4.9842609890276392e+04 },
    { 9.9999999999999994e-12, 4.9012238040299998e+00, 1.8827665404017159e+05 },
    { 9.9999999999999994e-12, 3.7634180175999998e+00, 1.4631413330323418e+05 },
    { 9.9999999999999994e-12, 1.3091693578700001e+00, 3.9629324344065390e+04 },
    { 9.9999999999999994e-12, 1.2128630197500001e+00, 1.2546481939083000e+04 },
    { 9.9999999999999994e-12, 3.3914647270899998e+00, 2.9428515568274001e+04 },
    { 9.9999999999999994e-12, 5.4137529577099999e+00, 2.2747290714874402e+04 },
    { 9.9999999999999994e-12, 3.0177971781299999e+00, 4.8847670626868203e+04 },
    { 9.9999999999999994e-12, 5.9868544995999995e-01, 1.1911685160956658e+05 },
    { 9.9999999999999994e-12, 3.7861174051700002e+00, 2.6068233380674399e+04 },
    { 9.9999999999999994e-12, 2.2703653974700000e+00, 6.5697557724739789e+04 },
    { 9.9999999999999994e-12, 4.8330122632999997e+00, 8.5034420025968182e+04 },
    { 9.9999999999999994e-12, 1.6798521077899999e+00, 9.1785460866313981e+04 },
    { 9.9999999999999994e-12, 9.4607737269000003e-01, 6.5831666774324796e+04 },
    { 9.9999999999999994e-12, 3.7090783280899999e+00, 2.5668418497699000e+04 },
    { 9.9999999999999994e-12, 5.1138297930299998e+00, 1.0594068546158058e+05 },
    { 9.9999999999999994e-12, 2.5524089963000001e+00, 2.3969139281195799e+04 },
    { 9.9999999999999994e-12, 5.5819628253099998e+00, 5.3399624123927002e+04 },
    { 9.9999999999999994e-12, 5.0513892101800000e+00, 8.6143798578061585e+04 },
    { 9.9999999999999994e-12, 2.2555647362100002e+00, 8.0596905817594597e+04 },
    { 9.9999999999999994e-12, 2.1311360055000000e-01, 1.1691776426627958e+05 },
    { 9.9999999999999994e-12, 1.5666340456100001e+00, 4.9527351457675388e+04 },
    { 9.9999999999999994e-12, 8.6126131152999996e-01, 2.6091784476932200e+04 },
    { 9.9999999999999994e-12, 1.8570138310300000e+00, 1.0276253967101299e+05 },
    { 9.9999999999999994e-12, 5.2215422456500002e+00, 4.2153969003049002e+04 },
    { 9.9999999999999994e-12, 3.5836754391499999e+00, 1.0021837280099400e+04 },
    { 9.9999999999999994e-12, 2.0905385415600000e+00, 5.2643771273502804e+04 },
    { 9.9999999999999994e-12, 5.7954403720899998e+00, 1.0657036967048359e+05 },
    { 9.9999999999999994e-12, 1.9147046140099999e+00, 6.0055895436487393e+04 },
    { 9.9999999999999994e-12, 2.8568051326499999e+00, 7.7410513042970590e+04 },
    { 9.9999999999999994e-12, 3.7694619407799999e+00, 3.5472744149649399e+04 },
    { 9.9999999999999994e-12, 5.0357557440200003e+00, 1.6703062133498999e+04 },
    { 9.9999999999999994e-12, 4.7864312064899996e+00, 5.1109310805958390e+04 },
    { 9.9999999999999994e-12, 5.7669802141000004e-01, 9.5247705572179184e+04 },
    { 9.9999999999999994e-12, 2.9018056985100000e+00, 2.5234706759822191e+04 },
    { 9.9999999999999994e-12, 1.5820140462200001e+00, 9.0695752075120385e+04 },
    { 9.9999999999999994e-12, 2.6874786754500000e+00, 2.6107572902473989e+04 },
    { 9.9999999999999994e-12, 4.3021174607499999e+00, 5.1123537899959993e+04 },
    { 9.9999999999999994e-12, 5.6178360020699998e+00, 2.6084021806216198e+04 },
    { 9.9999999999999994e-12, 3.8542134057099999e+00, 1.0329934418310839e+05 },
    { 0.0000000000000000e+00, 1.6782884764099999e+00, 1.1112232316754239e+05 },
    { 0.0000000000000000e+00, 4.3226951706600003e+00, 1.1678365521669458e+05 },
    { 0.0000000000000000e+00, 1.5768089024100000e+00, 1.0223284870591838e+05 },
    { 0.0000000000000000e+00, 3.8621205666599998e+00, 7.7829997686845789e+04 },
    { 0.0000000000000000e+00, 1.0790179246200000e+00, 5.3242301760338400e+04 },
    { 0.0000000000000000e+00, 5.6644461332700002e+00, 4.5892730433156990e+04 },
    { 0.0000000000000000e+00, 2.2008555802399998e+00, 8.5502385016322587e+04 },
    { 0.0000000000000000e+00, 9.8299558571000001e-01, 5.2329585093633192e+04 },
    { 0.0000000000000000e+00, 1.3297255491000000e+00, 9.7112936974696786e+04 },
    { 0.0000000000000000e+00, 1.1259234201099999e+00, 7.0269180982698381e+04 },
    { 0.0000000000000000e+00, 3.0427323146699998e+00, 1.3041984594697120e+05 },
    { 0.0000000000000000e+00, 2.3169210668500000e+00, 2.0759384658050019e+05 },
    { 0.0000000000000000e+00, 4.7743492160000001e-01, 7.8903606711036584e+04 },
};

static const VSOP87Term VSOP87A_MERCURY_Y3[59] = {
    { 1.8231000000000000e-07, 4.3514118391799999e+00, 7.8263709424722590e+04 },
    { 1.7840000000000001e-07, 1.4541906802000000e+00, 5.2175806283148399e+04 },
    { 1.5722000000000000e-07, 0.0000000000000000e+00, 0.0000000000000000e+00 },
    { 1.0723000000000000e-07, 1.0935349010700000e+00, 1.0435161256629678e+05 },
    { 8.8349999999999994e-08, 4.6273921422199997e+00, 2.6087903141574199e+04 },
    { 4.9730000000000003e-08, 4.1380564887200002e+00, 1.3043951570787099e+05 },
    { 2.0319999999999999e-08, 9.0409186617000004e-01, 1.5652741884944518e+05 },
    { 7.6799999999999999e-09, 3.9548140806499998e+00, 1.8261532199101939e+05 },
    { 2.7499999999999998e-09, 7.2281686312000004e-01, 2.0870322513259359e+05 },
    { 9.5000000000000003e-10, 3.7735785715899999e+00, 2.3479112827416777e+05 },
    { 3.1999999999999998e-10, 5.3816590044000001e-01, 2.6087903141574195e+05 },
    { 2.3000000000000001e-10, 1.3003983547900000e+00, 5.1066427731055002e+04 },
    { 1.2999999999999999e-10, 4.3512617007100003e+00, 7.7154330872629187e+04 },
    { 1.0999999999999999e-10, 4.2705857384900003e+00, 5.3285184835241802e+04 },
    { 1.0000000000000000e-10, 3.5847412517900001e+00, 2.8696693455731618e+05 },
    { 6.0000000000000000e-11, 1.1193941436199999e+00, 1.0324223401420339e+05 },
    { 3.9999999999999998e-11, 1.0691158093399999e+00, 7.9373087976815994e+04 },
    { 3.0000000000000000e-11, 3.5802564283999999e-01, 3.1305483769889036e+05 },
    { 3.0000000000000000e-11, 4.9769444800000002e+00, 2.4978524589480799e+04 },
    { 1.9999999999999999e-11, 4.1695855482799997e+00, 1.2933013715577759e+05 },
    { 1.9999999999999999e-11, 4.9803408015099997e+00, 4.6514474233996203e+04 },
    { 1.9999999999999999e-11, 7.5178980879000001e-01, 2.7197281693667599e+04 },
    { 1.9999999999999999e-11, 1.7973546247500001e+00, 7.2602377375570388e+04 },
    { 1.9999999999999999e-11, 3.9702044096000000e+00, 5.1220206541539788e+04 },
    { 9.9999999999999994e-12, 5.7925948804700003e+00, 5.2705497248242988e+04 },
    { 9.9999999999999994e-12, 5.7810627847999996e+00, 2.5028521211384999e+04 },
    { 9.9999999999999994e-12, 4.8123524997400002e+00, 9.8690280517144580e+04 },
    { 9.9999999999999994e-12, 5.8325393673000003e-01, 7.7308109683113988e+04 },
    { 9.9999999999999994e-12, 6.2560762381100004e+00, 2.7147285071763388e+04 },
    { 9.9999999999999994e-12, 1.3563951648899999e+00, 4.9957049178961599e+04 },
    { 9.9999999999999994e-12, 1.2793027654500000e+00, 5.3131406024757001e+04 },
    { 9.9999999999999994e-12, 2.5302976957399999e+00, 4.0853142184844000e+04 },
    { 9.9999999999999994e-12, 3.4099921565600000e+00, 3.3914274084046460e+05 },
    { 9.9999999999999994e-12, 4.1491420452499996e+00, 1.0546099111839019e+05 },
    { 9.9999999999999994e-12, 9.4394345298000004e-01, 1.5541804029735178e+05 },
    { 9.9999999999999994e-12, 2.7847798175600000e+00, 5.3235188213337591e+04 },
    { 9.9999999999999994e-12, 2.4414381239999998e+00, 7.8793400389817180e+04 },
    { 9.9999999999999994e-12, 5.7299004889900003e+00, 7.9323091354911783e+04 },
    { 9.9999999999999994e-12, 3.7596975798600001e+00, 5.6727759780207198e+04 },
    { 9.9999999999999994e-12, 4.1803587062599998e+00, 5.4394563387335191e+04 },
    { 9.9999999999999994e-12, 8.3186420507000003e-01, 1.1093785520934000e+03 },
    { 0.0000000000000000e+00, 6.0221940850599998e+00, 5.1646115318053788e+04 },
    { 0.0000000000000000e+00, 9.8563675821999996e-01, 2.7676976036857999e+04 },
    { 0.0000000000000000e+00, 4.6382064700900001e+00, 2.7043502883182799e+04 },
    { 0.0000000000000000e+00, 4.8808814558800000e+00, 2.4498830246290399e+04 },
    { 0.0000000000000000e+00, 5.0938401746900004e+00, 4.5405095681902800e+04 },
    { 0.0000000000000000e+00, 5.5268160318000004e-01, 8.2815662921781390e+04 },
    { 0.0000000000000000e+00, 4.0624256609499998e+00, 3.0639856638632999e+04 },
    { 0.0000000000000000e+00, 2.2930607278499999e+00, 2.0426571092422000e+04 },
    { 0.0000000000000000e+00, 2.8372446964200000e+00, 2.6617594106668799e+04 },
    { 0.0000000000000000e+00, 4.3582462553400001e+00, 7.6674636529438780e+04 },
    { 0.0000000000000000e+00, 3.8303332137699999e+00, 5.6613320491521999e+03 },
    { 0.0000000000000000e+00, 3.2449010894000002e-01, 2.1329909543799999e+02 },
    { 0.0000000000000000e+00, 2.7320488172599999e+00, 1.6983996147456601e+04 },
    { 0.0000000000000000e+00, 9.4024466665999995e-01, 2.5132303399965600e+04 },
    { 0.0000000000000000e+00, 5.5928861471999998e+00, 6.6941045326418185e+04 },
    { 0.0000000000000000e+00, 6.2124464231700003e+00, 1.4765239043269799e+04 },
    { 0.0000000000000000e+00, 5.0407994882300002e+00, 2.3869146037387400e+04 },
    { 0.0000000000000000e+00, 4.4212392122599997e+00, 7.6044952320535798e+04 },
};

static const VSOP87Term VSOP87A_MERCURY_Y4[15] = {
    { 4.4800000000000002e-09, 3.1415926535900001e+00, 0.0000000000000000e+00 },
    { 2.9899999999999998e-09, 2.9172532957900001e+00, 7.8263709424722590e+04 },
    { 2.4100000000000002e-09, 5.8823368497099997e+00, 1.0435161256629678e+05 },
    { 2.3400000000000002e-09, 1.1142561910000000e-01, 5.2175806283148399e+04 },
    { 1.4500000000000000e-09, 2.6200396956800001e+00, 1.3043951570787099e+05 },
    { 7.2999999999999996e-10, 5.6586077610799999e+00, 1.5652741884944518e+05 },
    { 5.6000000000000003e-10, 3.1945598651900000e+00, 2.6087903141574199e+04 },
    { 3.3000000000000000e-10, 2.4204430488300002e+00, 1.8261532199101939e+05 },
    { 1.4000000000000001e-10, 5.4630012476700003e+00, 2.0870322513259359e+05 },
    { 5.0000000000000002e-11, 2.2323984902700000e+00, 2.3479112827416777e+05 },
    { 1.9999999999999999e-11, 5.2806135862100003e+00, 2.6087903141574195e+05 },
    { 9.9999999999999994e-12, 2.0414046051399999e+00, 2.8696693455731618e+05 },
    { 0.0000000000000000e+00, 6.1102443193399996e+00, 5.1066427731055002e+04 },
    { 0.0000000000000000e+00, 2.7704533576900001e+00, 7.7154330872629187e+04 },
    { 0.0000000000000000e+00, 2.6450272257799998e+00, 5.3285184835241802e+04 },
};

static const VSOP87Term VSOP87A_MERCURY_Y5[10] = {
    { 5.0000000000000002e-11, 4.3636647996100004e+00, 1.0435161256629678e+05 },
    { 3.9999999999999998e-11, 1.2981820472600001e+00, 7.8263709424722590e+04 },
    { 3.9999999999999998e-11, 0.0000000000000000e+00, 0.0000000000000000e+00 },
    { 3.0000000000000000e-11, 1.1021003929199999e+00, 1.3043951570787099e+05 },
    { 3.0000000000000000e-11, 2.3280441883399998e+00, 2.6087903141574199e+04 },
    { 3.0000000000000000e-11, 3.9246569918800001e+00, 5.2175806283148399e+04 },
    { 1.9999999999999999e-11, 4.1333526383599999e+00, 1.5652741884944518e+05 },
    { 9.9999999999999994e-12, 8.8060083636999997e-01, 1.8261532199101939e+05 },
    { 9.9999999999999994e-12, 3.9189866367200001e+00, 2.0870322513259359e+05 },
    { 0.0000000000000000e+00, 6.7280389008999997e-01, 2.3479112827416777e+05 },
};

static const VSOP87Term VSOP87A_MERCURY_Z0[598] = {
    { 4.6076653260000003e-02, 1.9929508196700001e+00, 2.6087903141574199e+04 },
    { 7.0873436500000003e-03, 3.1415926535900001e+00, 0.0000000000000000e+00 },
    { 4.6917161699999996e-03, 5.0421574276400003e+00, 5.2175806283148399e+04 },
    { 7.1626394999999997e-04, 1.8089425607100000e+00, 7.8263709424722590e+04 },
    { 1.2957446000000001e-04, 4.8592203201000004e+00, 1.0435161256629678e+05 },
    { 2.5750020000000000e-05, 1.6264673154500000e+00, 1.3043951570787099e+05 },
    { 5.4325899999999996e-06, 4.6769886016699997e+00, 1.5652741884944518e+05 },
    { 1.1946200000000001e-06, 1.4443799409700000e+00, 1.8261532199101939e+05 },
    { 8.2635000000000004e-07, 2.0312796193799998e+00, 2.4978524589480799e+04 },
    { 7.9477000000000002e-07, 4.9444284934300002e+00, 2.7197281693667599e+04 },
    { 4.4712000000000000e-07, 5.2659463739000001e-01, 1.0593819301891999e+03 },
    { 4.3131000000000002e-07, 5.6692241296299999e+00, 2.0426571092422000e+04 },
    { 3.4676999999999997e-07, 2.3673617087399998e+00, 2.5028521211384999e+04 },
    { 3.2337000000000002e-07, 1.2909963109900000e+00, 3.1749235190726398e+04 },
    { 2.7464999999999999e-07, 3.5768007240399999e+00, 5.6613320491521999e+03 },
    { 2.7080999999999997e-07, 4.4949935696300001e+00, 2.0870322513259359e+05 },
    { 2.3561000000000000e-07, 1.8236102121500000e+00, 5.3285184835241802e+04 },
    { 2.0977999999999999e-07, 4.4613155523300003e+00, 4.5519534970588002e+03 },
    { 2.0601999999999999e-07, 1.1276855719900001e+00, 1.1093785520934000e+03 },
    { 2.0251999999999999e-07, 1.1722157720000000e-01, 5.1116424352959199e+04 },
    { 1.6632000000000000e-07, 5.0911886482500002e+00, 5.1066427731055002e+04 },
    { 1.4149999999999999e-07, 4.4248569828399997e+00, 5.7837138332300601e+04 },
    { 1.1609000000000000e-07, 3.4881279246800001e+00, 2.1535949644515400e+04 },
    { 9.8970000000000005e-08, 2.8094029291800000e+00, 2.7043502883182799e+04 },
    { 9.2340000000000005e-08, 3.8044581762099998e+00, 5.2969096509459996e+02 },
    { 8.3799999999999996e-08, 2.4682009310300002e+00, 4.6514474233996203e+04 },
    { 7.9500000000000004e-08, 4.4767912340999999e-01, 1.0213285546211000e+04 },
    { 7.8339999999999999e-08, 4.0800628542700004e+00, 2.5132303399965600e+04 },
    { 7.1690000000000002e-08, 2.4386162528000002e+00, 4.7623852786089599e+04 },
    { 6.3119999999999999e-08, 3.6015622205100000e+00, 2.7147285071763388e+04 },
    { 6.2830000000000001e-08, 1.2624459214499999e+00, 2.3479112827416777e+05 },
    { 6.1889999999999994e-08, 3.2459831106300001e+00, 7.7204327494533383e+04 },
    { 5.8740000000000002e-08, 3.1851753049500000e+00, 1.4765239043269799e+04 },
    { 5.8439999999999999e-08, 4.8868491539400001e+00, 7.9373087976815994e+04 },
    { 5.7779999999999999e-08, 4.7451860371799999e+00, 3.0639856638632999e+04 },
    { 5.4639999999999998e-08, 1.8067146403000001e-01, 1.3521751441591399e+04 },
    { 5.2070000000000002e-08, 1.0436060920600001e+00, 4.2659819087599999e+02 },
    { 5.0839999999999997e-08, 2.8882607600100001e+00, 1.2566151699982800e+04 },
    { 4.8620000000000001e-08, 1.0521142570499999e+00, 1.5890728952838001e+03 },
    { 4.4700000000000003e-08, 5.9424377611999999e+00, 1.1322664098304400e+04 },
    { 4.0840000000000000e-08, 6.1073072194700000e+00, 4.1962520736937397e+04 },
    { 4.0460000000000000e-08, 3.0846917664600002e+00, 5.1646115318053788e+04 },
    { 3.9330000000000002e-08, 1.6079554480600000e+00, 2.5661304950698199e+04 },
    { 3.9209999999999998e-08, 1.1959237463200001e+00, 8.3925041473874793e+04 },
    { 3.7970000000000002e-08, 1.8623369453900001e+00, 7.7154330872629187e+04 },
    { 3.7389999999999999e-08, 2.0442882550900001e+00, 1.5874617595363199e+04 },
    { 3.7280000000000000e-08, 1.7433215418400001e+00, 2.4498830246290399e+04 },
    { 3.6909999999999998e-08, 3.5089318025199998e+00, 3.7410567239878597e+04 },
    { 3.4170000000000001e-08, 4.1083094919800001e+00, 3.9609654583165597e+04 },
    { 3.2299999999999998e-08, 5.3901579798399997e+00, 9.5559974160859997e+02 },
    { 3.0199999999999999e-08, 5.4676810844299997e+00, 2.5558212176479599e+04 },
    { 3.0169999999999998e-08, 6.0886366091699999e+00, 5.3131406024757001e+04 },
    { 2.6530000000000000e-08, 5.5215695028700003e+00, 2.6617594106668799e+04 },
    { 2.6440000000000001e-08, 5.6008952186799998e+00, 7.3711755927663791e+04 },
    { 2.5259999999999999e-08, 5.4911247870700004e+00, 6.2830758499914000e+03 },
    { 2.2169999999999999e-08, 5.8193089410300001e+00, 5.1749208092272391e+04 },
    { 2.1690000000000001e-08, 5.8584368872399999e+00, 5.0586733387864588e+04 },
    { 2.0359999999999998e-08, 2.7941190799200002e+00, 7.2386755916000002e+03 },
    { 1.9099999999999999e-08, 5.5430289273500000e+00, 7.2602377375570388e+04 },
    { 1.9040000000000000e-08, 2.4294494940000001e-02, 3.3326578733174203e+04 },
    { 1.8410000000000001e-08, 3.3198754226500000e+00, 3.6301188687785187e+04 },
    { 1.8139999999999999e-08, 5.8414302882000002e-01, 6.3498470381452789e+04 },
    { 1.6930000000000000e-08, 4.0226118849999999e-02, 1.0329223063610759e+05 },
    { 1.6050000000000001e-08, 8.4854708960000003e-01, 5.1220206541539788e+04 },
    { 1.6009999999999999e-08, 6.9333064930999999e-01, 9.1039069941176003e+03 },
    { 1.5950000000000001e-08, 5.6521661909100001e+00, 2.6068233380674399e+04 },
    { 1.5950000000000001e-08, 1.4723892163000001e+00, 2.6107572902473989e+04 },
    { 1.5600000000000001e-08, 9.6215455240000003e-01, 6.5697557724739789e+04 },
    { 1.4870000000000000e-08, 1.6695037677800000e+00, 1.0546099111839019e+05 },
    { 1.4850000000000001e-08, 4.3130766705600001e+00, 2.6087903141574195e+05 },
    { 1.3920000000000000e-08, 4.1896396082999998e-01, 1.8849227549974199e+04 },
    { 1.3860000000000000e-08, 4.2909271064999999e-01, 5.3235188213337591e+04 },
    { 1.3440000000000000e-08, 3.0518857926199998e+00, 6.8050423878511589e+04 },
    { 1.2580000000000001e-08, 2.6281712648800002e+00, 5.2705497248242988e+04 },
    { 1.2240000000000001e-08, 3.5654307978099999e+00, 1.9804827291582798e+04 },
    { 1.2180000000000001e-08, 6.1664287045300004e+00, 7.7734018459627987e+04 },
    { 1.1910000000000000e-08, 5.7671563195699997e+00, 1.9317192540328600e+04 },
    { 1.1549999999999999e-08, 1.8329152417000001e+00, 1.6983996147456601e+04 },
    { 1.1469999999999999e-08, 1.4006922675800000e+00, 5.6727759780207198e+04 },
    { 1.1040000000000000e-08, 1.3736536521600000e+00, 4.5892730433156990e+04 },
    { 1.0960000000000000e-08, 2.7778284556199999e+00, 2.6091784476932200e+04 },
    { 1.0960000000000000e-08, 4.3496978501200001e+00, 2.6084021806216198e+04 },
    { 1.0390000000000001e-08, 4.2500384700199998e+00, 1.1001294461544899e+05 },
    { 9.9100000000000007e-09, 7.7508695125000004e-01, 3.2858613742819791e+04 },
    { 9.3299999999999998e-09, 3.5200898423700000e+00, 6.7707106012455997e+03 },
    { 9.1299999999999997e-09, 4.9138683537499999e+00, 1.0324223401420339e+05 },
    { 9.0300000000000005e-09, 4.5153942754300003e+00, 3.4425749449653999e+03 },
    { 8.9399999999999993e-09, 6.2089194932999998e+00, 4.0853142184844000e+04 },
    { 8.8800000000000008e-09, 5.9640517293300004e+00, 3.8654054841556987e+04 },
    { 8.7399999999999992e-09, 3.3346253346800001e+00, 2.2645328196608789e+04 },
    { 8.5799999999999997e-09, 3.4819420303099999e+00, 5.9414481874748402e+04 },
    { 8.4300000000000000e-09, 2.1028389301299999e+00, 2.3869146037387400e+04 },
    { 8.1599999999999999e-09, 4.6854984642500002e+00, 2.8306660245760992e+04 },
    { 7.9799999999999993e-09, 5.9097972411699997e+00, 4.3071899289030800e+04 },
    { 7.5100000000000007e-09, 2.3828096157699998e+00, 9.9799659069237983e+04 },
    { 7.4700000000000001e-09, 2.8841372874200002e+00, 7.9219309166331193e+04 },
    { 7.3900000000000003e-09, 4.0389810045500001e+00, 2.6514501332450200e+04 },
    { 6.8800000000000002e-09, 4.1373584551000002e+00, 2.7676976036857999e+04 },
    { 6.8500000000000001e-09, 2.6899454160300000e+00, 7.6674636529438780e+04 },
    { 6.6800000000000001e-09, 2.6744757060300000e+00, 7.7837111233846590e+04 },
    { 6.6400000000000002e-09, 5.4061571360000000e+00, 2.1329909543799999e+02 },
    { 6.4300000000000003e-09, 2.6212576856999997e-01, 6.3989728631399998e+02 },
    { 6.3600000000000004e-09, 5.0127732926500004e+00, 7.1135470007999997e+00 },
    { 5.9399999999999998e-09, 4.7220480108300000e+00, 2.9530478086539599e+04 },
    { 5.7999999999999998e-09, 4.9131159360999999e-01, 6.2389091829359393e+04 },
    { 5.5999999999999997e-09, 5.8980434737999998e-01, 2.6080789594573391e+04 },
    { 4.9499999999999997e-09, 4.3509468615999998e+00, 5.8946516884393990e+04 },
    { 4.7300000000000000e-09, 3.1914765678800001e+00, 2.5874604046136199e+04 },
    { 4.5999999999999998e-09, 2.4100835360900001e+00, 2.5448005855260191e+04 },
    { 4.5800000000000003e-09, 2.3174613952800001e+00, 9.8690280517144580e+04 },
    { 4.5299999999999999e-09, 4.0264606419500000e+00, 9.1785460866313981e+04 },
    { 4.5200000000000001e-09, 3.1020557522600001e+00, 1.2938013377768178e+05 },
    { 4.4100000000000003e-09, 3.6012092280400001e+00, 8.9586373523026981e+04 },
    { 4.0599999999999996e-09, 1.5777630172299999e+00, 2.1187638603783998e+03 },
    { 3.9499999999999998e-09, 2.9737033271000002e-01, 2.5934124331089399e+04 },
    { 3.9300000000000003e-09, 3.0972525839700000e+00, 6.9159802430604992e+04 },
    { 3.9199999999999997e-09, 5.3546585935000002e-01, 2.6241681952059000e+04 },
    { 3.8199999999999996e-09, 4.7273147112900000e+00, 1.3154889425996438e+05 },
    { 3.8199999999999996e-09, 4.4919873784200002e+00, 7.1980633574731182e+04 },
    { 3.6800000000000001e-09, 3.8989217277199999e+00, 7.7308109683113988e+04 },
    { 3.5600000000000001e-09, 6.1230999953199996e+00, 9.4138327020085781e+04 },
    { 3.5600000000000001e-09, 1.0806284094000000e+00, 2.8696693455731618e+05 },
    { 3.5100000000000001e-09, 1.0474568213800000e+00, 2.2187571041868000e+03 },
    { 3.4499999999999999e-09, 5.7195786851400001e+00, 7.8793400389817180e+04 },
    { 3.2700000000000001e-09, 4.5240664377500002e+00, 5.2195476044048191e+04 },
    { 3.2700000000000001e-09, 2.9465188549999999e+00, 1.0382192160120218e+05 },
    { 3.2599999999999999e-09, 3.4994174689899999e+00, 7.9323091354911783e+04 },
    { 3.2500000000000002e-09, 2.4184486567299999e+00, 5.2156136522248598e+04 },
    { 3.1000000000000000e-09, 1.1065309462299999e+00, 2.3969139281195799e+04 },
    { 3.0000000000000000e-09, 3.5361259844899999e+00, 2.6301202237012199e+04 },
    { 2.9300000000000000e-09, 1.7489733394300000e+00, 5.4394563387335191e+04 },
    { 2.7999999999999998e-09, 3.4771168199200000e+00, 4.4937130691548402e+04 },
    { 2.7499999999999998e-09, 4.0537215580000003e+00, 1.4956319713459999e+02 },
    { 2.7400000000000001e-09, 1.0212854439900001e+00, 1.3610084775702318e+05 },
    { 2.7299999999999999e-09, 3.4374545554000002e-01, 5.1535908996834391e+04 },
    { 2.6400000000000000e-09, 4.4252768757499998e+00, 8.2815662921781390e+04 },
    { 2.5599999999999998e-09, 3.5754475922100002e+00, 1.0664954771900000e+03 },
    { 2.5300000000000002e-09, 6.7254115542000004e-01, 1.0522683831884001e+03 },
    { 2.5200000000000000e-09, 2.0778929505899999e+00, 4.8733231338182988e+04 },
    { 2.3300000000000000e-09, 1.5895629560500000e+00, 2.6095016688575000e+04 },
    { 2.2699999999999998e-09, 1.6829141329599999e+00, 1.2933013715577759e+05 },
    { 2.2400000000000001e-09, 5.8266354673200000e+00, 5.2179687618506403e+04 },
    { 2.2300000000000000e-09, 1.1156646011900000e+00, 5.2171924947790401e+04 },
    { 2.1799999999999999e-09, 5.8500966778199999e+00, 2.5619938151219802e+04 },
    { 2.1299999999999999e-09, 1.3198576448599999e+00, 2.6555868131928601e+04 },
    { 2.1100000000000000e-09, 2.5517942256900001e+00, 4.5405095681902800e+04 },
    { 2.0700000000000001e-09, 5.2669851569199997e+00, 2.5938339944439598e+04 },
    { 2.0700000000000001e-09, 2.7543317603599999e+00, 6.4741957983131193e+04 },
    { 2.0400000000000000e-09, 5.4421212234400000e+00, 1.2588756221081219e+05 },
    { 1.9900000000000000e-09, 3.0592670547899998e+00, 6.6941045326418185e+04 },
    { 1.9500000000000001e-09, 5.6118026607099996e+00, 2.5021407664384202e+04 },
    { 1.9500000000000001e-09, 5.9759651310699997e+00, 1.0530721230790539e+05 },
    { 1.9300000000000002e-09, 2.2050775896300001e+00, 2.5035634758385801e+04 },
    { 1.9099999999999998e-09, 2.4359410467000001e-01, 8.5502385016322587e+04 },
    { 1.9000000000000001e-09, 3.2544460507399999e+00, 1.3655860491176400e+04 },
    { 1.8899999999999999e-09, 5.7882364849999997e+00, 1.0276253967101299e+05 },
    { 1.8400000000000001e-09, 1.7448716324300000e+00, 5.1962507187710398e+04 },
    { 1.8400000000000001e-09, 1.6182858009700001e+00, 2.0760427033191401e+04 },
    { 1.8100000000000000e-09, 5.7521514081199996e+00, 1.0392501437542078e+05 },
    { 1.7700000000000001e-09, 5.3158601251900004e+00, 5.0057042422769999e+04 },
    { 1.7100000000000001e-09, 1.9919319713800001e+00, 2.6094531700474210e+04 },
    { 1.7100000000000001e-09, 5.1352042365999999e+00, 2.6081274582674188e+04 },
    { 1.6600000000000001e-09, 5.1670869465699996e+00, 4.9957049178961599e+04 },
    { 1.6399999999999999e-09, 3.6548689180400000e+00, 7.6266071275599998e+01 },
    { 1.6399999999999999e-09, 5.8298171326599997e+00, 1.2432042650397800e+04 },
    { 1.6399999999999999e-09, 9.1431121775000002e-01, 5.2602404474024399e+04 },
    { 1.5900000000000001e-09, 2.1097877432400001e+00, 3.2370978991565600e+04 },
    { 1.5500000000000000e-09, 5.3987938894100003e+00, 6.6812248533995999e+03 },
    { 1.5199999999999999e-09, 9.6120452018000002e-01, 5.3764879178432202e+04 },
    { 1.5000000000000000e-09, 9.0885651205999995e-01, 5.2389105378586399e+04 },
    { 1.5000000000000000e-09, 3.5880773418300000e+00, 8.8476994970933592e+04 },
    { 1.4900000000000000e-09, 2.1744423927500001e+00, 5.2168692736147590e+04 },
    { 1.4599999999999999e-09, 5.5720725119500001e+00, 5.8458882133139792e+04 },
    { 1.4100000000000001e-09, 1.4403357881500001e+00, 5.3274761083827998e+03 },
    { 1.3800000000000000e-09, 5.4022651941600000e+00, 2.6011637070298599e+04 },
    { 1.3000000000000001e-09, 3.1296398062300002e+00, 3.8519945791972001e+04 },
    { 1.2600000000000000e-09, 8.0278847447000001e-01, 1.1787336400788819e+05 },
    { 1.2400000000000001e-09, 1.4263305883599999e+00, 5.5618381228113802e+04 },
    { 1.2400000000000001e-09, 5.5907364501399996e+00, 7.4821134479757180e+04 },
    { 1.2300000000000001e-09, 2.9095927256000000e+00, 5.2026243086013797e+04 },
    { 1.2000000000000000e-09, 6.1589049848000004e+00, 1.5546803691925600e+05 },
    { 1.1800000000000001e-09, 3.9116770585100000e+00, 1.9406678288174600e+04 },
    { 1.1700000000000001e-09, 4.6811816247999998e+00, 5.2182919830149200e+04 },
    { 1.1400000000000000e-09, 3.3472906186300002e+00, 5.1109310805958390e+04 },
    { 1.1300000000000000e-09, 5.3728360212300004e+00, 1.2477818365871879e+05 },
    { 1.1200000000000001e-09, 6.2404476589900000e+00, 5.1123537899959993e+04 },
    { 1.0900000000000000e-09, 3.7062017000000003e-01, 1.1567427666460119e+05 },
    { 1.0800000000000000e-09, 5.3552361965399999e+00, 4.6848330174765601e+04 },
    { 1.0500000000000001e-09, 1.2694931386599999e+00, 9.8068536716305389e+04 },
    { 1.0100000000000000e-09, 1.5085965774600001e+00, 1.5763679740153858e+05 },
    { 9.6999999999999996e-10, 9.4285631414000004e-01, 8.5034420025968182e+04 },
    { 9.2000000000000003e-10, 2.8991928458800000e+00, 1.2022623016165999e+05 },
    { 9.0999999999999996e-10, 2.5005214985199999e+00, 1.0488130353139139e+05 },
    { 8.9000000000000003e-10, 6.6689996363000004e-01, 1.0339601282468818e+05 },
    { 8.7999999999999996e-10, 3.2423444332100000e+00, 2.6727800427888200e+04 },
    { 8.6999999999999999e-10, 5.9993011061799999e+00, 1.2990982474277639e+05 },
    { 8.6999999999999999e-10, 3.2929549456000000e+00, 3.3406124266997999e+03 },
    { 8.6000000000000003e-10, 4.1315142502400004e+00, 3.1305483769889036e+05 },
    { 8.3000000000000003e-10, 3.4688735672700002e+00, 7.7623812138408583e+04 },
    { 8.3000000000000003e-10, 4.5607612864400000e+00, 4.1948464387519999e+02 },
    { 8.1999999999999996e-10, 3.5848950504900001e+00, 5.2329585093633192e+04 },
    { 8.1999999999999996e-10, 2.7659872925000002e+00, 3.5191810135691798e+04 },
    { 8.0999999999999999e-10, 3.3488836224799998e+00, 5.2022027472663598e+04 },
    { 8.0000000000000003e-10, 1.6180082693000000e+00, 4.5494581429748789e+04 },
    { 8.0000000000000003e-10, 2.7627260079999999e-01, 1.0541099449648599e+05 },
    { 7.8999999999999996e-10, 5.9964505285199996e+00, 9.5247705572179184e+04 },
    { 7.5000000000000000e-10, 1.2916556098900001e+00, 7.8283379185622383e+04 },
    { 7.5000000000000000e-10, 9.3132779666999999e-01, 4.3371173787679999e+02 },
    { 7.4000000000000003e-10, 6.3932580906000003e-01, 6.4607848933546193e+04 },
    { 7.4000000000000003e-10, 5.4682340900200002e+00, 7.8244039663822798e+04 },
    { 7.2999999999999996e-10, 3.2181507230699999e+00, 5.2099540211872802e+04 },
    { 7.1000000000000003e-10, 4.0685589566999996e+00, 1.6218875089859738e+05 },
    { 6.6000000000000000e-10, 4.7115225569700003e+00, 8.0482466528909383e+04 },
    { 6.6000000000000000e-10, 2.9868320371100001e+00, 2.3754706748702189e+04 },
    { 6.5000000000000003e-10, 2.5068571899999997e-01, 7.1025033833122594e+04 },
    { 6.3999999999999996e-10, 1.1712903828300001e+00, 1.0890356606335558e+05 },
    { 6.2000000000000003e-10, 5.7315631578400001e+00, 8.5319638175199998e+02 },
    { 6.0999999999999996e-10, 4.5006615004599997e+00, 2.5668418497699000e+04 },
    { 5.9000000000000003e-10, 7.9111914847999998e-01, 7.9945284420241996e+03 },
    { 5.9000000000000003e-10, 4.9282215896099997e+00, 7.8050410329284583e+04 },
    { 5.7000000000000000e-10, 6.2364205403300002e+00, 1.6921656695024001e+03 },
    { 5.7000000000000000e-10, 4.6712138499300000e+00, 2.8206667001952599e+04 },
    { 5.7000000000000000e-10, 4.7338547541300002e+00, 1.5541804029735178e+05 },
    { 5.6000000000000003e-10, 2.0921589268899998e+00, 7.6144945564344191e+04 },
    { 5.6000000000000003e-10, 1.7330400824200001e+00, 2.5654191403697390e+04 },
    { 5.6000000000000003e-10, 5.8872078563199999e+00, 2.6037906519669988e+04 },
    { 5.6000000000000003e-10, 2.2155635772100002e+00, 1.5197546535238638e+05 },
    { 5.6000000000000003e-10, 4.7723742186400004e+00, 2.8421099534446199e+04 },
    { 5.4999999999999996e-10, 1.2337626985500001e+00, 2.6137899763478399e+04 },
    { 5.3000000000000003e-10, 6.0770948543300003e+00, 1.0021837280099400e+04 },
    { 5.1999999999999996e-10, 2.7463021159499998e+00, 1.3139511544947958e+05 },
    { 5.1999999999999996e-10, 2.5840329042599999e+00, 1.2885044281258718e+05 },
    { 5.1000000000000000e-10, 4.1640621624699996e+00, 7.8259828089364586e+04 },
    { 5.1000000000000000e-10, 2.5917247668300001e+00, 7.8267590760080580e+04 },
    { 5.1000000000000000e-10, 3.9046537847799998e+00, 7.4781598567299994e+01 },
    { 5.0000000000000003e-10, 5.8101313425100001e+00, 9.0829861124705378e+04 },
    { 4.8000000000000000e-10, 5.6458450244399998e+00, 7.1492998823476984e+04 },
    { 4.8000000000000000e-10, 2.5342752815799998e+00, 1.3001291751699499e+05 },
    { 4.7000000000000003e-10, 1.6548126943299999e+00, 1.8093374699549990e+04 },
    { 4.7000000000000003e-10, 6.1137708668600004e+00, 9.3028948467992392e+04 },
    { 4.7000000000000003e-10, 3.1107781399699999e+00, 1.6066065861474801e+04 },
    { 4.5000000000000000e-10, 3.3020820977800001e+00, 1.1159028815789679e+05 },
    { 4.5000000000000000e-10, 4.6127543698100002e+00, 2.3331963928720002e+03 },
    { 4.5000000000000000e-10, 5.0323597994900000e+00, 7.8256595877721789e+04 },
    { 4.3999999999999998e-10, 4.3619548229899996e+00, 5.2643771273502804e+04 },
    { 4.3000000000000001e-10, 4.0807492507300003e+00, 1.5961864422845999e+03 },
    { 4.3000000000000001e-10, 1.3191769322999999e-01, 1.5819593482830001e+03 },
    { 4.2000000000000000e-10, 2.9427633130199999e+00, 2.4395737472071800e+04 },
    { 4.2000000000000000e-10, 4.0211792582600001e+00, 7.8477008520160583e+04 },
    { 4.2000000000000000e-10, 2.3434121766999998e+00, 8.4546785274713984e+04 },
    { 4.0000000000000001e-10, 3.2376826373399998e+00, 2.5234706759822191e+04 },
    { 3.9000000000000000e-10, 3.9929663576299999e+00, 7.8690307615598591e+04 },
    { 3.9000000000000000e-10, 3.8970280197000001e-01, 1.1456489811250778e+05 },
    { 3.7999999999999998e-10, 1.9648560460200000e+00, 6.3278373931320004e+02 },
    { 3.7999999999999998e-10, 8.5372328003999998e-01, 2.6237466338708789e+04 },
    { 3.7999999999999998e-10, 1.9410487910900001e+00, 7.6044952320535798e+04 },
    { 3.7000000000000001e-10, 2.5861886477399998e+00, 5.1707841292793993e+04 },
    { 3.7000000000000001e-10, 3.7524844702100002e+00, 2.7140171524762591e+04 },
    { 3.7000000000000001e-10, 2.2277872453100001e+00, 2.5973463852888959e+04 },
    { 3.6000000000000000e-10, 6.0276404202400000e+00, 7.8114146227587989e+04 },
    { 3.6000000000000000e-10, 3.6116080292900001e+00, 4.8835193856448590e+04 },
    { 3.6000000000000000e-10, 3.6428632575999997e-01, 2.7154398618764200e+04 },
    { 3.6000000000000000e-10, 4.8975704387199999e+00, 2.6202342430259410e+04 },
    { 3.6000000000000000e-10, 4.0214248738100000e+00, 7.9852782320006387e+04 },
    { 3.4999999999999998e-10, 1.9483010118999999e-01, 7.7197213947532582e+04 },
    { 3.4999999999999998e-10, 1.8994816863899999e+00, 5.2169177724248388e+04 },
    { 3.4999999999999998e-10, 3.0903735559799999e+00, 7.7211441041534184e+04 },
    { 3.4999999999999998e-10, 5.8609513904600004e+00, 1.8207813988235212e+04 },
    { 3.4999999999999998e-10, 5.0403646823499999e+00, 5.2182434842048402e+04 },
    { 3.4999999999999998e-10, 2.5082133279800001e+00, 7.2936233316339785e+04 },
    { 3.4000000000000001e-10, 5.9002441047799996e+00, 1.0309277421860000e+02 },
    { 3.3000000000000000e-10, 4.2208778231500004e+00, 1.1112232316754239e+05 },
    { 3.3000000000000000e-10, 4.3095696973299997e+00, 1.4477351183200000e+04 },
    { 3.3000000000000000e-10, 3.8540735900500001e+00, 1.4396126714946239e+05 },
    { 3.1999999999999998e-10, 1.4527785558499999e+00, 7.8270822971723392e+04 },
    { 3.1999999999999998e-10, 4.7083349580700000e+00, 7.1582484571322988e+04 },
    { 3.1999999999999998e-10, 5.9384906630399996e+00, 5.1742094545271590e+04 },
    { 3.1999999999999998e-10, 2.9301671412100001e+00, 1.8155594006083018e+05 },
    { 3.1000000000000002e-10, 2.0993955814800001e+00, 2.6484548254729998e+03 },
    { 3.1000000000000002e-10, 6.2826722279999997e-02, 5.3680451209540001e+02 },
    { 3.1000000000000002e-10, 2.3333663996800000e+00, 5.1756321639273199e+04 },
    { 3.0000000000000000e-10, 1.7325551657999999e-01, 2.2434479570190001e+02 },
    { 3.0000000000000000e-10, 4.4657021301300004e+00, 8.1706284369687986e+04 },
    { 3.0000000000000000e-10, 5.5025093927000002e-01, 3.3967992294913187e+04 },
    { 2.8999999999999998e-10, 4.9968092839400002e+00, 2.4491716699289591e+04 },
    { 2.8999999999999998e-10, 2.1430493704900000e+00, 1.5086608680029298e+05 },
    { 2.8999999999999998e-10, 5.6292904284700001e+00, 4.4181277841124189e+04 },
    { 2.8000000000000002e-10, 3.4172306518800002e+00, 7.8800891533389904e+03 },
    { 2.8000000000000002e-10, 4.9518762133200003e+00, 9.5628915597059995e+02 },
    { 2.8000000000000002e-10, 1.5204300591800000e+00, 4.0565254324774200e+04 },
    { 2.8000000000000002e-10, 8.3354394743000004e-01, 4.2153969003049002e+04 },
    { 2.7000000000000000e-10, 5.2988442747600004e+00, 1.1624747044077999e+03 },
    { 2.7000000000000000e-10, 3.4444041835100001e+00, 1.4176217980617538e+05 },
    { 2.7000000000000000e-10, 4.8237947730300004e+00, 1.1020632121940000e+02 },
    { 2.7000000000000000e-10, 4.3293586022900001e+00, 1.2415643985787958e+05 },
    { 2.7000000000000000e-10, 2.7726983047900000e+00, 2.4505943793291190e+04 },
    { 2.7000000000000000e-10, 4.4616534988299996e+00, 3.1639186965660002e+02 },
    { 2.5999999999999998e-10, 1.1535299180100000e+00, 5.1322609901396390e+04 },
    { 2.5999999999999998e-10, 1.3577439886000001e-01, 2.2747290714874402e+04 },
    { 2.5999999999999998e-10, 4.5615134741899999e+00, 1.8372470054311279e+05 },
    { 2.5000000000000002e-10, 7.1097564554000003e-01, 5.0483640613645999e+04 },
    { 2.5000000000000002e-10, 5.3683551885599998e+00, 2.6013121543006899e+04 },
    { 2.5000000000000002e-10, 5.9553231264499997e+00, 1.4631413330323418e+05 },
    { 2.4000000000000000e-10, 2.9224801013700001e+00, 1.2133560871375339e+05 },
    { 2.4000000000000000e-10, 2.1654518097000000e+00, 3.2769127994973802e+04 },
    { 2.4000000000000000e-10, 5.5581330381300003e+00, 1.3096920667296558e+05 },
    { 2.3000000000000001e-10, 6.1694922342399998e+00, 3.9743763632750597e+04 },
    { 2.3000000000000001e-10, 2.7733313371900001e+00, 1.5599772788435058e+05 },
    { 2.3000000000000001e-10, 2.6296960044000001e-01, 1.0371171527998279e+05 },
    { 2.3000000000000001e-10, 5.6388735516099997e+00, 5.2257741809380002e+02 },
    { 2.3000000000000001e-10, 2.0637760135300001e+00, 1.0090903762133139e+05 },
    { 2.1999999999999999e-10, 2.9004780506799999e+00, 2.5863558345872290e+04 },
    { 2.1999999999999999e-10, 4.9237419293000001e-01, 2.3439448316101189e+04 },
    { 2.1999999999999999e-10, 3.9938514656000002e-01, 2.6164169212849800e+04 },
    { 2.1999999999999999e-10, 7.8209397570000005e-02, 7.8187443353446986e+04 },
    { 2.1999999999999999e-10, 3.7133062773500001e+00, 1.2948391596626239e+05 },
    { 2.1999999999999999e-10, 4.3542257394700004e+00, 2.6720686880887391e+04 },
    { 2.1999999999999999e-10, 1.6938106240399999e+00, 1.0657036967048359e+05 },
    { 2.1999999999999999e-10, 4.3715066813499996e+00, 1.4788665740644001e+03 },
    { 2.1000000000000000e-10, 3.3682117739900002e+00, 1.3757737998458001e+03 },
    { 2.1000000000000000e-10, 5.1249773222600004e+00, 1.1610551958374201e+04 },
    { 2.1000000000000000e-10, 5.1050515044100004e+00, 2.5977696820354791e+04 },
    { 2.1000000000000000e-10, 8.9581658307000001e-01, 3.3914274084046460e+05 },
    { 2.0000000000000001e-10, 4.7942407573099999e+00, 2.5780345520604598e+04 },
    { 2.0000000000000001e-10, 3.8274070589800000e+00, 2.4925428437166400e+04 },
    { 2.0000000000000001e-10, 4.2687838972699996e+00, 2.5131613985603592e+04 },
    { 2.0000000000000001e-10, 3.3285595597800000e+00, 1.3149889763806018e+05 },
    { 2.0000000000000001e-10, 1.2980508623700000e+00, 4.6796499035440002e+02 },
    { 2.0000000000000001e-10, 1.1725114675999999e-01, 5.2815703569462399e+04 },
    { 1.8999999999999999e-10, 2.3313213527399999e+00, 2.6395460762543789e+04 },
    { 1.8999999999999999e-10, 3.5229627877000003e-01, 7.8417488235207391e+04 },
    { 1.8999999999999999e-10, 2.8617693301400000e+00, 5.0579619840863787e+04 },
    { 1.8999999999999999e-10, 1.1691865573000000e-01, 7.8109930614237790e+04 },
    { 1.8999999999999999e-10, 8.4289230961999995e-01, 1.8827665404017159e+05 },
    { 1.8999999999999999e-10, 5.4267379272000005e-01, 5.0593846934865389e+04 },
    { 1.8999999999999999e-10, 5.9521481733600003e+00, 2.9428515568274001e+04 },
    { 1.8000000000000000e-10, 4.3382610558500003e+00, 1.0437128232719658e+05 },
    { 1.8000000000000000e-10, 1.6675682602799999e+00, 2.3888815798287189e+04 },
    { 1.8000000000000000e-10, 4.7252163889999999e-01, 2.7999102624791401e+04 },
    { 1.8000000000000000e-10, 2.2313609180900000e+00, 1.0433194280539699e+05 },
    { 1.8000000000000000e-10, 4.5550994962600004e+00, 5.1543022543835199e+04 },
    { 1.7000000000000001e-10, 4.3906310555100001e+00, 6.0055895436487393e+04 },
    { 1.7000000000000001e-10, 3.1766808411800000e+00, 7.0269180982698381e+04 },
    { 1.7000000000000001e-10, 1.7402922915200001e+00, 1.0413831347085879e+05 },
    { 1.7000000000000001e-10, 5.2441128846199998e+00, 1.0223284870591838e+05 },
    { 1.7000000000000001e-10, 3.2869705282399999e+00, 2.5565325723480400e+04 },
    { 1.7000000000000001e-10, 1.6041139068000001e+00, 2.6507387785449391e+04 },
    { 1.5999999999999999e-10, 2.8132092467000001e+00, 5.2101024684581091e+04 },
    { 1.5999999999999999e-10, 4.2071039331100000e+00, 1.3499146920492980e+05 },
    { 1.5999999999999999e-10, 3.3095771757999999e+00, 9.7112936974696786e+04 },
    { 1.5999999999999999e-10, 1.0265591824300000e+00, 2.6162684740141500e+04 },
    { 1.5999999999999999e-10, 3.6211392744499999e+00, 2.6610480559667991e+04 },
    { 1.5999999999999999e-10, 4.9576278407399998e+00, 2.5771511271917600e+04 },
    { 1.5999999999999999e-10, 3.5365827536199999e+00, 2.5551098629478791e+04 },
    { 1.5999999999999999e-10, 3.0545139203299998e+00, 2.6190995915792791e+04 },
    { 1.5999999999999999e-10, 1.1438048243000001e+00, 2.6404295011230799e+04 },
    { 1.5999999999999999e-10, 4.7770424751900000e+00, 2.4609036567509800e+04 },
    { 1.5000000000000000e-10, 5.7861347865299999e+00, 2.4712129341728400e+04 },
    { 1.5000000000000000e-10, 5.1501957594000003e-01, 1.3362449706799200e+04 },
    { 1.5000000000000000e-10, 5.4445766837900003e+00, 2.8286990484861199e+04 },
    { 1.5000000000000000e-10, 3.3369783925999998e+00, 2.0618554843720000e+02 },
    { 1.5000000000000000e-10, 3.0931980889499999e+00, 2.5984810367355600e+04 },
    { 1.5000000000000000e-10, 1.5051301742100001e+00, 1.8150594343892598e+05 },
    { 1.5000000000000000e-10, 2.1995889611099999e+00, 2.2759767485294011e+04 },
    { 1.5000000000000000e-10, 5.2788651451700002e+00, 1.7806336849396059e+05 },
    { 1.4000000000000001e-10, 1.5853597470000000e-01, 6.4701083331480004e+02 },
    { 1.4000000000000001e-10, 2.4445149416700001e+00, 2.6421759082343600e+04 },
    { 1.4000000000000001e-10, 4.6896775370199997e+00, 2.5754047200804798e+04 },
    { 1.4000000000000001e-10, 5.6812257493700002e+00, 1.5493834595416137e+05 },
    { 1.4000000000000001e-10, 6.1265480216699997e+00, 5.2278899057366987e+04 },
    { 1.2999999999999999e-10, 2.2878615940499998e+00, 1.2725453434775000e+04 },
    { 1.2999999999999999e-10, 5.8192238996699999e+00, 1.5748301859105378e+05 },
    { 1.2999999999999999e-10, 4.0316049845800004e+00, 2.4356780788641601e+04 },
    { 1.2999999999999999e-10, 4.4987884886799998e+00, 2.9416038797854391e+04 },
    { 1.2999999999999999e-10, 1.3394428875400000e+00, 5.2808590022461591e+04 },
    { 1.2999999999999999e-10, 6.3063491269000005e-01, 5.1639001771052986e+04 },
    { 1.2999999999999999e-10, 5.1980975221000003e-01, 5.1951461487446490e+04 },
    { 1.2999999999999999e-10, 1.1999669274999999e-01, 2.6294088690011391e+04 },
    { 1.2999999999999999e-10, 3.7271479520000000e-01, 7.4923096998022782e+04 },
    { 1.2999999999999999e-10, 1.4784298059800001e+00, 5.4294570143526791e+04 },
    { 1.2999999999999999e-10, 2.5851312777500000e+00, 1.1691776426627958e+05 },
    { 1.2999999999999999e-10, 5.5953722056000004e+00, 1.5610082065856917e+05 },
    { 1.2999999999999999e-10, 3.9650535124299999e+00, 6.8241872144623194e+04 },
    { 1.2999999999999999e-10, 4.0725071634800001e+00, 3.4282178474782799e+04 },
    { 1.2000000000000000e-10, 9.2918175451999996e-01, 1.0434773123093879e+05 },
    { 1.2000000000000000e-10, 1.6679860757700000e+00, 5.1013331578740588e+04 },
    { 1.2000000000000000e-10, 4.2489289032500004e+00, 5.2225802905052602e+04 },
    { 1.2000000000000000e-10, 5.6399025384300003e+00, 1.0435549390165479e+05 },
    { 1.2000000000000000e-10, 3.1050821770299999e+00, 2.7819025494506801e+04 },
    { 1.2000000000000000e-10, 5.5873702009399997e+00, 2.5881717593137000e+04 },
    { 1.2000000000000000e-10, 2.0129999307499999e+00, 5.1219517127177787e+04 },
    { 1.2000000000000000e-10, 6.2605710821100002e+00, 2.4176703658357001e+04 },
    { 1.2000000000000000e-10, 1.7302176537300000e+00, 1.0434449901929598e+05 },
    { 1.2000000000000000e-10, 5.4031685819400002e+00, 1.1063468841628819e+05 },
    { 1.2000000000000000e-10, 2.2518817542799998e+00, 5.2065599961928987e+04 },
    { 1.2000000000000000e-10, 1.5201900753099999e+00, 5.4509002676020398e+04 },
    { 1.0999999999999999e-10, 2.4220922062899999e+00, 9.7580901965051191e+04 },
    { 1.0999999999999999e-10, 4.7688579495400001e+00, 4.9527351457675388e+04 },
    { 1.0999999999999999e-10, 3.9259397804499998e+00, 2.6521614879451001e+04 },
    { 1.0999999999999999e-10, 2.8944427706600000e+00, 1.1911685160956658e+05 },
    { 1.0999999999999999e-10, 9.9253734696999996e-01, 2.5455119402261000e+04 },
    { 1.0999999999999999e-10, 6.9086368479999993e-02, 5.2072713508929788e+04 },
    { 1.0999999999999999e-10, 8.0982228739999995e-01, 1.0456491166173479e+05 },
    { 1.0999999999999999e-10, 2.6555926508500001e+00, 5.2125809661244188e+04 },
    { 1.0999999999999999e-10, 6.2157487046300002e+00, 6.1279713277265997e+04 },
    { 1.0999999999999999e-10, 2.1652262924899999e+00, 2.6624707653669589e+04 },
    { 1.0000000000000000e-10, 2.7736419359800002e+00, 7.7829997686845789e+04 },
    { 1.0000000000000000e-10, 2.8331438356600001e+00, 1.0420204936916218e+05 },
    { 1.0000000000000000e-10, 4.0596346473700002e+00, 2.6089387614282488e+04 },
    { 1.0000000000000000e-10, 6.1724721996599996e+00, 1.0329934418310839e+05 },
    { 1.0000000000000000e-10, 2.1532343430000000e-02, 1.3767819129947099e+05 },
    { 1.0000000000000000e-10, 3.1165083103900000e+00, 2.6086418668865899e+04 },
    { 1.0000000000000000e-10, 3.2728704943500002e+00, 1.0328511708910679e+05 },
    { 1.0000000000000000e-10, 1.3853226112799999e+00, 2.5446489579835201e+04 },
    { 1.0000000000000000e-10, 3.4751748596000001e+00, 1.4065280125408198e+05 },
    { 1.0000000000000000e-10, 2.8029950827899999e+00, 5.5516418709848200e+04 },
    { 1.0000000000000000e-10, 5.4327305922800004e+00, 1.2699694076290558e+05 },
    { 1.0000000000000000e-10, 1.0794428116000001e+00, 7.4299006053259995e+02 },
    { 1.0000000000000000e-10, 7.1259250112999994e-01, 1.3721022630911658e+05 },
    { 1.0000000000000000e-10, 1.1293483694299999e+00, 7.8731674415076996e+04 },
    { 1.0000000000000000e-10, 2.7349840051299998e+00, 3.2350541665740002e+02 },
    { 8.9999999999999999e-11, 4.9710408415000001e-01, 2.2218566345969998e+03 },
    { 8.9999999999999999e-11, 2.8882169171099998e+00, 9.0695752075120385e+04 },
    { 8.9999999999999999e-11, 4.8039134268700003e+00, 2.7311720982352810e+04 },
    { 8.9999999999999999e-11, 4.2337347441100004e+00, 5.2250587881715699e+04 },
    { 8.9999999999999999e-11, 5.7443725917000004e+00, 2.6729316703313190e+04 },
    { 8.9999999999999999e-11, 6.6918270135000002e-01, 5.2698383701242194e+04 },
    { 8.9999999999999999e-11, 1.5141398953100000e+00, 9.7670387712897180e+04 },
    { 8.9999999999999999e-11, 3.3390967840800001e+00, 1.9111994832171999e+03 },
    { 8.9999999999999999e-11, 6.2816784549199998e+00, 1.4742351185532758e+05 },
    { 8.9999999999999999e-11, 3.5700544141499999e+00, 5.0800032483302588e+04 },
    { 8.9999999999999999e-11, 5.5016071875100003e+00, 2.1990873432869998e+03 },
    { 8.9999999999999999e-11, 2.0884557190900002e+00, 5.1859414413491788e+04 },
    { 8.9999999999999999e-11, 4.2530823041900003e+00, 2.6222012191159200e+04 },
    { 8.9999999999999999e-11, 2.5015752049200000e+00, 2.4864085300795588e+04 },
    { 8.9999999999999999e-11, 2.8548237604300000e+00, 2.5953794091989192e+04 },
    { 8.9999999999999999e-11, 7.8685007134999996e-01, 1.0477821075717278e+05 },
    { 8.9999999999999999e-11, 5.4850676854699998e+00, 7.7844224780847391e+04 },
    { 8.9999999999999999e-11, 1.5736318894300001e+00, 2.4998194350380589e+04 },
    { 8.9999999999999999e-11, 2.5637060492099999e+00, 5.0696939709083992e+04 },
    { 8.9999999999999999e-11, 8.1310827208000003e-01, 1.0594068546158058e+05 },
    { 8.9999999999999999e-11, 5.0003486696500001e+00, 1.0213285546210999e+05 },
    { 8.9999999999999999e-11, 6.3097343517000004e-01, 1.7004917029103660e+05 },
    { 7.9999999999999995e-11, 2.4779465370299998e+00, 2.5440892308259390e+04 },
    { 7.9999999999999995e-11, 2.2977657684300001e+00, 2.3866046506977189e+04 },
    { 7.9999999999999995e-11, 4.6294455131900003e+00, 5.2492198152804987e+04 },
    { 7.9999999999999995e-11, 2.5954926789999999e+00, 2.6941099523326200e+04 },
    { 7.9999999999999995e-11, 4.9578024218100003e+00, 7.8257080865822587e+04 },
    { 7.9999999999999995e-11, 2.5569660975200001e+00, 1.7893627808365600e+04 },
    { 7.9999999999999995e-11, 4.5200236989700002e+00, 1.0435872611329758e+05 },
    { 7.9999999999999995e-11, 3.8252999846900000e+00, 7.6571543755220191e+04 },
    { 7.9999999999999995e-11, 6.5301230309000002e-01, 8.1942753332085995e+03 },
    { 7.9999999999999995e-11, 4.3069163772300003e+00, 7.7410513042970590e+04 },
    { 7.9999999999999995e-11, 5.6164872780800001e+00, 7.7795744434368185e+04 },
    { 7.9999999999999995e-11, 5.5349419683700001e+00, 2.0085575391590000e+03 },
    { 7.9999999999999995e-11, 3.0305318072600000e+00, 2.7780068811076591e+04 },
    { 7.9999999999999995e-11, 1.0907257979400000e+00, 9.4917560896980001e+02 },
    { 7.9999999999999995e-11, 5.2153707877300004e+00, 1.7695398994186718e+05 },
    { 7.9999999999999995e-11, 2.8804058828199999e+00, 3.6109740421673603e+04 },
    { 7.9999999999999995e-11, 3.9087918699099999e+00, 5.4087005766365597e+04 },
    { 7.9999999999999995e-11, 2.0116953934400001e+00, 3.1415379249957001e+04 },
    { 7.9999999999999995e-11, 6.0113682021399999e+00, 2.0764384320240439e+05 },
    { 7.9999999999999995e-11, 6.2422774486200003e+00, 3.8813356576349201e+04 },
    { 7.9999999999999995e-11, 1.3310833680800001e+00, 5.5503941939428587e+04 },
    { 7.9999999999999995e-11, 3.4874652022900001e+00, 5.3242301760338400e+04 },
    { 7.9999999999999995e-11, 5.5647733718700003e+00, 2.7177611932767799e+04 },
    { 7.9999999999999995e-11, 1.6647123568200000e+00, 5.2290245571833613e+04 },
    { 7.9999999999999995e-11, 3.9654139076099999e+00, 5.2325369480282992e+04 },
    { 7.9999999999999995e-11, 6.4215744063000002e-01, 5.3228074666336790e+04 },
    { 7.9999999999999995e-11, 1.8119246908100000e+00, 7.8270337983622594e+04 },
    { 7.0000000000000004e-11, 2.0930405577700002e+00, 2.5344913081041599e+04 },
    { 7.0000000000000004e-11, 3.7798267953700000e+00, 1.9669760899789999e+01 },
    { 7.0000000000000004e-11, 3.3080474167700000e+00, 2.3919142659291600e+04 },
    { 7.0000000000000004e-11, 4.0751314844999997e-01, 2.1116503133776000e+03 },
    { 7.0000000000000004e-11, 7.7215567463000001e-01, 2.6073676047572590e+04 },
    { 7.0000000000000004e-11, 5.3230642687899996e+00, 5.2595290927023590e+04 },
    { 7.0000000000000004e-11, 2.5702373401599998e+00, 2.4448833624386200e+04 },
    { 7.0000000000000004e-11, 1.4033036143199999e+00, 1.5024434299945380e+05 },
    { 7.0000000000000004e-11, 5.4099489395499996e+00, 2.5508215554575399e+04 },
    { 7.0000000000000004e-11, 1.2260207636500000e+00, 1.0779418751126219e+05 },
    { 7.0000000000000004e-11, 2.9982854411500002e+00, 2.2625658435709000e+04 },
    { 7.0000000000000004e-11, 3.3811352724299999e+00, 1.2546481939083000e+04 },
    { 7.0000000000000004e-11, 2.7525233060100001e+00, 1.7240203644480839e+05 },
    { 7.0000000000000004e-11, 9.3184314111000000e-01, 3.3281356562801898e+03 },
    { 7.0000000000000004e-11, 1.1236053840000000e-01, 2.5764397724916791e+04 },
    { 7.0000000000000004e-11, 5.6953776628400004e+00, 6.6653157466348392e+04 },
    { 7.0000000000000004e-11, 1.9823873786899999e+00, 8.4608283475120004e+02 },
    { 7.0000000000000004e-11, 5.2754943432900001e+00, 5.2061366994463169e+04 },
    { 7.0000000000000004e-11, 1.3230578495100001e+00, 2.0981260368468694e+05 },
    { 6.0000000000000000e-11, 2.7373562450900000e+00, 1.5377881048480000e+02 },
    { 6.0000000000000000e-11, 1.5780856187400001e+00, 4.9842609890276392e+04 },
    { 6.0000000000000000e-11, 5.8563997308799998e+00, 1.8208563102592478e+05 },
    { 6.0000000000000000e-11, 5.9882473657000004e+00, 7.6667522982437978e+04 },
    { 6.0000000000000000e-11, 2.7236197052800000e+00, 2.6709646942413401e+04 },
    { 6.0000000000000000e-11, 2.3159588698900002e+00, 1.5705710981453978e+05 },
    { 6.0000000000000000e-11, 1.7355706312200001e+00, 2.6667590728572999e+04 },
    { 6.0000000000000000e-11, 4.3786324278600004e+00, 2.5466159340735001e+04 },
    { 6.0000000000000000e-11, 3.1940316069999999e+00, 1.0427534649502118e+05 },
    { 6.0000000000000000e-11, 1.0982364626500001e+00, 2.5867490499135391e+04 },
    { 6.0000000000000000e-11, 6.5894058731000005e-01, 1.1678365521669458e+05 },
    { 6.0000000000000000e-11, 4.5222176172299999e+00, 2.7726972658762199e+04 },
    { 6.0000000000000000e-11, 9.8402274200000001e-02, 1.6785008294774959e+05 },
    { 6.0000000000000000e-11, 3.3660841507099999e+00, 1.2979961842155698e+05 },
    { 6.0000000000000000e-11, 3.6088434944799999e+00, 7.8903606711036584e+04 },
    { 6.0000000000000000e-11, 5.2639796681200002e+00, 7.5930513031850584e+04 },
    { 6.0000000000000000e-11, 3.2309747926100001e+00, 2.7669862489857191e+04 },
    { 6.0000000000000000e-11, 2.6056245819799999e+00, 4.4295717129809404e+04 },
    { 6.0000000000000000e-11, 5.7613123670500004e+00, 5.8857031136547987e+04 },
    { 6.0000000000000000e-11, 5.0453577398000005e-01, 1.5557181910783658e+05 },
    { 6.0000000000000000e-11, 4.1323503743599996e+00, 2.9550147847439392e+04 },
    { 6.0000000000000000e-11, 3.6581810709900000e+00, 7.6681750076439581e+04 },
    { 5.0000000000000002e-11, 4.6676133413100001e+00, 2.1258774073792001e+03 },
    { 5.0000000000000002e-11, 3.7618309459399999e+00, 2.8256663623856792e+04 },
    { 5.0000000000000002e-11, 5.6806075361500001e+00, 1.2655674786264001e+03 },
    { 5.0000000000000002e-11, 3.8727328322400001e+00, 2.1436455718174577e+05 },
    { 5.0000000000000002e-11, 3.9304765402699999e+00, 3.6523064398203877e+05 },
    { 5.0000000000000002e-11, 1.5643838482600001e+00, 6.0370081616356991e+04 },
    { 5.0000000000000002e-11, 9.1437456221000002e-01, 2.7684089583858800e+04 },
    { 5.0000000000000002e-11, 2.7830441326800002e+00, 2.5169972855592401e+04 },
    { 5.0000000000000002e-11, 2.6708595487500002e+00, 3.8133035637799999e+01 },
    { 5.0000000000000002e-11, 3.8625044982799999e+00, 5.2381991831585590e+04 },
    { 5.0000000000000002e-11, 4.3236177182100004e+00, 2.7005833427555990e+04 },
    { 5.0000000000000002e-11, 1.6378330646000001e+00, 3.4082431583598402e+04 },
    { 5.0000000000000002e-11, 9.0698561940000005e-02, 1.3541421202491190e+04 },
    { 5.0000000000000002e-11, 5.5066399307099996e+00, 2.6402089321443800e+04 },
    { 5.0000000000000002e-11, 5.8899745715499998e+00, 7.8188927826155283e+04 },
    { 5.0000000000000002e-11, 1.2942754579900000e+00, 7.7630925685409384e+04 },
    { 5.0000000000000002e-11, 1.5937864026099999e+00, 2.5773716961704591e+04 },
    { 5.0000000000000002e-11, 5.5398546681500003e+00, 2.4203001978156801e+04 },
    { 5.0000000000000002e-11, 1.1012188518499999e+00, 1.3045918546877075e+05 },
    { 5.0000000000000002e-11, 3.5586675942000001e+00, 5.2252072354423988e+04 },
    { 5.0000000000000002e-11, 3.3995978796199999e+00, 1.0450539137678158e+05 },
    { 5.0000000000000002e-11, 1.9417539342300001e+00, 1.2726810256271999e+03 },
    { 5.0000000000000002e-11, 1.6485365617800001e+00, 2.6198109462793600e+04 },
    { 5.0000000000000002e-11, 1.8946812000999999e-01, 1.4844727083000000e+00 },
    { 5.0000000000000002e-11, 2.1479391624800002e+00, 1.2832075184749259e+05 },
    { 5.0000000000000002e-11, 1.8643870855900000e+00, 5.1653228865054589e+04 },
    { 5.0000000000000002e-11, 3.3641857410700000e+00, 8.6457984757931190e+04 },
    { 3.9999999999999998e-11, 5.2255426593000003e+00, 2.8736357967047199e+04 },
    { 3.9999999999999998e-11, 5.4984365109800004e+00, 5.2712610795243789e+04 },
    { 3.9999999999999998e-11, 5.8115883263000001e-01, 2.6118230002578599e+04 },
    { 3.9999999999999998e-11, 5.4682947960999995e-01, 1.5758680077963436e+05 },
    { 3.9999999999999998e-11, 2.0421274089199999e+00, 1.4807872442630000e+02 },
    { 3.9999999999999998e-11, 5.0477449276600002e+00, 1.1223170171963578e+05 },
    { 3.9999999999999998e-11, 3.2488953004400001e+00, 2.6312247937276101e+04 },
    { 3.9999999999999998e-11, 3.0368101954000002e+00, 1.5104766984290001e+02 },
    { 3.9999999999999998e-11, 2.3608219351000001e-01, 2.6057576280569789e+04 },
    { 3.9999999999999998e-11, 2.7844958116599998e+00, 2.5138727532604400e+04 },
    { 3.9999999999999998e-11, 1.7475032163499999e+00, 2.7044192297544800e+04 },
    { 3.9999999999999998e-11, 5.2658559002900001e+00, 1.3831960486120995e+05 },
    { 3.9999999999999998e-11, 2.1636170505700001e+00, 7.8366802198941179e+04 },
    { 3.9999999999999998e-11, 3.6774568454900001e+00, 1.4853289040742096e+05 },
    { 3.9999999999999998e-11, 4.3981794976800002e+00, 7.8896493164035783e+04 },
    { 3.9999999999999998e-11, 3.1649536317300000e+00, 1.0419783375581198e+05 },
    { 3.9999999999999998e-11, 5.9672058362599998e+00, 1.4859801210651999e+03 },
    { 3.9999999999999998e-11, 4.9585037320199996e+00, 8.0382473285100990e+04 },
    { 3.9999999999999998e-11, 6.0942309597400000e+00, 5.1528795449833589e+04 },
    { 3.9999999999999998e-11, 2.2683272601600000e+00, 6.4901259717923393e+04 },
    { 3.9999999999999998e-11, 5.3723195632299996e+00, 9.9024136457913992e+04 },
    { 3.9999999999999998e-11, 5.8309998100599998e+00, 2.6411408558231600e+04 },
    { 3.9999999999999998e-11, 1.0107965655700000e+00, 1.6703062133498999e+04 },
    { 3.9999999999999998e-11, 2.6153251123099999e+00, 2.7250377845981991e+04 },
    { 3.9999999999999998e-11, 3.8438424623000000e+00, 6.5831666774324796e+04 },
    { 3.9999999999999998e-11, 5.0764783530599997e+00, 1.1882896374949679e+05 },
    { 3.9999999999999998e-11, 9.5691561960000004e-01, 1.7952584437210000e+03 },
    { 3.9999999999999998e-11, 3.2100269985200001e+00, 5.1969620734711192e+04 },
    { 3.9999999999999998e-11, 1.1185037385000001e+00, 5.2609518021025193e+04 },
    { 3.9999999999999998e-11, 3.7017604330200000e+00, 1.6376609444104519e+05 },
    { 3.9999999999999998e-11, 2.5084960028699999e+00, 2.2041264243879999e+02 },
    { 3.0000000000000000e-11, 3.6236951028700002e+00, 5.1852300866490987e+04 },
    { 3.0000000000000000e-11, 2.4273955075000000e-01, 1.2810631931499895e+05 },
    { 3.0000000000000000e-11, 5.2787705071000000e+00, 1.4227094001599999e+01 },
    { 3.0000000000000000e-11, 5.2962444223300000e+00, 5.1432816222615787e+04 },
    { 3.0000000000000000e-11, 1.9937251733300001e+00, 9.3848410080752001e+03 },
    { 3.0000000000000000e-11, 1.0854969432899999e+00, 9.2741060607922584e+04 },
    { 3.0000000000000000e-11, 4.5607105519399997e+00, 1.3041984594697120e+05 },
    { 3.0000000000000000e-11, 5.3542089835999995e-01, 2.5985940623308590e+04 },
    { 3.0000000000000000e-11, 8.6346795613000005e-01, 2.7972804304991590e+04 },
    { 3.0000000000000000e-11, 5.1369735028300001e+00, 2.6189865659839801e+04 },
    { 3.0000000000000000e-11, 3.8316290726800002e+00, 2.0759384658050019e+05 },
    { 3.0000000000000000e-11, 3.7040570094800001e+00, 1.9393798608932379e+05 },
    { 3.0000000000000000e-11, 1.8290755427500001e+00, 1.5308484390447979e+05 },
    { 3.0000000000000000e-11, 1.7567741410200000e+00, 1.5874617595363196e+05 },
    { 3.0000000000000000e-11, 3.4579536632000002e-01, 1.6107937234650398e+05 },
    { 3.0000000000000000e-11, 3.8548383017300001e+00, 3.9629324344065390e+04 },
    { 3.0000000000000000e-11, 8.2694378726999995e-01, 5.1868248662178798e+04 },
    { 3.0000000000000000e-11, 1.6328069810000001e-01, 2.7463676941419999e+04 },
    { 3.0000000000000000e-11, 4.0897164796299998e+00, 1.3022621661243298e+05 },
    { 3.0000000000000000e-11, 1.3185387326700000e+00, 2.0415127163553477e+05 },
    { 3.0000000000000000e-11, 3.2907629459300001e+00, 1.2511203959948818e+05 },
    { 3.0000000000000000e-11, 5.8252711702899997e+00, 4.9953949648551388e+04 },
    { 3.0000000000000000e-11, 1.8325867431200000e+00, 3.7698455099948398e+04 },
    { 3.0000000000000000e-11, 1.8470793636300000e+00, 2.6076857441310302e+04 },
    { 3.0000000000000000e-11, 3.9684094057800001e+00, 1.3043240216087017e+05 },
    { 3.0000000000000000e-11, 1.1928707780000001e-02, 2.6098948841838101e+04 },
    { 3.0000000000000000e-11, 2.2412850767800001e+00, 7.8160616650503987e+04 },
    { 3.0000000000000000e-11, 1.6745769186599999e+00, 1.8218872380014337e+05 },
    { 3.0000000000000000e-11, 1.3602142214799999e+00, 1.3672259155786238e+05 },
};

static const VSOP87Term VSOP87A_MERCURY_Z1[351] = {
    { 1.0872217699999999e-03, 3.9113475082500000e+00, 2.6087903141574199e+04 },
    { 5.7826621000000005e-04, 3.1415926535900001e+00, 0.0000000000000000e+00 },
    { 4.2973519999999999e-05, 2.5637304717700000e+00, 5.2175806283148399e+04 },
    { 2.4358329999999999e-05, 5.1126405059999998e-02, 7.8263709424722590e+04 },
    { 7.9569899999999995e-06, 3.2004108192200000e+00, 1.0435161256629678e+05 },
    { 2.2925099999999999e-06, 5.5843111000000003e-03, 1.3043951570787099e+05 },
    { 6.3404000000000003e-07, 3.0761284368399999e+00, 1.5652741884944518e+05 },
    { 1.7253000000000000e-07, 6.1390863495400003e+00, 1.8261532199101939e+05 },
    { 9.6219999999999997e-08, 3.7621894804000000e-01, 2.7197281693667599e+04 },
    { 6.2970000000000001e-08, 3.9728882750000000e-01, 2.4978524589480799e+04 },
    { 4.6620000000000003e-08, 2.9149269830700000e+00, 2.0870322513259359e+05 },
    { 2.1480000000000001e-08, 3.4857722186300002e+00, 5.3285184835241802e+04 },
    { 1.7520000000000001e-08, 3.0724687161999999e+00, 3.1749235190726398e+04 },
    { 1.7339999999999998e-08, 3.4801277028599999e+00, 5.1066427731055002e+04 },
    { 1.2560000000000000e-08, 5.9716324813900004e+00, 2.3479112827416777e+05 },
    { 9.1899999999999999e-09, 2.2024781129900002e+00, 5.1116424352959199e+04 },
    { 8.9299999999999996e-09, 2.8354964433499998e+00, 1.1093785520934000e+03 },
    { 8.3400000000000006e-09, 6.5008044610000004e-01, 2.5028521211384999e+04 },
    { 7.0600000000000000e-09, 4.0998943747499998e+00, 2.1535949644515400e+04 },
    { 5.9900000000000002e-09, 1.3070787841100000e+00, 5.6613320491521999e+03 },
    { 5.7299999999999999e-09, 4.2722633544199997e+00, 1.0593819301891999e+03 },
    { 5.4899999999999999e-09, 4.5577282764499998e+00, 2.7043502883182799e+04 },
    { 5.0300000000000002e-09, 2.6513156341999999e-01, 7.7154330872629187e+04 },
    { 3.9499999999999998e-09, 4.3207315642699999e+00, 4.7623852786089599e+04 },
    { 3.9000000000000002e-09, 3.0582923967500002e+00, 2.6107572902473989e+04 },
    { 3.7000000000000000e-09, 2.6188837520000002e-01, 7.9373087976815994e+04 },
    { 3.6500000000000000e-09, 2.5587601358000001e-01, 5.2969096509459996e+02 },
    { 3.4699999999999998e-09, 5.4294071198599996e+00, 3.7410567239878597e+04 },
    { 3.4200000000000002e-09, 6.2016216829999998e-02, 5.7837138332300601e+04 },
    { 3.3799999999999999e-09, 2.7436103133200000e+00, 2.6087903141574195e+05 },
    { 3.2900000000000000e-09, 2.5790814267000002e+00, 2.5132303399965600e+04 },
    { 3.2599999999999999e-09, 3.0988826764000001e+00, 2.0426571092422000e+04 },
    { 3.1899999999999999e-09, 4.0663879490400001e+00, 2.6068233380674399e+04 },
    { 3.1500000000000001e-09, 5.9773284501899999e+00, 2.6084021806216198e+04 },
    { 3.0300000000000001e-09, 2.9759778033400002e+00, 3.0639856638632999e+04 },
    { 2.8299999999999999e-09, 6.1887039139400004e+00, 1.5890728952838001e+03 },
    { 2.8200000000000002e-09, 7.2426739653000005e-01, 4.6514474233996203e+04 },
    { 2.6700000000000001e-09, 3.5017012818300000e+00, 2.5558212176479599e+04 },
    { 2.6700000000000001e-09, 1.1399044974799999e+00, 2.6091784476932200e+04 },
    { 2.3600000000000001e-09, 1.5764278442199999e+00, 2.7147285071763388e+04 },
    { 2.3600000000000001e-09, 2.2252535306499999e+00, 4.5519534970588002e+03 },
    { 2.3499999999999999e-09, 1.4970128940600000e+00, 1.4765239043269799e+04 },
    { 2.3199999999999998e-09, 2.6958930513800001e+00, 1.0213285546211000e+04 },
    { 1.9399999999999999e-09, 1.0217354284600000e+00, 5.0586733387864588e+04 },
    { 1.7500000000000000e-09, 1.2964063811000001e-01, 2.8306660245760992e+04 },
    { 1.6700000000000000e-09, 1.4184589057600001e+00, 5.3131406024757001e+04 },
    { 1.5799999999999999e-09, 4.7201169197799997e+00, 1.3521751441591399e+04 },
    { 1.4800000000000001e-09, 5.0941562343000002e-01, 2.3869146037387400e+04 },
    { 1.4800000000000001e-09, 2.6760627063200002e+00, 3.2858613742819791e+04 },
    { 1.4800000000000001e-09, 3.9755833354600001e+00, 2.6617594106668799e+04 },
    { 1.4599999999999999e-09, 3.3280908524799999e+00, 1.0324223401420339e+05 },
    { 1.4400000000000000e-09, 6.1747185403399998e+00, 3.9609654583165597e+04 },
    { 1.4300000000000000e-09, 5.6054926316399998e+00, 2.5661304950698199e+04 },
    { 1.4200000000000001e-09, 5.5153870561599998e+00, 7.7204327494533383e+04 },
    { 1.3600000000000001e-09, 2.4480362680300001e+00, 2.4498830246290399e+04 },
    { 1.3500000000000001e-09, 5.2902370274599999e+00, 1.5874617595363199e+04 },
    { 1.3500000000000001e-09, 4.5129411958999999e+00, 2.2645328196608789e+04 },
    { 1.3100000000000000e-09, 5.1039682738599996e+00, 5.1646115318053788e+04 },
    { 1.2799999999999999e-09, 4.1688533618200001e+00, 1.9317192540328600e+04 },
    { 1.2700000000000000e-09, 1.7470810745200001e+00, 4.1962520736937397e+04 },
    { 1.2300000000000001e-09, 2.9219981675400000e+00, 7.1135470007999997e+00 },
    { 1.1900000000000000e-09, 3.8935105071199998e+00, 7.2602377375570388e+04 },
    { 1.1800000000000001e-09, 5.3489982198400003e+00, 9.1039069941176003e+03 },
    { 1.1400000000000000e-09, 1.6157106154200001e+00, 5.1749208092272391e+04 },
    { 1.1100000000000001e-09, 5.6169340668199998e+00, 5.1220206541539788e+04 },
    { 1.0800000000000000e-09, 2.2998975495299998e+00, 1.9804827291582798e+04 },
    { 1.0500000000000001e-09, 2.3564312901800002e+00, 6.3498470381452789e+04 },
    { 1.0200000000000000e-09, 1.5140847395199999e+00, 4.3071899289030800e+04 },
    { 9.9000000000000010e-10, 2.9152880199400002e+00, 3.4425749449653999e+03 },
    { 9.0999999999999996e-10, 5.7982538714200000e+00, 2.8696693455731618e+05 },
    { 8.9999999999999999e-10, 3.1041171413700002e+00, 2.9530478086539599e+04 },
    { 8.6999999999999999e-10, 7.2759501704999996e-01, 9.5559974160859997e+02 },
    { 8.6999999999999999e-10, 6.0261834660500000e+00, 5.6727759780207198e+04 },
    { 8.6000000000000003e-10, 1.2685328559700000e+00, 5.2705497248242988e+04 },
    { 8.4999999999999996e-10, 4.4270372323700000e+00, 6.3989728631399998e+02 },
    { 8.3000000000000003e-10, 4.8615550553000002e+00, 5.3235188213337591e+04 },
    { 8.0999999999999999e-10, 5.0340529336999995e-01, 1.2566151699982800e+04 },
    { 7.7000000000000003e-10, 1.8551522096199999e+00, 1.0664954771900000e+03 },
    { 7.5000000000000000e-10, 2.3929547791700001e+00, 1.0522683831884001e+03 },
    { 7.4000000000000003e-10, 8.3774903676000001e-01, 5.2156136522248598e+04 },
    { 7.2000000000000000e-10, 5.1520065284200003e+00, 6.7707106012455997e+03 },
    { 7.1000000000000003e-10, 6.1072538463899999e+00, 5.2195476044048191e+04 },
    { 6.9999999999999996e-10, 1.3020706977700001e+00, 7.3711755927663791e+04 },
    { 6.9000000000000000e-10, 1.6761666006400000e+00, 1.1322664098304400e+04 },
    { 6.3999999999999996e-10, 2.3739028324400002e+00, 6.2830758499914000e+03 },
    { 6.2000000000000003e-10, 5.3772034528000001e-01, 2.5035634758385801e+04 },
    { 6.2000000000000003e-10, 5.2160707898499998e+00, 2.1329909543799999e+02 },
    { 6.0999999999999996e-10, 4.1987230189300000e+00, 5.2179687618506403e+04 },
    { 5.7999999999999996e-10, 2.7457973477700000e+00, 5.2171924947790401e+04 },
    { 5.7000000000000000e-10, 4.5515047182900004e+00, 4.0853142184844000e+04 },
    { 5.7000000000000000e-10, 2.7567932616299999e+00, 2.7676976036857999e+04 },
    { 5.7000000000000000e-10, 5.9848507207299999e+00, 5.8946516884393990e+04 },
    { 5.4999999999999996e-10, 3.3738830627400000e+00, 5.4394563387335191e+04 },
    { 5.4999999999999996e-10, 4.7986166262700003e+00, 2.5448005855260191e+04 },
    { 5.4000000000000000e-10, 3.1327490759800001e+00, 1.6983996147456601e+04 },
    { 5.4000000000000000e-10, 6.0358950682700003e+00, 2.6080789594573391e+04 },
    { 5.3000000000000003e-10, 3.3719501083100001e+00, 1.0546099111839019e+05 },
    { 5.1999999999999996e-10, 1.0806529248700001e+00, 2.5021407664384202e+04 },
    { 5.1999999999999996e-10, 5.9002592012299999e+00, 2.6095016688575000e+04 },
    { 5.0000000000000003e-10, 2.6670985671800000e+00, 2.2187571041868000e+03 },
    { 4.8000000000000000e-10, 2.4299105578400000e+00, 5.1535908996834391e+04 },
    { 4.8000000000000000e-10, 5.6028391392999999e-01, 2.1187638603783998e+03 },
    { 4.7000000000000003e-10, 3.3047498351600000e+00, 2.5874604046136199e+04 },
    { 4.6000000000000001e-10, 3.9571379020399999e+00, 7.6674636529438780e+04 },
    { 4.3999999999999998e-10, 3.3977842946800001e+00, 5.2168692736147590e+04 },
    { 4.3000000000000001e-10, 1.0468670720999999e-01, 1.2933013715577759e+05 },
    { 4.2000000000000000e-10, 5.1255029352400001e+00, 1.8849227549974199e+04 },
    { 4.0999999999999998e-10, 6.9895535279999998e-01, 9.8690280517144580e+04 },
    { 3.9000000000000000e-10, 1.3892572483800001e+00, 2.6301202237012199e+04 },
    { 3.9000000000000000e-10, 3.9276434111800000e+00, 4.8733231338182988e+04 },
    { 3.7999999999999998e-10, 5.1086735246800004e+00, 5.1109310805958390e+04 },
    { 3.7999999999999998e-10, 5.0824294519100004e+00, 4.2659819087599999e+02 },
    { 3.7000000000000001e-10, 4.0633556545999996e+00, 3.8654054841556987e+04 },
    { 3.6000000000000000e-10, 2.5865126139900001e+00, 3.3326578733174203e+04 },
    { 3.6000000000000000e-10, 2.3784980853700000e+00, 7.7308109683113988e+04 },
    { 3.6000000000000000e-10, 3.0154478969500000e+00, 6.5697557724739789e+04 },
    { 3.4999999999999998e-10, 2.5119964910100001e+00, 1.1001294461544899e+05 },
    { 3.4999999999999998e-10, 4.8150054360399999e+00, 6.9159802430604992e+04 },
    { 3.4000000000000001e-10, 3.5825041399400002e+00, 4.9957049178961599e+04 },
    { 3.4000000000000001e-10, 4.9803752408999999e+00, 8.3925041473874793e+04 },
    { 3.3000000000000000e-10, 2.9899839981500000e+00, 5.2182919830149200e+04 },
    { 3.3000000000000000e-10, 4.3315542078600000e+00, 7.8793400389817180e+04 },
    { 3.0000000000000000e-10, 2.2668996171800000e+00, 2.6514501332450200e+04 },
    { 3.0000000000000000e-10, 5.3205764802799997e+00, 5.9414481874748402e+04 },
    { 3.0000000000000000e-10, 4.4763645003499999e+00, 5.1123537899959993e+04 },
    { 2.8999999999999998e-10, 1.0950522744000001e-01, 3.2370978991565600e+04 },
    { 2.8000000000000002e-10, 1.6826624960200001e+00, 1.3655860491176400e+04 },
    { 2.8000000000000002e-10, 9.4446303339000004e-01, 4.5405095681902800e+04 },
    { 2.8000000000000002e-10, 1.7428279098499999e+00, 7.9323091354911783e+04 },
    { 2.8000000000000002e-10, 1.9971812562100000e+00, 2.3969139281195799e+04 },
    { 2.7000000000000000e-10, 4.3668310411000002e+00, 4.5892730433156990e+04 },
    { 2.7000000000000000e-10, 2.8099491907299998e+00, 8.2815662921781390e+04 },
    { 2.7000000000000000e-10, 2.8076690939999999e-01, 5.0057042422769999e+04 },
    { 2.7000000000000000e-10, 5.9499161174199999e+00, 3.6301188687785187e+04 },
    { 2.5000000000000002e-10, 3.8108558399899999e+00, 1.0329223063610759e+05 },
    { 2.4000000000000000e-10, 2.5684416758499999e+00, 3.1305483769889036e+05 },
    { 2.3000000000000001e-10, 5.0210000195199997e+00, 3.8519945791972001e+04 },
    { 2.3000000000000001e-10, 2.2972921238000001e+00, 2.5934124331089399e+04 },
    { 2.1999999999999999e-10, 5.4887995084399996e+00, 4.3371173787679999e+02 },
    { 2.1999999999999999e-10, 5.1174256539099998e+00, 5.2389105378586399e+04 },
    { 2.1000000000000000e-10, 6.1254530074500000e+00, 5.5618381228113802e+04 },
    { 2.1000000000000000e-10, 6.1179676662900002e+00, 4.1948464387519999e+02 },
    { 2.1000000000000000e-10, 4.4391407307800002e+00, 7.9219309166331193e+04 },
    { 2.0000000000000001e-10, 5.0204562576000002e+00, 6.2389091829359393e+04 },
    { 2.0000000000000001e-10, 1.4541784359900001e+00, 6.6941045326418185e+04 },
    { 1.8999999999999999e-10, 3.8913123118000001e+00, 7.8244039663822798e+04 },
    { 1.8000000000000000e-10, 1.1466656856899999e+00, 1.2432042650397800e+04 },
    { 1.8000000000000000e-10, 4.3545042634600000e+00, 2.3754706748702189e+04 },
    { 1.8000000000000000e-10, 2.8393859120000000e+00, 2.5668418497699000e+04 },
    { 1.8000000000000000e-10, 3.8080895085200002e+00, 2.5973463852888959e+04 },
    { 1.7000000000000001e-10, 1.2684639895800001e+00, 1.2938013377768178e+05 },
    { 1.7000000000000001e-10, 1.2129521472800000e+00, 2.5619938151219802e+04 },
    { 1.7000000000000001e-10, 4.8834145255900001e+00, 7.7837111233846590e+04 },
    { 1.7000000000000001e-10, 5.6452545244400003e+00, 1.3610084775702318e+05 },
    { 1.7000000000000001e-10, 5.8355650806800003e+00, 5.3764879178432202e+04 },
    { 1.5999999999999999e-10, 3.4367410970700001e+00, 2.5654191403697390e+04 },
    { 1.5999999999999999e-10, 3.6425542703499998e+00, 8.5319638175199998e+02 },
    { 1.5999999999999999e-10, 3.3178443715400001e+00, 2.6202342430259410e+04 },
    { 1.5000000000000000e-10, 1.2323301393699999e+00, 2.6727800427888200e+04 },
    { 1.5000000000000000e-10, 9.7248576910999995e-01, 7.8267590760080580e+04 },
    { 1.5000000000000000e-10, 1.9106340267199999e+00, 4.4937130691548402e+04 },
    { 1.5000000000000000e-10, 1.1148443148300000e+00, 7.4821134479757180e+04 },
    { 1.4000000000000001e-10, 5.4418292116000000e+00, 8.9586373523026981e+04 },
    { 1.4000000000000001e-10, 2.8742750005100000e+00, 7.8283379185622383e+04 },
    { 1.4000000000000001e-10, 2.4637605599899999e+00, 1.5961864422845999e+03 },
    { 1.4000000000000001e-10, 3.7712251008900002e+00, 1.2477818365871879e+05 },
    { 1.2999999999999999e-10, 4.9507407596600004e+00, 2.7154398618764200e+04 },
    { 1.2999999999999999e-10, 9.8498982943000002e-01, 6.4741957983131193e+04 },
    { 1.2999999999999999e-10, 3.1758665022599999e+00, 2.8421099534446199e+04 },
    { 1.2000000000000000e-10, 3.1578175475300001e+00, 1.5541804029735178e+05 },
    { 1.2000000000000000e-10, 5.7988066680100001e+00, 7.8259828089364586e+04 },
    { 1.2000000000000000e-10, 7.8485262109999998e-01, 5.1962507187710398e+04 },
    { 1.2000000000000000e-10, 4.3415206807100004e+00, 9.4138327020085781e+04 },
    { 1.2000000000000000e-10, 5.3806150304399996e+00, 5.2602404474024399e+04 },
    { 1.2000000000000000e-10, 5.5987850131499997e+00, 7.7623812138408583e+04 },
    { 1.2000000000000000e-10, 2.0589141568000000e-01, 7.8256595877721789e+04 },
    { 1.0999999999999999e-10, 1.1261576850000000e-02, 8.0482466528909383e+04 },
    { 1.0999999999999999e-10, 5.4247151716299999e+00, 1.0339601282468818e+05 },
    { 1.0999999999999999e-10, 1.9209183337400000e+00, 7.7197213947532582e+04 },
    { 1.0999999999999999e-10, 2.5104358367400001e+00, 7.7734018459627987e+04 },
    { 1.0999999999999999e-10, 2.6958992216900000e+00, 2.6137899763478399e+04 },
    { 1.0999999999999999e-10, 3.0468128561000002e-01, 6.8050423878511589e+04 },
    { 1.0999999999999999e-10, 1.0857959185399999e+00, 1.0488130353139139e+05 },
    { 1.0999999999999999e-10, 1.9506031520200000e+00, 1.5819593482830001e+03 },
    { 1.0999999999999999e-10, 2.3907174076300000e+00, 6.4607848933546193e+04 },
    { 1.0000000000000000e-10, 4.0532223176800004e+00, 2.6241681952059000e+04 },
    { 1.0000000000000000e-10, 1.3926558774000000e+00, 5.1742094545271590e+04 },
    { 1.0000000000000000e-10, 6.0817510579600000e+00, 7.8270822971723392e+04 },
    { 1.0000000000000000e-10, 3.0792827903600002e+00, 2.3331963928720002e+03 },
    { 1.0000000000000000e-10, 1.3638939129000001e+00, 7.7211441041534184e+04 },
    { 8.9999999999999999e-11, 5.5338873843799998e+00, 7.9945284420241996e+03 },
    { 8.9999999999999999e-11, 4.0604041041000000e-01, 2.4491716699289591e+04 },
    { 8.9999999999999999e-11, 1.9139278009300000e+00, 8.8476994970933592e+04 },
    { 8.9999999999999999e-11, 3.5558235819000000e-01, 7.6044952320535798e+04 },
    { 8.9999999999999999e-11, 4.4649054231300003e+00, 2.6037906519669988e+04 },
    { 8.9999999999999999e-11, 4.8522947905600002e+00, 1.0541099449648599e+05 },
    { 8.9999999999999999e-11, 2.5117548322999999e+00, 8.5034420025968182e+04 },
    { 8.9999999999999999e-11, 5.5584971452499996e+00, 2.7140171524762591e+04 },
    { 7.9999999999999995e-11, 4.6923598804899997e+00, 5.3680451209540001e+02 },
    { 7.9999999999999995e-11, 4.9443430628899998e+00, 6.3278373931320004e+02 },
    { 7.9999999999999995e-11, 4.9047863022999999e-01, 1.0276253967101299e+05 },
    { 7.9999999999999995e-11, 5.8548332720299996e+00, 1.0890356606335558e+05 },
    { 7.9999999999999995e-11, 4.0378554762799999e+00, 7.1492998823476984e+04 },
    { 7.9999999999999995e-11, 1.2215985800000001e-02, 5.3274761083827998e+03 },
    { 7.9999999999999995e-11, 4.4668324162399999e+00, 1.5546803691925600e+05 },
    { 7.9999999999999995e-11, 3.5049912273100001e+00, 2.8206667001952599e+04 },
    { 7.9999999999999995e-11, 3.2170138527700001e+00, 7.6144945564344191e+04 },
    { 7.9999999999999995e-11, 1.1431045599500000e+00, 1.0382192160120218e+05 },
    { 7.9999999999999995e-11, 5.4834070749799997e+00, 2.5234706759822191e+04 },
    { 7.9999999999999995e-11, 6.6014034659999998e-01, 5.1756321639273199e+04 },
    { 7.9999999999999995e-11, 5.4081102338999998e-01, 3.5191810135691798e+04 },
    { 7.9999999999999995e-11, 2.0606559881300002e+00, 7.1980633574731182e+04 },
    { 7.0000000000000004e-11, 5.7898880804199999e+00, 2.6555868131928601e+04 },
    { 7.0000000000000004e-11, 4.2785321296800003e+00, 1.8207813988235212e+04 },
    { 7.0000000000000004e-11, 2.0296219917800000e+00, 7.8477008520160583e+04 },
    { 7.0000000000000004e-11, 3.2078078729700001e+00, 1.8093374699549990e+04 },
    { 7.0000000000000004e-11, 3.2991745292500001e+00, 5.1322609901396390e+04 },
    { 7.0000000000000004e-11, 4.0444236857300000e+00, 2.5455119402261000e+04 },
    { 7.0000000000000004e-11, 7.0887703357999998e-01, 2.4505943793291190e+04 },
    { 7.0000000000000004e-11, 3.6768945010100000e+00, 1.2588756221081219e+05 },
    { 7.0000000000000004e-11, 4.4938634129999997e+00, 5.0579619840863787e+04 },
    { 7.0000000000000004e-11, 3.3385809006199998e+00, 5.8458882133139792e+04 },
    { 7.0000000000000004e-11, 9.1622586929000005e-01, 4.6848330174765601e+04 },
    { 7.0000000000000004e-11, 5.9795095252800001e+00, 9.9799659069237983e+04 },
    { 7.0000000000000004e-11, 5.5980655039200000e+00, 3.3914274084046460e+05 },
    { 7.0000000000000004e-11, 2.3589902540700001e+00, 3.3967992294913187e+04 },
    { 6.0000000000000000e-11, 4.7136223972200000e+00, 6.4701083331480004e+02 },
    { 6.0000000000000000e-11, 2.8481675557999999e+00, 8.1706284369687986e+04 },
    { 6.0000000000000000e-11, 4.5449773461299996e+00, 9.3028948467992392e+04 },
    { 6.0000000000000000e-11, 1.2372382661800001e+00, 4.4181277841124189e+04 },
    { 6.0000000000000000e-11, 1.2206731331400000e+00, 1.2022623016165999e+05 },
    { 6.0000000000000000e-11, 1.2090013340000001e+00, 2.6484548254729998e+03 },
    { 6.0000000000000000e-11, 2.4635511195099999e+00, 1.6218875089859738e+05 },
    { 5.0000000000000002e-11, 4.9680969079799997e+00, 5.2026243086013797e+04 },
    { 5.0000000000000002e-11, 3.2869072049299999e+00, 2.5938339944439598e+04 },
    { 5.0000000000000002e-11, 4.9969529623900000e+00, 7.1025033833122594e+04 },
    { 5.0000000000000002e-11, 3.8054020782400002e+00, 7.8050410329284583e+04 },
    { 5.0000000000000002e-11, 5.4627208165400001e+00, 2.6610480559667991e+04 },
    { 5.0000000000000002e-11, 6.3510871715999995e-01, 1.0433194280539699e+05 },
    { 5.0000000000000002e-11, 1.6321741539700001e+00, 9.1785460866313981e+04 },
    { 5.0000000000000002e-11, 4.3129415128100002e+00, 1.0021837280099400e+04 },
    { 5.0000000000000002e-11, 6.7943011611000004e-01, 5.2257741809380002e+02 },
    { 5.0000000000000002e-11, 2.5988816486899999e+00, 7.9852782320006387e+04 },
    { 5.0000000000000002e-11, 5.3270000698800004e+00, 2.5551098629478791e+04 },
    { 3.9999999999999998e-11, 5.7167819424499999e+00, 9.8068536716305389e+04 },
    { 3.9999999999999998e-11, 3.5209916493300000e+00, 2.6507387785449391e+04 },
    { 3.9999999999999998e-11, 5.3135339765599996e+00, 1.1787336400788819e+05 },
    { 3.9999999999999998e-11, 5.1319474337099997e+00, 7.8800891533389904e+03 },
    { 3.9999999999999998e-11, 4.8525905721799996e+00, 5.0593846934865389e+04 },
    { 3.9999999999999998e-11, 3.5820586809499999e+00, 4.5494581429748789e+04 },
    { 3.9999999999999998e-11, 5.6456514852999995e-01, 1.5086608680029298e+05 },
    { 3.9999999999999998e-11, 4.0548215611299998e+00, 1.0435549390165479e+05 },
    { 3.9999999999999998e-11, 7.8297474988000004e-01, 2.6081274582674188e+04 },
    { 3.9999999999999998e-11, 4.4261512227499997e+00, 5.2815703569462399e+04 },
    { 3.9999999999999998e-11, 4.3843490897699997e+00, 1.2990982474277639e+05 },
    { 3.9999999999999998e-11, 4.5278220799400000e+00, 3.9743763632750597e+04 },
    { 3.9999999999999998e-11, 2.2094119044500000e+00, 2.6521614879451001e+04 },
    { 3.9999999999999998e-11, 6.4407233905000005e-01, 2.2759767485294011e+04 },
    { 3.9999999999999998e-11, 8.7584552144000005e-01, 9.5247705572179184e+04 },
    { 3.9999999999999998e-11, 2.2221687399799999e+00, 5.1639001771052986e+04 },
    { 3.9999999999999998e-11, 4.0755936715700001e+00, 1.3096920667296558e+05 },
    { 3.9999999999999998e-11, 1.9940442365600000e+00, 5.3242301760338400e+04 },
    { 3.9999999999999998e-11, 1.0237777214000000e-01, 5.2290245571833613e+04 },
    { 3.9999999999999998e-11, 6.1900791099300001e+00, 1.8150594343892598e+05 },
    { 3.9999999999999998e-11, 4.1350993910199998e+00, 9.0829861124705378e+04 },
    { 3.9999999999999998e-11, 3.8110794859900001e+00, 7.2386755916000002e+03 },
    { 3.9999999999999998e-11, 2.2248720485800000e+00, 7.8690307615598591e+04 },
    { 3.9999999999999998e-11, 2.1293011176000001e-01, 2.9416038797854391e+04 },
    { 3.9999999999999998e-11, 4.6609484670599999e+00, 2.6190995915792791e+04 },
    { 3.9999999999999998e-11, 1.4413855231299999e+00, 5.2329585093633192e+04 },
    { 3.9999999999999998e-11, 3.9228294912699999e+00, 2.6094531700474210e+04 },
    { 3.9999999999999998e-11, 3.6706915226999998e-01, 2.6624707653669589e+04 },
    { 3.9999999999999998e-11, 2.6961475090999998e+00, 4.2153969003049002e+04 },
    { 3.9999999999999998e-11, 5.0247639263800004e+00, 1.1456489811250778e+05 },
    { 3.0000000000000000e-11, 5.3651424673300001e+00, 5.2099540211872802e+04 },
    { 3.0000000000000000e-11, 4.5709015034500000e+00, 1.0329934418310839e+05 },
    { 3.0000000000000000e-11, 2.1678294256399999e+00, 1.2948391596626239e+05 },
    { 3.0000000000000000e-11, 3.7043968282700002e+00, 2.6011637070298599e+04 },
    { 3.0000000000000000e-11, 3.3859336670700002e+00, 2.2747290714874402e+04 },
    { 3.0000000000000000e-11, 2.9068552202200002e+00, 1.0435872611329758e+05 },
    { 3.0000000000000000e-11, 3.8812083875300001e+00, 2.4864085300795588e+04 },
    { 3.0000000000000000e-11, 3.2319500999700002e+00, 2.7311720982352810e+04 },
    { 3.0000000000000000e-11, 5.9206616691000002e-01, 1.5197546535238638e+05 },
    { 3.0000000000000000e-11, 5.0226336585400002e+00, 1.3362449706799200e+04 },
    { 3.0000000000000000e-11, 4.4881275478899996e+00, 7.7829997686845789e+04 },
    { 3.0000000000000000e-11, 1.6136445675600000e+00, 1.3149889763806018e+05 },
    { 3.0000000000000000e-11, 2.3591853071100002e+00, 5.3228074666336790e+04 },
    { 3.0000000000000000e-11, 6.2394363129199997e+00, 5.4509002676020398e+04 },
    { 3.0000000000000000e-11, 5.9645447449100004e+00, 1.0437128232719658e+05 },
    { 3.0000000000000000e-11, 1.7234577038900001e+00, 2.0760427033191401e+04 },
    { 3.0000000000000000e-11, 5.0072748279000001e-01, 2.6941099523326200e+04 },
    { 3.0000000000000000e-11, 1.9697761144699999e+00, 2.5565325723480400e+04 },
    { 3.0000000000000000e-11, 5.1162418778500003e+00, 2.0618554843720000e+02 },
    { 3.0000000000000000e-11, 5.1349992272000000e-01, 1.3154889425996438e+05 },
    { 3.0000000000000000e-11, 3.9525647748899999e+00, 7.7844224780847391e+04 },
    { 3.0000000000000000e-11, 6.1974117562700002e+00, 1.4788665740644001e+03 },
    { 3.0000000000000000e-11, 4.9620259289000002e-01, 8.4546785274713984e+04 },
    { 3.0000000000000000e-11, 5.5085259266999997e-01, 5.2061366994463169e+04 },
    { 3.0000000000000000e-11, 3.6066050068700002e+00, 9.0695752075120385e+04 },
    { 3.0000000000000000e-11, 2.5582162551800001e+00, 1.0371171527998279e+05 },
    { 3.0000000000000000e-11, 6.0936070752599996e+00, 3.3406124266997999e+03 },
    { 3.0000000000000000e-11, 5.6512594289300004e+00, 5.2022027472663598e+04 },
    { 3.0000000000000000e-11, 3.2028924029499999e+00, 1.0657036967048359e+05 },
    { 3.0000000000000000e-11, 2.5181383803499999e+00, 1.0090903762133139e+05 },
    { 3.0000000000000000e-11, 2.6801087052099999e+00, 5.2643771273502804e+04 },
    { 3.0000000000000000e-11, 5.9209511857499999e+00, 4.9527351457675388e+04 },
    { 3.0000000000000000e-11, 1.3650027525899999e+00, 1.8155594006083018e+05 },
    { 3.0000000000000000e-11, 1.6059305604300000e+00, 1.4956319713459999e+02 },
    { 3.0000000000000000e-11, 3.4557086096999998e+00, 1.0213285546210999e+05 },
    { 3.0000000000000000e-11, 3.2562974938200000e+00, 1.0392501437542078e+05 },
    { 3.0000000000000000e-11, 5.2645884660700002e+00, 6.0055895436487393e+04 },
    { 1.9999999999999999e-11, 5.2716380464799997e+00, 1.0456491166173479e+05 },
    { 1.9999999999999999e-11, 9.1771435129000001e-01, 9.7580901965051191e+04 },
    { 1.9999999999999999e-11, 5.5927230069200000e+00, 2.7684089583858800e+04 },
    { 1.9999999999999999e-11, 4.9965613414699996e+00, 1.0328511708910679e+05 },
    { 1.9999999999999999e-11, 2.4415302212199999e+00, 6.6812248533995999e+03 },
    { 1.9999999999999999e-11, 6.1620335688200001e+00, 1.5763679740153858e+05 },
    { 1.9999999999999999e-11, 6.0700635539099999e+00, 1.1112232316754239e+05 },
    { 1.9999999999999999e-11, 1.2301894526799999e+00, 5.1543022543835199e+04 },
    { 1.9999999999999999e-11, 5.4483217333800003e+00, 1.8827665404017159e+05 },
    { 1.9999999999999999e-11, 3.3555584613499998e+00, 1.0434449901929598e+05 },
    { 1.9999999999999999e-11, 1.1141040033400000e+00, 3.1415379249957001e+04 },
    { 1.9999999999999999e-11, 1.2419697744900000e+00, 5.2125809661244188e+04 },
    { 1.9999999999999999e-11, 3.4823857204699999e+00, 2.5881717593137000e+04 },
    { 1.9999999999999999e-11, 2.4667713839299998e+00, 3.3281356562801898e+03 },
    { 1.9999999999999999e-11, 2.2036817569500000e+00, 1.4396126714946239e+05 },
    { 1.9999999999999999e-11, 4.8445939880899997e+00, 7.0269180982698381e+04 },
    { 1.9999999999999999e-11, 1.6808341332200001e+00, 2.5984810367355600e+04 },
    { 1.9999999999999999e-11, 2.4515794103799999e+00, 5.2698383701242194e+04 },
    { 1.9999999999999999e-11, 7.8207794200000003e-01, 1.3001291751699499e+05 },
    { 1.9999999999999999e-11, 9.8920727566000000e-01, 7.6266071275599998e+01 },
    { 1.9999999999999999e-11, 3.9256824537399999e+00, 9.5628915597059995e+02 },
    { 1.9999999999999999e-11, 2.6508224628399999e+00, 1.1610551958374201e+04 },
    { 1.9999999999999999e-11, 2.6874981495600001e+00, 1.2415643985787958e+05 },
    { 1.9999999999999999e-11, 4.3366595928300002e+00, 1.4631413330323418e+05 },
    { 1.9999999999999999e-11, 1.5981567999999999e-01, 7.7410513042970590e+04 },
    { 1.9999999999999999e-11, 3.2248774042000000e-01, 1.5599772788435058e+05 },
    { 1.9999999999999999e-11, 6.1280375275600001e+00, 1.0413831347085879e+05 },
    { 1.9999999999999999e-11, 2.1368583386000001e-01, 5.4294570143526791e+04 },
    { 1.9999999999999999e-11, 6.5535255590999997e-01, 1.1624747044077999e+03 },
    { 1.9999999999999999e-11, 3.2992977468400002e+00, 1.2885044281258718e+05 },
    { 1.9999999999999999e-11, 2.0177238141800000e+00, 1.3499146920492980e+05 },
    { 1.9999999999999999e-11, 5.3932839125300003e+00, 3.2769127994973802e+04 },
    { 1.9999999999999999e-11, 5.8606871358800001e+00, 1.6066065861474801e+04 },
    { 1.9999999999999999e-11, 2.2175188395199998e+00, 1.0309277421860000e+02 },
    { 1.9999999999999999e-11, 1.6002159588800000e+00, 5.1528795449833589e+04 },
    { 9.9999999999999994e-12, 4.1204573464800003e+00, 2.5440892308259390e+04 },
    { 9.9999999999999994e-12, 6.2723758876000000e+00, 1.0594068546158058e+05 },
    { 9.9999999999999994e-12, 1.1853909415199999e+00, 1.2546481939083000e+04 },
    { 9.9999999999999994e-12, 1.6039883189399999e+00, 2.6294088690011391e+04 },
    { 9.9999999999999994e-12, 3.9906075889400001e+00, 2.5867490499135391e+04 },
};

static const VSOP87Term VSOP87A_MERCURY_Z2[143] = {
    { 1.1850240000000000e-05, 0.0000000000000000e+00, 0.0000000000000000e+00 },
    { 1.0531180000000000e-05, 5.3797921435699996e+00, 2.6087903141574199e+04 },
    { 8.7112999999999999e-07, 4.2206932429999999e-01, 5.2175806283148399e+04 },
    { 4.9533999999999999e-07, 4.3805403976899999e+00, 7.8263709424722590e+04 },
    { 2.5684999999999997e-07, 1.4582744316200000e+00, 1.0435161256629678e+05 },
    { 1.0457000000000000e-07, 4.6222469173700000e+00, 1.3043951570787099e+05 },
    { 3.7569999999999998e-08, 1.4458995477600001e+00, 1.5652741884944518e+05 },
    { 1.2590000000000001e-08, 4.5297472893200004e+00, 1.8261532199101939e+05 },
    { 5.1000000000000002e-09, 2.0274155988899998e+00, 2.7197281693667599e+04 },
    { 4.0499999999999999e-09, 1.3190394210800001e+00, 2.0870322513259359e+05 },
    { 1.9399999999999999e-09, 4.8465639902900000e+00, 2.4978524589480799e+04 },
    { 1.2600000000000000e-09, 4.3848491848500002e+00, 2.3479112827416777e+05 },
    { 8.3999999999999999e-10, 5.1311220640100004e+00, 5.3285184835241802e+04 },
    { 8.0999999999999999e-10, 1.7799166933399999e+00, 5.1066427731055002e+04 },
    { 4.0999999999999998e-10, 5.9287816567300000e+00, 2.1535949644515400e+04 },
    { 3.9000000000000000e-10, 1.1638953356799999e+00, 2.6087903141574195e+05 },
    { 3.7999999999999998e-10, 4.5949408967999998e+00, 3.1749235190726398e+04 },
    { 3.1000000000000002e-10, 5.7051774626600000e+00, 2.7043502883182799e+04 },
    { 3.1000000000000002e-10, 4.9009298247600004e+00, 7.7154330872629187e+04 },
    { 2.5000000000000002e-10, 3.4205136896499999e+00, 5.1116424352959199e+04 },
    { 2.3000000000000001e-10, 5.1288870346700000e+00, 1.1093785520934000e+03 },
    { 2.0000000000000001e-10, 5.5166361826900001e+00, 5.2969096509459996e+02 },
    { 2.0000000000000001e-10, 1.7917379064000000e-01, 2.5028521211384999e+04 },
    { 1.5999999999999999e-10, 1.9039031396799999e+00, 2.8306660245760992e+04 },
    { 1.5000000000000000e-10, 1.0570179532499999e+00, 2.0426571092422000e+04 },
    { 1.2999999999999999e-10, 1.8182207788400000e+00, 2.5132303399965600e+04 },
    { 1.2000000000000000e-10, 5.7343045555899996e+00, 2.2645328196608789e+04 },
    { 1.2000000000000000e-10, 8.3014470454000000e-01, 3.7410567239878597e+04 },
    { 1.2000000000000000e-10, 4.2225235344699996e+00, 2.8696693455731618e+05 },
    { 1.0999999999999999e-10, 5.1741432847400004e+00, 2.3869146037387400e+04 },
    { 1.0999999999999999e-10, 1.7057348180600000e+00, 1.0324223401420339e+05 },
    { 1.0999999999999999e-10, 2.3896990103700002e+00, 5.0586733387864588e+04 },
    { 1.0000000000000000e-10, 5.8971971692200000e+00, 4.7623852786089599e+04 },
    { 1.0000000000000000e-10, 6.2532110951800002e+00, 4.5519534970588002e+03 },
    { 1.0000000000000000e-10, 4.8049800558199998e+00, 1.5890728952838001e+03 },
    { 1.0000000000000000e-10, 3.6648949642400002e+00, 2.5558212176479599e+04 },
    { 8.9999999999999999e-11, 4.9466496208299997e+00, 2.6107572902473989e+04 },
    { 8.9999999999999999e-11, 1.9424224315000000e+00, 7.9373087976815994e+04 },
    { 8.9999999999999999e-11, 4.5082398553200003e+00, 3.2858613742819791e+04 },
    { 7.9999999999999995e-11, 4.7732322003099998e+00, 4.6514474233996203e+04 },
    { 7.9999999999999995e-11, 6.0443983845200000e+00, 2.6068233380674399e+04 },
    { 7.0000000000000004e-11, 2.3875123773300002e+00, 5.3131406024757001e+04 },
    { 7.0000000000000004e-11, 1.5912063728200001e+00, 2.6084021806216198e+04 },
    { 7.0000000000000004e-11, 5.5046174970299999e+00, 1.4765239043269799e+04 },
    { 7.0000000000000004e-11, 3.1123551430599998e+00, 2.6091784476932200e+04 },
    { 7.0000000000000004e-11, 2.5235917480299999e+00, 1.9317192540328600e+04 },
    { 6.0000000000000000e-11, 2.4450602168800000e+00, 2.6617594106668799e+04 },
    { 6.0000000000000000e-11, 5.8636360320099996e+00, 2.5874604046136199e+04 },
    { 6.0000000000000000e-11, 2.5269222888599998e+00, 2.1329909543799999e+02 },
    { 6.0000000000000000e-11, 5.4902491701200002e+00, 2.5973463852888959e+04 },
    { 5.0000000000000002e-11, 4.5057721671400000e+00, 5.1220206541539788e+04 },
    { 5.0000000000000002e-11, 3.3635647447500001e+00, 4.3071899289030800e+04 },
    { 5.0000000000000002e-11, 1.6156448496800000e+00, 2.6202342430259410e+04 },
    { 5.0000000000000002e-11, 1.2315017350599999e+00, 3.4425749449653999e+03 },
    { 5.0000000000000002e-11, 3.1422089222600000e+00, 1.0213285546211000e+04 },
    { 3.9999999999999998e-11, 4.9998459261699999e+00, 5.4394563387335191e+04 },
    { 3.9999999999999998e-11, 3.5682290843400000e+00, 9.1039069941176003e+03 },
    { 3.9999999999999998e-11, 3.4110492958199998e+00, 2.5661304950698199e+04 },
    { 3.9999999999999998e-11, 2.0498699398200002e+00, 7.2602377375570388e+04 },
    { 3.9999999999999998e-11, 4.7858647858200003e+00, 1.2933013715577759e+05 },
    { 3.9999999999999998e-11, 1.2328541506999999e+00, 3.9609654583165597e+04 },
    { 3.9999999999999998e-11, 1.9956635680199999e+00, 2.6095016688575000e+04 },
    { 3.9999999999999998e-11, 1.3538051989800000e+00, 2.9530478086539599e+04 },
    { 3.9999999999999998e-11, 6.0798286358900002e+00, 7.1135470007999997e+00 },
    { 3.9999999999999998e-11, 2.6585418783999998e-01, 5.1646115318053788e+04 },
    { 3.9999999999999998e-11, 2.7778258616999998e+00, 4.1962520736937397e+04 },
    { 3.9999999999999998e-11, 1.0965092910800001e+00, 5.6613320491521999e+03 },
    { 3.9999999999999998e-11, 1.4254465460000001e+00, 1.0593819301891999e+03 },
    { 3.9999999999999998e-11, 1.6696918983000000e+00, 3.0639856638632999e+04 },
    { 3.9999999999999998e-11, 2.5687095974199998e+00, 6.3989728631399998e+02 },
    { 3.0000000000000000e-11, 1.9757737983400001e+00, 4.9957049178961599e+04 },
    { 3.0000000000000000e-11, 6.0217181686299996e+00, 5.2705497248242988e+04 },
    { 3.0000000000000000e-11, 1.2415742256700000e+00, 5.7837138332300601e+04 },
    { 3.0000000000000000e-11, 4.2413924780699999e+00, 5.1535908996834391e+04 },
    { 3.0000000000000000e-11, 3.7898088748399998e+00, 1.0664954771900000e+03 },
    { 3.0000000000000000e-11, 4.3118309545100004e+00, 2.2187571041868000e+03 },
    { 3.0000000000000000e-11, 2.7171631884699998e+00, 2.6080789594573391e+04 },
    { 3.0000000000000000e-11, 3.0829537245299998e+00, 3.3326578733174203e+04 },
    { 3.0000000000000000e-11, 2.8312452913100001e+00, 4.0853142184844000e+04 },
    { 3.0000000000000000e-11, 3.9809410681299999e+00, 6.3498470381452789e+04 },
    { 3.0000000000000000e-11, 6.1199801907299998e+00, 3.6301188687785187e+04 },
    { 3.0000000000000000e-11, 9.9287952932000001e-01, 3.1305483769889036e+05 },
    { 3.0000000000000000e-11, 4.4723261039100004e+00, 2.4498830246290399e+04 },
    { 3.0000000000000000e-11, 4.3944708569399999e+00, 5.6727759780207198e+04 },
    { 3.0000000000000000e-11, 1.2949695678799999e+00, 5.8946516884393990e+04 },
    { 3.0000000000000000e-11, 5.7851853787599996e+00, 4.8733231338182988e+04 },
    { 3.0000000000000000e-11, 5.6904573361499997e+00, 2.1187638603783998e+03 },
    { 3.0000000000000000e-11, 5.7832680784599999e+00, 7.7204327494533383e+04 },
    { 1.9999999999999999e-11, 3.1462917618900001e+00, 5.3235188213337591e+04 },
    { 1.9999999999999999e-11, 5.8575501874100002e+00, 2.6301202237012199e+04 },
    { 1.9999999999999999e-11, 2.4058067837800001e+00, 6.2830758499914000e+03 },
    { 1.9999999999999999e-11, 7.4752177961999999e-01, 9.5559974160859997e+02 },
    { 1.9999999999999999e-11, 3.2352138137200002e+00, 1.6983996147456601e+04 },
    { 1.9999999999999999e-11, 2.5166029271000001e+00, 4.2659819087599999e+02 },
    { 1.9999999999999999e-11, 6.0194942542999996e-01, 3.8519945791972001e+04 },
    { 1.9999999999999999e-11, 4.2539782120299998e+00, 1.0522683831884001e+03 },
    { 1.9999999999999999e-11, 5.2282782325300001e+00, 7.6674636529438780e+04 },
    { 1.9999999999999999e-11, 1.1067683484399999e+00, 7.7308109683113988e+04 },
    { 1.9999999999999999e-11, 5.5915355792600003e+00, 2.3754706748702189e+04 },
    { 1.9999999999999999e-11, 1.1046591263000000e-01, 1.3655860491176400e+04 },
    { 1.9999999999999999e-11, 5.6265876868399998e+00, 4.5405095681902800e+04 },
    { 1.9999999999999999e-11, 4.5191826978599998e+00, 1.5874617595363199e+04 },
    { 1.9999999999999999e-11, 5.8139959402799999e+00, 2.7147285071763388e+04 },
    { 1.9999999999999999e-11, 4.4553698813000002e+00, 5.5618381228113802e+04 },
    { 1.9999999999999999e-11, 1.6788633704400000e+00, 1.0329223063610759e+05 },
    { 1.9999999999999999e-11, 8.0870638069000000e-01, 6.7707106012455997e+03 },
    { 1.9999999999999999e-11, 5.3105046289500004e+00, 9.8690280517144580e+04 },
    { 1.9999999999999999e-11, 1.6283079881800000e+00, 5.0057042422769999e+04 },
    { 1.9999999999999999e-11, 2.8566998069100000e+00, 7.8793400389817180e+04 },
    { 1.9999999999999999e-11, 1.4503737005999999e+00, 2.7676976036857999e+04 },
    { 9.9999999999999994e-12, 6.0178974692900002e+00, 2.5448005855260191e+04 },
    { 9.9999999999999994e-12, 4.7817433455999998e+00, 1.8849227549974199e+04 },
    { 9.9999999999999994e-12, 1.5753913591399999e+00, 1.5541804029735178e+05 },
    { 9.9999999999999994e-12, 1.7512021286399999e+00, 8.5319638175199998e+02 },
    { 9.9999999999999994e-12, 5.6423988424899996e+00, 2.6514501332450200e+04 },
    { 9.9999999999999994e-12, 2.8429528515800002e+00, 5.1749208092272391e+04 },
    { 9.9999999999999994e-12, 1.2314884196100000e+00, 8.2815662921781390e+04 },
    { 9.9999999999999994e-12, 3.5267802169800002e+00, 1.3521751441591399e+04 },
    { 9.9999999999999994e-12, 4.7060750358299996e+00, 5.2290245571833613e+04 },
    { 9.9999999999999994e-12, 1.5381872374900001e+00, 2.8421099534446199e+04 },
    { 9.9999999999999994e-12, 2.6712735729999999e-02, 7.9323091354911783e+04 },
    { 9.9999999999999994e-12, 4.9792310169000000e+00, 7.6044952320535798e+04 },
    { 9.9999999999999994e-12, 3.3059571699600001e+00, 5.2389105378586399e+04 },
    { 9.9999999999999994e-12, 1.1854808560900001e+00, 1.2477818365871879e+05 },
    { 9.9999999999999994e-12, 4.5896137954300000e+00, 1.2566151699982800e+04 },
    { 9.9999999999999994e-12, 3.9810320441600000e+00, 3.3914274084046460e+05 },
    { 9.9999999999999994e-12, 2.2251931515000001e+00, 5.2061366994463169e+04 },
    { 9.9999999999999994e-12, 4.8866221147999997e+00, 1.0488130353139139e+05 },
    { 9.9999999999999994e-12, 2.5358492002499999e+00, 3.8654054841556987e+04 },
    { 9.9999999999999994e-12, 2.9159416050100000e+00, 8.3925041473874793e+04 },
    { 9.9999999999999994e-12, 2.4312806487900001e+00, 7.7734018459627987e+04 },
    { 9.9999999999999994e-12, 3.1905677152799998e+00, 5.1962507187710398e+04 },
    { 9.9999999999999994e-12, 6.1276539132999996e+00, 6.6941045326418185e+04 },
    { 9.9999999999999994e-12, 4.0244182067700001e+00, 1.2432042650397800e+04 },
    { 9.9999999999999994e-12, 2.4299471964600001e+00, 5.2602404474024399e+04 },
    { 9.9999999999999994e-12, 1.2245485702300001e+00, 2.8206667001952599e+04 },
    { 9.9999999999999994e-12, 4.8303732915699999e+00, 1.2938013377768178e+05 },
    { 9.9999999999999994e-12, 6.4473152247999999e-01, 1.1001294461544899e+05 },
    { 9.9999999999999994e-12, 2.9924819002600000e+00, 7.9945284420241996e+03 },
    { 9.9999999999999994e-12, 4.4965883390699997e+00, 7.8267590760080580e+04 },
    { 9.9999999999999994e-12, 4.7174923862200000e+00, 6.4741957983131193e+04 },
    { 9.9999999999999994e-12, 4.5452282955500003e+00, 1.1322664098304400e+04 },
    { 9.9999999999999994e-12, 5.5817696807999999e+00, 7.8477008520160583e+04 },
};

static const VSOP87Term VSOP87A_MERCURY_Z3[28] = {
    { 2.1392000000000000e-07, 2.1280427845999998e+00, 2.6087903141574199e+04 },
    { 1.7872000000000000e-07, 0.0000000000000000e+00, 0.0000000000000000e+00 },
    { 2.9189999999999999e-08, 5.4108583618399999e+00, 5.2175806283148399e+04 },
    { 1.1640000000000000e-08, 2.4332496691199998e+00, 7.8263709424722590e+04 },
    { 6.4899999999999997e-09, 5.9182882001900001e+00, 1.0435161256629678e+05 },
    { 3.3799999999999999e-09, 2.9072718886300000e+00, 1.3043951570787099e+05 },
    { 1.5300000000000001e-09, 6.0690278542099998e+00, 1.5652741884944518e+05 },
    { 6.3000000000000000e-10, 2.9004050658999998e+00, 1.8261532199101939e+05 },
    { 2.4000000000000000e-10, 5.9941243316400001e+00, 2.0870322513259359e+05 },
    { 8.9999999999999999e-11, 2.7898570091999999e+00, 2.3479112827416777e+05 },
    { 8.9999999999999999e-11, 3.8501441042700000e+00, 2.7197281693667599e+04 },
    { 7.0000000000000004e-11, 9.4845759850000000e-01, 2.4978524589480799e+04 },
    { 3.0000000000000000e-11, 5.8522732324500000e+00, 2.6087903141574195e+05 },
    { 9.9999999999999994e-12, 1.4606697107900000e+00, 1.1093785520934000e+03 },
    { 9.9999999999999994e-12, 5.6333081746099998e+00, 5.1066427731055002e+04 },
    { 9.9999999999999994e-12, 2.6366479339200000e+00, 2.8696693455731618e+05 },
    { 9.9999999999999994e-12, 3.1254828818100000e+00, 7.7154330872629187e+04 },
    { 9.9999999999999994e-12, 1.0273460254000000e-01, 1.0593819301891999e+03 },
    { 9.9999999999999994e-12, 5.3895020246399996e+00, 2.0426571092422000e+04 },
    { 9.9999999999999994e-12, 4.2764430677699998e+00, 2.5132303399965600e+04 },
    { 9.9999999999999994e-12, 1.8575037242100001e+00, 5.3285184835241802e+04 },
    { 0.0000000000000000e+00, 1.2672284450200000e+00, 1.5890728952838001e+03 },
    { 0.0000000000000000e+00, 1.4831541416700000e+00, 2.7043502883182799e+04 },
    { 0.0000000000000000e+00, 5.1010264399999998e-03, 1.0324223401420339e+05 },
    { 0.0000000000000000e+00, 2.6222106415000002e-01, 3.1749235190726398e+04 },
    { 0.0000000000000000e+00, 4.2976316786400002e+00, 4.5519534970588002e+03 },
    { 0.0000000000000000e+00, 1.6242782840200001e+00, 2.1535949644515400e+04 },
    { 0.0000000000000000e+00, 2.5562635171800001e+00, 4.6514474233996203e+04 },
};

static const VSOP87Term VSOP87A_MERCURY_Z4[10] = {
    { 5.8699999999999998e-09, 4.8644269931500004e+00, 2.6087903141574199e+04 },
    { 2.8299999999999999e-09, 3.1415926535900001e+00, 0.0000000000000000e+00 },
    { 5.4999999999999996e-10, 3.0408210724600000e+00, 5.2175806283148399e+04 },
    { 2.5999999999999998e-10, 5.3815113361000000e-01, 7.8263709424722590e+04 },
    { 1.5000000000000000e-10, 4.0606532070699997e+00, 1.0435161256629678e+05 },
    { 8.9999999999999999e-11, 1.1489379152899999e+00, 1.3043951570787099e+05 },
    { 5.0000000000000002e-11, 4.3856535917499997e+00, 1.5652741884944518e+05 },
    { 1.9999999999999999e-11, 1.2500234121700000e+00, 1.8261532199101939e+05 },
    { 9.9999999999999994e-12, 4.3819294216199998e+00, 2.0870322513259359e+05 },
    { 0.0000000000000000e+00, 1.1651508528900001e+00, 2.3479112827416777e+05 },
};

static const VSOP87Term VSOP87A_MERCURY_Z5[7] = {
    { 3.9999999999999998e-11, 0.0000000000000000e+00, 0.0000000000000000e+00 },
    { 1.9999999999999999e-11, 5.4284977789399997e+00, 2.6087903141574199e+04 },
    { 1.9999999999999999e-11, 1.1901108749800000e+00, 5.2175806283148399e+04 },
    { 9.9999999999999994e-12, 4.8414797790300002e+00, 7.8263709424722590e+04 },
    { 0.0000000000000000e+00, 2.1434333195500002e+00, 1.0435161256629678e+05 },
    { 0.0000000000000000e+00, 5.6195023048500001e+00, 1.3043951570787099e+05 },
    { 0.0000000000000000e+00, 2.7688049573200000e+00, 1.5652741884944518e+05 },
};

static const VSOP87Block VSOP87A_MERCURY_X[6] = {
    { VSOP87A_MERCURY_X0, 1449 },
    { VSOP87A_MERCURY_X1, 792 },
    { VSOP87A_MERCURY_X2, 299 },
    { VSOP87A_MERCURY_X3, 54 },
    { VSOP87A_MERCURY_X4, 15 },
    { VSOP87A_MERCURY_X5, 10 },
};

static const VSOP87Block VSOP87A_MERCURY_Y[6] = {
    { VSOP87A_MERCURY_Y0, 1438 },
    { VSOP87A_MERCURY_Y1, 782 },
    { VSOP87A_MERCURY_Y2, 299 },
    { VSOP87A_MERCURY_Y3, 59 },
    { VSOP87A_MERCURY_Y4, 15 },
    { VSOP87A_MERCURY_Y5, 10 },
};

static const VSOP87Block VSOP87A_MERCURY_Z[6] = {
    { VSOP87A_MERCURY_Z0, 598 },
    { VSOP87A_MERCURY_Z1, 351 },
    { VSOP87A_MERCURY_Z2, 143 },
    { VSOP87A_MERCURY_Z3, 28 },
    { VSOP87A_MERCURY_Z4, 10 },
    { VSOP87A_MERCURY_Z5, 7 },
};

static const int32 VSOP87A_MERCURY_TRUNC_COUNT[11][3][6] = {
    {
        { 1449, 792, 299, 54, 15, 10 },
        { 1438, 782, 299, 59, 15, 10 },
        { 598, 351, 143, 28, 10, 7 },
    },
    {
        { 170, 56, 13, 7, 0, 0 },
        { 171, 58, 13, 7, 0, 0 },
        { 83, 15, 8, 4, 0, 0 },
    },
    {
        { 102, 19, 8, 5, 0, 0 },
        { 104, 19, 8, 5, 0, 0 },
        { 38, 10, 6, 2, 0, 0 },
    },
    {
        { 74, 17, 8, 4, 0, 0 },
        { 75, 17, 8, 4, 0, 0 },
        { 23, 8, 6, 2, 0, 0 },
    },
    {
        { 34, 10, 6, 0, 0, 0 },
        { 35, 10, 6, 0, 0, 0 },
        { 10, 7, 3, 0, 0, 0 },
    },
    {
        { 21, 8, 6, 0, 0, 0 },
        { 21, 8, 6, 0, 0, 0 },
        { 8, 6, 2, 0, 0, 0 },
    },
    {
        { 10, 7, 3, 0, 0, 0 },
        { 10, 7, 4, 0, 0, 0 },
        { 7, 5, 2, 0, 0, 0 },
    },
    {
        { 7, 6, 2, 0, 0, 0 },
        { 7, 6, 2, 0, 0, 0 },
        { 6, 4, 2, 0, 0, 0 },
    },
    {
        { 6, 5, 0, 0, 0, 0 },
        { 6, 5, 0, 0, 0, 0 },
        { 5, 2, 0, 0, 0, 0 },
    },
    {
        { 6, 4, 0, 0, 0, 0 },
        { 6, 4, 0, 0, 0, 0 },
        { 5, 2, 0, 0, 0, 0 },
    },
    {
        { 5, 2, 0, 0, 0, 0 },
        { 5, 1, 0, 0, 0, 0 },
        { 3, 1, 0, 0, 0, 0 },
    },
};

