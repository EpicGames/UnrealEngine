// Copyright Epic Games, Inc. All Rights Reserved.

// Auto-generated VSOP87A header for NEPTUNE (rectangular XYZ, J2000)
// Source: IMCCE/BDL VSOP87A.neptune file
// Generated by vsop87a_header_gen_imcce.py

#pragma once

#include "VSOP87A_Common.h"

static const VSOP87Term VSOP87A_NEPTUNE_X0[772] = {
    { 3.0058900044760001e+01, 5.3121134002900003e+00, 3.8133035637799999e+01 },
    { 2.7080164222000003e-01, 3.1415926535900001e+00, 0.0000000000000000e+00 },
    { 1.5726094556000000e-01, 1.1319072675000000e-01, 3.6648562929500002e+01 },
    { 1.4935120126000001e-01, 1.0849940301800001e+00, 3.9617508346100003e+01 },
    { 1.3505661755000001e-01, 3.5007897563400001e+00, 7.6266071275599998e+01 },
    { 2.5973138140000001e-02, 1.9959030141200000e+00, 1.4844727083000000e+00 },
    { 1.0740407080000000e-02, 5.3850293867200003e+00, 7.4781598567299994e+01 },
    { 8.2379328700000008e-03, 1.4322158186200000e+00, 3.5164090221199999e+01 },
    { 8.1758881299999996e-03, 7.8180174031000005e-01, 2.9689454166000000e+00 },
    { 5.6553491799999998e-03, 5.9896490761300001e+00, 4.1101981054399999e+01 },
    { 4.9571910700000003e-03, 5.9948143567000001e-01, 5.2969096509459996e+02 },
    { 3.0752590700000000e-03, 4.0023311011000001e-01, 7.3297125859000005e+01 },
    { 2.7225355100000000e-03, 8.7443494386999998e-01, 2.1329909543799999e+02 },
    { 1.3588721900000000e-03, 5.5467657781600002e+00, 7.7750543983900002e+01 },
    { 9.0965703999999997e-04, 1.6891024611500001e+00, 1.1439910691340000e+02 },
    { 6.9040539000000001e-04, 5.8346912352000002e+00, 4.4534181248999998e+00 },
    { 6.0813556000000003e-04, 2.6258995837999999e+00, 3.3679617512900002e+01 },
    { 5.4690827000000003e-04, 1.5579999666100000e+00, 7.1812653150700001e+01 },
    { 2.8889260000000000e-04, 4.7896682602700000e+00, 4.2586453762700003e+01 },
    { 1.2749153000000000e-04, 2.7371926964500002e+00, 1.1143016149680000e+02 },
    { 1.2614732000000001e-04, 3.5700251643400001e+00, 1.1291463420510000e+02 },
    { 1.2013993999999999e-04, 9.4912933495999996e-01, 1.0593819301891999e+03 },
    { 8.0043180000000005e-05, 1.6396562626000000e+00, 1.0846121608020000e+02 },
    { 7.5733830000000005e-05, 1.0011329853000001e-01, 4.2659819087599999e+02 },
    { 7.5406499999999997e-05, 2.7778347785499999e+00, 7.0328180442399997e+01 },
    { 6.4648419999999997e-05, 4.6258006601300004e+00, 5.9378908332000000e+00 },
    { 5.5658600000000003e-05, 3.8250218595300001e+00, 3.2195144804599998e+01 },
    { 4.7324340000000000e-05, 4.0972397719100000e+00, 7.9235016692200006e+01 },
    { 4.6543610000000000e-05, 1.0385887980000000e-01, 3.7611770776000000e+01 },
    { 4.5572470000000002e-05, 1.0971266931699999e+00, 3.8654300499599998e+01 },
    { 4.3225500000000001e-05, 2.3774478018799998e+00, 3.8084851528000002e+01 },
    { 4.3155390000000003e-05, 5.1047314205600003e+00, 3.8181219747599997e+01 },
    { 4.2496739999999999e-05, 5.6332447582300000e+00, 2.8571808082200000e+01 },
    { 4.0890360000000001e-05, 1.9942904824400001e+00, 3.7169827791300001e+01 },
    { 3.9518480000000003e-05, 2.2999693411000002e+00, 9.8899988524600005e+01 },
    { 3.9204120000000001e-05, 5.4926378486500003e+00, 3.9096243484299997e+01 },
    { 3.3227350000000003e-05, 4.6879859193800000e+00, 4.1927856940000003e+00 },
    { 3.2600949999999998e-05, 1.8183965287799999e+00, 1.4510977900969999e+02 },
    { 3.1082920000000002e-05, 1.8443454340900001e+00, 4.7694263193399998e+01 },
    { 2.7234419999999999e-05, 3.8229628594100000e+00, 1.0994568878849999e+02 },
    { 2.5229380000000001e-05, 4.6629612691200002e+00, 3.1219908396260001e+02 },
    { 2.0844769999999999e-05, 1.5682167136900000e+00, 1.4956319713459999e+02 },
    { 2.0836820000000002e-05, 2.8367696181099999e+00, 1.3703302416240001e+02 },
    { 1.9454620000000000e-05, 4.1532682528800002e+00, 2.0618554843720000e+02 },
    { 1.8918920000000001e-05, 7.4998855535999998e-01, 2.2041264243879999e+02 },
    { 1.8874300000000000e-05, 3.2048541779200002e+00, 3.5685355082999997e+01 },
    { 1.8267000000000000e-05, 3.5802431864900002e+00, 4.4070926471000000e+01 },
    { 1.6812549999999999e-05, 4.2756030428200003e+00, 4.0580716192600001e+01 },
    { 1.6489849999999999e-05, 4.0699066659099996e+00, 3.0056280790500001e+01 },
    { 1.6150630000000000e-05, 2.9106383501000002e+00, 1.0697674337190000e+02 },
    { 1.5604290000000002e-05, 5.5865739143000004e-01, 3.7872403206900003e+01 },
    { 1.5457050000000001e-05, 6.4028780671999996e-01, 3.8393668068700002e+01 },
    { 1.5333830000000001e-05, 1.1773221166500001e+00, 3.8021161053199997e+01 },
    { 1.5275260000000000e-05, 2.1736402459999998e-02, 3.8244910222400001e+01 },
    { 1.4484390000000001e-05, 1.9881431725900001e+00, 1.8175834193919999e+02 },
    { 1.4347980000000001e-05, 7.2658718863000005e-01, 5.2257741809380002e+02 },
    { 1.4074220000000000e-05, 3.6174328866600001e+00, 5.3680451209540001e+02 },
    { 1.3879220000000001e-05, 3.7181433095199998e+00, 1.1588357962169999e+02 },
    { 1.2715430000000001e-05, 2.7441298122900002e+00, 3.3940249943799998e+01 },
    { 1.2657969999999999e-05, 3.4203727544700002e+00, 4.6209790485100001e+01 },
    { 1.2289390000000001e-05, 2.7887821179199999e+00, 7.2073285581600004e+01 },
    { 1.2203410000000000e-05, 2.6479144958399998e+00, 1.4807872442630000e+02 },
    { 1.1700780000000000e-05, 3.9859468904100002e+00, 8.0767548473000002e+00 },
    { 1.0807950000000000e-05, 4.7548563601899998e+00, 4.2325821331800000e+01 },
    { 9.4272199999999998e-06, 3.9986107667700002e+00, 6.8843707734099993e+01 },
    { 7.2201300000000001e-06, 6.1681178156599996e+00, 1.5253214255120000e+02 },
    { 7.1496899999999996e-06, 3.0912163150700001e+00, 1.4362530630140000e+02 },
    { 6.7954199999999999e-06, 2.1567368135599998e+00, 2.1840690486870000e+02 },
    { 6.3383400000000002e-06, 3.4169264817000000e+00, 7.4223635415000002e+00 },
    { 6.0826500000000001e-06, 4.4956370085400001e+00, 3.5212274331000003e+01 },
    { 5.6630599999999998e-06, 5.0280988227999996e+00, 3.0710672096300001e+01 },
    { 5.5447300000000000e-06, 2.9863439777599998e+00, 4.1053796944600002e+01 },
    { 5.2803799999999997e-06, 3.4505105111000001e-01, 9.6320784650000002e-01 },
    { 4.9323100000000000e-06, 3.8261318704699998e+00, 1.4659425171800001e+02 },
    { 4.6314000000000002e-06, 2.7443664379600001e+00, 3.1540753498800001e+01 },
    { 4.5161900000000002e-06, 4.6275704052200002e+00, 4.4725317776799997e+01 },
    { 4.3963800000000002e-06, 1.2967804829900000e+00, 1.5890728952838001e+03 },
    { 4.0609200000000001e-06, 5.5108421366300000e+00, 6.5922821389999999e+00 },
    { 4.0541099999999997e-06, 5.3194762900899999e+00, 3.1019488636999998e+01 },
    { 3.8239799999999996e-06, 2.1680708235000001e-01, 4.8736514376280002e+02 },
    { 3.4616399999999999e-06, 3.2650747070500001e+00, 1.8027386923090000e+02 },
    { 3.3708200000000001e-06, 1.7233996942900001e+00, 5.2126486179999998e-01 },
    { 3.3625000000000000e-06, 2.1482443486000000e+00, 4.5246582638600003e+01 },
    { 3.3323299999999998e-06, 2.3286186217300000e+00, 2.5505546779820000e+02 },
    { 3.1398899999999999e-06, 2.7270478680100001e+00, 3.8846515523820000e+02 },
    { 3.1319099999999999e-06, 3.0198616141499999e+00, 4.1948464387519999e+02 },
    { 3.0684300000000001e-06, 2.6265122389500002e+00, 6.0766952886799999e+01 },
    { 3.0604800000000000e-06, 1.7514437706899999e+00, 6.2830758499914000e+03 },
    { 3.0450300000000000e-06, 5.6325835775100002e+00, 6.3989728631399998e+02 },
    { 3.0149100000000000e-06, 5.1777876348099996e+00, 8.4342826122899993e+01 },
    { 2.8540400000000000e-06, 1.7471112530099999e+00, 1.1020632121940000e+02 },
    { 2.8384100000000001e-06, 3.3622629850800001e+00, 1.2530172972200001e+01 },
    { 2.6805999999999998e-06, 3.3085217737799999e+00, 3.6760437514099998e+01 },
    { 2.6593999999999999e-06, 4.1195450663799997e+00, 9.4498282327580000e+02 },
    { 2.6475999999999999e-06, 4.1272408344500002e+00, 3.9505633761500000e+01 },
    { 2.4274299999999998e-06, 2.0647252293800000e+00, 1.4014645680499999e+01 },
    { 2.3967200000000000e-06, 5.5843257386000000e-01, 2.7087335373900000e+01 },
    { 2.3307500000000001e-06, 4.1573807400099998e+00, 1.0549227066360000e+02 },
    { 2.2782200000000002e-06, 2.1305726749699998e+00, 1.0309277421860000e+02 },
    { 2.2697099999999999e-06, 4.5927838047999998e+00, 2.7406604832480002e+02 },
    { 2.2615400000000001e-06, 2.8384451299500002e+00, 8.0719489400499995e+01 },
    { 2.1598800000000001e-06, 2.6591769072500000e+00, 3.1639186965660002e+02 },
    { 2.0323000000000000e-06, 6.0054564923999998e-01, 4.1529185818119998e+02 },
    { 1.9099799999999998e-06, 2.3241977489600001e+00, 6.9152524274800001e+01 },
    { 1.8528200000000000e-06, 3.3149248227900001e+00, 1.7516605980020000e+02 },
    { 1.7918900000000001e-06, 3.9579251702799998e+00, 1.8324281464750001e+02 },
    { 1.7782000000000000e-06, 3.1725701576100001e+00, 1.0213285546211000e+04 },
    { 1.7700300000000000e-06, 9.2406134380000005e-02, 2.4118389957300000e+01 },
    { 1.7681000000000001e-06, 3.4406766566700000e+00, 2.1692243216040001e+02 },
    { 1.7646499999999999e-06, 1.4731264750000000e-01, 3.6536688344900000e+01 },
    { 1.7520900000000001e-06, 1.1257572185100000e+00, 3.9729382930699998e+01 },
    { 1.6903700000000000e-06, 2.4941709679100001e+00, 2.9170403072770000e+02 },
    { 1.5656099999999999e-06, 4.1915386797299998e+00, 6.3735898303399999e+01 },
    { 1.5479700000000001e-06, 1.7484036771100000e+00, 7.7229279122099996e+01 },
    { 1.4834600000000000e-06, 1.6801000039600000e+00, 1.5104766984290001e+02 },
    { 1.4633400000000001e-06, 4.8102778978899998e+00, 1.1045700263900001e+01 },
    { 1.4463799999999999e-06, 6.3080734163999996e-01, 4.9178735901700001e+01 },
    { 1.3545900000000001e-06, 3.2965870656399998e+00, 9.5612275556000004e+00 },
    { 1.3475900000000000e-06, 5.4817752094100003e+00, 7.5744806413800006e+01 },
    { 1.2944200000000000e-06, 2.3690712280100001e+00, 4.5555399179299997e+01 },
    { 1.2917400000000001e-06, 5.9194900308299996e+00, 4.3240845068500001e+01 },
    { 1.2904899999999999e-06, 3.8068488285900002e+00, 3.6861011250600001e+01 },
    { 1.2409200000000000e-06, 3.3312196832000002e-01, 8.2858353414600003e+01 },
    { 1.2291499999999999e-06, 3.6743355215900002e+00, 3.9405060024999997e+01 },
    { 1.2122600000000000e-06, 5.1058428245599998e+00, 3.7920587316700001e+01 },
    { 1.2113800000000000e-06, 1.4966621323300000e+00, 3.3025226207099998e+01 },
    { 1.2011700000000001e-06, 2.3605303240400000e+00, 3.8345483958899997e+01 },
    { 1.1961700000000001e-06, 5.0181735400300003e+00, 2.5602862665600000e+01 },
    { 1.1620500000000000e-06, 5.5636456827099998e+00, 6.3278373931320004e+02 },
    { 1.1110900000000000e-06, 5.2144747307800001e+00, 6.7359235025800004e+01 },
    { 1.1005000000000001e-06, 4.3389676603899998e+00, 1.4214083359310001e+02 },
    { 1.0886499999999999e-06, 1.2850780040000001e-02, 1.1624747044077999e+03 },
    { 1.0683899999999999e-06, 5.7301677365000003e-01, 5.1078094306999997e+00 },
    { 1.0445200000000000e-06, 6.2604874692200001e+00, 4.3371173787679999e+02 },
    { 1.0354000000000001e-06, 5.2563465676899996e+00, 4.0841348623499997e+01 },
    { 1.0291700000000000e-06, 2.2953265956200002e+00, 3.5424722652100002e+01 },
    { 1.0025400000000000e-06, 2.5270327951199998e+00, 4.5342489381899998e+02 },
    { 9.7241000000000006e-07, 6.1767708024300001e+00, 1.0522683831884001e+03 },
    { 9.6664000000000003e-07, 4.1139765526799996e+00, 2.1989137757699999e+02 },
    { 9.4921999999999997e-07, 3.6129602344900000e+00, 2.5357099508990001e+02 },
    { 9.2946999999999997e-07, 5.4542446675800003e+00, 6.2251425595100002e+01 },
    { 9.2567000000000000e-07, 3.2145649905300000e+00, 3.2716409666399997e+01 },
    { 8.9067999999999998e-07, 2.7691906547900000e+00, 6.5220371011699996e+01 },
    { 8.9059999999999999e-07, 1.3293888650000001e+00, 2.5143213107579999e+02 },
    { 8.7258000000000000e-07, 2.6676053010000000e+00, 3.2835259365719998e+02 },
    { 8.6613000000000004e-07, 1.3801209118700000e+00, 1.0038446123289999e+02 },
    { 8.5710000000000002e-07, 5.9368119892399998e+00, 1.0601353552540000e+02 },
    { 8.5104000000000003e-07, 4.7901819309100002e+00, 3.6436114608399997e+01 },
    { 8.0200000000000001e-07, 2.6919980075300001e+00, 3.9829956667200001e+01 },
    { 7.7347999999999996e-07, 5.3114627739599998e+00, 5.4166259714000002e+00 },
    { 7.5165000000000004e-07, 2.6221144253000001e-01, 9.7415515816300001e+01 },
    { 7.3979999999999999e-07, 5.5643958558000000e+00, 2.7083129857000001e+00 },
    { 7.2986000000000005e-07, 1.8235550701200001e+00, 2.2633917249000000e+01 },
    { 7.2806999999999998e-07, 4.2305225421700001e+00, 4.3549661609200001e+01 },
    { 7.2027000000000003e-07, 1.1366561346000000e-01, 3.6696747039300000e+01 },
    { 7.1608999999999995e-07, 2.0567924610100001e+00, 6.6253120356299996e+02 },
    { 6.9405000000000005e-07, 3.9575484111299999e+00, 1.0664954771900000e+03 },
    { 6.8141000000000001e-07, 3.9565995636300002e+00, 6.9010986796999996e+00 },
    { 6.7998000000000003e-07, 3.8862604543999999e+00, 2.2434479570190001e+02 },
    { 6.6446999999999997e-07, 3.4169977046200000e+00, 8.4608283475120004e+02 },
    { 6.5817000000000002e-07, 1.4282189665900000e+00, 3.6127298067700004e+01 },
    { 6.5519999999999999e-07, 4.0099666637000002e+00, 6.6704843720000000e+01 },
    { 6.5196999999999998e-07, 9.4675037786000005e-01, 2.4476805547999998e+00 },
    { 6.3661999999999999e-07, 2.2115583539400001e+00, 8.9068362497999995e+00 },
    { 6.2934000000000000e-07, 1.5366671380300001e+00, 2.9207613067999998e+00 },
    { 6.0042000000000003e-07, 6.2359129597800003e+00, 2.9226199388000001e+01 },
    { 5.9503999999999996e-07, 4.2837117451299997e+00, 2.5653994050649999e+02 },
    { 5.7327000000000000e-07, 8.8500496531999995e-01, 1.5499118388799999e+01 },
    { 5.7029000000000001e-07, 4.4733652638299999e+00, 9.4917560896980001e+02 },
    { 5.6594000000000000e-07, 6.1853017732800000e+00, 2.2137585028530000e+02 },
    { 5.5782000000000000e-07, 4.5192944247900000e+00, 1.7878939652259999e+02 },
    { 5.2496000000000001e-07, 3.7777441417400000e+00, 2.9021955801939998e+02 },
    { 5.0470000000000001e-07, 1.0838652708400001e+00, 3.6909195360399998e+01 },
    { 4.9423000000000005e-07, 3.8824723028100001e+00, 5.2549817940059995e+02 },
    { 4.8927000000000004e-07, 1.2723220826099999e+00, 1.9010580526599998e+01 },
    { 4.8296999999999999e-07, 6.2009177689300001e+00, 3.3406124266997999e+03 },
    { 4.7341000000000000e-07, 2.8133861063399999e+00, 3.6500115658670001e+02 },
    { 4.6773999999999998e-07, 8.3941291962999998e-01, 3.9321532631000000e+00 },
    { 4.5759000000000001e-07, 4.7598875641899996e+00, 4.9155792945680003e+02 },
    { 4.5642000000000001e-07, 2.1762762556299999e+00, 1.1736805233000000e+02 },
    { 4.5277000000000000e-07, 2.1515321589000000e-01, 1.2238402773999999e+00 },
    { 4.3900000000000000e-07, 2.2494356838999998e+00, 3.2503961345299999e+01 },
    { 4.2227999999999999e-07, 5.6469094091700001e+00, 7.6787336137400004e+01 },
    { 4.2160000000000002e-07, 4.4865661462100004e+00, 3.1231936958100000e+01 },
    { 4.1712999999999998e-07, 2.1621386605400001e+00, 1.0212488945514000e+03 },
    { 4.0830999999999998e-07, 2.5809575872399999e+00, 3.1071461125430000e+02 },
    { 4.0526000000000002e-07, 3.7261107270000003e-02, 2.5802441321480001e+02 },
    { 3.9603999999999999e-07, 4.6007283215800001e+00, 5.6363121503840000e+02 },
    { 3.8967000000000002e-07, 5.0538159307399999e+00, 7.1135470007999997e+00 },
    { 3.8879999999999999e-07, 3.2938119897900000e+00, 7.6314255385400003e+01 },
    { 3.8879999999999999e-07, 5.6654650955999997e-01, 7.6217887165799993e+01 },
    { 3.8844000000000000e-07, 9.8004799212000004e-01, 4.1756372360199997e+01 },
    { 3.8617999999999998e-07, 2.3171683280300002e+00, 5.0663208609999998e+01 },
    { 3.8369000000000002e-07, 3.6602739792000000e-01, 3.9356875915200000e+01 },
    { 3.8281000000000002e-07, 3.0795321775000001e-01, 5.6772584022999997e+00 },
    { 3.8212999999999999e-07, 2.8030398995000000e-01, 7.5302863429100000e+01 },
    { 3.8163000000000001e-07, 5.2336737683600001e+00, 4.3762109930299999e+01 },
    { 3.7776999999999997e-07, 4.4597718303400002e+00, 2.9318850343600002e+02 },
    { 3.7422000000000003e-07, 2.8462420622700000e+00, 1.7526107818300002e+01 },
    { 3.5904000000000002e-07, 1.7178502657000000e-01, 4.4923210812500002e+02 },
    { 3.5690000000000003e-07, 2.1777987404600001e+00, 7.0849445304200003e+01 },
    { 3.5246999999999998e-07, 2.0766798411000001e-01, 3.4509698915400001e+01 },
    { 3.4974000000000001e-07, 3.5468467207800001e+00, 3.1368355667089997e+02 },
    { 3.4964000000000000e-07, 2.0123650540200000e+00, 3.6233367223999999e+00 },
    { 3.4672000000000002e-07, 4.4829400838500000e+00, 3.1931263096340001e+02 },
    { 3.4456000000000001e-07, 1.6582104782600000e+00, 3.0508553696180002e+02 },
    { 3.4351999999999999e-07, 1.0828898298600000e+00, 3.3467169191799996e+01 },
    { 3.3509000000000002e-07, 5.6565242241800000e+00, 6.7880499887599996e+01 },
    { 3.3084999999999997e-07, 4.5956024354500000e+00, 3.4200882374700001e+01 },
    { 3.2754999999999999e-07, 5.5786964743000000e+00, 9.1786441523799994e+01 },
    { 3.2206000000000001e-07, 2.2993764438800000e+00, 7.8713751830400000e+01 },
    { 3.1496000000000001e-07, 4.1079890282999996e+00, 2.1979525943199999e+01 },
    { 3.1314000000000001e-07, 2.5112021559900000e+00, 1.8472728735580000e+02 },
    { 2.8785999999999998e-07, 4.6505596853700002e+00, 2.1543795945209999e+02 },
    { 2.8458999999999998e-07, 1.1299014227000000e-01, 4.2798902083800002e+01 },
    { 2.8084999999999998e-07, 2.8345055870800002e+00, 1.1306332694800000e+01 },
    { 2.7524000000000002e-07, 3.4297503439699999e+00, 6.9364972595899999e+01 },
    { 2.6754000000000003e-07, 3.9478159218600002e+00, 3.2686812094890001e+02 },
    { 2.6520000000000002e-07, 3.0917290380000001e-01, 1.8621176006409999e+02 },
    { 2.6519000000000001e-07, 6.2026697637600003e+00, 5.7255490749000003e+01 },
    { 2.5982000000000001e-07, 5.3891379184300003e+00, 1.0400779795530001e+02 },
    { 2.5245000000000001e-07, 2.9772093721899999e+00, 4.0164971951619998e+02 },
    { 2.4635999999999999e-07, 2.3656687992599998e+00, 2.0495053234899999e+01 },
    { 2.4397999999999999e-07, 4.6360367059199996e+00, 3.2983706636549999e+02 },
    { 2.3863000000000001e-07, 2.8305954715999998e-01, 2.9467297614429998e+02 },
    { 2.3432000000000000e-07, 4.2473138639499997e+00, 3.5275964805800001e+01 },
    { 2.3421999999999999e-07, 9.1668385937999997e-01, 1.3554855145409999e+02 },
    { 2.3164000000000000e-07, 2.5910300956899999e+00, 6.8189316428300003e+01 },
    { 2.2793999999999999e-07, 6.1844806068000002e-01, 2.2752618943959999e+02 },
    { 2.2719000000000001e-07, 2.7346561788499999e+00, 7.9186832582400001e+01 },
    { 2.2713000000000000e-07, 5.7077545273099997e+00, 1.4365407527325999e+03 },
    { 2.2616999999999999e-07, 3.4625177647799998e+00, 3.6600378819699998e+01 },
    { 2.2482000000000000e-07, 7.2349643384999995e-01, 3.9569324236299998e+01 },
    { 2.1423000000000001e-07, 8.7822750242000003e-01, 3.9665692455900000e+01 },
    { 2.0858000000000001e-07, 3.2357748250400000e+00, 4.0990106469799997e+01 },
    { 2.0858000000000001e-07, 4.6750525194400003e+00, 5.8739963457300000e+01 },
    { 2.0671000000000000e-07, 5.3062566121900003e+00, 8.5827298831199997e+01 },
    { 2.0468999999999999e-07, 2.8499182794200002e+00, 2.0225339517410001e+02 },
    { 2.0326999999999999e-07, 4.3314851789999997e-02, 3.7478644332000002e+01 },
    { 2.0326999999999999e-07, 1.1556797609600000e+00, 3.8787426943600003e+01 },
    { 2.0302000000000000e-07, 8.9746397481999995e-01, 2.0936694217490000e+02 },
    { 2.0113000000000000e-07, 4.0143854869600002e+00, 2.1723124870109999e+02 },
    { 1.9952000000000000e-07, 5.2663001976699997e+00, 6.4959738580800007e+01 },
    { 1.9831000000000001e-07, 6.1423100257199996e+00, 1.2991947716160001e+02 },
    { 1.9388000000000001e-07, 1.6213381322700000e+00, 2.1187638603783998e+03 },
    { 1.9161999999999999e-07, 2.4985817880500001e+00, 8.1373880706300000e+01 },
    { 1.8916000000000001e-07, 5.4700215268600001e+00, 4.0138773207900002e+01 },
    { 1.8822000000000000e-07, 6.2008225976100002e+00, 3.2350541665740002e+02 },
    { 1.8699999999999999e-07, 1.1221961388299999e+00, 5.9282480178500002e+01 },
    { 1.8608999999999999e-07, 3.3878233309699999e+00, 5.4286545332400003e+01 },
    { 1.8461999999999999e-07, 9.3487935576000003e-01, 1.3851749687070000e+02 },
    { 1.7728000000000001e-07, 2.7744359847600002e+00, 1.7637982402900001e+01 },
    { 1.7322000000000001e-07, 1.6935057173300001e+00, 7.3818390720799997e+01 },
    { 1.6404999999999999e-07, 5.6930917352000003e+00, 7.3587651353180001e+02 },
    { 1.6187999999999999e-07, 2.5735679110600000e+00, 3.7499896191399998e+01 },
    { 1.6187999999999999e-07, 4.9086120088699996e+00, 3.8766175084200000e+01 },
    { 1.6170000000000000e-07, 2.6151158013999998e-01, 7.2333918012500007e+01 },
    { 1.6166000000000001e-07, 4.9640577663299998e+00, 8.5319638175199998e+02 },
    { 1.5974000000000000e-07, 7.9917645855999997e-01, 5.1546387109299997e+02 },
    { 1.5724000000000000e-07, 1.8892586701700000e+00, 1.5401661525950001e+02 },
    { 1.5475999999999999e-07, 4.7166756719700000e+00, 3.6648562929500002e+02 },
    { 1.5463000000000000e-07, 3.8897340746100002e+00, 8.3769726336320002e+02 },
    { 1.5424000000000001e-07, 6.0985852464099999e+00, 2.0910630974399999e+02 },
    { 1.5400000000000000e-07, 4.8957050905299999e+00, 5.5771018040700000e+01 },
    { 1.5360000000000000e-07, 1.3489685932000000e+00, 8.3855713880000007e+00 },
    { 1.5346000000000000e-07, 2.1258870448999998e+00, 7.4299006053259995e+02 },
    { 1.5003000000000000e-07, 6.0683729959199999e+00, 2.0606927819500001e+01 },
    { 1.4964000000000001e-07, 4.1282348469199999e+00, 3.6351668387839999e+02 },
    { 1.4928000000000000e-07, 7.8464894996000001e-01, 3.3988434053600002e+01 },
    { 1.4889000000000001e-07, 9.7067379700000000e-01, 5.2147681318300002e+01 },
    { 1.4840000000000001e-07, 6.2836398949000005e-01, 4.3810294040099997e+01 },
    { 1.4744999999999999e-07, 1.5517483460799999e+00, 8.2203962108799999e+01 },
    { 1.4352000000000000e-07, 2.0345079826600001e+00, 7.3557758289899994e+01 },
    { 1.4292000000000001e-07, 5.6654718305399996e+00, 7.0066423920080001e+02 },
    { 1.4252999999999999e-07, 4.8740241601100003e+00, 2.5208652238159999e+02 },
    { 1.3971000000000000e-07, 5.1125615514699998e+00, 7.6526703706500001e+01 },
    { 1.3965999999999999e-07, 4.4540403273900004e+00, 2.3593301268700000e+02 },
    { 1.3771000000000001e-07, 4.4939789447300003e+00, 7.6377945860200001e+01 },
    { 1.3771000000000001e-07, 5.6495648197100001e+00, 7.6154196690999996e+01 },
    { 1.3663000000000000e-07, 4.3266174775999999e-01, 3.3132153907380001e+02 },
    { 1.3656000000000000e-07, 5.3398401597599996e+00, 1.9122455111200001e+01 },
    { 1.3631999999999999e-07, 2.8668363624700000e+00, 4.5034134317499998e+01 },
    { 1.3308000000000000e-07, 3.1090723396100000e+00, 4.3829828244570001e+02 },
    { 1.2914000000000000e-07, 2.3269719617700000e+00, 1.0728555991259999e+02 },
    { 1.2911000000000000e-07, 4.1434577053999999e-01, 4.2277637222000003e+01 },
    { 1.2793999999999999e-07, 5.9005533593899999e+00, 2.6826702943000001e+01 },
    { 1.2269000000000000e-07, 4.9756051303000000e+00, 7.6005438844699995e+01 },
    { 1.2251000000000000e-07, 1.1882441108899999e+00, 4.7039871887600000e+01 },
    { 1.2137000000000001e-07, 2.3301677412299999e+00, 4.2065188900899997e+01 },
    { 1.2109000000000001e-07, 2.1014243045600001e+00, 3.2764593776200002e+01 },
    { 1.1769000000000000e-07, 4.2418161845700002e+00, 2.7555052103309998e+02 },
    { 1.1622000000000000e-07, 6.1757259414999999e-01, 5.3362311835770004e+02 },
    { 1.1498000000000000e-07, 5.5804642016599999e+00, 1.4065636088479999e+02 },
    { 1.1490000000000000e-07, 3.7665381438800001e+00, 5.2575881183150000e+02 },
    { 1.1480999999999999e-07, 2.2554735364399998e+00, 3.7302954235300000e+01 },
    { 1.1480999999999999e-07, 5.2267063834999998e+00, 3.8963117040299998e+01 },
    { 1.1455000000000000e-07, 1.2118510333400001e+00, 5.2916970023279998e+02 },
    { 1.1419000000000001e-07, 4.8011883937700004e+00, 4.0313419222450000e+02 },
    { 1.1035000000000001e-07, 3.1646861108600000e+00, 5.3021222995640005e+02 },
    { 1.0956999999999999e-07, 4.4045124299999996e+00, 1.9907200143640000e+02 },
    { 1.0691999999999999e-07, 4.3515949000200003e+00, 1.8769623277240001e+02 },
    { 1.0361000000000000e-07, 1.4864296480000001e-02, 1.1950691634410001e+02 },
    { 1.0275000000000000e-07, 2.3381690767299999e+00, 5.6782400073240001e+02 },
    { 1.0240000000000000e-07, 2.9184673870000002e+00, 8.0458856969600006e+01 },
    { 1.0223000000000000e-07, 5.2252219942700000e+00, 1.2099138905240000e+02 },
    { 1.0218000000000001e-07, 1.5937912520000000e-01, 6.5874762317500000e+01 },
    { 1.0188999999999999e-07, 2.5898549646900002e+00, 2.1458261081400000e+01 },
    { 1.0031000000000000e-07, 5.3807557972600000e+00, 4.3501477499400004e+01 },
    { 9.9079999999999998e-08, 3.5064913207199999e+00, 5.4391805909619995e+02 },
    { 9.8140000000000004e-08, 2.4878691859900002e+00, 2.1149444540699999e+01 },
    { 9.7530000000000000e-08, 8.8878589600000002e-02, 1.6805251279940001e+02 },
    { 9.6779999999999997e-08, 6.0734434507100001e+00, 9.5628915597059995e+02 },
    { 9.0260000000000002e-08, 4.7778484212399999e+00, 3.4793262315539999e+02 },
    { 9.0040000000000004e-08, 3.7183459802000002e-01, 1.0186893394120000e+02 },
    { 8.9859999999999999e-08, 5.6289901162399998e+00, 1.4788665740644001e+03 },
    { 8.9859999999999999e-08, 4.2935262631900004e+00, 4.0016524680790002e+02 },
    { 8.7670000000000004e-08, 2.0426411565099998e+00, 6.2439816792520003e+02 },
    { 8.7359999999999996e-08, 3.9924740321100001e+00, 4.5936278465219999e+02 },
    { 8.6630000000000002e-08, 4.0550335952900003e+00, 2.7703499374140000e+02 },
    { 8.5209999999999993e-08, 2.0759016203199998e+00, 9.5979227217800002e+01 },
    { 8.4540000000000001e-08, 2.5237785048800001e+00, 1.4414657116320001e+02 },
    { 8.4180000000000003e-08, 1.4851541520600000e+00, 7.7489911552999999e+01 },
    { 8.4139999999999998e-08, 2.4700578100800001e+00, 3.7715882254339999e+02 },
    { 8.3620000000000004e-08, 6.2230895069000003e-01, 7.0996483432549996e+02 },
    { 8.3570000000000000e-08, 3.5447021961999998e+00, 4.8025159676200002e+02 },
    { 8.3360000000000000e-08, 4.0706659654999999e-01, 1.6921656695024001e+03 },
    { 8.1709999999999995e-08, 1.8092107055800000e+00, 1.4227094001599999e+01 },
    { 8.0919999999999999e-08, 4.5119457614099998e+00, 4.8588067105450000e+02 },
    { 7.9590000000000000e-08, 3.8379928039000000e-01, 1.7097327410619999e+02 },
    { 7.9269999999999994e-08, 9.3746230599000002e-01, 9.0684978763799995e+02 },
    { 7.9100000000000000e-08, 1.4261722946700000e+00, 5.2650957135689998e+02 },
    { 7.9090000000000002e-08, 5.0484947966500000e+00, 2.8873508531110002e+02 },
    { 7.9090000000000002e-08, 1.6436822118300001e+00, 3.6387930498599999e+01 },
    { 7.9090000000000002e-08, 2.6969050545100002e+00, 3.9878140776999999e+01 },
    { 7.9080000000000004e-08, 2.9032046319500000e+00, 5.3287235883230005e+02 },
    { 7.8370000000000006e-08, 3.6694469486999998e-01, 1.0930615477020999e+03 },
    { 7.8209999999999997e-08, 5.6100595105200002e+00, 4.1555249061209997e+02 },
    { 7.8050000000000001e-08, 5.6735932926199997e+00, 3.5033211960040001e+02 },
    { 7.8030000000000005e-08, 3.8548005895799999e+00, 3.1490739694830000e+02 },
    { 7.7540000000000005e-08, 7.2926728610000005e-02, 4.9447869076360001e+02 },
    { 7.7359999999999999e-08, 8.3413065439000000e-01, 1.2655674786264001e+03 },
    { 7.7000000000000001e-08, 4.4281162163500003e+00, 1.1802244363580000e+02 },
    { 7.5709999999999994e-08, 2.8173803070900001e+00, 2.3463998651499999e+01 },
    { 7.5489999999999996e-08, 3.2557423229300002e+00, 4.7494684537519998e+02 },
    { 7.4460000000000006e-08, 5.5864440185099999e+00, 9.8756927703849999e+02 },
    { 7.4040000000000007e-08, 1.0029655033400000e+00, 7.5042230998199997e+01 },
    { 7.3720000000000001e-08, 4.8871304197100001e+00, 5.4807810194200002e+01 },
    { 7.1079999999999999e-08, 1.2636213623500001e+00, 1.1731986822019999e+02 },
    { 7.0920000000000002e-08, 1.3432697790000000e-02, 4.5290362895720000e+02 },
    { 7.0669999999999997e-08, 1.9291066892399999e+00, 4.5394615868080001e+02 },
    { 7.0389999999999997e-08, 8.7709076060000002e-01, 3.6797010200329998e+02 },
    { 6.9699999999999995e-08, 5.7798871210499998e+00, 1.7730492381430000e+02 },
    { 6.9450000000000003e-08, 8.9579514200000005e-03, 1.5819593482830001e+03 },
    { 6.9380000000000003e-08, 5.6033995171999997e+00, 6.4701083331480004e+02 },
    { 6.8789999999999996e-08, 4.4795113816999999e-01, 2.1648048917570000e+02 },
    { 6.7659999999999998e-08, 1.5164047270700001e+00, 5.5659143456099997e+01 },
    { 6.7109999999999997e-08, 4.4626838048700002e+00, 2.1011770170029999e+02 },
    { 6.6990000000000006e-08, 2.9504604123100000e+00, 2.1568968960162001e+03 },
    { 6.6940000000000003e-08, 5.5308827771600004e+00, 4.8884961647109998e+02 },
    { 6.5540000000000003e-08, 2.7333269638000002e-01, 9.1278767847120002e+02 },
    { 6.4749999999999994e-08, 2.5236327803399998e+00, 5.9083042499689998e+02 },
    { 6.4259999999999994e-08, 5.0063472897799999e+00, 1.0749800823370001e+02 },
    { 6.2979999999999999e-08, 3.6497186704900000e+00, 2.7166655187980001e+02 },
    { 6.2149999999999997e-08, 5.2172593855000002e+00, 4.3978275515399997e+02 },
    { 6.1309999999999998e-08, 6.2812755168300001e+00, 6.3182399402126002e+03 },
    { 6.1270000000000005e-08, 3.6876521738500001e+00, 1.3288842257819999e+02 },
    { 6.0629999999999994e-08, 4.7604250486700002e+00, 7.4958234267160003e+02 },
    { 6.0080000000000006e-08, 4.5929432883999999e+00, 7.1293377974210000e+02 },
    { 5.9940000000000006e-08, 5.8926777149599996e+00, 9.3270914232099997e+01 },
    { 5.9470000000000002e-08, 2.4960465249100001e+00, 5.2213939318786201e+04 },
    { 5.9219999999999997e-08, 4.3713441617199997e+00, 2.8659622129700000e+02 },
    { 5.9079999999999997e-08, 1.1363521147700000e+00, 2.7741726679700001e+01 },
    { 5.8999999999999999e-08, 4.9260353503600003e+00, 5.2061407176234992e+04 },
    { 5.8719999999999999e-08, 4.9261366210600004e+00, 7.8623090560109995e+02 },
    { 5.8710000000000001e-08, 3.4099031969999999e-02, 5.2846712481719999e+02 },
    { 5.8210000000000003e-08, 1.4198194759000000e+00, 1.0248449636432200e+04 },
    { 5.7579999999999997e-08, 2.8234405884900000e+00, 2.9795648359600001e+01 },
    { 5.7480000000000003e-08, 2.9383329374499998e+00, 1.6922816896699999e+02 },
    { 5.7189999999999998e-08, 4.9625207381700003e+00, 3.2176031151820001e+02 },
    { 5.6949999999999997e-08, 5.9378823661600002e+00, 2.2286032299359999e+02 },
    { 5.6850000000000003e-08, 1.7115139774899999e+00, 1.1653954657146000e+03 },
    { 5.6830000000000000e-08, 5.9640596365700000e+00, 3.8135160823740000e+02 },
    { 5.6790000000000001e-08, 5.9177588296300003e+00, 1.6983591097100000e+01 },
    { 5.6710000000000003e-08, 5.7312591516099998e+00, 6.7480074410429995e+02 },
    { 5.6540000000000002e-08, 3.0404489308999998e+00, 1.2604284735620600e+04 },
    { 5.6370000000000001e-08, 1.4931346796000000e-01, 7.3879727483859995e+02 },
    { 5.5719999999999999e-08, 8.7437668538000002e-01, 2.9104963942190000e+02 },
    { 5.5369999999999999e-08, 4.0882285275000002e-01, 6.2479117597701998e+03 },
    { 5.5080000000000001e-08, 4.2772425439699999e+00, 1.7787437278589999e+02 },
    { 5.4539999999999997e-08, 5.9354330367400001e+00, 7.1144930703379998e+02 },
    { 5.4020000000000003e-08, 4.4313805730000000e-01, 2.8362727588040002e+02 },
    { 5.3799999999999999e-08, 5.0944571970999997e+00, 8.2287946853059998e+02 },
    { 5.3480000000000000e-08, 1.4246817055900001e+00, 2.4639654819099999e+01 },
    { 5.3179999999999997e-08, 5.2269743923899998e+00, 4.4283374792099998e+01 },
    { 5.3090000000000001e-08, 1.8302516817600001e+00, 1.0178121455989800e+04 },
    { 5.2959999999999999e-08, 1.3083292093100001e+00, 2.8117959532560002e+02 },
    { 5.2700000000000002e-08, 3.4674283467999998e+00, 6.1671922703697001e+03 },
    { 5.2439999999999998e-08, 2.3367500956199998e+00, 7.3345309968799995e+01 },
    { 5.2089999999999999e-08, 4.0219546182899997e+00, 1.6540326338646000e+03 },
    { 5.1889999999999997e-08, 5.7734353632099999e+00, 3.5181659230870002e+02 },
    { 5.1679999999999997e-08, 4.7748540655799996e+00, 2.5520079140841790e+04 },
    { 5.1459999999999999e-08, 8.1172839255999996e-01, 4.6518607025800002e+01 },
    { 5.1249999999999999e-08, 8.7728916069999996e-01, 1.3406407874580000e+02 },
    { 5.1140000000000000e-08, 2.5263546939900001e+00, 1.1201488830759999e+03 },
    { 5.1060000000000002e-08, 4.6652102631399996e+00, 2.4431858407499999e+02 },
    { 5.1000000000000000e-08, 4.4947275072200004e+00, 4.3681380973739999e+02 },
    { 5.0799999999999998e-08, 4.3216777176800001e+00, 5.6511568774670002e+02 },
    { 5.0479999999999999e-08, 3.9507950411100001e+00, 3.4496367773880002e+02 },
    { 5.0409999999999999e-08, 2.5218332913800001e+00, 2.7258157561650000e+02 },
    { 5.0039999999999997e-08, 5.3392408006000003e-01, 7.4260333705500003e+01 },
    { 5.0020000000000001e-08, 7.3296305202000001e-01, 2.0736120460480001e+02 },
    { 4.9840000000000002e-08, 4.8908817125299997e+00, 1.0097401966589299e+04 },
    { 4.9819999999999999e-08, 7.9282576065999999e-01, 7.4661339725500000e+02 },
    { 4.9490000000000002e-08, 4.4279331610600003e+00, 1.4746737883704000e+03 },
    { 4.9439999999999998e-08, 2.3170372349299999e+00, 6.0176425067620005e+02 },
    { 4.8730000000000000e-08, 4.6500648916999996e+00, 5.2802072624099999e+01 },
    { 4.8510000000000002e-08, 1.5893768633100001e+00, 6.3518323575820000e+02 },
    { 4.8009999999999998e-08, 5.4948304297100004e+00, 1.4669036986300000e+01 },
    { 4.7979999999999997e-08, 5.2392886963000000e+00, 5.2995159752550001e+02 },
    { 4.7890000000000001e-08, 2.6869123384200000e+00, 6.2747898792640001e+02 },
    { 4.7650000000000000e-08, 6.2733256137000000e+00, 3.2455777235500001e+01 },
    { 4.7540000000000001e-08, 1.4368787800800000e+00, 6.2097787241324004e+03 },
    { 4.7110000000000003e-08, 4.3747301673000000e-01, 5.2252072354423988e+04 },
    { 4.6989999999999999e-08, 5.8682544133299999e+00, 2.1478356814630001e+02 },
    { 4.6870000000000002e-08, 2.5592731682299998e+00, 5.7143616164400001e+01 },
    { 4.6469999999999999e-08, 3.7789433311699998e+00, 1.3757737998458001e+03 },
    { 4.6460000000000000e-08, 2.8777518516299998e+00, 1.3286183635523599e+04 },
    { 4.6339999999999997e-08, 1.7913809213000000e+00, 3.2276977924947000e+03 },
    { 4.6290000000000000e-08, 9.7704654493999998e-01, 1.2247586176070000e+02 },
    { 4.5930000000000002e-08, 1.4821025070999999e-01, 1.0254416802454000e+03 },
    { 4.5860000000000002e-08, 5.2420702929300003e+00, 1.8084331820250000e+02 },
    { 4.5849999999999997e-08, 1.6380866198399999e+00, 1.2726810256271999e+03 },
    { 4.5769999999999999e-08, 4.3669152689699997e+00, 2.6087903141574199e+04 },
    { 4.5660000000000000e-08, 2.5286672620499999e+00, 3.9557870223899999e+02 },
    { 4.5569999999999997e-08, 1.3876816810199999e+00, 5.9853467282869997e+02 },
    { 4.5569999999999997e-08, 6.2751795646900002e+00, 2.5974988507369089e+04 },
    { 4.5459999999999998e-08, 6.1747931989899998e+00, 3.0423420369989998e+02 },
    { 4.5279999999999999e-08, 5.8788999999999995e-07, 1.5615547927360001e+02 },
    { 4.5260000000000003e-08, 4.5540183009500002e+00, 5.2554636351040006e+02 },
    { 4.5079999999999997e-08, 1.5826379449700001e+00, 4.5490936652729999e+02 },
    { 4.4920000000000001e-08, 9.9127019649999995e-02, 5.2177290755856688e+04 },
    { 4.4829999999999998e-08, 2.4400897573000000e+00, 3.8427236954419999e+02 },
    { 4.4820000000000000e-08, 1.7054978936900000e+00, 3.4236723933770003e+02 },
    { 4.4510000000000000e-08, 2.4208188713799998e+00, 3.1982696483500000e+01 },
    { 4.4400000000000001e-08, 4.1804191495499996e+00, 4.9033408917940000e+02 },
    { 4.4300000000000000e-08, 5.3791651224699999e+00, 5.2943033266370003e+02 },
    { 4.4269999999999999e-08, 3.0015620824400000e+00, 1.8918070548070000e+02 },
    { 4.4239999999999998e-08, 1.3784963395600001e+00, 2.0388438056784202e+04 },
    { 4.3530000000000000e-08, 4.2424548638399999e+00, 1.0452906281710000e+02 },
    { 4.3480000000000003e-08, 1.5242053493400001e+00, 2.0312171985508590e+04 },
    { 4.3429999999999999e-08, 6.6907601244000003e-01, 3.2027583880989999e+02 },
    { 4.3359999999999999e-08, 5.8010298240499996e+00, 5.3653405886000002e+01 },
    { 4.3299999999999997e-08, 4.9957358051299998e+00, 3.2324478422649997e+02 },
    { 4.3040000000000000e-08, 6.3784520720000004e-01, 1.2567636172691100e+04 },
    { 4.2930000000000001e-08, 4.6587397222600000e+00, 4.6470422915999997e+01 },
    { 4.2879999999999998e-08, 5.1173355710799999e+00, 2.3905941636200001e+01 },
    { 4.2839999999999998e-08, 4.9741498256599996e+00, 3.9654191008549998e+02 },
    { 4.2799999999999999e-08, 4.4246929651300002e+00, 2.7463549729639999e+02 },
    { 4.2699999999999999e-08, 4.2448124645399998e+00, 1.5777665625889999e+03 },
    { 4.2680000000000002e-08, 2.5926906505899998e+00, 3.7390799283650000e+02 },
    { 4.2310000000000000e-08, 5.3591418882999999e+00, 3.1280121067900001e+01 },
    { 4.2100000000000000e-08, 3.4066196874600001e+00, 5.1159540830470002e+02 },
    { 4.1999999999999999e-08, 2.0873003352300001e+00, 9.4349835056749998e+02 },
    { 4.1920000000000001e-08, 2.9273630154000002e-01, 2.7727884259674001e+03 },
    { 4.1890000000000000e-08, 6.1649996887500000e+00, 1.0160830151030000e+02 },
    { 4.1820000000000000e-08, 1.1176344934100000e+00, 6.6064432548323002e+03 },
    { 4.1810000000000002e-08, 5.1173524308099996e+00, 5.6660016045500004e+02 },
    { 4.1780000000000001e-08, 4.9042681403700001e+00, 9.7892307321960004e+02 },
    { 4.1500000000000001e-08, 3.8631937887399999e+00, 2.7199209958499999e+01 },
    { 4.1350000000000003e-08, 2.5941117780299998e+00, 5.0286426215159997e+02 },
    { 4.1290000000000001e-08, 3.4462403986400001e+00, 2.6048285633228090e+04 },
    { 4.1140000000000003e-08, 5.5060999292400004e+00, 7.7110429422779998e+02 },
    { 4.1140000000000003e-08, 1.8743674169300000e+00, 1.0495600702027000e+03 },
    { 4.1129999999999998e-08, 5.7005313908000002e+00, 1.4148644228730001e+02 },
    { 4.1010000000000001e-08, 1.4089017442500000e+00, 8.7583029900099996e+02 },
    { 4.0949999999999999e-08, 3.9427917449000000e-01, 3.5543992903110001e+02 },
    { 4.0900000000000002e-08, 3.4930540427499999e+00, 5.3117543780289998e+02 },
    { 4.0840000000000000e-08, 3.3581637094899999e+00, 7.6562397778160005e+02 },
    { 4.0749999999999998e-08, 3.7478142240199999e+00, 3.8161224066829999e+02 },
    { 4.0730000000000001e-08, 1.8029210803000001e-01, 9.4943624140070006e+02 },
    { 4.0639999999999999e-08, 1.0332454782900000e+00, 6.1305437074401998e+03 },
    { 4.0600000000000000e-08, 6.0608787739599999e+00, 5.1626416456500003e+01 },
    { 4.0439999999999997e-08, 2.8011690840900001e+00, 2.5596345212117401e+04 },
    { 4.0399999999999998e-08, 5.1337849007500003e+00, 4.3319047301500001e+02 },
    { 4.0219999999999999e-08, 1.7207880965400000e+00, 1.4775945496771999e+03 },
    { 4.0210000000000001e-08, 1.4455396364000000e-01, 6.6034743094157002e+03 },
    { 4.0170000000000002e-08, 5.3860971846199996e+00, 5.2174321810440088e+04 },
    { 4.0119999999999998e-08, 2.1508086558500001e+00, 5.2360129639400000e+01 },
    { 4.0030000000000002e-08, 4.7238254042200003e+00, 9.4755386940400001e+01 },
    { 3.9979999999999998e-08, 1.0964584561199999e+00, 4.0461866493280002e+02 },
    { 3.9839999999999998e-08, 5.4093206086599999e+00, 4.7643131808350000e+02 },
    { 3.9430000000000003e-08, 5.6804667159999997e+00, 9.8311585891360005e+02 },
    { 3.9379999999999999e-08, 2.1324047525899998e+00, 2.2582926841020000e+02 },
    { 3.9370000000000001e-08, 1.2413599087200000e+00, 1.6173840709351000e+03 },
    { 3.9179999999999997e-08, 4.2059233131699996e+00, 3.7567434983509997e+02 },
    { 3.9150000000000003e-08, 4.2745485946999997e+00, 1.5961864422845999e+03 },
    { 3.9039999999999998e-08, 4.2229141199999999e-03, 3.7004527554260000e+02 },
    { 3.8759999999999998e-08, 4.3445904479299999e+00, 1.4297091499560000e+02 },
    { 3.8549999999999998e-08, 5.9056751589000001e-01, 5.1838463239980001e+02 },
    { 3.8500000000000001e-08, 5.9253823304399997e+00, 1.2564667227274500e+04 },
    { 3.8470000000000000e-08, 4.0878353562700003e+00, 3.0923013854599998e+02 },
    { 3.8390000000000002e-08, 2.4565123374600000e+00, 1.0060753403659801e+04 },
    { 3.8280000000000003e-08, 2.2807762060800001e+00, 2.1384770163205999e+03 },
    { 3.8229999999999999e-08, 2.8094781941300000e+00, 1.0139988420352000e+04 },
    { 3.8110000000000002e-08, 3.3295958822900000e+00, 7.3297125859000005e+02 },
    { 3.7980000000000000e-08, 1.5487417608000000e+00, 3.5692440173940003e+02 },
    { 3.7930000000000003e-08, 8.8752668901999998e-01, 1.0457724692690000e+02 },
    { 3.7919999999999998e-08, 7.5463890542000001e-01, 1.1169079392770000e+02 },
    { 3.7889999999999997e-08, 4.2935290947500002e+00, 3.1018797192920001e+03 },
    { 3.7830000000000002e-08, 2.6937653478000001e-01, 1.6144151255185000e+03 },
    { 3.7639999999999998e-08, 4.3223597859699998e+00, 1.6508356738280000e+02 },
    { 3.7520000000000001e-08, 5.8157199762299996e+00, 3.0360106425350000e+02 },
    { 3.7499999999999998e-08, 3.4794819661199998e+00, 2.3501798895030001e+02 },
    { 3.7450000000000001e-08, 4.7089668288300004e+00, 1.0257023126762999e+03 },
    { 3.7259999999999997e-08, 5.5317699448000002e-01, 1.0962429414398000e+03 },
    { 3.7009999999999999e-08, 4.8566338776500002e+00, 4.5220105354160000e+02 },
    { 3.6960000000000001e-08, 1.8260315066799999e+00, 1.4557162401699999e+01 },
    { 3.6820000000000001e-08, 5.5056840127599997e+00, 3.6203221117010003e+02 },
    { 3.6739999999999997e-08, 4.9264943248999998e-01, 1.8376407950929999e+02 },
    { 3.6699999999999998e-08, 1.5174640430899999e+00, 5.7846191580000003e+01 },
    { 3.6400000000000002e-08, 2.2273593184600000e+00, 2.5650752071965999e+03 },
    { 3.6230000000000001e-08, 2.2945639272200000e+00, 1.6653389665594000e+03 },
    { 3.5940000000000003e-08, 6.0672407488699998e+00, 2.1744369702220001e+02 },
    { 3.5780000000000000e-08, 3.4108141486000001e+00, 5.6214674233009998e+02 },
    { 3.5630000000000002e-08, 5.3746119263100001e+00, 1.2451752593069399e+04 },
    { 3.5240000000000003e-08, 9.9824225459999999e-02, 1.1653797092750000e+02 },
    { 3.5089999999999999e-08, 8.5163274980000003e-01, 1.4407335384266000e+03 },
    { 3.5050000000000000e-08, 1.4922991844400000e+00, 4.5884151979040001e+02 },
    { 3.4900000000000001e-08, 4.4073340948700004e+00, 3.2642617796420001e+02 },
    { 3.4900000000000001e-08, 2.8508227568500000e+00, 6.6412755085590004e+02 },
    { 3.4770000000000000e-08, 2.0810129512400000e+00, 6.6523951654869995e+02 },
    { 3.4749999999999997e-08, 5.1931932869399997e+00, 3.2538364824060000e+02 },
    { 3.4660000000000001e-08, 4.7389987391400004e+00, 1.1511683717129999e+03 },
    { 3.4469999999999997e-08, 3.5396368307900001e+00, 4.9989531673499999e+02 },
    { 3.4459999999999999e-08, 2.5558115457300001e+00, 1.5098378785916000e+03 },
    { 3.4450000000000001e-08, 2.5642565941200002e+00, 8.9132941738980003e+02 },
    { 3.4289999999999998e-08, 4.7342229387800003e+00, 5.2980283967920002e+02 },
    { 3.4280000000000000e-08, 1.2413479847800000e+00, 2.3517761117586001e+03 },
    { 3.4200000000000002e-08, 9.8621583590999995e-01, 3.0274973099160002e+02 },
    { 3.4170000000000001e-08, 3.0710235492400000e+00, 1.0551891444952000e+03 },
    { 3.4130000000000002e-08, 2.4724122827500001e+00, 9.5931043107999997e+01 },
    { 3.4119999999999997e-08, 1.2700696443099999e+00, 7.7258876693610000e+02 },
    { 3.4060000000000002e-08, 5.8897874388100000e+00, 5.2957909051000001e+02 },
    { 3.4020000000000003e-08, 4.8530752218200002e+00, 8.9908209940600003e+01 },
    { 3.4000000000000000e-08, 2.9121972390700002e+00, 5.3383556667879998e+02 },
    { 3.3979999999999997e-08, 3.5539905621900001e+00, 7.6961982151949996e+02 },
    { 3.3960000000000001e-08, 5.5779034264899998e+00, 5.2820649238630006e+02 },
    { 3.3950000000000003e-08, 1.4087268392100001e+00, 5.2401370669230005e+02 },
    { 3.3850000000000002e-08, 3.0663786918000002e-01, 6.6109491496450005e+02 },
    { 3.3670000000000003e-08, 1.9174358000700000e+00, 2.0037478847900001e+01 },
    { 3.3650000000000000e-08, 1.7431487941999999e+00, 6.4260559929969997e+02 },
    { 3.3640000000000002e-08, 4.6379292745400003e+00, 2.8490544972429998e+03 },
    { 3.3629999999999997e-08, 2.3183870730299998e+00, 2.4549424024260000e+02 },
    { 3.3570000000000002e-08, 4.7265882645600001e+00, 1.2274344429886000e+03 },
    { 3.3489999999999997e-08, 1.7432591237499999e+00, 6.7183179868770003e+02 },
    { 3.3470000000000001e-08, 4.9153591174700004e+00, 2.0351789493854700e+04 },
    { 3.3400000000000001e-08, 1.5384852484600000e+00, 7.0588812873300000e+01 },
    { 3.3400000000000001e-08, 3.1213496077300000e+00, 1.7224529849340001e+02 },
    { 3.3370000000000000e-08, 4.8080933886999997e+00, 3.5840887444769999e+02 },
    { 3.3309999999999999e-08, 8.1282265496999995e-01, 1.0134050529518800e+04 },
    { 3.3290000000000002e-08, 5.9192542424300001e+00, 6.6396027179300006e+01 },
    { 3.3279999999999998e-08, 6.4683172590000004e-02, 5.1960847267719998e+02 },
    { 3.3260000000000001e-08, 2.1230383654299998e+00, 4.4985950207700000e+01 },
    { 3.3260000000000001e-08, 2.3408860291900000e+00, 3.4613933390640000e+02 },
    { 3.3139999999999998e-08, 1.6485666294000001e+00, 9.3935374898329997e+02 },
    { 3.3099999999999999e-08, 6.3021323746000002e-01, 1.5861039498672001e+03 },
    { 3.3060000000000000e-08, 6.1109509469500001e+00, 7.4809786996330001e+02 },
    { 3.3029999999999999e-08, 1.1644217651700000e+00, 2.4161027108930000e+02 },
    { 3.2990000000000000e-08, 1.5502724918999999e-01, 1.4040849754971000e+03 },
    { 3.2980000000000002e-08, 3.7071499295999999e-01, 1.0252332524700000e+02 },
    { 3.2910000000000002e-08, 1.7903863908100000e+00, 2.6695250132400002e+02 },
    { 3.2780000000000000e-08, 3.6895104628800000e+00, 9.4201387785919997e+02 },
    { 3.2759999999999997e-08, 1.9566404108800000e+00, 1.3066212755710001e+03 },
    { 3.2749999999999999e-08, 2.2015285458500000e+00, 7.6981676347560006e+02 },
    { 3.2700000000000002e-08, 1.0990491157500000e+00, 7.8404935289700006e+01 },
    { 3.2649999999999998e-08, 1.5495032550700001e+00, 2.4490896972800002e+01 },
    { 3.2640000000000000e-08, 9.8644966742999995e-01, 1.5071777497157000e+03 },
    { 3.2600000000000001e-08, 1.9753430023500000e+00, 5.3776771994190005e+02 },
    { 3.2590000000000003e-08, 5.6226194325599996e+00, 6.5812596954790001e+02 },
    { 3.2570000000000000e-08, 1.5247428380900001e+00, 3.0571867640820000e+02 },
    { 3.2539999999999999e-08, 6.2494781688899996e+00, 1.0197644218431000e+03 },
    { 3.2500000000000000e-08, 3.5040212485100000e+00, 7.0904981058880003e+02 },
    { 3.2420000000000002e-08, 3.4378180947899999e+00, 9.7866244078870000e+02 },
    { 3.2409999999999997e-08, 6.1312431844899997e+00, 3.5304043258610000e+02 },
    { 3.2310000000000003e-08, 4.6136852724999997e+00, 4.0106476417959999e+02 },
    { 3.2269999999999997e-08, 1.4427352845800001e+00, 4.0610313764109998e+02 },
    { 3.2269999999999997e-08, 5.7384243874400003e+00, 6.2038408332992003e+03 },
    { 3.2140000000000002e-08, 3.9423645775699998e+00, 2.0348820548438089e+04 },
    { 3.2089999999999998e-08, 5.6517277141899998e+00, 1.5126611373299999e+01 },
    { 3.1960000000000003e-08, 5.8428596998300000e+00, 3.6286229257259998e+02 },
    { 3.1949999999999998e-08, 5.6236366123000003e+00, 3.5330106501699998e+02 },
    { 3.1900000000000001e-08, 2.2496692083999998e+00, 2.8214280317210000e+02 },
    { 3.1860000000000002e-08, 1.7407312534399999e+00, 1.4904193227280001e+02 },
    { 3.1839999999999999e-08, 3.9378687713499998e+00, 5.5406998748280000e+02 },
    { 3.1760000000000001e-08, 3.2973109487999999e+00, 3.0115338369870000e+02 },
    { 3.1730000000000000e-08, 5.4664175274100000e+00, 1.4011160300805000e+03 },
    { 3.1699999999999999e-08, 5.3683800464999998e-01, 1.0113788504551000e+03 },
    { 3.1680000000000003e-08, 2.2901953167200002e+00, 3.8994962794650002e+02 },
    { 3.1590000000000000e-08, 4.1444169977899996e+00, 1.9779997704920001e+02 },
    { 3.1580000000000002e-08, 1.1815204972300000e+00, 5.3977345751200005e+02 },
    { 3.1489999999999999e-08, 4.8734813281299996e+00, 2.2796813242429999e+02 },
    { 3.1480000000000001e-08, 5.3390085619200001e+00, 1.1228571960617001e+03 },
    { 3.1440000000000002e-08, 1.9826796050000001e-02, 1.5042088042990999e+03 },
    { 3.1340000000000001e-08, 2.4356724269900001e+00, 9.5920991727740000e+02 },
    { 3.1270000000000001e-08, 3.0451144735900000e+00, 2.2530800354839999e+02 },
    { 3.1219999999999998e-08, 4.8735236747600004e+00, 1.2303552042954000e+03 },
    { 3.1189999999999997e-08, 3.7316327890699998e+00, 1.4350562800242999e+03 },
    { 3.1160000000000002e-08, 2.3172542333599999e+00, 6.1716456884946001e+03 },
    { 3.1119999999999997e-08, 1.1103572312000001e+00, 5.2523754696970002e+02 },
    { 3.1070000000000000e-08, 5.3824032504600003e+00, 2.8683682666799999e+01 },
    { 3.1030000000000001e-08, 5.2595457867200004e+00, 8.5952803146010001e+02 },
    { 3.0969999999999999e-08, 1.3826811235400001e+00, 2.5950888592310002e+02 },
    { 3.0909999999999997e-08, 1.8824426266100001e+00, 3.7901580226720000e+02 },
    { 3.0890000000000001e-08, 6.0901544497500000e+00, 3.4051025961390002e+02 },
    { 3.0570000000000002e-08, 4.3231512280200004e+00, 3.5478553772530000e+02 },
    { 3.0570000000000002e-08, 1.3458822091600000e+00, 5.2690198039499997e+01 },
    { 3.0570000000000002e-08, 6.1362977107700001e+00, 2.3575873236100001e+01 },
    { 3.0530000000000003e-08, 3.2122717556599998e+00, 9.4646729598410002e+02 },
    { 3.0519999999999998e-08, 3.2453752251400001e+00, 4.6350738623640001e+02 },
    { 3.0479999999999999e-08, 4.6567465772399999e+00, 4.7346237266690002e+02 },
    { 3.0449999999999998e-08, 4.5897093461600003e+00, 4.7733083545519997e+02 },
    { 3.0349999999999997e-08, 5.9822524729900000e+00, 3.3061896365820002e+02 },
    { 3.0320000000000003e-08, 4.2538305456899996e+00, 2.3798693436689999e+02 },
    { 3.0309999999999998e-08, 2.7512717478100002e+00, 1.5645115277816958e+05 },
    { 3.0260000000000001e-08, 5.3178244297999999e-01, 4.6473122651379998e+02 },
    { 3.0220000000000002e-08, 1.8727173651500000e+00, 4.6053844081979997e+02 },
    { 3.0120000000000001e-08, 2.4750674481000001e-01, 3.4736317418380003e+02 },
    { 3.0120000000000001e-08, 2.3854632727000000e-01, 2.9912639426919998e+02 },
    { 3.0110000000000003e-08, 5.6313348080400001e+00, 6.3815218117480003e+02 },
    { 3.0050000000000001e-08, 3.0237433514399998e+00, 1.9295831548977001e+03 },
    { 3.0029999999999998e-08, 5.6541222259000001e+00, 4.1677633088950000e+02 },
    { 3.0010000000000002e-08, 5.6240042303899997e+00, 1.2747179660680000e+02 },
    { 2.9989999999999999e-08, 2.0688661105600001e+00, 1.9266142094811000e+03 },
    { 2.9950000000000000e-08, 3.5571957883600001e+00, 1.3995378546920000e+02 },
    { 2.9940000000000002e-08, 2.9888408610000000e-01, 6.6812248533995999e+03 },
    { 2.9919999999999999e-08, 3.7409127210499999e+00, 1.0101855384714199e+04 },
    { 2.9830000000000003e-08, 2.1184054542899999e+00, 6.0150361824530000e+02 },
    { 2.9830000000000003e-08, 9.6467854341000003e-01, 7.8326196018450003e+02 },
    { 2.9760000000000000e-08, 1.4540795950600001e+00, 1.0143477958717000e+03 },
    { 2.9720000000000001e-08, 3.0228215717000001e-01, 5.6904784100979998e+02 },
    { 2.9690000000000000e-08, 2.9102863088499999e+00, 4.8295990974770001e+02 },
    { 2.9670000000000000e-08, 1.3803435540000001e-02, 6.1503391543000001e+00 },
    { 2.9650000000000001e-08, 5.3132110509799997e+00, 4.5194042111070002e+02 },
    { 2.9639999999999999e-08, 1.0468043404800000e+00, 4.8858898404019999e+02 },
    { 2.9600000000000000e-08, 2.2014303561599999e+00, 2.4619824329779999e+03 },
    { 2.9510000000000001e-08, 5.5807887707599999e+00, 3.3727801622699999e+01 },
    { 2.9510000000000001e-08, 5.0429838027600002e+00, 4.2538269652899999e+01 },
    { 2.9460000000000000e-08, 6.2712530631599996e+00, 1.8647239249500001e+02 },
    { 2.9440000000000001e-08, 5.8079512054400002e+00, 4.9718700374930000e+02 },
    { 2.9379999999999999e-08, 9.2092579208000003e-01, 6.0027977796790003e+02 },
    { 2.9379999999999999e-08, 5.4943023332500003e+00, 1.0128633231634000e+03 },
    { 2.9340000000000000e-08, 4.0023540612800002e+00, 7.9889407997999996e+01 },
    { 2.9329999999999999e-08, 3.5673941445800001e+00, 1.5494553869377000e+03 },
    { 2.9239999999999999e-08, 8.5106841380000006e-02, 3.9409422953069998e+02 },
    { 2.9220000000000000e-08, 1.1247958610100000e+00, 2.4400976753430001e+02 },
    { 2.9169999999999999e-08, 2.2586297979299998e+00, 8.3379618276399995e+01 },
    { 2.9060000000000000e-08, 6.9691325437999996e-01, 1.7582045110600001e+02 },
    { 2.8909999999999999e-08, 5.9652623806699996e+00, 5.1775174302800004e+01 },
    { 2.8819999999999999e-08, 3.7627023240000002e-02, 3.4088276662940001e+02 },
    { 2.8710000000000000e-08, 4.2343991659000002e+00, 1.5467100656529999e+02 },
    { 2.8550000000000001e-08, 6.0415184323700002e+00, 4.1406801790380001e+02 },
    { 2.8500000000000000e-08, 5.7565117344300001e+00, 3.1665250208750001e+02 },
    { 2.8489999999999999e-08, 5.9594588271099997e+00, 2.1395348674380000e+02 },
    { 2.8450000000000000e-08, 4.1744969556599996e+00, 4.2537435059860002e+02 },
    { 2.8410000000000001e-08, 6.2004452498699996e+00, 6.2291369521690001e+02 },
    { 2.8410000000000001e-08, 1.8982272203399999e+00, 2.0806308247406000e+03 },
    { 2.8250000000000002e-08, 5.7206677819499996e+00, 1.1311945833399000e+03 },
    { 2.8229999999999999e-08, 1.2863195510700001e+00, 4.1114725659700002e+02 },
    { 2.8209999999999999e-08, 5.6686272902899999e+00, 4.0269224923979999e+02 },
    { 2.8180000000000002e-08, 3.6189868932000002e+00, 4.9066861317099999e+01 },
    { 2.8170000000000000e-08, 3.2670619675700001e+00, 2.1037833413120001e+02 },
    { 2.8170000000000000e-08, 5.8692224046800003e+00, 4.5071658083329999e+02 },
    { 2.8150000000000001e-08, 7.5877175602000002e-01, 1.3715810141518000e+03 },
    { 2.8139999999999999e-08, 4.4717046601100003e+00, 2.8808069400530002e+02 },
    { 2.8100000000000000e-08, 5.1480133131699999e+00, 1.5920418407003999e+03 },
    { 2.8089999999999999e-08, 5.6707815786799998e+00, 3.1782815825509999e+02 },
    { 2.8050000000000000e-08, 4.6555030099800003e+00, 1.4380252254408999e+03 },
    { 2.7940000000000001e-08, 1.8851805484699999e+00, 3.4470304530790003e+02 },
    { 2.7800000000000001e-08, 4.5650718618499999e+00, 2.8511174858869998e+02 },
    { 2.7770000000000000e-08, 1.6585698777600000e+00, 2.3864132567269999e+02 },
    { 2.7759999999999998e-08, 5.7490523249300001e+00, 1.1312708252620000e+02 },
    { 2.7739999999999999e-08, 1.9896729189500000e+00, 5.6622351302600002e+01 },
    { 2.7730000000000001e-08, 6.7900305992999999e-01, 3.4173409989129999e+02 },
    { 2.7710000000000001e-08, 4.9289309050999996e+00, 6.0887779767699999e+02 },
    { 2.7660000000000001e-08, 2.6925112728500000e+00, 5.7798007470199998e+01 },
    { 2.7649999999999999e-08, 2.3257830772800001e+00, 4.1237109687440000e+02 },
    { 2.7640000000000001e-08, 4.9822077243200003e+00, 4.1083844005629999e+02 },
    { 2.7560000000000000e-08, 3.0263425921099998e+00, 1.4761582610787000e+03 },
    { 2.7450000000000001e-08, 1.2530885700600001e+00, 5.9194239068970001e+02 },
    { 2.7350000000000000e-08, 4.7719704874700000e+00, 1.2059761819072000e+03 },
    { 2.7330000000000001e-08, 3.5071592942900001e+00, 2.6247783133970000e+02 },
    { 2.7240000000000001e-08, 9.3671377497999997e-01, 2.2942733789700000e+01 },
    { 2.7220000000000002e-08, 2.1649502971099999e+00, 4.9439368332599997e+01 },
    { 2.7190000000000001e-08, 1.9895709646699999e+00, 1.7782618867610000e+02 },
    { 2.7179999999999999e-08, 3.9443267111200000e+00, 1.0820158474381999e+03 },
    { 2.7039999999999999e-08, 4.4899487088000001e+00, 2.6099335863139999e+02 },
    { 2.7039999999999999e-08, 4.8206613593299998e+00, 3.8283608094570002e+02 },
    { 2.7010000000000002e-08, 3.9490073806899999e+00, 2.0058492261732999e+03 },
    { 2.7000000000000000e-08, 4.4546688101600003e+00, 9.6340270297140000e+02 },
    { 2.6980000000000001e-08, 1.6937161111200001e+00, 7.8149310317809184e+04 },
    { 2.6960000000000001e-08, 2.9348143290000000e-02, 1.7071264167530001e+02 },
    { 2.6950000000000000e-08, 3.5785969749200000e+00, 6.2845603226997000e+03 },
    { 2.6810000000000000e-08, 1.1180932144900000e+00, 2.4948471359799999e+01 },
    { 2.6770000000000001e-08, 1.3489257210000001e+00, 6.7383753625780002e+02 },
    { 2.6729999999999998e-08, 4.6003982304099997e+00, 5.3169670266469996e+02 },
    { 2.6709999999999999e-08, 2.6793960714599998e+00, 4.0966278388870001e+02 },
    { 2.6600000000000000e-08, 4.3081795741000001e-01, 5.7345307502490004e+02 },
    { 2.6550000000000000e-08, 2.7208899093699999e+00, 4.8439619834619998e+02 },
    { 2.6470000000000001e-08, 3.8525997587499998e+00, 4.1826080359780002e+02 },
    { 2.6440000000000001e-08, 1.0196288380900000e+00, 5.0835032409220003e+02 },
    { 2.6390000000000000e-08, 1.7794846549900001e+00, 1.5580534066468001e+03 },
    { 2.6300000000000001e-08, 1.1269349381500000e+00, 2.7515475996916002e+03 },
    { 2.6289999999999999e-08, 5.0662483303000005e-01, 8.8150755740329998e+02 },
    { 2.6280000000000001e-08, 2.6804144414600000e+00, 3.0534616939270001e+02 },
    { 2.6190000000000002e-08, 2.3022354058699999e+00, 5.2161421024729998e+02 },
    { 2.6130000000000000e-08, 4.5611909284100003e+00, 5.6188610989920005e+02 },
    { 2.6110000000000000e-08, 4.0771678014699999e+00, 2.1334727954780001e+02 },
    { 2.6070000000000001e-08, 4.2179018464400002e+00, 1.5128068240082000e+03 },
    { 2.6070000000000001e-08, 5.9478851617400004e+00, 1.1553611574070001e+03 },
    { 2.6029999999999999e-08, 8.1228898724999998e-01, 2.1325091132820000e+02 },
    { 2.5909999999999998e-08, 1.7633980847999999e+00, 1.1907858800591000e+03 },
    { 2.5860000000000001e-08, 1.8137012982200000e+00, 9.5209637027660006e+02 },
    { 2.5830000000000000e-08, 6.1476124575700002e+00, 4.1109907248719998e+02 },
    { 2.5480000000000001e-08, 5.8531923132900001e+00, 1.6242369855049999e+03 },
    { 2.5460000000000001e-08, 3.7866189641600001e+00, 4.2271422172270002e+02 },
    { 2.5410000000000001e-08, 4.7362553343200000e+00, 9.0833426034629997e+02 },
    { 2.5290000000000000e-08, 5.9005799785000002e-01, 8.0083625211260005e+02 },
    { 2.5130000000000001e-08, 2.6760348053100000e+00, 4.0721510333390000e+02 },
    { 2.5060000000000001e-08, 5.0160943918300003e+00, 1.4478470854274001e+03 },
    { 2.5030000000000000e-08, 5.3193229612800002e+00, 1.0126508748423000e+03 },
    { 2.4940000000000000e-08, 7.9353995080999995e-01, 1.1878169346425000e+03 },
    { 2.4850000000000001e-08, 1.7293103672200001e+00, 3.9357296466890000e+02 },
    { 2.4710000000000001e-08, 1.2301049836000000e+00, 4.1135970491810002e+02 },
    { 2.4710000000000001e-08, 5.8533419543700003e+00, 8.7853861198669995e+02 },
    { 2.4710000000000001e-08, 3.9368115084699999e+00, 6.5818966002269997e+02 },
    { 2.4660000000000000e-08, 6.4176464596000005e-01, 2.0373786788240000e+02 },
    { 2.4649999999999999e-08, 2.7522496302300001e+00, 5.3213864564940002e+02 },
    { 2.4599999999999999e-08, 9.8834029379999999e-01, 1.7368158709190001e+02 },
    { 2.4490000000000000e-08, 6.2639168056100001e+00, 5.4527502581759995e+02 },
    { 2.4369999999999999e-08, 5.6706534939999997e-02, 1.8983509678650000e+02 },
    { 2.4360000000000001e-08, 4.3957194138300002e+00, 1.6257214582132999e+03 },
    { 2.4190000000000000e-08, 6.0063238030799999e+00, 5.2768522752449996e+02 },
    { 2.4089999999999999e-08, 1.1680709715399999e+00, 7.0636996983100005e+01 },
    { 2.3969999999999999e-08, 6.0549411058700002e+00, 8.4189004905720003e+02 },
    { 2.3800000000000001e-08, 8.6688341144000003e-01, 4.2951895218279998e+02 },
    { 2.3800000000000001e-08, 5.7019071988599999e+00, 3.9868077409960000e+02 },
    { 2.3730000000000001e-08, 4.9647336694800002e+00, 1.3109513332920000e+02 },
    { 2.3700000000000000e-08, 2.4236554873999999e-01, 1.3284023846840000e+02 },
    { 2.3689999999999999e-08, 4.3301369609000000e+00, 4.1922401144430000e+02 },
    { 2.3619999999999999e-08, 5.3478661635900000e+00, 9.0536531492970005e+02 },
    { 2.3530000000000000e-08, 2.6498713356999999e+00, 2.7851946644970002e+02 },
    { 2.3510000000000000e-08, 1.0098958978899999e+00, 4.8762577619370001e+02 },
    { 2.3460000000000000e-08, 8.5350513181999998e-01, 5.2137673247510589e+04 },
    { 2.3450000000000002e-08, 2.8067816990100001e+00, 4.5417565334440002e+02 },
    { 2.3429999999999999e-08, 5.3041427217999999e+00, 5.7074476203920005e+02 },
    { 2.3409999999999999e-08, 1.7246260926300001e+00, 4.4068227252570000e+02 },
    { 2.3400000000000001e-08, 5.4199348688800004e+00, 4.5267413429359999e+02 },
    { 2.3359999999999999e-08, 4.1736621217399996e+00, 4.9549008271989999e+02 },
    { 2.3289999999999999e-08, 1.7592368726300001e+00, 1.4776427337870000e+03 },
    { 2.3269999999999999e-08, 1.9469516120899999e+00, 4.9007345674850001e+02 },
    { 2.3240000000000002e-08, 8.5149013777000004e-01, 1.5157757694248000e+03 },
    { 2.2900000000000000e-08, 3.4111928830800000e+00, 4.5543063138909997e+02 },
    { 2.2659999999999999e-08, 3.3241549009300000e+00, 6.2147740661839998e+02 },
    { 2.2510000000000001e-08, 1.8460362603900000e+00, 2.2894549679899999e+01 },
    { 2.2469999999999999e-08, 4.9944880639899996e+00, 6.3570450061999998e+02 },
    { 2.2350000000000002e-08, 4.9387207963500002e+00, 8.7710232338820003e+02 },
    { 2.2320000000000001e-08, 2.4216950854100001e+00, 1.5837366516480000e+02 },
    { 2.2280000000000002e-08, 1.4871984610000000e-01, 2.9747464249800000e+01 },
    { 2.2259999999999999e-08, 2.0997773228900001e+00, 4.7582388608800002e+01 },
    { 2.2259999999999999e-08, 6.1626401064799996e+00, 1.1893014073508000e+03 },
    { 2.2040000000000001e-08, 3.2046722254100000e+00, 4.4312995308050000e+02 },
    { 2.1480000000000001e-08, 6.1063369837600003e+00, 2.4872381809010000e+02 },
};

static const VSOP87Term VSOP87A_NEPTUNE_X1[330] = {
    { 2.5584026100000001e-03, 2.0193568679500000e+00, 3.6648562929500002e+01 },
    { 2.4312529900000000e-03, 5.4621490287299999e+00, 3.9617508346100003e+01 },
    { 1.1839816800000001e-03, 2.8825184506100001e+00, 7.6266071275599998e+01 },
    { 3.7965448999999999e-04, 3.1415926535900001e+00, 0.0000000000000000e+00 },
    { 2.1924704999999999e-04, 3.2015616415200001e+00, 3.5164090221199999e+01 },
    { 1.7459807999999999e-04, 4.2634939881699996e+00, 4.1101981054399999e+01 },
    { 1.3130617000000000e-04, 5.3642496184799997e+00, 2.9689454166000000e+00 },
    { 5.0865270000000000e-05, 1.9237735472899999e+00, 3.8133035637799999e+01 },
    { 4.8997179999999998e-05, 2.0934949781299999e+00, 7.3297125859000005e+01 },
    { 2.7452670000000000e-05, 4.0625281866699998e+00, 7.7750543983900002e+01 },
    { 2.2044140000000001e-05, 4.3885563952100002e+00, 3.3679617512900002e+01 },
    { 2.1687190000000001e-05, 4.1176801256299997e+00, 4.4534181248999998e+00 },
    { 1.5722020000000001e-05, 1.0760661158900000e+00, 1.1439910691340000e+02 },
    { 1.3440220000000000e-05, 3.0380205905100000e+00, 4.2586453762700003e+01 },
    { 1.2855420000000000e-05, 6.0236755499700001e+00, 7.4781598567299994e+01 },
    { 9.2096900000000002e-06, 3.3442297094300000e+00, 7.1812653150700001e+01 },
    { 8.9745800000000007e-06, 4.2706634208200001e+00, 4.2659819087599999e+02 },
    { 8.6524700000000002e-06, 1.6660094983100000e+00, 3.7611770776000000e+01 },
    { 8.4996299999999995e-06, 5.8159954474899997e+00, 3.8654300499599998e+01 },
    { 7.7634499999999993e-06, 5.8468844321600004e+00, 2.0618554843720000e+02 },
    { 7.5330599999999997e-06, 5.3319395095799997e+00, 2.2041264243879999e+02 },
    { 6.3755100000000001e-06, 5.4107772829099998e+00, 1.4844727083000000e+00 },
    { 6.0783699999999997e-06, 1.0459953392000000e-01, 1.0593819301891999e+03 },
    { 5.7137900000000004e-06, 2.4311457597700001e+00, 5.2257741809380002e+02 },
    { 5.6275500000000000e-06, 1.9144515601600001e+00, 5.3680451209540001e+02 },
    { 5.0115799999999999e-06, 1.7133641658400001e+00, 2.8571808082200000e+01 },
    { 4.7777400000000003e-06, 4.4160617137399996e+00, 9.8899988524600005e+01 },
    { 4.5402000000000002e-06, 1.7144634795600000e+00, 3.5685355082999997e+01 },
    { 4.1005699999999998e-06, 5.7658000477000000e+00, 4.0580716192600001e+01 },
    { 3.6689900000000001e-06, 5.7675571411400002e+00, 4.7694263193399998e+01 },
    { 3.0110500000000002e-06, 2.8879806898300000e+00, 5.9378908332000000e+00 },
    { 2.6100500000000001e-06, 5.5834158825899998e+00, 3.2195144804599998e+01 },
    { 2.0850200000000001e-06, 4.5040350840699999e+00, 7.0328180442399997e+01 },
    { 1.7263399999999999e-06, 4.4166810013899998e+00, 3.3940249943799998e+01 },
    { 1.6173000000000000e-06, 7.9247603616999995e-01, 3.1019488636999998e+01 },
    { 1.5687999999999999e-06, 2.5963615896399999e+00, 7.9235016692200006e+01 },
    { 1.5254499999999999e-06, 5.8200211091999998e-01, 3.0056280790500001e+01 },
    { 1.5077400000000000e-06, 3.0395569385700001e+00, 4.2325821331800000e+01 },
    { 1.3457399999999999e-06, 3.9220827353999999e-01, 4.5246582638600003e+01 },
    { 1.3284099999999999e-06, 1.6257522392099999e+00, 8.0767548473000002e+00 },
    { 1.1590600000000000e-06, 1.8184706574000000e+00, 4.4070926471000000e+01 },
    { 1.1163900000000000e-06, 6.3975084586999997e-01, 4.6209790485100001e+01 },
    { 1.1029300000000000e-06, 6.2656104061500004e+00, 3.5212274331000003e+01 },
    { 1.0904700000000000e-06, 3.0891618825000000e+00, 1.1291463420510000e+02 },
    { 1.0045900000000000e-06, 1.2132093382900000e+00, 4.1053796944600002e+01 },
    { 8.8217999999999998e-07, 4.2212210214499999e+00, 6.0766952886799999e+01 },
    { 8.6328000000000004e-07, 4.7984076731999998e+00, 4.1948464387519999e+02 },
    { 7.1814000000000000e-07, 3.5055340988800001e+00, 6.3989728631399998e+02 },
    { 5.8138999999999999e-07, 1.1079923828800000e+00, 2.1329909543799999e+02 },
    { 5.3242000000000000e-07, 4.4651174862799996e+00, 4.3371173787679999e+02 },
    { 5.1602000000000004e-07, 4.0200121382200003e+00, 6.9152524274800001e+01 },
    { 5.1114000000000001e-07, 6.0343973434400002e+00, 1.1143016149680000e+02 },
    { 4.8159000000000001e-07, 6.2209254622000003e+00, 2.9207613067999998e+00 },
    { 4.4382000000000001e-07, 4.4285052615999998e-01, 1.5890728952838001e+03 },
    { 4.2529999999999998e-07, 1.6729133512900001e+00, 7.4223635415000002e+00 },
    { 4.2058999999999998e-07, 1.6884305259600001e+00, 4.4725317776799997e+01 },
    { 4.0839999999999998e-07, 5.5485072512300002e+00, 1.0994568878849999e+02 },
    { 3.9462000000000002e-07, 5.8405094801199997e+00, 3.1540753498800001e+01 },
    { 3.9309999999999999e-07, 9.4345147545999997e-01, 1.2530172972200001e+01 },
    { 3.8295999999999998e-07, 4.3033513776500003e+00, 4.1927856940000003e+00 },
    { 3.7940000000000002e-07, 2.7382209638399999e+00, 6.5922821389999999e+00 },
    { 3.7655000000000002e-07, 5.9879832238499997e+00, 1.4014645680499999e+01 },
    { 3.6951999999999999e-07, 5.7214579276200004e+00, 6.8843707734099993e+01 },
    { 3.6847999999999997e-07, 2.8556484887100000e+00, 8.4342826122899993e+01 },
    { 3.5231000000000001e-07, 6.2532540209900001e+00, 2.7406604832480002e+02 },
    { 3.3598999999999998e-07, 5.1632981098999997e-01, 3.0710672096300001e+01 },
    { 3.3570000000000002e-07, 6.0456575829499997e+00, 1.4956319713459999e+02 },
    { 3.3084000000000002e-07, 2.4585688083099999e+00, 1.1588357962169999e+02 },
    { 3.3024000000000002e-07, 4.1268484493999997e+00, 5.2969096509459996e+02 },
    { 3.1315999999999998e-07, 1.7293016894200000e+00, 1.0522683831884001e+03 },
    { 3.1035000000000002e-07, 2.9509956209600001e+00, 2.7087335373900000e+01 },
    { 3.0543000000000000e-07, 4.0210047296100004e+00, 2.2633917249000000e+01 },
    { 3.0123000000000000e-07, 2.3351270843899998e+00, 1.0664954771900000e+03 },
    { 2.6640000000000000e-07, 2.5131138951000001e+00, 2.4118389957300000e+01 },
    { 2.5605999999999998e-07, 2.2816942972000001e-01, 6.3735898303399999e+01 },
    { 2.4422000000000000e-07, 4.3542887579300000e+00, 1.0601353552540000e+02 },
    { 2.2151000000000000e-07, 4.4809083871600004e+00, 1.0697674337190000e+02 },
    { 2.2126000000000001e-07, 1.2600709834499999e+00, 1.7516605980020000e+02 },
    { 2.1126000000000000e-07, 3.6935153185899998e+00, 3.1639186965660002e+02 },
    { 1.9878000000000001e-07, 1.0180582698500000e+00, 3.2716409666399997e+01 },
    { 1.9870000000000000e-07, 9.4595354812999999e-01, 5.4166259714000002e+00 },
    { 1.9754000000000001e-07, 4.4900021146900002e+00, 4.9178735901700001e+01 },
    { 1.9347000000000000e-07, 5.6349672679699996e+00, 1.5253214255120000e+02 },
    { 1.9249000000000000e-07, 6.0142396056600003e+00, 6.9010986796999996e+00 },
    { 1.8246000000000001e-07, 3.2262794651000002e+00, 7.2073285581600004e+01 },
    { 1.8175000000000000e-07, 5.2015941499900000e+00, 2.2752618943959999e+02 },
    { 1.7239000000000000e-07, 1.8394173571000000e-01, 1.1306332694800000e+01 },
    { 1.6978000000000000e-07, 1.1070345183500001e+00, 9.1786441523799994e+01 },
    { 1.6780999999999999e-07, 3.2051891186499999e+00, 3.0508553696180002e+02 },
    { 1.6590000000000000e-07, 2.7683740769599998e+00, 3.3467169191799996e+01 },
    { 1.6511000000000000e-07, 2.5472934819100002e+00, 4.3240845068500001e+01 },
    { 1.6400000000000001e-07, 5.0242612549399999e+00, 3.3025226207099998e+01 },
    { 1.5976000000000000e-07, 1.6504410529400000e+00, 6.2251425595100002e+01 },
    { 1.5948000000000000e-07, 1.1374932037200001e+00, 2.5602862665600000e+01 },
    { 1.5620999999999999e-07, 6.6768589767999997e-01, 9.5612275556000004e+00 },
    { 1.5590000000000001e-07, 5.4895704503299996e+00, 3.9405060024999997e+01 },
    { 1.5590000000000001e-07, 1.9926094696000001e+00, 3.6861011250600001e+01 },
    { 1.5468999999999999e-07, 1.9931734816999999e-01, 4.3549661609200001e+01 },
    { 1.5454000000000001e-07, 2.7555529132899999e+00, 3.1931263096340001e+02 },
    { 1.5069000000000001e-07, 1.1761439988000000e+00, 7.9186832582400001e+01 },
    { 1.4471000000000000e-07, 4.0529532027999998e-01, 1.0846121608020000e+02 },
    { 1.3461000000000001e-07, 3.8576878410700002e+00, 8.2858353414600003e+01 },
    { 1.3293000000000001e-07, 4.7151195308600000e+00, 4.2798902083800002e+01 },
    { 1.2925000000000000e-07, 1.1036422297499999e+00, 1.1020632121940000e+02 },
    { 1.2751000000000001e-07, 6.1425113639999995e-01, 4.5555399179299997e+01 },
    { 1.2713000000000000e-07, 2.5167517750999999e+00, 5.1546387109299997e+02 },
    { 1.2616000000000001e-07, 3.2130331787799999e+00, 7.1135470007999997e+00 },
    { 1.2587000000000000e-07, 2.3645561390499998e+00, 1.1045700263900001e+01 },
    { 1.2373000000000001e-07, 2.4210344452000001e+00, 3.1231936958100000e+01 },
    { 1.1724000000000000e-07, 6.1090742692499997e+00, 6.3278373931320004e+02 },
    { 1.1361999999999999e-07, 1.5325049431000000e+00, 1.2991947716160001e+02 },
    { 1.1199000000000000e-07, 1.4687616655800000e+00, 4.1529185818119998e+02 },
    { 1.1029000000000000e-07, 3.8252606782199998e+00, 5.1078094306999997e+00 },
    { 1.0966000000000000e-07, 3.5379014729499998e+00, 1.0038446123289999e+02 },
    { 1.0927000000000000e-07, 5.5898559181099996e+00, 1.3703302416240001e+02 },
    { 1.0632000000000000e-07, 4.6647489140599996e+00, 1.4362530630140000e+02 },
    { 1.0323000000000001e-07, 2.4723702413500002e+00, 9.7415515816300001e+01 },
    { 9.9229999999999996e-08, 2.5123326620000003e-01, 7.0849445304200003e+01 },
    { 9.7180000000000001e-08, 5.6562770870000003e+00, 2.5143213107579999e+02 },
    { 9.6009999999999997e-08, 9.6952666661999998e-01, 8.0719489400499995e+01 },
    { 9.4479999999999996e-08, 5.9577087065700001e+00, 1.9907200143640000e+02 },
    { 8.6910000000000002e-08, 3.8613071630200002e+00, 3.2503961345299999e+01 },
    { 8.1289999999999995e-08, 1.0629247804499999e+00, 5.3287235883230005e+02 },
    { 8.0799999999999996e-08, 3.2704651712900001e+00, 5.2650957135689998e+02 },
    { 8.0729999999999996e-08, 1.8060284540400000e+00, 5.4391805909619995e+02 },
    { 7.9240000000000000e-08, 6.1297155105599996e+00, 7.5744806413800006e+01 },
    { 7.8260000000000001e-08, 4.0240103808600001e+00, 7.6787336137400004e+01 },
    { 7.7929999999999997e-08, 5.8747757621399996e+00, 1.0309277421860000e+02 },
    { 7.7140000000000001e-08, 5.0813506365599999e+00, 6.5220371011699996e+01 },
    { 7.6980000000000005e-08, 1.7875116227999999e-01, 1.4227094001599999e+01 },
    { 7.3360000000000003e-08, 5.0048881883699998e+00, 1.5499118388799999e+01 },
    { 7.2260000000000000e-08, 6.1080092563699999e+00, 9.4917560896980001e+02 },
    { 7.1340000000000002e-08, 3.6209012961300000e+00, 4.3762109930299999e+01 },
    { 6.6460000000000001e-08, 5.7955527945099998e+00, 1.0549227066360000e+02 },
    { 6.4230000000000000e-08, 5.0471132627799999e+00, 6.7803032195180003e+02 },
    { 6.2940000000000006e-08, 5.3403647313000002e-01, 6.6704843720000000e+01 },
    { 6.2390000000000005e-08, 5.7806655779899998e+00, 1.7526107818300002e+01 },
    { 6.1049999999999994e-08, 1.5442596721699999e+00, 6.9364972595899999e+01 },
    { 5.9989999999999997e-08, 8.5410630839000001e-01, 6.4959738580800007e+01 },
    { 5.9279999999999999e-08, 5.5984051110199999e+00, 1.4659425171800001e+02 },
    { 5.9100000000000000e-08, 6.1180615536999996e+00, 6.1686767430780001e+03 },
    { 5.8570000000000001e-08, 1.0725315143900001e+00, 1.4414657116320001e+02 },
    { 5.8110000000000002e-08, 1.7082466911000000e+00, 1.4097140497896000e+03 },
    { 5.7600000000000000e-08, 1.9622941643900000e+00, 4.4923210812500002e+02 },
    { 5.5619999999999998e-08, 1.2640317145000000e+00, 1.0098886439297599e+04 },
    { 5.4830000000000002e-08, 4.2989065510800000e+00, 4.2065188900899997e+01 },
    { 5.4830000000000002e-08, 3.1832733688500001e+00, 3.4200882374700001e+01 },
    { 5.4720000000000003e-08, 2.8438810733700000e+00, 8.5319638175199998e+02 },
    { 5.4560000000000000e-08, 4.8225532018699999e+00, 1.8027386923090000e+02 },
    { 5.3249999999999997e-08, 4.3975958592700000e+00, 1.9010580526599998e+01 },
    { 5.3220000000000003e-08, 3.6701467599100002e+00, 1.1046285128278000e+03 },
    { 5.2500000000000000e-08, 4.8779879953399998e+00, 9.6320784650000002e-01 },
    { 5.1949999999999999e-08, 5.0972652026600000e+00, 2.0936694217490000e+02 },
    { 5.1340000000000002e-08, 6.1035148199400000e+00, 2.1723124870109999e+02 },
    { 5.1280000000000000e-08, 2.5427997235399999e+00, 1.0175152510573200e+04 },
    { 5.1270000000000002e-08, 6.9501864499999996e-02, 1.4117762574660000e+02 },
    { 5.0729999999999998e-08, 3.5820644050300001e+00, 1.4985797300065999e+03 },
    { 5.0269999999999999e-08, 4.4395339393300004e+00, 2.1458261081400000e+01 },
    { 4.9840000000000002e-08, 3.0317555866400001e+00, 1.0904014188261999e+03 },
    { 4.9100000000000003e-08, 5.0478717994000002e+00, 4.5034134317499998e+01 },
    { 4.8909999999999999e-08, 1.2793491232900001e+00, 5.2549817940059995e+02 },
    { 4.8230000000000002e-08, 2.7849025343600000e+00, 4.1800017116689997e+02 },
    { 4.8020000000000003e-08, 3.3357325825999999e+00, 3.9096243484299997e+01 },
    { 4.7580000000000000e-08, 3.9624068289799999e+00, 3.2350541665740002e+02 },
    { 4.7309999999999998e-08, 2.7782832282299998e+00, 3.6127298067700004e+01 },
    { 4.7309999999999998e-08, 1.5623040381100000e+00, 4.0138773207900002e+01 },
    { 4.7290000000000002e-08, 3.7676208253199999e+00, 4.2096911658350001e+02 },
    { 4.6999999999999997e-08, 6.2339474058500004e+00, 5.0663208609999998e+01 },
    { 4.6889999999999998e-08, 1.2427109339500000e+00, 1.0679799498983000e+03 },
    { 4.5540000000000003e-08, 4.7092899246000002e+00, 3.1071461125430000e+02 },
    { 4.4360000000000001e-08, 6.1726597703800001e+00, 2.3593301268700000e+02 },
    { 4.4320000000000002e-08, 2.4063789219900000e+00, 4.1555249061209997e+02 },
    { 4.4220000000000002e-08, 1.1460540121900000e+00, 2.9170403072770000e+02 },
    { 4.4139999999999997e-08, 3.5749516038400002e+00, 1.0623508756058000e+03 },
    { 4.3830000000000003e-08, 1.4547123826899999e+00, 4.6350738623640001e+02 },
    { 4.3670000000000000e-08, 1.7115357653300001e+00, 5.8739963457300000e+01 },
    { 4.2850000000000003e-08, 5.9755508408900004e+00, 1.0507839104801001e+03 },
    { 4.2270000000000001e-08, 1.4871002191600000e+00, 5.3414438321950001e+02 },
    { 4.1540000000000000e-08, 3.5567555068200001e+00, 5.1838463239980001e+02 },
    { 4.1299999999999999e-08, 3.7769943165700002e+00, 9.5979227217800002e+01 },
    { 4.1089999999999999e-08, 1.7385383923300000e+00, 4.1677633088950000e+02 },
    { 4.0930000000000003e-08, 3.8331678056500000e+00, 2.4476805547999998e+00 },
    { 4.0879999999999999e-08, 5.4573879207699996e+00, 4.2341679713830001e+02 },
    { 4.0539999999999998e-08, 4.0288309267700004e+00, 4.2977958461370002e+02 },
    { 4.0530000000000000e-08, 3.3168225817899999e+00, 9.2107688163959995e+02 },
    { 4.0480000000000002e-08, 4.6817239454199999e+00, 9.7993446517589996e+02 },
    { 4.0390000000000000e-08, 3.7998885642300002e+00, 1.6230131452276000e+03 },
    { 4.0130000000000003e-08, 1.3969045441800001e+00, 1.0427534649502118e+05 },
    { 4.0019999999999997e-08, 5.0145950217799999e+00, 2.1648048917570000e+02 },
    { 3.9620000000000000e-08, 4.0208399453999997e+00, 9.7918370565049997e+02 },
    { 3.9540000000000002e-08, 3.8630176543200001e+00, 1.1312708252620000e+02 },
    { 3.9449999999999999e-08, 2.7996965937699998e+00, 5.9282480178500002e+01 },
    { 3.9440000000000001e-08, 3.7837445500500002e+00, 1.1950691634410001e+02 },
    { 3.9330000000000002e-08, 6.1810796810699999e+00, 2.1011770170029999e+02 },
    { 3.8929999999999999e-08, 2.6840875602900001e+00, 2.0076892246579999e+02 },
    { 3.8379999999999997e-08, 1.9981988533299999e+00, 9.8014691349700001e+02 },
    { 3.8140000000000003e-08, 1.9682807805700000e+00, 4.4928029223480002e+02 },
    { 3.7809999999999999e-08, 6.0370709836199996e+00, 1.4214083359310001e+02 },
    { 3.7569999999999998e-08, 3.5991585861100002e+00, 2.9615744885260000e+02 },
    { 3.7179999999999999e-08, 2.2957714216000000e-01, 1.0650110044817000e+03 },
    { 3.7149999999999998e-08, 6.0215218200000002e+00, 2.5435289238260000e+02 },
    { 3.7079999999999999e-08, 3.4199717946199999e+00, 6.7880499887599996e+01 },
    { 3.6629999999999998e-08, 7.0030067491000003e-01, 1.5580534066468001e+03 },
    { 3.6529999999999997e-08, 5.0833426768400001e+00, 1.1314552157707999e+03 },
    { 3.6510000000000001e-08, 2.2531801432499998e+00, 1.1272624300768000e+03 },
    { 3.6190000000000002e-08, 3.0043039180300002e+00, 6.4701083331480004e+02 },
    { 3.6099999999999999e-08, 3.0447721898300002e+00, 5.4807810194200002e+01 },
    { 3.5380000000000003e-08, 3.0903221684200002e+00, 5.7255490749000003e+01 },
    { 3.4900000000000001e-08, 5.2599636259800002e+00, 9.8704801217670001e+02 },
    { 3.4879999999999999e-08, 1.2386146069499999e+00, 1.8769623277240001e+02 },
    { 3.4660000000000001e-08, 1.9028320922399999e+00, 2.1455905633214002e+03 },
    { 3.4289999999999998e-08, 1.1156266427999999e+00, 7.3587651353180001e+02 },
    { 3.3960000000000001e-08, 1.6546608706199999e+00, 2.9226199388000001e+01 },
    { 3.3650000000000000e-08, 5.2357816847300001e+00, 4.8025159676200002e+02 },
    { 3.3530000000000003e-08, 3.8738766560000002e+00, 1.0289551617486601e+04 },
    { 3.3530000000000003e-08, 3.1969195823500001e+00, 1.5763995198190000e+02 },
    { 3.3479999999999999e-08, 5.1224455653599996e+00, 6.2845603226997000e+03 },
    { 3.3470000000000001e-08, 5.7063923016600002e+00, 1.5104766984290001e+02 },
    { 3.3390000000000003e-08, 3.6963347304199998e+00, 4.4334240140160000e+02 },
    { 3.3209999999999998e-08, 2.3053588436100001e+00, 4.5024350008130000e+02 },
    { 3.3169999999999999e-08, 1.9617448817700001e+00, 5.0015594916589998e+02 },
    { 3.3120000000000001e-08, 3.5984652910999998e-01, 1.5438263126452000e+03 },
    { 3.3069999999999998e-08, 1.4237482550000000e-02, 2.1543795945209999e+02 },
    { 3.2929999999999998e-08, 4.5590733299200004e+00, 3.8994962794650002e+02 },
    { 3.2920000000000000e-08, 3.4317097377000000e-01, 8.0794979911339999e+02 },
    { 3.2859999999999998e-08, 3.3970045701200000e+00, 4.3519621058510000e+02 },
    { 3.2730000000000003e-08, 6.2821512161000004e-01, 6.7359235025800004e+01 },
    { 3.2689999999999997e-08, 3.8905262684399999e+00, 1.7368158709190001e+02 },
    { 3.2539999999999999e-08, 1.0607096721700000e+00, 2.6826702943000001e+01 },
    { 3.2530000000000001e-08, 2.4486357944399999e+00, 4.8592885516429999e+02 },
    { 3.2290000000000000e-08, 1.4363369319099999e+00, 1.0509963588011999e+03 },
    { 3.2280000000000002e-08, 4.2456565828999998e-01, 5.2360129639400000e+01 },
    { 3.2280000000000002e-08, 6.2756457943999999e-01, 2.3905941636200001e+01 },
    { 3.2259999999999999e-08, 7.7463438424999997e-01, 1.0400779795530001e+02 },
    { 3.2140000000000002e-08, 2.4079004143499998e+00, 5.6930847344070003e+02 },
    { 3.2049999999999999e-08, 5.2249193707800003e+00, 2.7555052103309998e+02 },
    { 3.1949999999999998e-08, 1.2946515430200001e+00, 5.3653405886000002e+01 },
    { 3.1919999999999997e-08, 3.1223536378599999e+00, 6.5961044225620003e+02 },
    { 3.1879999999999998e-08, 1.0580854834000000e+00, 3.6137781986430002e+02 },
    { 3.1709999999999997e-08, 2.5949973316000002e-01, 1.0214770018919300e+04 },
    { 3.1620000000000001e-08, 3.5627438867599999e+00, 5.9465070367540000e+02 },
    { 3.1559999999999999e-08, 4.6648737447700004e+00, 4.3208425281659999e+02 },
    { 3.1389999999999999e-08, 4.6707812370099999e+00, 1.5912442469019999e+02 },
    { 3.1380000000000000e-08, 5.3514633317299998e+00, 3.4736317418380003e+02 },
    { 3.1200000000000001e-08, 5.9174092514999996e+00, 4.5660628755670001e+02 },
    { 3.1170000000000001e-08, 3.1717282546900001e+00, 4.5756949540319999e+02 },
    { 3.1109999999999999e-08, 1.2325644492200001e+00, 1.0141353475506000e+03 },
    { 3.0969999999999999e-08, 9.0779595391000001e-01, 1.4717530270636000e+03 },
    { 3.0969999999999999e-08, 2.8504141282800002e+00, 1.3954869557879999e+03 },
    { 3.0750000000000001e-08, 3.2441759118200002e+00, 3.9265794093220001e+02 },
    { 3.0580000000000000e-08, 1.5217690897700000e+00, 6.2815913772831000e+03 },
    { 3.0360000000000002e-08, 4.5935275350999999e-01, 9.4795176869240004e+02 },
    { 3.0309999999999998e-08, 4.1302158675799996e+00, 4.0610313764109998e+02 },
    { 3.0209999999999997e-08, 1.3929174454200000e+00, 3.2835259365719998e+02 },
    { 3.0160000000000000e-08, 1.6436984958000000e+00, 5.2575881183150000e+02 },
    { 3.0150000000000002e-08, 4.1645606538199997e+00, 1.8079513409270001e+02 },
    { 3.0129999999999999e-08, 3.0660398321300000e+00, 1.2099138905240000e+02 },
    { 3.0110000000000003e-08, 7.6186974874000002e-01, 2.1187638603783998e+03 },
    { 3.0080000000000002e-08, 8.7346015646999997e-01, 2.5505546779820000e+02 },
    { 3.0040000000000003e-08, 3.9553426203200002e+00, 1.0551891444952000e+03 },
    { 2.9819999999999998e-08, 2.6938104313400002e+00, 5.3362311835770004e+02 },
    { 2.9720000000000001e-08, 5.8239051164699998e+00, 9.8608480433019997e+02 },
    { 2.9690000000000000e-08, 4.6229960164600001e+00, 4.9447869076360001e+02 },
    { 2.9679999999999998e-08, 2.8517058164500000e+00, 6.1307058337100000e+02 },
    { 2.9399999999999999e-08, 4.3679590981000000e+00, 3.8549620982160002e+02 },
    { 2.9169999999999999e-08, 5.9199884298500001e+00, 1.1731986822019999e+02 },
    { 2.9049999999999999e-08, 5.2858370971399999e+00, 5.7345307502490004e+02 },
    { 2.9049999999999999e-08, 2.9472163538400000e+00, 1.0211801073502700e+04 },
    { 2.9029999999999999e-08, 5.2663311542500004e+00, 2.0225339517410001e+02 },
    { 2.9000000000000002e-08, 5.8003638785399998e+00, 5.5922598102330005e+02 },
    { 2.8959999999999999e-08, 2.2907178517800002e+00, 1.6850521225016000e+03 },
    { 2.8900000000000001e-08, 3.5579648131299999e+00, 8.3477650205639998e+02 },
    { 2.8889999999999999e-08, 3.2714099533700001e+00, 4.9304240216509999e+02 },
    { 2.8839999999999999e-08, 8.7269814612999996e-01, 3.8427236954419999e+02 },
    { 2.8760000000000001e-08, 2.1021602577200000e+00, 8.4481081036399996e+02 },
    { 2.8509999999999999e-08, 4.4771975051700004e+00, 1.8647239249500001e+02 },
    { 2.8450000000000000e-08, 4.9855325966999997e+00, 2.1692243216040001e+02 },
    { 2.8320000000000001e-08, 4.5010847666200000e+00, 4.1974527630609998e+02 },
    { 2.8270000000000001e-08, 1.9393979423900001e+00, 4.2956713629260003e+02 },
    { 2.8250000000000002e-08, 2.8875430880400001e+00, 2.2286032299359999e+02 },
    { 2.8209999999999999e-08, 2.5791837224099998e+00, 8.3058371636239997e+02 },
    { 2.7459999999999999e-08, 2.7023678935000000e+00, 3.2642617796420001e+02 },
    { 2.7280000000000000e-08, 2.3698056630600002e+00, 4.3222726516850003e+02 },
    { 2.6890000000000001e-08, 3.4466085474000002e-01, 7.4299006053259995e+02 },
    { 2.6740000000000000e-08, 7.2178755843999998e-01, 1.5837366516480000e+02 },
    { 2.6619999999999999e-08, 5.9676831103200003e+00, 7.7229279122099996e+01 },
    { 2.6610000000000001e-08, 5.5742494785999996e-01, 3.1368355667089997e+02 },
    { 2.6490000000000001e-08, 2.4886074924499999e+00, 4.2537435059860002e+02 },
    { 2.6340000000000000e-08, 1.7767163721499999e+00, 9.4646729598410002e+02 },
    { 2.6330000000000002e-08, 4.5698639721100003e+00, 3.5181659230870002e+02 },
    { 2.6309999999999999e-08, 5.7814523643099998e+00, 2.0085575391590000e+03 },
    { 2.6130000000000000e-08, 1.0397275114500000e+00, 1.8175834193919999e+02 },
    { 2.6009999999999999e-08, 2.5871432333800000e+00, 3.6351668387839999e+02 },
    { 2.5970000000000000e-08, 2.6176811173200001e+00, 1.0213285546211000e+04 },
    { 2.5950000000000001e-08, 1.9009278417100000e+00, 8.5827298831199997e+01 },
    { 2.5869999999999999e-08, 2.2257572989999998e-02, 4.0106476417959999e+02 },
    { 2.5810000000000001e-08, 1.2012186287400000e+00, 1.3411226285559999e+02 },
    { 2.5780000000000000e-08, 3.9826319335999999e-01, 3.9557870223899999e+02 },
    { 2.5520000000000000e-08, 1.8036312071800000e+00, 1.0625633239269000e+03 },
    { 2.5500000000000000e-08, 3.0213161098399999e+00, 1.8621176006409999e+02 },
    { 2.5450000000000000e-08, 1.7545375593799999e+00, 1.3554855145409999e+02 },
    { 2.5419999999999999e-08, 2.6192170671300001e+00, 4.8147543703939999e+02 },
    { 2.5100000000000000e-08, 3.0087181483699998e+00, 4.1114725659700002e+02 },
    { 2.4960000000000000e-08, 4.8534251206799999e+00, 1.4711551657979999e+02 },
    { 2.4940000000000000e-08, 5.7449778352900003e+00, 4.5761767951299998e+02 },
    { 2.4690000000000001e-08, 3.8714488203099999e+00, 7.0777778620159995e+02 },
    { 2.4640000000000001e-08, 3.7340938083299999e+00, 1.4148644228730001e+02 },
    { 2.4590000000000000e-08, 2.1714821286200001e+00, 4.5097721326419997e+02 },
    { 2.4540000000000000e-08, 4.8499102894200004e+00, 1.9577298761969999e+02 },
    { 2.4400000000000000e-08, 2.9130306780600002e+00, 5.2523754696970002e+02 },
    { 2.4389999999999999e-08, 6.1695672596400000e+00, 4.6298612137459997e+02 },
    { 2.4059999999999998e-08, 9.9201934800000002e-01, 2.4846318565920001e+02 },
    { 2.3879999999999999e-08, 5.3243303995100000e+00, 5.9884348936940000e+02 },
    { 2.3870000000000001e-08, 3.6126962480799998e+00, 6.5412438031559998e+02 },
    { 2.3820000000000001e-08, 1.6988047669899999e+00, 1.7665053250849999e+02 },
    { 2.3650000000000000e-08, 4.3155583715099999e+00, 1.0728555991259999e+02 },
    { 2.3639999999999999e-08, 3.0594092632800001e+00, 5.6214674233009998e+02 },
    { 2.3380000000000002e-08, 4.3038278305700004e+00, 2.1033015002139999e+02 },
    { 2.3300000000000000e-08, 5.2936108973999998e+00, 1.1802244363580000e+02 },
    { 2.3250000000000000e-08, 4.9663483376400004e+00, 4.4312995308050000e+02 },
    { 2.3050000000000001e-08, 4.5007160703000002e+00, 1.0335340664950000e+02 },
    { 2.3029999999999998e-08, 4.3982209088799999e+00, 2.0806308247406000e+03 },
    { 2.3000000000000001e-08, 6.2755528396000004e-01, 9.8629725265130003e+02 },
    { 2.2609999999999999e-08, 4.8206596188799997e+00, 6.4409007200799999e+02 },
    { 2.2609999999999999e-08, 3.6750071954400001e+00, 2.9747464249800000e+01 },
    { 2.2490000000000002e-08, 4.6641290866000001e-01, 3.8135160823740000e+02 },
    { 2.2469999999999999e-08, 1.6839375649899999e+00, 1.7219711438360000e+02 },
    { 2.2329999999999999e-08, 5.9442559504599997e+00, 7.0904981058880003e+02 },
    { 2.2010000000000000e-08, 2.7849636483500002e+00, 6.0595703637019994e+02 },
    { 2.1600000000000002e-08, 9.7578185075000001e-01, 6.6672398925699997e+02 },
};

static const VSOP87Term VSOP87A_NEPTUNE_X2[102] = {
    { 5.3711380000000001e-05, 0.0000000000000000e+00, 0.0000000000000000e+00 },
    { 4.5362830000000001e-05, 5.0270075183599996e+00, 3.6648562929500002e+01 },
    { 4.3507660000000001e-05, 2.4542025430400001e+00, 3.9617508346100003e+01 },
    { 3.0929650000000000e-05, 6.2250463031000003e-01, 3.8133035637799999e+01 },
    { 2.1637029999999998e-05, 1.7921816836800000e+00, 7.6266071275599998e+01 },
    { 3.9086799999999998e-06, 5.6764348397999997e+00, 3.5164090221199999e+01 },
    { 3.0133900000000000e-06, 1.8173725886000001e+00, 4.1101981054399999e+01 },
    { 2.0456200000000001e-06, 2.4663755689300002e+00, 2.9689454166000000e+00 },
    { 1.5974800000000000e-06, 1.2589434385200000e+00, 2.0618554843720000e+02 },
    { 1.5579200000000001e-06, 3.6241666786000000e+00, 2.2041264243879999e+02 },
    { 1.1787900000000000e-06, 4.1269693789600002e+00, 5.2257741809380002e+02 },
    { 1.1447799999999999e-06, 5.9702177640000000e-02, 3.5685355082999997e+01 },
    { 1.1289100000000000e-06, 2.1785515638000000e-01, 5.3680451209540001e+02 },
    { 1.0594700000000000e-06, 1.1393599364000000e+00, 4.0580716192600001e+01 },
    { 9.7352000000000000e-07, 2.4248347596900000e+00, 2.1329909543799999e+02 },
    { 7.6870000000000005e-07, 5.2825495987100002e+00, 5.2969096509459996e+02 },
    { 7.5616999999999997e-07, 5.0063174277600000e+00, 7.3297125859000005e+01 },
    { 6.5425000000000003e-07, 2.4917597309400001e+00, 4.2659819087599999e+02 },
    { 5.8848000000000004e-07, 3.6832119544000003e-01, 3.3679617512900002e+01 },
    { 4.2437999999999999e-07, 1.7285820944400001e+00, 4.4534181248999998e+00 },
    { 3.8851999999999999e-07, 5.1303075158100002e+00, 1.0593819301891999e+03 },
    { 3.8263999999999999e-07, 8.9872407936999998e-01, 4.2586453762700003e+01 },
    { 3.7332000000000001e-07, 3.2511973284200000e+00, 7.4781598567299994e+01 },
    { 3.5530000000000001e-07, 6.1444315126399998e+00, 9.8899988524600005e+01 },
    { 3.5469000000000001e-07, 2.4987988160099999e+00, 3.1019488636999998e+01 },
    { 3.3708000000000000e-07, 1.7905425482000001e-01, 1.1439910691340000e+02 },
    { 3.1376999999999998e-07, 3.3961478053000000e+00, 2.8571808082200000e+01 },
    { 2.9746000000000002e-07, 4.9766278952400000e+00, 4.5246582638600003e+01 },
    { 2.4148999999999997e-07, 4.0838962726099997e+00, 4.7694263193399998e+01 },
    { 1.4231999999999999e-07, 2.7405377993500002e+00, 4.3371173787679999e+02 },
    { 1.3386999999999999e-07, 5.6770278350299996e+00, 7.1812653150700001e+01 },
    { 1.3243999999999999e-07, 6.1211704089600003e+00, 6.0766952886799999e+01 },
    { 1.3213999999999999e-07, 4.0945689746000002e-01, 4.1948464387519999e+02 },
    { 1.0889000000000000e-07, 6.0626890790300001e+00, 3.3940249943799998e+01 },
    { 9.5739999999999996e-08, 5.8003943883200000e+00, 6.9152524274800001e+01 },
    { 9.2550000000000005e-08, 1.4758801552600000e+00, 4.2325821331800000e+01 },
    { 9.2399999999999994e-08, 1.6663927066899999e+00, 6.3989728631399998e+02 },
    { 8.5290000000000005e-08, 1.5084125234200001e+00, 7.1135470007999997e+00 },
    { 7.3640000000000003e-08, 3.4977396126000002e+00, 2.2752618943959999e+02 },
    { 7.2610000000000000e-08, 1.7295213925000000e-01, 1.0975149658270000e+03 },
    { 7.1820000000000004e-08, 1.4519040564200001e+00, 3.2195144804599998e+01 },
    { 6.6759999999999997e-08, 4.8554089882999998e+00, 2.5854567807659998e+02 },
    { 6.5480000000000001e-08, 2.2734693181800001e+00, 3.0056280790500001e+01 },
    { 6.3510000000000004e-08, 7.9204424944999996e-01, 1.0664954771900000e+03 },
    { 5.6360000000000003e-08, 6.2041130201000003e+00, 8.0767548473000002e+00 },
    { 5.5869999999999997e-08, 3.6092809923000000e+00, 1.0522683831884001e+03 },
    { 5.4189999999999998e-08, 1.8124280905300001e+00, 1.0601353552540000e+02 },
    { 5.3790000000000001e-08, 5.8633842805300000e+00, 2.2633917249000000e+01 },
    { 5.2450000000000003e-08, 6.0561203055700004e+00, 4.4070926471000000e+01 },
    { 5.0950000000000003e-08, 4.2047459784900001e+00, 5.1546387109299997e+02 },
    { 5.0820000000000001e-08, 5.1593872340600004e+00, 4.6209790485100001e+01 },
    { 4.8310000000000000e-08, 1.2641672031900000e+00, 2.1478356814630001e+02 },
    { 4.6880000000000000e-08, 2.5213914087400000e+00, 2.2189711514710001e+02 },
    { 4.6070000000000002e-08, 4.8910182252099998e+00, 5.4940412103680001e+02 },
    { 4.4969999999999998e-08, 1.0060503053000001e+00, 7.7750543983900002e+01 },
    { 4.2569999999999997e-08, 5.5142844837900000e+00, 2.0470107572890001e+02 },
    { 4.2020000000000002e-08, 6.2325971543599996e+00, 3.1219908396260001e+02 },
    { 4.1810000000000002e-08, 2.6897719725000000e+00, 9.4498282327580000e+02 },
    { 4.0429999999999999e-08, 1.2206994705000001e+00, 1.9907200143640000e+02 },
    { 4.0410000000000003e-08, 4.9659002163699997e+00, 3.0508553696180002e+02 },
    { 4.0149999999999999e-08, 5.6188723972999997e+00, 5.3287235883230005e+02 },
    { 4.0130000000000003e-08, 5.5633782457000001e+00, 1.5890728952838001e+03 },
    { 3.9739999999999997e-08, 2.0491128546900002e+00, 3.3610502559880001e+02 },
    { 3.9710000000000003e-08, 5.0011763931899997e+00, 5.2650957135689998e+02 },
    { 3.9319999999999997e-08, 3.6624630348300000e+00, 9.1786441523799994e+01 },
    { 3.9160000000000001e-08, 2.1419237232999999e-01, 2.0767002114549999e+02 },
    { 3.7329999999999997e-08, 1.0185620341199999e+00, 3.1931263096340001e+02 },
    { 3.3960000000000001e-08, 2.7910613183000001e+00, 3.5744566660120000e+02 },
    { 3.3460000000000003e-08, 4.6430262391300001e+00, 1.1624747044077999e+03 },
    { 3.3400000000000001e-08, 6.5707077966000005e-01, 5.3117543780289998e+02 },
    { 3.3279999999999998e-08, 1.0428240806000000e-01, 5.4391805909619995e+02 },
    { 3.2880000000000001e-08, 1.4495881113100000e+00, 4.0817831118039999e+02 },
    { 3.2520000000000003e-08, 3.2386914983400001e+00, 1.2991947716160001e+02 },
    { 3.2320000000000001e-08, 6.1711444467099996e+00, 3.8135160823740000e+02 },
    { 3.1839999999999999e-08, 1.2698225936900001e+00, 1.0309277421860000e+02 },
    { 3.0080000000000002e-08, 1.4288636846000000e-01, 3.4321857259960001e+02 },
    { 2.9989999999999999e-08, 1.2787311702099999e+00, 9.3786927627499995e+02 },
    { 2.9670000000000000e-08, 1.0447763237500001e+00, 8.6030992875280003e+02 },
    { 2.9480000000000000e-08, 2.9609547490199999e+00, 1.4956319713459999e+02 },
    { 2.9370000000000001e-08, 1.5815716013700001e+00, 2.3593301268700000e+02 },
    { 2.8920000000000000e-08, 2.2018075025299999e+00, 3.9557870223899999e+02 },
    { 2.8240000000000000e-08, 9.7334571054999997e-01, 6.6672398925699997e+02 },
    { 2.7899999999999998e-08, 1.9043077877400001e+00, 4.8736514376280002e+02 },
    { 2.7500000000000001e-08, 1.7665755175700000e+00, 5.6363121503840000e+02 },
    { 2.7000000000000000e-08, 1.7273075522800001e+00, 2.7406604832480002e+02 },
    { 2.6980000000000001e-08, 1.6947054096300000e+00, 1.0141353475506000e+03 },
    { 2.6840000000000001e-08, 5.7401790863800004e+00, 4.5490936652729999e+02 },
    { 2.6370000000000001e-08, 5.2610362482899999e+00, 1.7516605980020000e+02 },
    { 2.6169999999999999e-08, 5.3344552130200000e+00, 9.5209637027660006e+02 },
    { 2.5819999999999999e-08, 4.8526698652000000e-01, 1.0736090241908000e+03 },
    { 2.5690000000000000e-08, 4.2262459874099996e+00, 4.6473122651379998e+02 },
    { 2.5480000000000001e-08, 2.0442842674300001e+00, 9.9022940591439999e+02 },
    { 2.5419999999999999e-08, 4.2229666606800000e+00, 1.2655674786264001e+03 },
    { 2.5250000000000001e-08, 6.1685462162000002e+00, 2.6826702943000001e+01 },
    { 2.5089999999999998e-08, 1.7441231168700000e+00, 2.1723124870109999e+02 },
    { 2.5030000000000000e-08, 3.4100010689999999e-01, 4.2511371816769997e+02 },
    { 2.5019999999999998e-08, 3.1572270391299999e+00, 2.0936694217490000e+02 },
    { 2.5010000000000000e-08, 1.6741318966500001e+00, 6.6964475056380002e+02 },
    { 2.4809999999999998e-08, 5.5675394385299999e+00, 6.5541765656220002e+02 },
    { 2.4299999999999999e-08, 1.7165769878999999e+00, 4.5342489381899998e+02 },
    { 2.3429999999999999e-08, 1.9359848184899999e+00, 9.8163138620530003e+02 },
    { 2.2729999999999999e-08, 5.5976023361199996e+00, 4.9155792945680003e+02 },
};

static const VSOP87Term VSOP87A_NEPTUNE_X3[33] = {
    { 1.9270299999999999e-06, 8.3849647679999995e-01, 3.6648562929500002e+01 },
    { 1.8221800000000000e-06, 3.6067577276000001e-01, 3.9617508346100003e+01 },
    { 1.8087999999999999e-06, 0.0000000000000000e+00, 0.0000000000000000e+00 },
    { 1.3107300000000000e-06, 3.7297217576500001e+00, 3.8133035637799999e+01 },
    { 8.5921999999999999e-07, 4.2809925886000000e-01, 7.6266071275599998e+01 },
    { 2.2385000000000001e-07, 2.9734758180199998e+00, 2.0618554843720000e+02 },
    { 2.1976000000000000e-07, 1.9306996719800000e+00, 3.5164090221199999e+01 },
    { 2.1096000000000000e-07, 1.8984645821099999e+00, 2.2041264243879999e+02 },
    { 1.9807000000000000e-07, 5.5514802992699996e+00, 4.1101981054399999e+01 },
    { 1.6219000000000000e-07, 5.8488116962300003e+00, 5.2257741809380002e+02 },
    { 1.5816000000000001e-07, 4.7851801408399997e+00, 5.3680451209540001e+02 },
    { 1.3456000000000001e-07, 3.0790365229999998e-02, 7.1135470007999997e+00 },
    { 6.7720000000000000e-08, 1.1967887335300000e+00, 6.9152524274800001e+01 },
    { 4.5230000000000002e-08, 4.1870136189400000e+00, 3.1019488636999998e+01 },
    { 4.5170000000000000e-08, 3.2889317168600001e+00, 4.5246582638600003e+01 },
    { 3.6559999999999998e-08, 6.0147675762999997e-01, 9.6888876491200006e+02 },
    { 3.6080000000000003e-08, 1.4024958698300001e+00, 9.8899988524600005e+01 },
    { 3.4980000000000000e-08, 3.2747758351299998e+00, 4.0817831118039999e+02 },
    { 3.4520000000000001e-08, 6.8243992106999996e-01, 4.2659819087599999e+02 },
    { 3.3039999999999997e-08, 2.2474655847300000e+00, 4.2240540518199998e+02 },
    { 3.3029999999999999e-08, 2.0788994992099998e+00, 4.9867147645760002e+02 },
    { 3.1989999999999997e-08, 1.6732803221000001e+00, 1.5199203710090001e+03 },
    { 3.0619999999999999e-08, 3.1178151390200002e+00, 4.8444438245600003e+02 },
    { 2.8749999999999999e-08, 5.2251242690500002e+00, 8.5319638175199998e+02 },
    { 2.6890000000000001e-08, 8.9931829837999999e-01, 4.4082528487759998e+02 },
    { 2.6460000000000000e-08, 5.6168663180299996e+00, 4.6202291352809999e+02 },
    { 2.6390000000000000e-08, 4.5729158440000001e-01, 3.0508553696180002e+02 },
    { 2.5419999999999999e-08, 5.7634798837699996e+00, 4.4482687410990002e+02 },
    { 2.4820000000000000e-08, 7.6026715845000004e-01, 4.3371173787679999e+02 },
    { 2.4730000000000000e-08, 5.6333677666600002e+00, 3.1931263096340001e+02 },
    { 2.3569999999999999e-08, 4.6206172916999999e-01, 4.4779581952650000e+02 },
    { 2.2790000000000001e-08, 4.6371047126600002e+00, 4.5905396811150001e+02 },
    { 2.1470000000000000e-08, 2.6361017341899999e+00, 1.7516605980020000e+02 },
};

static const VSOP87Term VSOP87A_NEPTUNE_X4[7] = {
    { 4.3340000000000003e-08, 5.4164710162800001e+00, 4.3371173787679999e+02 },
    { 4.2289999999999997e-08, 1.4324484492800000e+00, 5.1546387109299997e+02 },
    { 3.5469999999999999e-08, 5.7556217305699997e+00, 9.9022940591439999e+02 },
    { 3.1550000000000001e-08, 4.8532385600100003e+00, 4.6765198782060003e+02 },
    { 3.0169999999999998e-08, 6.4784331959999997e-02, 2.2752618943959999e+02 },
    { 2.2980000000000001e-08, 4.7099122864099998e+00, 2.0618554843720000e+02 },
    { 2.2490000000000002e-08, 1.5040734848000001e-01, 2.2041264243879999e+02 },
};

static const VSOP87Term VSOP87A_NEPTUNE_X5[1] = {
    { 0.0, 0.0, 0.0 }
};

static const VSOP87Term VSOP87A_NEPTUNE_Y0[746] = {
    { 3.0060563516649999e+01, 3.7408629471400001e+00, 3.8133035637799999e+01 },
    { 3.0205857682999998e-01, 3.1415926535900001e+00, 0.0000000000000000e+00 },
    { 1.5706589372999999e-01, 4.8253997012900003e+00, 3.6648562929500002e+01 },
    { 1.4936165806000001e-01, 5.7969490066500002e+00, 3.9617508346100003e+01 },
    { 1.3506391797000000e-01, 1.9295303488300000e+00, 7.6266071275599998e+01 },
    { 2.5842507490000000e-02, 4.2549700754000003e-01, 1.4844727083000000e+00 },
    { 1.0737397720000000e-02, 3.8137172853300001e+00, 7.4781598567299994e+01 },
    { 8.1518758300000004e-03, 5.4942977582600001e+00, 2.9689454166000000e+00 },
    { 5.8219929500000003e-03, 6.1963371893600003e+00, 3.5164090221199999e+01 },
    { 5.6557641200000001e-03, 4.4184300901500002e+00, 4.1101981054399999e+01 },
    { 4.9558104699999996e-03, 5.3120582578400004e+00, 5.2969096509459996e+02 },
    { 3.0452520299999999e-03, 5.1104811366099998e+00, 7.3297125859000005e+01 },
    { 2.7264029800000002e-03, 5.5860369078499996e+00, 2.1329909543799999e+02 },
    { 1.3589738499999999e-03, 3.9755375096400001e+00, 7.7750543983900002e+01 },
    { 9.0970871000000005e-04, 1.1783619888000001e-01, 1.1439910691340000e+02 },
    { 6.8790260999999995e-04, 4.2639199715100000e+00, 4.4534181248999998e+00 },
    { 2.8893354999999998e-04, 3.2184897503199998e+00, 4.2586453762700003e+01 },
    { 2.0081558999999999e-04, 1.1978791608499999e+00, 3.3679617512900002e+01 },
    { 1.2828707999999999e-04, 1.1674005344300000e+00, 1.1143016149680000e+02 },
    { 1.2613583000000000e-04, 1.9977733293400000e+00, 1.1291463420510000e+02 },
    { 1.2012961000000000e-04, 5.6615756380400004e+00, 1.0593819301891999e+03 },
    { 8.7685800000000000e-05, 3.2348715695000001e+00, 1.0846121608020000e+02 },
    { 7.5817880000000003e-05, 4.8116916839600004e+00, 4.2659819087599999e+02 },
    { 6.4392649999999994e-05, 3.0545325995099999e+00, 5.9378908332000000e+00 },
    { 5.2979780000000003e-05, 7.9002313989999995e-01, 7.1812653150700001e+01 },
    { 4.7334830000000002e-05, 2.5262019464200001e+00, 7.9235016692200006e+01 },
    { 4.6507079999999999e-05, 4.8154098329400004e+00, 3.7611770776000000e+01 },
    { 4.5572470000000002e-05, 5.8095155231800000e+00, 3.8654300499599998e+01 },
    { 4.3225500000000001e-05, 8.0665145881000000e-01, 3.8084851528000002e+01 },
    { 4.3155390000000003e-05, 3.5339350684099999e+00, 3.8181219747599997e+01 },
    { 4.2476430000000001e-05, 4.0635533650399998e+00, 2.8571808082200000e+01 },
    { 4.0890360000000001e-05, 4.2349446479000002e-01, 3.7169827791300001e+01 },
    { 3.9325149999999997e-05, 3.9160759281500002e+00, 3.9096243484299997e+01 },
    { 3.9301349999999999e-05, 3.8661417817400001e+00, 9.8899988524600005e+01 },
    { 3.3732809999999997e-05, 3.3961625565000002e+00, 1.4510977900969999e+02 },
    { 3.3239910000000001e-05, 3.1167427438500002e+00, 4.1927856940000003e+00 },
    { 3.1126359999999998e-05, 2.7319642944000000e-01, 4.7694263193399998e+01 },
    { 2.6709440000000002e-05, 2.3123527541600000e+00, 1.0994568878849999e+02 },
    { 2.5230420000000000e-05, 6.2340074518500002e+00, 3.1219908396260001e+02 },
    { 2.0869070000000001e-05, 6.2831362446099996e+00, 1.4956319713459999e+02 },
    { 2.0577940000000001e-05, 4.3855250578099998e+00, 1.3703302416240001e+02 },
    { 1.9671470000000001e-05, 2.5663477253200000e+00, 2.0618554843720000e+02 },
    { 1.8942610000000001e-05, 5.4627482525800000e+00, 2.2041264243879999e+02 },
    { 1.8888270000000001e-05, 1.6336433132399999e+00, 3.5685355082999997e+01 },
    { 1.8263900000000000e-05, 2.0093830596600002e+00, 4.4070926471000000e+01 },
    { 1.7327389999999999e-05, 4.1451850083400004e+00, 7.0328180442399997e+01 },
    { 1.7209540000000001e-05, 4.4940080513399998e+00, 1.0697674337190000e+02 },
    { 1.6812580000000001e-05, 2.7048031857899999e+00, 4.0580716192600001e+01 },
    { 1.6474739999999999e-05, 2.5001025496299998e+00, 3.0056280790500001e+01 },
    { 1.5591930000000001e-05, 5.2711484687799999e+00, 3.7872403206900003e+01 },
    { 1.5457050000000001e-05, 5.3526766943900004e+00, 3.8393668068700002e+01 },
    { 1.5333830000000001e-05, 5.8897111358999998e+00, 3.8021161053199997e+01 },
    { 1.5275260000000000e-05, 4.7341253439499997e+00, 3.8244910222400001e+01 },
    { 1.4670420000000001e-05, 3.5622646376999998e+00, 1.8175834193919999e+02 },
    { 1.4352740000000000e-05, 5.4429201317200002e+00, 5.2257741809380002e+02 },
    { 1.4049910000000000e-05, 2.0461108833899999e+00, 5.3680451209540001e+02 },
    { 1.3880240000000001e-05, 2.1479283041200001e+00, 1.1588357962169999e+02 },
    { 1.3141159999999999e-05, 1.8038644336200000e+00, 4.6209790485100001e+01 },
    { 1.2429290000000001e-05, 1.1024217356599999e+00, 3.3940249943799998e+01 },
    { 1.2370270000000000e-05, 1.0862219966800000e+00, 1.4807872442630000e+02 },
    { 1.2279260000000000e-05, 1.2133465184300001e+00, 7.2073285581600004e+01 },
    { 1.1117079999999999e-05, 5.5166992023899999e+00, 8.0767548473000002e+00 },
    { 1.0808070000000000e-05, 3.1840166143499999e+00, 4.2325821331800000e+01 },
    { 7.3076299999999997e-06, 4.6663380154200000e+00, 1.4362530630140000e+02 },
    { 7.2201499999999998e-06, 4.5972201465799998e+00, 1.5253214255120000e+02 },
    { 6.8501099999999997e-06, 3.7301758543300001e+00, 2.1840690486870000e+02 },
    { 6.3180700000000001e-06, 1.8463200964900000e+00, 7.4223635415000002e+00 },
    { 6.0882499999999997e-06, 2.9243066216300000e+00, 3.5212274331000003e+01 },
    { 5.5310600000000000e-06, 1.4149935734300001e+00, 4.1053796944600002e+01 },
    { 5.1508800000000004e-06, 1.9220956580099999e+00, 9.6320784650000002e-01 },
    { 4.6323100000000004e-06, 1.1732483537700000e+00, 3.1540753498800001e+01 },
    { 4.3614800000000000e-06, 6.0117733258600001e+00, 1.5890728952838001e+03 },
    { 3.9391600000000001e-06, 3.7810083301100001e+00, 3.1019488636999998e+01 },
    { 3.9009299999999998e-06, 7.9231945944000004e-01, 6.5922821389999999e+00 },
    { 3.8216000000000000e-06, 1.7892813396500000e+00, 4.8736514376280002e+02 },
    { 3.4986200000000000e-06, 1.9787949769999999e-01, 5.2126486179999998e-01 },
    { 3.4928400000000001e-06, 4.3203119924299997e+00, 6.0766952886799999e+01 },
    { 3.4902900000000000e-06, 4.8385896872999998e+00, 1.8027386923090000e+02 },
    { 3.4301000000000000e-06, 5.4805196298299999e+00, 6.8843707734099993e+01 },
    { 3.3617299999999999e-06, 5.7726127805000005e-01, 4.5246582638600003e+01 },
    { 3.3417000000000001e-06, 3.9009419019400000e+00, 2.5505546779820000e+02 },
    { 3.2180700000000002e-06, 3.2943328741000002e+00, 4.4725317776799997e+01 },
    { 3.1649700000000002e-06, 1.1470282628599999e+00, 3.8846515523820000e+02 },
    { 3.1634800000000002e-06, 1.3857203263000000e+00, 4.1948464387519999e+02 },
    { 3.0670999999999998e-06, 4.0619292226399999e+00, 6.3989728631399998e+02 },
    { 3.0617700000000001e-06, 1.8466278211000001e-01, 6.2830758499914000e+03 },
    { 2.9593000000000002e-06, 3.6117309489400000e+00, 8.4342826122899993e+01 },
    { 2.9585499999999998e-06, 1.7950850811999999e-01, 1.1020632121940000e+02 },
    { 2.8490499999999999e-06, 1.7897404260700001e+00, 1.2530172972200001e+01 },
    { 2.6913099999999998e-06, 3.4352455427900002e+00, 1.0309277421860000e+02 },
    { 2.6805999999999998e-06, 1.7377259325900001e+00, 3.6760437514099998e+01 },
    { 2.6573800000000000e-06, 5.6702968623099999e+00, 9.4498282327580000e+02 },
    { 2.6475999999999999e-06, 2.5564440160299999e+00, 3.9505633761500000e+01 },
    { 2.5739800000000001e-06, 4.1148364577000001e+00, 3.2195144804599998e+01 },
    { 2.5218000000000001e-06, 5.2438917549799999e+00, 2.7087335373900000e+01 },
    { 2.4910200000000001e-06, 5.7354799625200004e+00, 1.0549227066360000e+02 },
    { 2.4239400000000000e-06, 4.9323094251999999e-01, 1.4014645680499999e+01 },
    { 2.2722600000000000e-06, 6.1611595044699996e+00, 2.7406604832480002e+02 },
    { 2.2611900000000002e-06, 1.2670772752299999e+00, 8.0719489400499995e+01 },
    { 2.1334100000000000e-06, 1.0780127382600000e+00, 3.1639186965660002e+02 },
    { 2.0277099999999999e-06, 4.9619598398400004e+00, 1.7516605980020000e+02 },
    { 2.0259899999999999e-06, 2.1708350901800002e+00, 4.1529185818119998e+02 },
    { 1.9321199999999998e-06, 2.5383969249899998e+00, 1.4659425171800001e+02 },
    { 1.9027900000000000e-06, 7.5836937752000000e-01, 6.9152524274800001e+01 },
    { 1.7892599999999999e-06, 5.0086045668799999e+00, 2.1692243216040001e+02 },
    { 1.7800500000000000e-06, 1.6095005921600001e+00, 1.0213285546211000e+04 },
    { 1.7646499999999999e-06, 4.8597021904300002e+00, 3.6536688344900000e+01 },
    { 1.7520900000000001e-06, 5.8381456321799998e+00, 3.9729382930699998e+01 },
    { 1.6891600000000000e-06, 4.0676630941100003e+00, 2.9170403072770000e+02 },
    { 1.6889499999999999e-06, 2.5089346215999999e+00, 6.3735898303399999e+01 },
    { 1.5814000000000000e-06, 3.2516943106500000e+00, 1.1045700263900001e+01 },
    { 1.5498700000000000e-06, 1.8057956756999999e-01, 7.7229279122099996e+01 },
    { 1.5246500000000001e-06, 1.7890470658999999e-01, 3.0710672096300001e+01 },
    { 1.4695399999999999e-06, 1.2002838388000001e-01, 1.5104766984290001e+02 },
    { 1.4472699999999999e-06, 5.3420506294300001e+00, 4.9178735901700001e+01 },
    { 1.4234000000000000e-06, 3.8542978912699999e+00, 7.5744806413800006e+01 },
    { 1.4092700000000000e-06, 3.8163063133400001e+00, 6.2251425595100002e+01 },
    { 1.2929100000000000e-06, 7.9819261590000001e-01, 4.5555399179299997e+01 },
    { 1.2904899999999999e-06, 2.2360529837200001e+00, 3.6861011250600001e+01 },
    { 1.2765100000000000e-06, 1.6179774013599999e+00, 2.4118389957300000e+01 },
    { 1.2291499999999999e-06, 2.1035386868299999e+00, 3.9405060024999997e+01 },
    { 1.2122600000000000e-06, 3.5350465725900002e+00, 3.7920587316700001e+01 },
    { 1.2113900000000001e-06, 6.2088690348800002e+00, 3.3025226207099998e+01 },
    { 1.2058000000000000e-06, 8.2135679726999999e-01, 3.8345483958899997e+01 },
    { 1.1277499999999999e-06, 5.9186264657300001e+00, 1.4214083359310001e+02 },
    { 1.1238299999999999e-06, 3.9900905808800000e+00, 6.3278373931320004e+02 },
    { 1.0698999999999999e-06, 4.7310056927600002e+00, 1.1624747044077999e+03 },
    { 1.0663600000000000e-06, 2.1465234921700000e+00, 5.1078094306999997e+00 },
    { 1.0495800000000000e-06, 4.6901578635699996e+00, 4.3371173787679999e+02 },
    { 1.0433799999999999e-06, 7.1903696521000005e-01, 3.5424722652100002e+01 },
    { 1.0377400000000001e-06, 5.2398540817899999e+00, 8.2858353414600003e+01 },
    { 1.0354099999999999e-06, 3.6855519355999999e+00, 4.0841348623499997e+01 },
    { 9.8751999999999999e-07, 4.1040232532700003e+00, 4.5342489381899998e+02 },
    { 9.6668000000000008e-07, 4.5998842597199996e+00, 1.0522683831884001e+03 },
    { 9.5561000000000007e-07, 5.1798941725100001e+00, 2.5357099508990001e+02 },
    { 9.3279000000000003e-07, 5.6889410064100003e+00, 2.1989137757699999e+02 },
    { 9.0518999999999997e-07, 1.1887025090400001e+00, 6.5220371011699996e+01 },
    { 8.9782000000000003e-07, 4.2269501930700004e+00, 3.2835259365719998e+02 },
    { 8.8318000000000005e-07, 5.4182579710200001e+00, 1.8324281464750001e+02 },
    { 8.8316999999999999e-07, 6.0600445447700002e+00, 2.5143213107579999e+02 },
    { 8.6784999999999998e-07, 3.4302262179300000e+00, 2.9207613067999998e+00 },
    { 8.6636000000000004e-07, 1.7207557169000001e+00, 3.2716409666399997e+01 },
    { 8.5104000000000003e-07, 3.2193861895000002e+00, 3.6436114608399997e+01 },
    { 8.4713999999999995e-07, 1.2408388035500000e+00, 1.0601353552540000e+02 },
    { 8.0650999999999999e-07, 4.4611977941400003e+00, 4.3240845068500001e+01 },
    { 8.0200000000000001e-07, 1.1212010595499999e+00, 3.9829956667200001e+01 },
    { 7.9477999999999997e-07, 1.8451019849000001e+00, 9.7415515816300001e+01 },
    { 7.8248000000000000e-07, 2.4940371723200001e+00, 2.4476805547999998e+00 },
    { 7.6683999999999995e-07, 3.7309041496400002e+00, 5.4166259714000002e+00 },
    { 7.3954000000000002e-07, 3.9809964614500002e+00, 2.7083129857000001e+00 },
    { 7.2027000000000003e-07, 4.8260548884499999e+00, 3.6696747039300000e+01 },
    { 7.1436000000000005e-07, 3.6306170758700000e+00, 6.6253120356299996e+02 },
    { 7.0037000000000002e-07, 2.6608014100099999e+00, 4.3549661609200001e+01 },
    { 6.8912999999999998e-07, 2.3896923985499998e+00, 1.0664954771900000e+03 },
    { 6.7769000000000003e-07, 2.3138750823000001e+00, 2.2434479570190001e+02 },
    { 6.6879999999999995e-07, 2.3664305806199999e+00, 6.9010986796999996e+00 },
    { 6.6787000000000001e-07, 1.8614124275299999e+00, 8.4608283475120004e+02 },
    { 6.6578999999999998e-07, 5.8808196559799999e+00, 2.5653994050649999e+02 },
    { 6.5815999999999996e-07, 6.1405995390000001e+00, 3.6127298067700004e+01 },
    { 6.5264000000000005e-07, 3.4500358991700000e+00, 2.2633917249000000e+01 },
    { 6.3401000000000005e-07, 2.4814555982400002e+00, 6.6704843720000000e+01 },
    { 6.2933000000000004e-07, 6.4057863199999998e-01, 8.9068362497999995e+00 },
    { 6.1017999999999998e-07, 1.5967468779700000e+00, 9.5612275556000004e+00 },
    { 5.7112000000000005e-07, 5.5970836993299997e+00, 1.5499118388799999e+01 },
    { 5.6657999999999998e-07, 6.0991696369600001e+00, 1.7878939652259999e+02 },
    { 5.5372999999999997e-07, 2.8725447328499998e+00, 9.4917560896980001e+02 },
    { 5.3145999999999997e-07, 5.3517021967799998e+00, 2.9021955801939998e+02 },
    { 5.0999000000000003e-07, 6.2410315056699996e+00, 8.1373880706300000e+01 },
    { 5.0175000000000000e-07, 5.7947249975500004e+00, 3.6909195360399998e+01 },
    { 4.9655000000000004e-07, 4.8945385113000001e-01, 6.7359235025800004e+01 },
    { 4.9126000000000002e-07, 5.9967698907400004e+00, 1.9010580526599998e+01 },
    { 4.9103000000000002e-07, 4.3839874995799999e+00, 3.6500115658670001e+02 },
    { 4.7264999999999999e-07, 4.6135621118900003e+00, 3.3406124266997999e+03 },
    { 4.6730000000000001e-07, 1.2038701485100001e+00, 1.8472728735580000e+02 },
    { 4.5695999999999998e-07, 6.0387908344999996e-01, 1.1736805233000000e+02 },
    { 4.4133000000000000e-07, 6.8971110137000002e-01, 3.2503961345299999e+01 },
    { 4.3867000000000000e-07, 5.6045258127000004e+00, 3.9321532631000000e+00 },
    { 4.3747000000000001e-07, 6.5030388704999997e-01, 1.0212488945514000e+03 },
    { 4.3441999999999999e-07, 4.6475859374899997e+00, 2.2137585028530000e+02 },
    { 4.3440000000000003e-07, 6.0592634380800003e+00, 2.9318850343600002e+02 },
    { 4.2791999999999998e-07, 2.9900586706999999e+00, 5.6363121503840000e+02 },
    { 4.2301999999999999e-07, 9.9480139810000001e-01, 3.5033211960040001e+02 },
    { 4.2227999999999999e-07, 4.0761130823800000e+00, 7.6787336137400004e+01 },
    { 4.2142999999999998e-07, 4.1607047740800001e+00, 3.1071461125430000e+02 },
    { 3.9999000000000002e-07, 5.3306255424100000e+00, 5.2549817940059995e+02 },
    { 3.9136999999999999e-07, 1.1867764184600000e+00, 1.7526107818300002e+01 },
    { 3.8879999999999999e-07, 1.7230156630000000e+00, 7.6314255385400003e+01 },
    { 3.8879999999999999e-07, 5.2789354899400003e+00, 7.6217887165799993e+01 },
    { 3.8644000000000001e-07, 7.4635128856999999e-01, 5.0663208609999998e+01 },
    { 3.8592000000000000e-07, 5.0314424701200000e+00, 5.6772584022999997e+00 },
    { 3.8369000000000002e-07, 5.0784159227199996e+00, 3.9356875915200000e+01 },
    { 3.8212999999999999e-07, 4.9926925646200004e+00, 7.5302863429100000e+01 },
    { 3.8164000000000002e-07, 3.6628728849000001e+00, 4.3762109930299999e+01 },
    { 3.7226000000000003e-07, 1.7694973897399999e+00, 4.4923210812500002e+02 },
    { 3.6370000000000001e-07, 5.9701725549999995e-01, 7.0849445304200003e+01 },
    { 3.6104000000000001e-07, 3.2621571949999999e+00, 3.0508553696180002e+02 },
    { 3.5778999999999998e-07, 5.1621737976000004e+00, 3.1368355667089997e+02 },
    { 3.5246999999999998e-07, 4.9200579100699997e+00, 3.4509698915400001e+01 },
    { 3.4933999999999998e-07, 3.5861144114900001e+00, 3.6233367223999999e+00 },
    { 3.4351999999999999e-07, 5.7952805507400003e+00, 3.3467169191799996e+01 },
    { 3.4218000000000001e-07, 6.0503158810700004e+00, 3.1931263096340001e+02 },
    { 3.2863000000000000e-07, 8.6693212611000003e-01, 9.1786441523799994e+01 },
    { 3.2429000000000000e-07, 7.3124755748000003e-01, 7.8713751830400000e+01 },
    { 3.1916000000000001e-07, 3.0758304700300001e+00, 1.0038446123289999e+02 },
    { 3.1274999999999999e-07, 2.5363404908899998e+00, 2.1979525943199999e+01 },
    { 3.1031999999999999e-07, 3.0719239201400002e+00, 3.4200882374700001e+01 },
    { 3.0923999999999998e-07, 4.8203826623000001e+00, 2.5802441321480001e+02 },
    { 3.0193000000000000e-07, 3.8943453321199999e+00, 1.3851749687070000e+02 },
    { 2.9542999999999999e-07, 4.0046609771300004e+00, 7.1135470007999997e+00 },
    { 2.9234999999999999e-07, 6.9125716428999995e-01, 1.0400779795530001e+02 },
    { 2.8574000000000002e-07, 6.2300712127500004e+00, 2.1543795945209999e+02 },
    { 2.8458999999999998e-07, 4.8253770150599999e+00, 4.2798902083800002e+01 },
    { 2.7995999999999997e-07, 1.2605964610800000e+00, 1.1306332694800000e+01 },
    { 2.7749999999999999e-07, 7.8341232059999999e-01, 2.0495053234899999e+01 },
    { 2.7380999999999999e-07, 5.5337498773600000e+00, 3.2686812094890001e+02 },
    { 2.6742000000000000e-07, 2.0677751686999999e+00, 6.9364972595899999e+01 },
    { 2.6622000000000001e-07, 6.2303265192000001e+00, 3.2983706636549999e+02 },
    { 2.6519000000000001e-07, 4.6318687670600003e+00, 5.7255490749000003e+01 },
    { 2.6150000000000001e-07, 4.5464743182499996e+00, 4.0164971951619998e+02 },
    { 2.5867000000000003e-07, 1.4541930711500000e+00, 2.9226199388000001e+01 },
    { 2.5169000000000000e-07, 5.6819015506800001e+00, 4.1756372360199997e+01 },
    { 2.4877999999999998e-07, 5.0524861838400001e+00, 1.8621176006409999e+02 },
    { 2.4722000000000001e-07, 6.1246284144999996e-01, 6.7880499887599996e+01 },
    { 2.4532999999999999e-07, 2.4331246598199998e+00, 1.3554855145409999e+02 },
    { 2.3857000000000000e-07, 1.1272037362600000e+00, 7.9186832582400001e+01 },
    { 2.3104999999999999e-07, 1.0278942153199999e+00, 6.8189316428300003e+01 },
    { 2.2994000000000001e-07, 5.3308820782800002e+00, 2.2752618943959999e+02 },
    { 2.2707999999999999e-07, 9.6197477785999996e-01, 1.4365407527325999e+03 },
    { 2.2660999999999999e-07, 2.6732435286300000e+00, 3.5275964805800001e+01 },
    { 2.2616999999999999e-07, 1.8917214371200000e+00, 3.6600378819699998e+01 },
    { 2.2482000000000000e-07, 5.4358844843399998e+00, 3.9569324236299998e+01 },
    { 2.1423000000000001e-07, 5.5906164830599998e+00, 3.9665692455900000e+01 },
    { 2.0858000000000001e-07, 1.6649774300700000e+00, 4.0990106469799997e+01 },
    { 2.0858000000000001e-07, 3.1042516354999998e+00, 5.8739963457300000e+01 },
    { 2.0723999999999999e-07, 3.7645182451900001e+00, 8.5827298831199997e+01 },
    { 2.0452999999999999e-07, 5.6025190491199997e+00, 2.0936694217490000e+02 },
    { 2.0394000000000001e-07, 4.7145363436699999e+00, 2.5602862665600000e+01 },
    { 2.0347000000000001e-07, 1.5746118546500001e+00, 7.9889407997999996e+01 },
    { 2.0326999999999999e-07, 4.7557038321700000e+00, 3.7478644332000002e+01 },
    { 2.0326999999999999e-07, 5.8680687413500001e+00, 3.8787426943600003e+01 },
    { 1.9947000000000000e-07, 2.4578098509599999e+00, 2.1723124870109999e+02 },
    { 1.9425000000000001e-07, 1.3925695627800001e+00, 1.2991947716160001e+02 },
    { 1.8916000000000001e-07, 3.8992237640599998e+00, 4.0138773207900002e+01 },
    { 1.8890999999999999e-07, 1.7924835694500001e+00, 5.4286545332400003e+01 },
    { 1.8848000000000000e-07, 5.9286413689999998e-02, 2.1187638603783998e+03 },
    { 1.8627000000000001e-07, 4.6566127924899998e+00, 3.2350541665740002e+02 },
    { 1.7964000000000001e-07, 1.0386339790000001e+00, 2.0225339517410001e+02 },
    { 1.7462000000000001e-07, 1.2266713362999999e-01, 7.3818390720799997e+01 },
    { 1.7358000000000000e-07, 2.9610786264700000e+00, 1.9122455111200001e+01 },
    { 1.7267000000000000e-07, 5.4633559791900002e+00, 8.3855713880000007e+00 },
    { 1.6679000000000000e-07, 3.6443311079999999e-02, 3.6648562929500002e+02 },
    { 1.6366000000000000e-07, 3.3921589134999999e+00, 8.5319638175199998e+02 },
    { 1.6323000000000001e-07, 1.1443847031500001e+00, 2.0910630974399999e+02 },
    { 1.6187999999999999e-07, 1.0027715842600000e+00, 3.7499896191399998e+01 },
    { 1.6187999999999999e-07, 3.3378156820800000e+00, 3.8766175084200000e+01 },
    { 1.6035999999999999e-07, 5.6276724275200003e+00, 5.1546387109299997e+02 },
    { 1.5927000000000000e-07, 4.4540986388099997e+00, 2.0606927819500001e+01 },
    { 1.5769999999999999e-07, 3.1796217275999999e-01, 1.5401661525950001e+02 },
    { 1.5706999999999999e-07, 3.5034345913200000e+00, 7.6005438844699995e+01 },
    { 1.5703000000000000e-07, 5.4604004673500004e+00, 8.3769726336320002e+02 },
    { 1.5639000000000001e-07, 5.1200692501400002e+00, 2.9467297614429998e+02 },
    { 1.5519000000000000e-07, 5.0047189623900001e+00, 7.2333918012500007e+01 },
    { 1.5410999999999999e-07, 4.2026345311600002e+00, 7.3587651353180001e+02 },
    { 1.5258999999999999e-07, 5.7149859319600003e+00, 3.6351668387839999e+02 },
    { 1.5043999999999999e-07, 5.6837915933699996e+00, 5.2147681318300002e+01 },
    { 1.4984000000000000e-07, 4.3043880913599999e+00, 1.4414657116320001e+02 },
    { 1.4928000000000000e-07, 5.4970393030800002e+00, 3.3988434053600002e+01 },
    { 1.4840000000000001e-07, 5.3407509730899996e+00, 4.3810294040099997e+01 },
    { 1.4786000000000000e-07, 4.3719851868999998e-01, 7.3557758289899994e+01 },
    { 1.4739000000000000e-07, 6.2629495824300001e+00, 8.2203962108799999e+01 },
    { 1.4623000000000001e-07, 5.7477300147999999e-01, 7.4299006053259995e+02 },
    { 1.4466000000000000e-07, 2.0879849179000001e-01, 2.5208652238159999e+02 },
    { 1.4382000000000000e-07, 3.5604726885600000e+00, 5.5771018040700000e+01 },
    { 1.4368000000000000e-07, 2.1813948377200001e+00, 6.4959738580800007e+01 },
    { 1.4104000000000000e-07, 6.0320873853699997e+00, 2.3593301268700000e+02 },
    { 1.4023000000000001e-07, 4.7171111781899997e+00, 4.3829828244570001e+02 },
    { 1.3971000000000000e-07, 3.5417652246800002e+00, 7.6526703706500001e+01 },
    { 1.3771000000000001e-07, 2.9231826179299998e+00, 7.6377945860200001e+01 },
    { 1.3771000000000001e-07, 4.0787684929200001e+00, 7.6154196690999996e+01 },
    { 1.3631999999999999e-07, 1.2960362320100001e+00, 4.5034134317499998e+01 },
    { 1.3512999999999999e-07, 9.1285117035999996e-01, 7.0066423920080001e+02 },
    { 1.3262000000000001e-07, 8.8619404716000005e-01, 1.4065636088479999e+02 },
    { 1.2911000000000000e-07, 5.1267331637300000e+00, 4.2277637222000003e+01 },
    { 1.2653999999999999e-07, 7.8790100003999997e-01, 1.0728555991259999e+02 },
    { 1.2557999999999999e-07, 2.3512887312399999e+00, 1.7637982402900001e+01 },
    { 1.2545000000000000e-07, 2.4512229657700000e+00, 5.9282480178500002e+01 },
    { 1.2251000000000000e-07, 5.9006293720600000e+00, 4.7039871887600000e+01 },
    { 1.2244000000000000e-07, 2.6252844741799999e+00, 1.9907200143640000e+02 },
    { 1.2137999999999999e-07, 7.5939056043999997e-01, 4.2065188900899997e+01 },
    { 1.2109000000000001e-07, 5.3062972106999995e-01, 3.2764593776200002e+01 },
    { 1.1480999999999999e-07, 3.6559100566999998e+00, 3.8963117040299998e+01 },
    { 1.1480999999999999e-07, 6.8467720964000001e-01, 3.7302954235300000e+01 },
    { 1.1450000000000000e-07, 5.9227775369700000e+00, 5.2916970023279998e+02 },
    { 1.1388000000000000e-07, 5.8233648003600003e+00, 2.7555052103309998e+02 },
    { 1.1368999999999999e-07, 4.2299088464799999e+00, 4.9155792945680003e+02 },
    { 1.1367000000000000e-07, 2.2029674229600000e+00, 5.2575881183150000e+02 },
    { 1.1260000000000000e-07, 5.3209661151300001e+00, 5.3362311835770004e+02 },
    { 1.1037000000000000e-07, 1.5936753689500001e+00, 5.3021222995640005e+02 },
    { 1.0515000000000000e-07, 5.0001299713999998e+00, 2.9747464249800000e+01 },
    { 1.0240000000000000e-07, 1.3476709246400000e+00, 8.0458856969600006e+01 },
    { 1.0236000000000000e-07, 1.0223409386100000e+00, 5.6782400073240001e+02 },
    { 1.0188999999999999e-07, 1.0190614433000000e+00, 2.1458261081400000e+01 },
    { 1.0031000000000000e-07, 3.8099573660499999e+00, 4.3501477499400004e+01 },
    { 9.8480000000000006e-08, 4.0629707807899997e+00, 2.1149444540699999e+01 },
    { 9.7609999999999999e-08, 4.4716218180000000e+00, 9.5628915597059995e+02 },
    { 9.7530000000000000e-08, 1.9435849555500000e+00, 5.4391805909619995e+02 },
    { 9.5789999999999999e-08, 2.8315813260499998e+00, 1.8769623277240001e+02 },
    { 9.4370000000000004e-08, 3.6777972876699998e+00, 1.2099138905240000e+02 },
    { 9.1790000000000003e-08, 3.2017645884200001e+00, 3.4793262315539999e+02 },
    { 9.1609999999999998e-08, 1.6165080423700000e+00, 1.6805251279940001e+02 },
    { 9.1320000000000000e-08, 2.7141964250000000e-01, 4.0313419222450000e+02 },
    { 8.8380000000000001e-08, 6.0129548280000000e-01, 3.1982696483500000e+01 },
    { 8.8230000000000003e-08, 5.8940412736300001e+00, 4.0016524680790002e+02 },
    { 8.8150000000000005e-08, 6.0293455237500000e+00, 1.1802244363580000e+02 },
    { 8.7999999999999994e-08, 4.9151992929799997e+00, 7.3879727483859995e+02 },
    { 8.7250000000000004e-08, 4.7809414857999999e-01, 4.8884961647109998e+02 },
    { 8.5889999999999997e-08, 3.5509276740100000e+00, 6.2439816792520003e+02 },
    { 8.4180000000000003e-08, 6.1975431324499999e+00, 7.7489911552999999e+01 },
    { 8.4110000000000003e-08, 1.7246794270100001e+00, 6.5874762317500000e+01 },
    { 8.3980000000000002e-08, 4.0445623150700003e+00, 3.7715882254339999e+02 },
    { 8.3420000000000002e-08, 4.0485972284000002e+00, 1.4788665740644001e+03 },
    { 8.3039999999999994e-08, 2.3900308432099999e+00, 1.3406407874580000e+02 },
    { 8.2990000000000004e-08, 4.0644822208100004e+00, 4.1555249061209997e+02 },
    { 8.2030000000000001e-08, 1.7640371797300001e+00, 2.6826702943000001e+01 },
    { 8.1969999999999999e-08, 3.6958483365300001e+00, 9.5979227217800002e+01 },
    { 8.1190000000000001e-08, 5.0426430532700000e+00, 4.8025159676200002e+02 },
    { 8.1120000000000001e-08, 4.8848238064900000e+00, 4.7494684537519998e+02 },
    { 8.0599999999999994e-08, 1.9686345842899999e+00, 1.7097327410619999e+02 },
    { 8.0369999999999998e-08, 5.7049131223499998e+00, 1.2655674786264001e+03 },
    { 7.9730000000000000e-08, 6.1519114233499996e+00, 5.2650957135689998e+02 },
    { 7.9090000000000002e-08, 7.2885885030000000e-02, 3.6387930498599999e+01 },
    { 7.9090000000000002e-08, 1.1261087277199999e+00, 3.9878140776999999e+01 },
    { 7.8919999999999995e-08, 5.3214509917899999e+00, 3.3132153907380001e+02 },
    { 7.8510000000000006e-08, 5.1349671086399997e+00, 1.6921656695024001e+03 },
    { 7.8339999999999999e-08, 3.9352619436000003e-01, 2.8873508531110002e+02 },
    { 7.8320000000000003e-08, 1.3327955837700001e+00, 5.3287235883230005e+02 },
    { 7.7429999999999999e-08, 5.9825069424999997e+00, 4.8588067105450000e+02 },
    { 7.5709999999999994e-08, 1.2465881273899999e+00, 2.3463998651499999e+01 },
    { 7.5030000000000004e-08, 8.0961405134999997e-01, 6.0176425067620005e+02 },
    { 7.4630000000000001e-08, 2.4873232109000001e+00, 9.0684978763799995e+02 },
    { 7.4460000000000006e-08, 1.6437660973599999e+00, 4.9447869076360001e+02 },
    { 7.4040000000000007e-08, 5.7152943813399997e+00, 7.5042230998199997e+01 },
    { 7.3720000000000001e-08, 3.3163302035700002e+00, 5.4807810194200002e+01 },
    { 7.3360000000000003e-08, 6.0563495647999996e+00, 5.5659143456099997e+01 },
    { 7.2479999999999998e-08, 4.0168738886700002e+00, 6.7480074410429995e+02 },
    { 7.1659999999999994e-08, 5.9710005918400002e+00, 1.1731986822019999e+02 },
    { 7.0920000000000002e-08, 1.5841036791500001e+00, 4.5290362895720000e+02 },
    { 7.0539999999999995e-08, 3.5038552224899999e+00, 4.5394615868080001e+02 },
    { 7.0510000000000001e-08, 4.7552196502899999e+00, 1.0749800823370001e+02 },
    { 6.9940000000000003e-08, 4.0368319222500002e+00, 6.4701083331480004e+02 },
    { 6.9419999999999996e-08, 5.1602349716699996e+00, 2.1648048917570000e+02 },
    { 6.8330000000000004e-08, 2.8915249286900000e+00, 2.1011770170029999e+02 },
    { 6.7850000000000002e-08, 2.0197940152000000e+00, 3.2027583880989999e+02 },
    { 6.7270000000000006e-08, 4.7969479934099999e+00, 1.5819593482830001e+03 },
    { 6.6979999999999995e-08, 1.3797025722600000e+00, 2.1568968960162001e+03 },
    { 6.5719999999999996e-08, 1.6389994338700000e+00, 1.0551891444952000e+03 },
    { 6.4749999999999994e-08, 9.5285677251000001e-01, 5.9083042499689998e+02 },
    { 6.4589999999999998e-08, 2.7442800875500000e+00, 5.6188610989920005e+02 },
    { 6.4519999999999998e-08, 3.4070543237600002e+00, 3.2176031151820001e+02 },
    { 6.4490000000000004e-08, 2.1035823079000000e+00, 7.0996483432549996e+02 },
    { 6.4150000000000002e-08, 1.0078329631500000e+00, 5.7143616164400001e+01 },
    { 6.2649999999999995e-08, 3.0232028423699999e+00, 7.1293377974210000e+02 },
    { 6.2569999999999997e-08, 3.1897063651100002e+00, 7.4958234267160003e+02 },
    { 6.2540000000000003e-08, 5.3152957573900004e+00, 3.1231936958100000e+01 },
    { 6.1449999999999998e-08, 3.6813164063100001e+00, 9.5931043107999997e+01 },
    { 6.0990000000000005e-08, 4.3939474853300000e+00, 3.8135160823740000e+02 },
    { 6.0549999999999996e-08, 3.3559208646299998e+00, 7.8623090560109995e+02 },
    { 6.0240000000000002e-08, 2.7059173686200002e+00, 1.6153509694600000e+01 },
    { 5.9830000000000000e-08, 1.1803615916100001e+00, 9.3270914232099997e+01 },
    { 5.9580000000000001e-08, 9.3024333893000000e-01, 5.2213939318786201e+04 },
    { 5.8999999999999999e-08, 2.1363110756000001e-01, 5.2061407176234992e+04 },
    { 5.8330000000000000e-08, 5.1785992060300003e+00, 6.6430918177617996e+03 },
    { 5.7579999999999997e-08, 1.2526468464600000e+00, 2.9795648359600001e+01 },
    { 5.7490000000000001e-08, 4.7094071396199997e+00, 6.3182399402126002e+03 },
    { 5.7210000000000001e-08, 8.7653097312999995e-01, 1.1950691634410001e+02 },
    { 5.7079999999999999e-08, 3.6159587367600001e+00, 1.0186893394120000e+02 },
    { 5.6970000000000000e-08, 5.3079088956699998e+00, 3.2455777235500001e+01 },
    { 5.6890000000000002e-08, 2.7079604190000000e+00, 2.7741726679700001e+01 },
    { 5.6790000000000001e-08, 4.3469665056300002e+00, 1.6983591097100000e+01 },
    { 5.6690000000000000e-08, 5.8371555280999998e-01, 3.2538364824060000e+02 },
    { 5.6640000000000003e-08, 1.4227860016999999e-01, 1.1653954657146000e+03 },
    { 5.6550000000000000e-08, 1.4693970444200000e+00, 1.2604284735620600e+04 },
    { 5.6410000000000000e-08, 4.5309305236800004e+00, 1.6922816896699999e+02 },
    { 5.5859999999999999e-08, 5.1213123987199998e+00, 6.2479117597701998e+03 },
    { 5.5719999999999999e-08, 5.5867744509500001e+00, 2.9104963942190000e+02 },
    { 5.5630000000000002e-08, 1.0707662895400001e+00, 1.7730492381430000e+02 },
    { 5.5059999999999998e-08, 2.7012546305599998e+00, 1.7787437278589999e+02 },
    { 5.4970000000000002e-08, 3.5236976820599999e+00, 8.2287946853059998e+02 },
    { 5.4550000000000002e-08, 6.1336956509200000e+00, 1.0248449636432200e+04 },
    { 5.4319999999999999e-08, 4.2431054729900000e+00, 2.2286032299359999e+02 },
    { 5.3710000000000003e-08, 2.8352153613900000e+00, 2.8117959532560002e+02 },
    { 5.3559999999999998e-08, 2.6035194623000002e-01, 1.0178121455989800e+04 },
    { 5.3480000000000000e-08, 6.1370731346199996e+00, 2.4639654819099999e+01 },
    { 5.3309999999999999e-08, 6.1762888418099999e+00, 4.3681380973739999e+02 },
    { 5.3179999999999997e-08, 3.6561756177800002e+00, 4.4283374792099998e+01 },
    { 5.2700000000000002e-08, 5.0382295726999997e+00, 6.1671922703697001e+03 },
    { 5.2439999999999998e-08, 7.6594899514000003e-01, 7.3345309968799995e+01 },
    { 5.1580000000000003e-08, 6.1970021510000001e-02, 2.5520079140841790e+04 },
    { 5.1459999999999999e-08, 5.5241138826700000e+00, 4.6518607025800002e+01 },
    { 5.1020000000000003e-08, 5.9510611888199998e+00, 1.0452906281710000e+02 },
    { 5.1020000000000003e-08, 4.1008643068800001e+00, 2.7258157561650000e+02 },
    { 4.9840000000000002e-08, 1.7849748817000000e-01, 1.0097401966589299e+04 },
    { 4.8820000000000003e-08, 5.4005520843100001e+00, 2.4697871295089999e+02 },
    { 4.8639999999999997e-08, 4.3754043251099999e+00, 3.5181659230870002e+02 },
    { 4.8319999999999999e-08, 4.1790311879199997e+00, 7.1144930703379998e+02 },
    { 4.8140000000000000e-08, 3.9317249117299999e+00, 2.7703499374140000e+02 },
    { 4.7979999999999997e-08, 3.6685123500699999e+00, 5.2995159752550001e+02 },
    { 4.7890000000000001e-08, 1.1161363305900001e+00, 6.2747898792640001e+02 },
    { 4.7440000000000000e-08, 2.2573170741999999e+00, 3.0927832265580003e+02 },
    { 4.7440000000000000e-08, 2.3087185254799998e+00, 2.0388438056784202e+04 },
    { 4.7390000000000003e-08, 4.4967401674099996e+00, 7.4809786996330001e+02 },
    { 4.7110000000000003e-08, 5.1498823160800002e+00, 5.2252072354423988e+04 },
    { 4.6870000000000002e-08, 5.9702296080000002e-02, 1.5467100656529999e+02 },
    { 4.6649999999999998e-08, 5.0747542188999999e+00, 5.1159540830470002e+02 },
    { 4.6590000000000002e-08, 1.9178665241699999e+00, 2.8362727588040002e+02 },
    { 4.6460000000000000e-08, 4.4485278594600004e+00, 1.3286183635523599e+04 },
    { 4.6339999999999997e-08, 2.8670669630600001e+00, 2.6087903141574199e+04 },
    { 4.6329999999999999e-08, 3.3622388324400001e+00, 3.2276977924947000e+03 },
    { 4.6290000000000000e-08, 5.6895256517600004e+00, 1.2247586176070000e+02 },
    { 4.5940000000000000e-08, 4.8605937492200004e+00, 1.0254416802454000e+03 },
    { 4.5919999999999997e-08, 3.6701946488099999e+00, 1.8084331820250000e+02 },
    { 4.5809999999999998e-08, 2.4022029572600001e+00, 1.3757737998458001e+03 },
    { 4.5779999999999997e-08, 4.9871920877799996e+00, 6.3666770846650002e+02 },
    { 4.5559999999999999e-08, 1.5628527572699999e+00, 2.5974988507369089e+04 },
    { 4.5260000000000003e-08, 2.9832350424900000e+00, 5.2554636351040006e+02 },
    { 4.4950000000000002e-08, 2.3476453014800001e+00, 1.0820158474381999e+03 },
    { 4.4939999999999997e-08, 2.9110873403399999e+00, 8.7583029900099996e+02 },
    { 4.4920000000000001e-08, 4.8115185066699997e+00, 5.2177290755856688e+04 },
    { 4.4820000000000000e-08, 1.3472188586000000e-01, 3.4236723933770003e+02 },
    { 4.4479999999999999e-08, 5.9169500384999996e-01, 4.3978275515399997e+02 },
    { 4.4419999999999997e-08, 4.7781272569999998e-02, 1.2726810256271999e+03 },
    { 4.4300000000000000e-08, 3.8083886058599998e+00, 5.2943033266370003e+02 },
    { 4.4269999999999999e-08, 1.4307866075700000e+00, 1.8918070548070000e+02 },
    { 4.4139999999999997e-08, 1.1964101987900000e+00, 3.9557870223899999e+02 },
    { 4.4110000000000003e-08, 7.8391353364000005e-01, 1.4669036986300000e+01 },
    { 4.4069999999999997e-08, 6.0160419866000003e+00, 1.4746737883704000e+03 },
    { 4.4050000000000001e-08, 2.7863998404000001e+00, 5.2523754696970002e+02 },
    { 4.3480000000000003e-08, 3.0949841774300002e+00, 2.0312171985508590e+04 },
    { 4.3359999999999999e-08, 1.0888450678699999e+00, 5.3653405886000002e+01 },
    { 4.3229999999999997e-08, 3.4019631558199999e+00, 3.9654191008549998e+02 },
    { 4.3170000000000002e-08, 5.4045162973000005e-01, 3.4347920503050000e+02 },
    { 4.3040000000000000e-08, 5.3502367023300001e+00, 1.2567636172691100e+04 },
    { 4.2930000000000001e-08, 3.0879399286700000e+00, 4.6470422915999997e+01 },
    { 4.2680000000000002e-08, 1.0219015720900000e+00, 3.7390799283650000e+02 },
    { 4.2629999999999999e-08, 1.7161597714600001e+00, 1.1653797092750000e+02 },
    { 4.2559999999999999e-08, 5.8186140694899997e+00, 1.5777665625889999e+03 },
    { 4.2310000000000000e-08, 3.7883477352899999e+00, 3.1280121067900001e+01 },
    { 4.2240000000000000e-08, 3.8550593350200000e+00, 3.8427236954419999e+02 },
    { 4.2240000000000000e-08, 1.9630176767600001e+00, 1.7071264167530001e+02 },
    { 4.1960000000000000e-08, 1.2899428127300001e+00, 2.5596345212117401e+04 },
    { 4.1829999999999998e-08, 2.2807710962200001e+00, 5.6066226962179996e+02 },
    { 4.1820000000000000e-08, 1.8629042687199999e+00, 2.7727884259674001e+03 },
    { 4.1789999999999999e-08, 2.6881535287900000e+00, 6.6064432548323002e+03 },
    { 4.1719999999999999e-08, 3.6672788433900001e+00, 9.4349835056749998e+02 },
    { 4.1500000000000001e-08, 2.2924007224800000e+00, 2.7199209958499999e+01 },
    { 4.1369999999999999e-08, 1.0228305239800000e+00, 5.0286426215159997e+02 },
    { 4.1129999999999998e-08, 3.4452131951700000e+00, 1.0495600702027000e+03 },
    { 4.1070000000000003e-08, 1.7888822878999999e-01, 9.7892307321960004e+02 },
    { 4.1059999999999998e-08, 1.4432502545000001e+00, 2.2945260513260001e+02 },
    { 4.0949999999999999e-08, 4.9253110856199998e+00, 7.6562397778160005e+02 },
    { 4.0900000000000002e-08, 3.1115572153900000e+00, 5.2802072624099999e+01 },
    { 4.0770000000000000e-08, 3.5616907418300001e+00, 4.3319047301500001e+02 },
    { 4.0639999999999999e-08, 2.6040055731899998e+00, 6.1305437074401998e+03 },
    { 4.0620000000000002e-08, 1.2848036394500000e+00, 5.3117543780289998e+02 },
    { 4.0600000000000000e-08, 2.8620135883000000e+00, 2.7463549729639999e+02 },
    { 4.0600000000000000e-08, 4.4900792298400001e+00, 5.1626416456500003e+01 },
    { 4.0429999999999999e-08, 2.8682269523500001e+00, 2.4431858407499999e+02 },
    { 4.0359999999999999e-08, 2.5969566816100000e+00, 3.2324478422649997e+02 },
    { 4.0350000000000001e-08, 1.1556367103900000e+00, 1.0128633231634000e+03 },
    { 4.0250000000000000e-08, 3.8147441298799998e+00, 5.2174321810440088e+04 },
    { 4.0219999999999999e-08, 1.5001208870999999e-01, 1.4775945496771999e+03 },
    { 4.0210000000000001e-08, 1.7153309015800000e+00, 6.6034743094157002e+03 },
    { 4.0119999999999998e-08, 5.8003601547999994e-01, 5.2360129639400000e+01 },
    { 3.9750000000000002e-08, 3.4419597019400001e+00, 1.4557162401699999e+01 },
    { 3.9629999999999998e-08, 3.6592505754100002e+00, 6.3815218117480003e+02 },
    { 3.9620000000000000e-08, 1.4405569164300001e+00, 1.5615547927360001e+02 },
    { 3.9340000000000000e-08, 2.8118787393700000e+00, 1.6173840709351000e+03 },
    { 3.9179999999999997e-08, 5.7763847391100001e+00, 3.7567434983509997e+02 },
    { 3.9130000000000000e-08, 2.1991453062200002e+00, 3.4077089204480001e+02 },
    { 3.9039999999999998e-08, 1.5749952211100000e+00, 3.7004527554260000e+02 },
    { 3.9039999999999998e-08, 5.0189095932900001e+00, 5.5800214074589996e+02 },
    { 3.8959999999999999e-08, 5.6672552831700003e+00, 3.0923013854599998e+02 },
    { 3.8890000000000000e-08, 1.9271094333000001e+00, 3.1879136610159998e+02 },
    { 3.8780000000000001e-08, 2.7340081308099999e+00, 2.4846318565920001e+02 },
    { 3.8689999999999998e-08, 3.9853854961700002e+00, 5.5503319532930004e+02 },
    { 3.8579999999999999e-08, 4.3540255760099997e+00, 1.2564667227274500e+04 },
    { 3.8470000000000000e-08, 5.2769931170999995e-01, 2.2582926841020000e+02 },
    { 3.8390000000000002e-08, 4.0272725116299997e+00, 1.0060753403659801e+04 },
    { 3.8309999999999997e-08, 3.4647837293100001e+00, 3.5840887444769999e+02 },
    { 3.8290000000000001e-08, 4.4509657075999998e-01, 4.5220105354160000e+02 },
    { 3.8280000000000003e-08, 3.8515522139099998e+00, 2.1384770163205999e+03 },
    { 3.8280000000000003e-08, 2.8164258208900002e+00, 6.3570450061999998e+02 },
    { 3.8260000000000000e-08, 2.7146879455700001e+00, 1.5961864422845999e+03 },
    { 3.7919999999999998e-08, 5.4670317031300000e+00, 1.1169079392770000e+02 },
    { 3.7889999999999997e-08, 5.8643051025800004e+00, 3.1018797192920001e+03 },
    { 3.7840000000000000e-08, 3.1380342378299999e+00, 1.0257023126762999e+03 },
    { 3.7830000000000002e-08, 1.8401534715700001e+00, 1.6144151255185000e+03 },
    { 3.7679999999999997e-08, 2.7906088929299999e+00, 2.4400976753430001e+02 },
    { 3.7669999999999999e-08, 2.2635426685600000e+00, 7.4661339725500000e+02 },
    { 3.7620000000000002e-08, 3.8068694795100000e+00, 1.7219711438360000e+02 },
    { 3.7590000000000001e-08, 3.9016036092499999e+00, 6.1716456884946001e+03 },
    { 3.7529999999999999e-08, 5.8933401262300000e+00, 1.6508356738280000e+02 },
    { 3.7499999999999998e-08, 5.6251444560000001e+00, 3.6797010200329998e+02 },
    { 3.7469999999999997e-08, 1.0946693659400000e+00, 3.0360106425350000e+02 },
    { 3.7359999999999998e-08, 3.5598531040700001e+00, 1.4776427337870000e+03 },
    { 3.7259999999999997e-08, 5.2655862938300002e+00, 1.0962429414398000e+03 },
    { 3.7179999999999999e-08, 3.9445523266500002e+00, 1.5438263126452000e+03 },
    { 3.6790000000000001e-08, 3.1125617804500001e+00, 6.3518323575820000e+02 },
    { 3.6729999999999999e-08, 3.0882060706800001e+00, 5.7846191580000003e+01 },
    { 3.6650000000000001e-08, 3.5255106219700001e+00, 1.4904193227280001e+02 },
    { 3.6589999999999999e-08, 1.1906840350500001e+00, 3.4736317418380003e+02 },
    { 3.6400000000000002e-08, 3.7981353262900002e+00, 2.5650752071965999e+03 },
    { 3.6379999999999999e-08, 1.6535096840500001e+00, 3.5692440173940003e+02 },
    { 3.6300000000000001e-08, 1.9541973141300000e+00, 1.0252332524700000e+02 },
    { 3.6279999999999998e-08, 3.5419183870500000e+00, 2.4549424024260000e+02 },
    { 3.6230000000000001e-08, 7.2378614023999999e-01, 1.6653389665594000e+03 },
    { 3.5910000000000002e-08, 3.5298561036999998e+00, 6.9780716836880003e+02 },
    { 3.5889999999999999e-08, 1.2800893431200000e+00, 2.1395348674380000e+02 },
    { 3.5880000000000001e-08, 1.3347114075900000e+00, 5.3383556667879998e+02 },
    { 3.5859999999999998e-08, 4.8768942952099996e+00, 2.6048285633228090e+04 },
    { 3.5819999999999999e-08, 4.6988387021499998e+00, 3.0423420369989998e+02 },
    { 3.5630000000000002e-08, 6.6219107190000004e-01, 1.2451752593069399e+04 },
    { 3.5290000000000000e-08, 3.5206998917500001e+00, 6.0887779767699999e+02 },
    { 3.5229999999999998e-08, 4.7711970937499997e+00, 5.1960847267719998e+02 },
    { 3.4900000000000001e-08, 1.2800467490200000e+00, 6.6412755085590004e+02 },
    { 3.4739999999999999e-08, 3.4335423929100002e+00, 2.0884567731310000e+02 },
    { 3.4709999999999998e-08, 3.1677753529500001e+00, 1.1582819187138000e+03 },
    { 3.4650000000000003e-08, 2.6448647499999998e-02, 1.1511683717129999e+03 },
    { 3.4450000000000001e-08, 5.7992543307000000e-01, 3.8554439393140001e+02 },
    { 3.4440000000000003e-08, 1.9705004641499999e+00, 4.9989531673499999e+02 },
    { 3.4440000000000003e-08, 9.9359024004999996e-01, 8.9132941738980003e+02 },
    { 3.4289999999999998e-08, 3.1634489942799999e+00, 5.2980283967920002e+02 },
    { 3.4280000000000000e-08, 2.8121239926100001e+00, 2.3517761117586001e+03 },
    { 3.4189999999999997e-08, 9.3085492323999997e-01, 4.1677633088950000e+02 },
    { 3.4149999999999998e-08, 5.9564116426300000e+00, 3.2642617796420001e+02 },
    { 3.4119999999999997e-08, 5.9824789436600003e+00, 7.7258876693610000e+02 },
    { 3.4060000000000002e-08, 4.3190135299500003e+00, 5.2957909051000001e+02 },
    { 3.4020000000000003e-08, 5.3332268312300002e+00, 1.5861039498672001e+03 },
    { 3.3820000000000001e-08, 1.8771540130800000e+00, 6.6109491496450005e+02 },
    { 3.3750000000000001e-08, 4.9201591549000003e+00, 1.1201488830759999e+03 },
    { 3.3729999999999998e-08, 2.7134181716000000e-01, 4.0610313764109998e+02 },
    { 3.3670000000000003e-08, 1.2638842778399999e+00, 1.2747179660680000e+02 },
    { 3.3589999999999998e-08, 5.3299322085099998e+00, 1.0101855384714199e+04 },
    { 3.3570000000000002e-08, 3.9115174501399999e+00, 3.4613933390640000e+02 },
    { 3.3540000000000001e-08, 1.7083659229000001e-01, 6.4260559929969997e+02 },
    { 3.3440000000000000e-08, 2.0269311779999999e-01, 2.0351789493854700e+04 },
    { 3.3430000000000002e-08, 3.3673133907900001e+00, 2.6695250132400002e+02 },
    { 3.3400000000000001e-08, 3.8835943790499998e+00, 7.6981676347560006e+02 },
    { 3.3339999999999999e-08, 4.1721873675900003e+00, 1.6540326338646000e+03 },
    { 3.3309999999999999e-08, 2.3836067914600001e+00, 1.0134050529518800e+04 },
    { 3.3260000000000001e-08, 5.5223927311999998e-01, 4.4985950207700000e+01 },
    { 3.3220000000000002e-08, 2.5460686610800001e+00, 4.8592885516429999e+02 },
    { 3.3130000000000000e-08, 3.2190307557100000e+00, 9.3935374898329997e+02 },
    { 3.3130000000000000e-08, 3.0572788293399999e+00, 2.8490544972429998e+03 },
    { 3.3039999999999997e-08, 5.4554106885599998e+00, 5.0015594916589998e+02 },
    { 3.2969999999999997e-08, 1.7255503524800000e+00, 1.4040849754971000e+03 },
    { 3.2910000000000002e-08, 2.8288818818200001e+00, 2.9764192156090002e+02 },
    { 3.2780000000000000e-08, 5.2602014808500002e+00, 9.4201387785919997e+02 },
    { 3.2759999999999997e-08, 3.5274164187100001e+00, 1.3066212755710001e+03 },
    { 3.2719999999999998e-08, 3.2896945552900001e+00, 3.8458118608490003e+02 },
    { 3.2649999999999998e-08, 6.2618922354500004e+00, 2.4490896972800002e+01 },
    { 3.2609999999999999e-08, 2.5569643326399998e+00, 1.5071777497157000e+03 },
    { 3.2590000000000003e-08, 9.1021107092999998e-01, 6.5812596954790001e+02 },
    { 3.2570000000000000e-08, 6.2371718273600001e+00, 3.0571867640820000e+02 },
    { 3.2570000000000000e-08, 1.2340668985600001e+00, 6.6396027179300006e+01 },
    { 3.2520000000000003e-08, 1.9289189269800000e+00, 7.0904981058880003e+02 },
    { 3.2409999999999997e-08, 5.0090992934500003e+00, 9.7866244078870000e+02 },
    { 3.2360000000000000e-08, 3.4802204672100001e+00, 2.0037478847900001e+01 },
    { 3.2310000000000003e-08, 6.1844639313799998e+00, 4.0106476417959999e+02 },
    { 3.2280000000000002e-08, 1.0260116995699999e+00, 6.2038408332992003e+03 },
    { 3.2220000000000000e-08, 4.4969678784299996e+00, 1.3411226285559999e+02 },
    { 3.2140000000000002e-08, 5.5131415163100002e+00, 2.0348820548438089e+04 },
    { 3.2089999999999998e-08, 6.1974204215000004e+00, 1.4380252254408999e+03 },
    { 3.2019999999999998e-08, 3.2943878175000001e+00, 8.9908209940600003e+01 },
    { 3.1989999999999997e-08, 3.6904197519799999e+00, 8.5952803146010001e+02 },
    { 3.1960000000000003e-08, 4.2720836919999998e+00, 3.6286229257259998e+02 },
    { 3.1760000000000001e-08, 4.8680879701900004e+00, 3.0115338369870000e+02 },
    { 3.1730000000000000e-08, 7.5400916255999995e-01, 1.4011160300805000e+03 },
    { 3.1720000000000002e-08, 2.6567702843599998e+00, 7.8404935289700006e+01 },
    { 3.1699999999999999e-08, 2.1076142611500002e+00, 1.0113788504551000e+03 },
    { 3.1590000000000000e-08, 5.4609579721900001e+00, 1.0335340664950000e+02 },
    { 3.1590000000000000e-08, 5.7154060951599996e+00, 1.9779997704920001e+02 },
    { 3.1580000000000002e-08, 5.8939276235799998e+00, 5.3977345751200005e+02 },
    { 3.1529999999999998e-08, 4.3618074033300003e+00, 5.1775174302800004e+01 },
    { 3.1440000000000002e-08, 1.5906037350800000e+00, 1.5042088042990999e+03 },
    { 3.1270000000000001e-08, 1.4743384657600000e+00, 2.2530800354839999e+02 },
    { 3.1179999999999999e-08, 5.9846142351999996e+00, 6.9483822295220000e+02 },
    { 3.1170000000000001e-08, 5.3071401302399996e+00, 1.4350562800242999e+03 },
    { 3.1140000000000000e-08, 4.8888256375000000e-01, 5.3776771994190005e+02 },
    { 3.1090000000000002e-08, 2.2209445810899999e+00, 5.1838463239980001e+02 },
    { 3.1070000000000000e-08, 3.8116098042800002e+00, 2.8683682666799999e+01 },
    { 3.1070000000000000e-08, 4.5814805968999996e+00, 2.1744369702220001e+02 },
    { 3.0920000000000002e-08, 3.8298355063499998e+00, 8.1506334611420004e+02 },
    { 3.0909999999999997e-08, 3.1166661878000002e-01, 3.7901580226720000e+02 },
    { 3.0839999999999997e-08, 1.3744923949200001e+00, 3.4051025961390002e+02 },
    { 3.0810000000000003e-08, 4.5013417661200004e+00, 6.2567019231239999e+02 },
    { 3.0670000000000003e-08, 2.8705866109199998e+00, 4.5884151979040001e+02 },
    { 3.0640000000000002e-08, 1.3823970452000001e-01, 3.1787634236489998e+02 },
    { 3.0570000000000002e-08, 6.0582711895500001e+00, 5.2690198039499997e+01 },
    { 3.0570000000000002e-08, 4.5655013839700000e+00, 2.3575873236100001e+01 },
    { 3.0500000000000002e-08, 3.3052353820399998e+00, 1.2303552042954000e+03 },
    { 3.0489999999999997e-08, 2.6961855318399999e+00, 2.4161027108930000e+02 },
    { 3.0489999999999997e-08, 2.1322593892000001e+00, 1.5128068240082000e+03 },
    { 3.0460000000000003e-08, 2.4286996690299998e+00, 1.5494553869377000e+03 },
    { 3.0440000000000000e-08, 7.8066290549999995e-01, 3.6945457471160000e+02 },
    { 3.0330000000000001e-08, 4.4138745586799999e+00, 3.3061896365820002e+02 },
    { 3.0309999999999998e-08, 4.3220477556399999e+00, 1.5645115277816958e+05 },
    { 3.0260000000000001e-08, 8.3985577440000000e-02, 4.7346237266690002e+02 },
    { 3.0260000000000001e-08, 4.6610227484199998e+00, 7.8474643289280004e+02 },
    { 3.0050000000000001e-08, 4.7221264556699998e+00, 3.5304043258610000e+02 },
    { 3.0040000000000003e-08, 4.5944116503499997e+00, 1.9295831548977001e+03 },
    { 2.9989999999999999e-08, 3.6396424757100001e+00, 1.9266142094811000e+03 },
    { 2.9980000000000001e-08, 5.5977530738299999e+00, 4.7733083545519997e+02 },
    { 2.9950000000000000e-08, 5.1281856875300003e+00, 1.3995378546920000e+02 },
    { 2.9919999999999999e-08, 4.1528954575399997e+00, 5.4174670747800000e+01 },
    { 2.9799999999999999e-08, 5.2525223627899997e+00, 4.6473122651379998e+02 },
    { 2.9749999999999998e-08, 7.1741646582999996e-01, 9.5920991727740000e+02 },
    { 2.9749999999999998e-08, 3.0247624029200000e+00, 1.0143477958717000e+03 },
    { 2.9720000000000001e-08, 5.0146914565199996e+00, 5.6904784100979998e+02 },
    { 2.9679999999999998e-08, 2.8747446782399999e+00, 2.8511174858869998e+02 },
    { 2.9670000000000000e-08, 4.7261966677100000e+00, 6.1503391543000001e+00 },
    { 2.9639999999999999e-08, 3.3420429523899999e+00, 1.1239336934329999e+02 },
    { 2.9630000000000001e-08, 4.9402512547199997e+00, 2.9912639426919998e+02 },
    { 2.9609999999999998e-08, 3.7722063639900001e+00, 2.4619824329779999e+03 },
    { 2.9519999999999999e-08, 4.6978208030599999e+00, 1.8647239249500001e+02 },
    { 2.9510000000000001e-08, 4.0099924439599999e+00, 3.3727801622699999e+01 },
    { 2.9510000000000001e-08, 3.4721874759700002e+00, 4.2538269652899999e+01 },
    { 2.9420000000000001e-08, 2.4725743965300002e+00, 7.8225576389084788e+04 },
    { 2.9329999999999999e-08, 5.6591075791799996e+00, 3.1490739694830000e+02 },
    { 2.9230000000000001e-08, 1.5112295245200000e+00, 2.6610116806209999e+02 },
    { 2.9180000000000001e-08, 6.8789446711000002e-01, 8.3379618276399995e+01 },
    { 2.9060000000000000e-08, 1.9559195427300000e+00, 1.7224529849340001e+02 },
    { 2.9000000000000002e-08, 3.8888952963799999e+00, 7.7110429422779998e+02 },
    { 2.8979999999999999e-08, 5.6673428196100000e+00, 3.9409422953069998e+02 },
    { 2.8810000000000001e-08, 2.6922661424499998e+00, 2.3798693436689999e+02 },
    { 2.8780000000000000e-08, 1.1806907404900000e+00, 4.5071658083329999e+02 },
    { 2.8690000000000001e-08, 6.2234532555600000e+00, 9.0833426034629997e+02 },
    { 2.8590000000000000e-08, 3.5126194232699999e+00, 6.6523951654869995e+02 },
    { 2.8559999999999999e-08, 1.4887866935500000e+00, 6.2291369521690001e+02 },
    { 2.8539999999999999e-08, 2.5742293019400000e+00, 5.9194239068970001e+02 },
    { 2.8220000000000001e-08, 6.7588438863000000e-01, 9.5358084298489996e+02 },
    { 2.8209999999999999e-08, 4.0978512824599997e+00, 4.0269224923979999e+02 },
    { 2.8190000000000000e-08, 2.2706189893800000e+00, 1.7582045110600001e+02 },
    { 2.8180000000000002e-08, 2.0481857648899999e+00, 4.9066861317099999e+01 },
    { 2.8159999999999999e-08, 1.6965125048500000e+00, 2.1037833413120001e+02 },
    { 2.8100000000000000e-08, 3.5772487559499999e+00, 1.5920418407003999e+03 },
    { 2.8089999999999999e-08, 9.5837285337000000e-01, 3.1782815825509999e+02 },
    { 2.8060000000000001e-08, 3.9757419004000001e+00, 2.3905941636200001e+01 },
    { 2.7879999999999999e-08, 3.4873307438000001e-01, 1.9131956949479999e+02 },
    { 2.7789999999999999e-08, 3.2294289277899999e+00, 2.3864132567269999e+02 },
    { 2.7689999999999998e-08, 2.9298666931500001e+00, 2.8808069400530002e+02 },
    { 2.7630000000000000e-08, 2.6992068136000003e-01, 4.1083844005629999e+02 },
    { 2.7630000000000000e-08, 3.8724905795400000e+00, 4.6053844081979997e+02 },
    { 2.7459999999999999e-08, 5.1455748356699997e+00, 3.5330106501699998e+02 },
    { 2.7420000000000000e-08, 4.2690205846699998e+00, 5.7798007470199998e+01 },
    { 2.7400000000000001e-08, 3.2010521986199998e+00, 1.2059761819072000e+03 },
    { 2.7330000000000001e-08, 1.9363595648300000e+00, 2.6247783133970000e+02 },
    { 2.7249999999999999e-08, 5.1731305903899996e+00, 7.4260333705500003e+01 },
    { 2.7240000000000001e-08, 5.6491069263700000e+00, 2.2942733789700000e+01 },
    { 2.7220000000000002e-08, 5.9414923439000000e-01, 4.9439368332599997e+01 },
    { 2.7190000000000001e-08, 1.7835896761800001e+00, 1.4000196957899999e+02 },
    { 2.7179999999999999e-08, 3.5603000290600000e+00, 1.7782618867610000e+02 },
    { 2.7100000000000001e-08, 3.1536443047799998e+00, 5.6485505531579997e+02 },
    { 2.7030000000000001e-08, 9.8862354259999996e-01, 1.1312708252620000e+02 },
    { 2.7000000000000000e-08, 2.8838928023300001e+00, 9.6340270297140000e+02 },
    { 2.6989999999999999e-08, 4.2864714875600001e+00, 1.6242369855049999e+03 },
    { 2.6980000000000001e-08, 3.2644949325699999e+00, 7.8149310317809184e+04 },
    { 2.6910000000000001e-08, 2.0100639843999999e+00, 6.2845603226997000e+03 },
    { 2.6820000000000001e-08, 2.3495979297999998e+00, 4.2271422172270002e+02 },
    { 2.6810000000000000e-08, 5.8304860009099997e+00, 2.4948471359799999e+01 },
    { 2.6770000000000001e-08, 4.9764661344399999e+00, 5.2846712481719999e+02 },
    { 2.6759999999999999e-08, 2.3734342326999999e-01, 2.5950888592310002e+02 },
    { 2.6689999999999999e-08, 3.0294052140200001e+00, 5.3169670266469996e+02 },
    { 2.6659999999999999e-08, 5.1446058525299998e+00, 5.7345307502490004e+02 },
    { 2.6650000000000000e-08, 2.1998122217099998e+00, 5.5406998748280000e+02 },
    { 2.6600000000000000e-08, 4.2523596916499997e+00, 3.0534616939270001e+02 },
    { 2.6560000000000001e-08, 4.2514735153299998e+00, 4.0966278388870001e+02 },
    { 2.6550000000000000e-08, 4.2916694692500004e+00, 4.8439619834619998e+02 },
    { 2.6440000000000001e-08, 5.7320381374400000e+00, 5.0835032409220003e+02 },
    { 2.6400000000000001e-08, 5.5973237470799999e+00, 2.1246901403550001e+02 },
    { 2.6300000000000001e-08, 5.8393442374999998e+00, 2.7515475996916002e+03 },
    { 2.6289999999999999e-08, 1.4539859978500000e+00, 4.1109907248719998e+02 },
    { 2.6280000000000001e-08, 2.8564938822000001e+00, 4.5936278465219999e+02 },
    { 2.6250000000000000e-08, 7.3055263024999995e-01, 5.2161421024729998e+02 },
    { 2.6180000000000000e-08, 4.9377054391400002e+00, 6.6812248533995999e+03 },
    { 2.6169999999999999e-08, 2.5048368859100001e+00, 2.1334727954780001e+02 },
    { 2.6130000000000000e-08, 3.7083949948699999e+00, 5.5774150831500003e+02 },
    { 2.6090000000000001e-08, 5.5233036069099999e+00, 2.1325091132820000e+02 },
    { 2.5889999999999999e-08, 3.3339177537300002e+00, 1.1907858800591000e+03 },
    { 2.5769999999999998e-08, 3.3941414762200002e+00, 1.4478470854274001e+03 },
    { 2.5710000000000000e-08, 5.7680147381799998e+00, 6.0027977796790003e+02 },
    { 2.5520000000000000e-08, 4.7267817130400003e+00, 1.8376407950929999e+02 },
    { 2.5390000000000001e-08, 5.7528821223300000e+00, 4.2537435059860002e+02 },
    { 2.5329999999999999e-08, 2.1611525907000000e+00, 8.0083625211260005e+02 },
    { 2.5310000000000000e-08, 1.2486557163900001e+00, 9.0935108261899998e+01 },
    { 2.5310000000000000e-08, 5.2350255719899996e+00, 2.0373786788240000e+02 },
    { 2.5310000000000000e-08, 5.1407275146400000e+00, 4.9549008271989999e+02 },
    { 2.5080000000000000e-08, 3.0415893383500001e+00, 2.6099335863139999e+02 },
    { 2.5040000000000001e-08, 1.4265813550899999e+00, 4.6447059408289999e+02 },
    { 2.4940000000000000e-08, 2.3643168866000002e+00, 1.1878169346425000e+03 },
    { 2.4760000000000001e-08, 1.2416860493999999e+00, 5.3213864564940002e+02 },
    { 2.4710000000000001e-08, 2.8008836292399999e+00, 4.1135970491810002e+02 },
    { 2.4710000000000001e-08, 4.2420427127300000e+00, 4.0721510333390000e+02 },
    { 2.4599999999999999e-08, 3.6613945912500001e+00, 1.2528018664345000e+04 },
    { 2.4509999999999999e-08, 2.0586387819000000e+00, 4.8762577619370001e+02 },
    { 2.4430000000000001e-08, 2.8261263496800000e+00, 1.6257214582132999e+03 },
    { 2.4249999999999999e-08, 1.3398071663100000e+00, 1.3376407642080001e+03 },
    { 2.4140000000000000e-08, 4.4355915399399999e+00, 5.2768522752449996e+02 },
    { 2.4089999999999999e-08, 5.0518259873500000e+00, 1.4266209845489999e+02 },
    { 2.3980000000000000e-08, 5.7045981728299999e+00, 1.5524243323543001e+03 },
    { 2.3980000000000000e-08, 1.3653764120000000e-02, 1.2566151699982800e+04 },
    { 2.3809999999999999e-08, 8.0137306059999996e-02, 1.4117762574660000e+02 },
    { 2.3800000000000001e-08, 1.0135620353499999e+00, 9.8311585891360005e+02 },
    { 2.3800000000000001e-08, 2.4376620224600001e+00, 4.2951895218279998e+02 },
    { 2.3779999999999999e-08, 2.8996522033100001e+00, 9.5209637027660006e+02 },
    { 2.3730000000000001e-08, 2.5239158359000002e-01, 1.3109513332920000e+02 },
    { 2.3730000000000001e-08, 1.8129788223600001e+00, 1.3284023846840000e+02 },
    { 2.3689999999999999e-08, 5.9009157206999996e+00, 4.1922401144430000e+02 },
    { 2.3610000000000001e-08, 6.3552853686999999e-01, 9.0536531492970005e+02 },
    { 2.3590000000000001e-08, 7.7313093683000000e-01, 6.7383753625780002e+02 },
    { 2.3450000000000002e-08, 4.3775577068400002e+00, 4.5417565334440002e+02 },
    { 2.3440000000000000e-08, 3.9018577215099999e+00, 1.3257960603750001e+02 },
    { 2.3409999999999999e-08, 2.2301147452699999e+00, 3.4173409989129999e+02 },
    { 2.3400000000000001e-08, 7.0752556953000001e-01, 4.5267413429359999e+02 },
    { 2.3390000000000000e-08, 3.2951426416000000e+00, 4.4068227252570000e+02 },
    { 2.3320000000000000e-08, 5.9367898439199998e+00, 7.0636996983100005e+01 },
    { 2.3190000000000001e-08, 2.8189163617699999e+00, 2.6011637070298599e+04 },
    { 2.3059999999999999e-08, 4.0405225850899997e+00, 6.2464272870618997e+03 },
    { 2.3029999999999998e-08, 2.3007006611300000e+00, 1.3715810141518000e+03 },
    { 2.2950000000000000e-08, 6.3099781113999998e-01, 4.9718700374930000e+02 },
    { 2.2900000000000000e-08, 4.9820072577500003e+00, 4.5543063138909997e+02 },
    { 2.2840000000000001e-08, 1.5360967595499999e+00, 2.5060204967330000e+02 },
    { 2.2560000000000002e-08, 3.5111894514699999e+00, 9.8163138620530003e+02 },
    { 2.2550000000000000e-08, 4.8973422745399997e+00, 6.2147740661839998e+02 },
    { 2.2539999999999999e-08, 3.4170200542400000e+00, 2.2894549679899999e+01 },
    { 2.2500000000000000e-08, 3.7232537056999998e-01, 7.0588812873300000e+01 },
    { 2.2460000000000001e-08, 2.0143288038199998e+00, 3.8161224066829999e+02 },
    { 2.2350000000000002e-08, 1.6062911793600001e+00, 1.8983509678650000e+02 },
    { 2.2320000000000001e-08, 3.9924710932399998e+00, 1.5837366516480000e+02 },
    { 2.2309999999999999e-08, 3.3710629856500001e+00, 8.7710232338820003e+02 },
    { 2.2259999999999999e-08, 5.2897697548000000e-01, 4.7582388608800002e+01 },
    { 2.2040000000000001e-08, 4.7754482332399997e+00, 4.4312995308050000e+02 },
    { 2.2010000000000000e-08, 1.4502476271300000e+00, 1.1893014073508000e+03 },
    { 2.1850000000000000e-08, 5.4259729386600002e+00, 4.9867147645760002e+02 },
    { 2.1780000000000000e-08, 2.9845151750999999e+00, 6.2434583416452997e+03 },
    { 2.1730000000000000e-08, 5.4592353362899999e+00, 1.0176636983281500e+04 },
};

static const VSOP87Term VSOP87A_NEPTUNE_Y1[325] = {
    { 3.5294749300000001e-03, 3.1415926535900001e+00, 0.0000000000000000e+00 },
    { 2.5612549300000000e-03, 4.4757496816999998e-01, 3.6648562929500002e+01 },
    { 2.4314772499999999e-03, 3.8909979869600000e+00, 3.9617508346100003e+01 },
    { 1.1842720499999999e-03, 1.3112802703699999e+00, 7.6266071275599998e+01 },
    { 2.1936701999999999e-04, 1.6312408759100001e+00, 3.5164090221199999e+01 },
    { 1.7462331999999999e-04, 2.6922990296600000e+00, 4.1101981054399999e+01 },
    { 1.2992380000000000e-04, 3.7957863300199999e+00, 2.9689454166000000e+00 },
    { 4.9451170000000003e-05, 5.1727080684000004e-01, 7.3297125859000005e+01 },
    { 2.7459210000000000e-05, 2.4917831108200001e+00, 7.7750543983900002e+01 },
    { 2.1454810000000001e-05, 2.5476844729099999e+00, 4.4534181248999998e+00 },
    { 1.5722890000000001e-05, 5.7885335071100004e+00, 1.1439910691340000e+02 },
    { 1.5657250000000000e-05, 2.8984626627200001e+00, 3.3679617512900002e+01 },
    { 1.4582689999999999e-05, 1.6183554269900000e+00, 3.8133035637799999e+01 },
    { 1.3437310000000000e-05, 1.4671262210900000e+00, 4.2586453762700003e+01 },
    { 1.2934590000000001e-05, 4.4586806108200001e+00, 7.4781598567299994e+01 },
    { 9.2029300000000000e-06, 1.7750776890100000e+00, 7.1812653150700001e+01 },
    { 8.9849999999999995e-06, 2.6984015976900002e+00, 4.2659819087599999e+02 },
    { 8.6598699999999995e-06, 9.5563148850000001e-02, 3.7611770776000000e+01 },
    { 8.4996299999999995e-06, 4.2451989335900002e+00, 3.8654300499599998e+01 },
    { 7.8009899999999997e-06, 4.2719264019800001e+00, 2.0618554843720000e+02 },
    { 7.5474400000000002e-06, 3.7613802590500001e+00, 2.2041264243879999e+02 },
    { 6.0697499999999998e-06, 4.8193234151000004e+00, 1.0593819301891999e+03 },
    { 5.7228399999999999e-06, 8.5667831684999995e-01, 5.2257741809380002e+02 },
    { 5.5923599999999998e-06, 3.4587880894000000e-01, 5.3680451209540001e+02 },
    { 5.3597300000000001e-06, 2.9305379702000001e+00, 1.4844727083000000e+00 },
    { 5.0099799999999998e-06, 1.4254169132000000e-01, 2.8571808082200000e+01 },
    { 4.7186799999999997e-06, 5.9689350528900000e+00, 9.8899988524600005e+01 },
    { 4.5392999999999999e-06, 1.4360437422000000e-01, 3.5685355082999997e+01 },
    { 4.1005699999999998e-06, 4.1950026995999998e+00, 4.0580716192600001e+01 },
    { 3.6689900000000001e-06, 4.1967579906600001e+00, 4.7694263193399998e+01 },
    { 2.9921400000000000e-06, 1.3149758733700001e+00, 5.9378908332000000e+00 },
    { 1.7046599999999999e-06, 2.8925285867700001e+00, 3.3940249943799998e+01 },
    { 1.6283999999999999e-06, 5.5194041931599997e+00, 3.1019488636999998e+01 },
    { 1.5661800000000000e-06, 1.0237422492999999e+00, 7.9235016692200006e+01 },
    { 1.5255300000000000e-06, 5.2947847530800001e+00, 3.0056280790500001e+01 },
    { 1.5077599999999999e-06, 1.4687453327600000e+00, 4.2325821331800000e+01 },
    { 1.3465800000000000e-06, 5.1038627683900000e+00, 4.5246582638600003e+01 },
    { 1.3037900000000001e-06, 3.2030170311899999e+00, 8.0767548473000002e+00 },
    { 1.2102399999999999e-06, 5.3086064735400003e+00, 4.6209790485100001e+01 },
    { 1.1593000000000001e-06, 2.4759979383000000e-01, 4.4070926471000000e+01 },
    { 1.1029300000000000e-06, 4.6948150664300003e+00, 3.5212274331000003e+01 },
    { 1.0836500000000000e-06, 1.5228978665499999e+00, 1.1291463420510000e+02 },
    { 9.9369999999999990e-07, 5.9317520297200002e+00, 4.1053796944600002e+01 },
    { 9.0584999999999998e-07, 4.2339627374099997e+00, 3.2195144804599998e+01 },
    { 8.9894000000000003e-07, 5.8823420340799997e+00, 6.0766952886799999e+01 },
    { 8.7642999999999997e-07, 3.2327288035800001e+00, 4.1948464387519999e+02 },
    { 7.2648999999999999e-07, 1.9354137774800000e+00, 6.3989728631399998e+02 },
    { 6.6255000000000001e-07, 3.2452226121900001e+00, 7.0328180442399997e+01 },
    { 5.3547000000000002e-07, 2.8945018292600002e+00, 4.3371173787679999e+02 },
    { 5.1313999999999995e-07, 2.4437036388100002e+00, 6.9152524274800001e+01 },
    { 4.8991000000000002e-07, 1.5623942272400000e+00, 2.9207613067999998e+00 },
    { 4.6502999999999998e-07, 4.4200524375999999e+00, 1.1143016149680000e+02 },
    { 4.4988000000000000e-07, 5.2820187112500001e+00, 2.1329909543799999e+02 },
    { 4.4005999999999998e-07, 4.0580937023799999e+00, 1.0994568878849999e+02 },
    { 4.3855999999999998e-07, 5.1618807321800002e+00, 1.5890728952838001e+03 },
    { 4.2570000000000001e-07, 1.0123710589000000e-01, 7.4223635415000002e+00 },
    { 4.2478000000000002e-07, 1.8700183295999999e+00, 5.2969096509459996e+02 },
    { 4.1772000000000001e-07, 2.6766494024999998e+00, 4.1927856940000003e+00 },
    { 3.9579999999999998e-07, 5.6615377082400000e+00, 1.2530172972200001e+01 },
    { 3.9462000000000002e-07, 4.2697150303900004e+00, 3.1540753498800001e+01 },
    { 3.8034999999999999e-07, 4.3095922354300003e+00, 6.5922821389999999e+00 },
    { 3.7948000000000001e-07, 4.4168443834400000e+00, 1.4014645680499999e+01 },
    { 3.7352000000000003e-07, 1.5907885121300001e+00, 2.7406604832480002e+02 },
    { 3.5716000000000000e-07, 1.2802118012899999e+00, 8.4342826122899993e+01 },
    { 3.4040000000000000e-07, 4.4658983548900002e+00, 1.4956319713459999e+02 },
    { 3.3108000000000002e-07, 8.8651710157999997e-01, 1.1588357962169999e+02 },
    { 3.3038999999999998e-07, 1.3695771832300001e+00, 2.7087335373900000e+01 },
    { 3.1100999999999998e-07, 1.6833819871000000e-01, 1.0522683831884001e+03 },
    { 3.0599000000000000e-07, 5.5928138974900001e+00, 2.2633917249000000e+01 },
    { 2.9880000000000000e-07, 7.6687803071000005e-01, 1.0664954771900000e+03 },
    { 2.8690000000000001e-07, 6.1334153024000004e+00, 1.0697674337190000e+02 },
    { 2.3139000000000001e-07, 5.9084546036000001e+00, 1.0601353552540000e+02 },
    { 2.2861999999999999e-07, 6.2725630263100003e+00, 4.4725317776799997e+01 },
    { 2.2438000000000000e-07, 4.9620671223099997e+00, 6.3735898303399999e+01 },
    { 2.2382000000000000e-07, 2.7428866192500001e+00, 1.7516605980020000e+02 },
    { 2.1675000000000001e-07, 2.2172675915800002e+00, 3.1639186965660002e+02 },
    { 2.1638000000000001e-07, 5.6560113208500002e+00, 5.4166259714000002e+00 },
    { 2.0370999999999999e-07, 6.0674799651800004e+00, 6.2251425595100002e+01 },
    { 1.9878000000000001e-07, 5.7304495188800004e+00, 3.2716409666399997e+01 },
    { 1.9754000000000001e-07, 2.9192040576100000e+00, 4.9178735901700001e+01 },
    { 1.9579000000000001e-07, 4.0847957985700001e+00, 2.4118389957300000e+01 },
    { 1.9347000000000000e-07, 4.0642021634700001e+00, 1.5253214255120000e+02 },
    { 1.9233000000000000e-07, 4.4432344774799999e+00, 6.9010986796999996e+00 },
    { 1.8371000000000000e-07, 3.6302197398299998e+00, 2.2752618943959999e+02 },
    { 1.7244000000000000e-07, 4.8954891527899997e+00, 1.1306332694800000e+01 },
    { 1.7013999999999999e-07, 2.6727524345500000e+00, 9.1786441523799994e+01 },
    { 1.6590000000000000e-07, 1.1975797699199999e+00, 3.3467169191799996e+01 },
    { 1.6537999999999999e-07, 4.7175247258699997e+00, 3.0508553696180002e+02 },
    { 1.6400000000000001e-07, 3.4534665937600000e+00, 3.3025226207099998e+01 },
    { 1.6105999999999999e-07, 5.5563734970300001e+00, 1.1020632121940000e+02 },
    { 1.5990000000000000e-07, 5.8639457283600001e+00, 7.9186832582400001e+01 },
    { 1.5590000000000001e-07, 3.9187741235400000e+00, 3.9405060024999997e+01 },
    { 1.5590000000000001e-07, 4.2181314280999999e-01, 3.6861011250600001e+01 },
    { 1.5468999999999999e-07, 4.9117034586099999e+00, 4.3549661609200001e+01 },
    { 1.4553999999999999e-07, 4.2529539968299996e+00, 3.1931263096340001e+02 },
    { 1.4467000000000001e-07, 5.1503925121299998e+00, 1.0846121608020000e+02 },
    { 1.3293000000000001e-07, 3.1443206863900000e+00, 4.2798902083800002e+01 },
    { 1.3036000000000001e-07, 8.2144801893999997e-01, 1.1045700263900001e+01 },
    { 1.2917000000000001e-07, 4.0962148706099999e+00, 9.7415515816300001e+01 },
    { 1.2809999999999999e-07, 1.0258629984500001e+00, 5.1546387109299997e+02 },
    { 1.2751000000000001e-07, 5.3266371006800002e+00, 4.5555399179299997e+01 },
    { 1.2060999999999999e-07, 1.6342714905400000e+00, 7.2073285581600004e+01 },
    { 1.1499000000000000e-07, 3.0521779555199999e+00, 1.2991947716160001e+02 },
    { 1.1456000000000000e-07, 4.7288786901300002e+00, 6.3278373931320004e+02 },
    { 1.1438000000000000e-07, 1.5388812960000000e-02, 1.4362530630140000e+02 },
    { 1.1107000000000000e-07, 7.5452281255999998e-01, 4.3240845068500001e+01 },
    { 1.1036000000000000e-07, 5.3968816369699999e+00, 5.1078094306999997e+00 },
    { 1.0719000000000000e-07, 3.0702244540299999e+00, 4.1529185818119998e+02 },
    { 1.0565999999999999e-07, 4.9925558727399997e+00, 7.0849445304200003e+01 },
    { 1.0082000000000000e-07, 4.0234298928800003e+00, 2.5143213107579999e+02 },
    { 9.6009999999999997e-08, 5.6819124146100002e+00, 8.0719489400499995e+01 },
    { 9.0569999999999996e-08, 2.5151309648700000e+00, 8.2858353414600003e+01 },
    { 9.0330000000000002e-08, 4.4605490632100002e+00, 1.9907200143640000e+02 },
    { 8.9589999999999997e-08, 3.9208232472000000e+00, 1.4659425171800001e+02 },
    { 8.9459999999999995e-08, 6.1401023565999999e+00, 6.9364972595899999e+01 },
    { 8.6910000000000002e-08, 2.2905126560200002e+00, 3.2503961345299999e+01 },
    { 8.2700000000000006e-08, 1.1381083474700000e+00, 1.0549227066360000e+02 },
    { 8.1089999999999994e-08, 1.7064038699799999e+00, 5.2650957135689998e+02 },
    { 8.0890000000000005e-08, 5.7753300594999999e+00, 5.3287235883230005e+02 },
    { 7.9980000000000005e-08, 5.9300556113500003e+00, 6.2449428143535997e+03 },
    { 7.9800000000000000e-08, 4.5275821993600003e+00, 9.5612275556000004e+00 },
    { 7.9420000000000006e-08, 2.4228394666000000e-01, 5.4391805909619995e+02 },
    { 7.8260000000000001e-08, 2.4532140540600000e+00, 7.6787336137400004e+01 },
    { 7.7140000000000001e-08, 3.5105746287300001e+00, 6.5220371011699996e+01 },
    { 7.7039999999999994e-08, 4.3484563877599998e+00, 1.0309277421860000e+02 },
    { 7.6980000000000005e-08, 4.8911604616300002e+00, 1.4227094001599999e+01 },
    { 7.5639999999999994e-08, 1.0657172047300001e+00, 1.0175152510573200e+04 },
    { 7.3360000000000003e-08, 3.4340965847599998e+00, 1.5499118388799999e+01 },
    { 7.2149999999999994e-08, 2.5944111916499999e+00, 7.1135470007999997e+00 },
    { 7.1700000000000000e-08, 4.6020436008000001e+00, 9.4917560896980001e+02 },
    { 7.1340000000000002e-08, 2.0501027525200000e+00, 4.3762109930299999e+01 },
    { 7.0599999999999997e-08, 5.5884640483799997e+00, 6.4959738580800007e+01 },
    { 6.7290000000000002e-08, 2.5682705388299998e+00, 1.4414657116320001e+02 },
    { 6.5729999999999994e-08, 6.2800622438999998e+00, 2.5602862665600000e+01 },
    { 6.4280000000000004e-08, 3.4762143598600002e+00, 6.7803032195180003e+02 },
    { 6.3409999999999997e-08, 5.2491205162699996e+00, 6.6704843720000000e+01 },
    { 6.2390000000000005e-08, 4.2098723062000003e+00, 1.7526107818300002e+01 },
    { 5.9389999999999998e-08, 4.7421800105500003e+00, 7.5744806413800006e+01 },
    { 5.9100000000000000e-08, 1.4056490603400000e+00, 6.1686767430780001e+03 },
    { 5.5649999999999999e-08, 1.2692687928699999e+00, 8.5319638175199998e+02 },
    { 5.5619999999999998e-08, 2.8348045980099998e+00, 1.0098886439297599e+04 },
    { 5.5229999999999999e-08, 1.6906422296000001e-01, 1.8027386923090000e+02 },
    { 5.4830000000000002e-08, 2.7281102242899999e+00, 4.2065188900899997e+01 },
    { 5.4830000000000002e-08, 1.6124770420500001e+00, 3.4200882374700001e+01 },
    { 5.4410000000000002e-08, 3.4192861588199999e+00, 4.4923210812500002e+02 },
    { 5.3900000000000000e-08, 3.5463623485500002e+00, 8.1373880706300000e+01 },
    { 5.3249999999999997e-08, 2.8268029452999999e+00, 1.9010580526599998e+01 },
    { 5.3209999999999998e-08, 2.0994318310000000e+00, 1.1046285128278000e+03 },
    { 5.3200000000000000e-08, 3.3631343606000002e+00, 1.4097140497896000e+03 },
    { 5.2980000000000002e-08, 3.5143574399799999e+00, 2.0936694217490000e+02 },
    { 5.2500000000000000e-08, 1.6559323231000000e-01, 9.6320784650000002e-01 },
    { 5.2220000000000000e-08, 4.9301662636500003e+00, 3.9096243484299997e+01 },
    { 5.2070000000000002e-08, 2.0647681665399999e+00, 3.1368355667089997e+02 },
    { 5.1819999999999997e-08, 3.1616808303999999e+00, 5.2549817940059995e+02 },
    { 5.1280000000000000e-08, 4.5373558056999999e+00, 2.1723124870109999e+02 },
    { 5.1270000000000002e-08, 1.6402883831999999e+00, 1.4117762574660000e+02 },
    { 5.0729999999999998e-08, 5.1528404128599998e+00, 1.4985797300065999e+03 },
    { 5.0269999999999999e-08, 2.8687404779699999e+00, 2.1458261081400000e+01 },
    { 4.9819999999999999e-08, 1.4613204451099999e+00, 1.0904014188261999e+03 },
    { 4.9100000000000003e-08, 3.4770726849800000e+00, 4.5034134317499998e+01 },
    { 4.8319999999999999e-08, 1.2134886528200000e+00, 4.1800017116689997e+02 },
    { 4.7880000000000003e-08, 2.9295013934199998e+00, 1.3703302416240001e+02 },
    { 4.7309999999999998e-08, 1.2074869014300000e+00, 3.6127298067700004e+01 },
    { 4.7309999999999998e-08, 6.2746930184999998e+00, 4.0138773207900002e+01 },
    { 4.7290000000000002e-08, 2.1968293287499998e+00, 4.2096911658350001e+02 },
    { 4.6999999999999997e-08, 4.6631368774700004e+00, 5.0663208609999998e+01 },
    { 4.6889999999999998e-08, 5.9551031565799999e+00, 1.0679799498983000e+03 },
    { 4.6159999999999998e-08, 6.0131635545100002e+00, 2.9170403072770000e+02 },
    { 4.5260000000000003e-08, 1.7099018685000000e+00, 9.2107688163959995e+02 },
    { 4.4759999999999998e-08, 1.4646375119599999e+00, 2.3593301268700000e+02 },
    { 4.4320000000000002e-08, 8.3560291415999999e-01, 4.1555249061209997e+02 },
    { 4.4139999999999997e-08, 2.0041641900200000e+00, 1.0623508756058000e+03 },
    { 4.3670000000000000e-08, 1.4073509949999999e-01, 5.8739963457300000e+01 },
    { 4.3630000000000001e-08, 2.3676647947100000e+00, 3.2350541665740002e+02 },
    { 4.3630000000000001e-08, 1.5413194966299999e+00, 4.9304240216509999e+02 },
    { 4.3410000000000003e-08, 1.5069679503000000e+00, 3.6648562929500002e+02 },
    { 4.3049999999999998e-08, 4.8233548411499996e+00, 6.7880499887599996e+01 },
    { 4.2930000000000001e-08, 4.4042309044600003e+00, 1.0507839104801001e+03 },
    { 4.2270000000000001e-08, 6.1995949973500002e+00, 5.3414438321950001e+02 },
    { 4.2249999999999998e-08, 3.2886910369300000e+00, 4.1677633088950000e+02 },
    { 4.1649999999999999e-08, 4.4635069662399998e+00, 5.9282480178500002e+01 },
    { 4.1589999999999997e-08, 1.3477801652999999e-01, 5.0015594916589998e+02 },
    { 4.1390000000000002e-08, 5.1953135771000003e+00, 5.1838463239980001e+02 },
    { 4.0879999999999999e-08, 3.8866119129399999e+00, 4.2341679713830001e+02 },
    { 4.0760000000000002e-08, 6.6240216055000001e-01, 1.1272624300768000e+03 },
    { 4.0579999999999997e-08, 3.4400204229000000e+00, 2.1648048917570000e+02 },
    { 4.0569999999999999e-08, 6.4472041755999998e-01, 4.8592885516429999e+02 },
    { 4.0539999999999998e-08, 2.4580549189399998e+00, 4.2977958461370002e+02 },
    { 4.0480000000000002e-08, 6.2524999532500001e+00, 9.7993446517589996e+02 },
    { 4.0390000000000000e-08, 5.3706645720599999e+00, 1.6230131452276000e+03 },
    { 4.0170000000000002e-08, 6.1607035935700001e+00, 3.1071461125430000e+02 },
    { 4.0130000000000003e-08, 2.9676805520100000e+00, 1.0427534649502118e+05 },
    { 3.9850000000000003e-08, 4.6088256927600000e+00, 2.1011770170029999e+02 },
    { 3.9620000000000000e-08, 5.5916159532299998e+00, 9.7918370565049997e+02 },
    { 3.9559999999999998e-08, 2.4588917456299999e+00, 8.3769726336320002e+02 },
    { 3.9410000000000000e-08, 5.4272203001200001e+00, 1.1312708252620000e+02 },
    { 3.9340000000000000e-08, 2.2431403739500002e+00, 6.7359235025800004e+01 },
    { 3.8870000000000003e-08, 7.9597881268000004e-01, 9.1396333463880001e+02 },
    { 3.8500000000000001e-08, 5.3946814706700001e+00, 9.5979227217800002e+01 },
    { 3.8379999999999997e-08, 3.5689111481700002e+00, 9.8014691349700001e+02 },
    { 3.7590000000000001e-08, 2.3920613122600001e+00, 1.0749800823370001e+02 },
    { 3.7569999999999998e-08, 2.0284024392500002e+00, 2.9615744885260000e+02 },
    { 3.7259999999999997e-08, 4.9414614640699996e+00, 1.0650110044817000e+03 },
    { 3.7149999999999998e-08, 1.3091125206500001e+00, 2.5435289238260000e+02 },
    { 3.6769999999999998e-08, 1.4448152741900000e+00, 6.4701083331480004e+02 },
    { 3.6640000000000003e-08, 1.6966235780100001e+00, 8.0794979911339999e+02 },
    { 3.6599999999999997e-08, 3.5209286500400001e+00, 4.4928029223480002e+02 },
    { 3.6330000000000002e-08, 4.6973445258200002e+00, 4.9155792945680003e+02 },
    { 3.6260000000000002e-08, 1.6140462750200000e+00, 2.1543795945209999e+02 },
    { 3.6249999999999997e-08, 1.8538674434300000e+00, 1.0227333672597000e+03 },
    { 3.6099999999999999e-08, 1.4739718778099999e+00, 5.4807810194200002e+01 },
    { 3.5920000000000000e-08, 4.7546131413999998e-01, 3.8135160823740000e+02 },
    { 3.5380000000000003e-08, 1.5195207198900000e+00, 5.7255490749000003e+01 },
    { 3.4900000000000001e-08, 5.4755432663000003e-01, 9.8704801217670001e+02 },
    { 3.4879999999999999e-08, 5.9510003308300004e+00, 1.8769623277240001e+02 },
    { 3.4849999999999998e-08, 3.8092744589999998e-02, 6.8843707734099993e+01 },
    { 3.4660000000000001e-08, 3.4736081000699999e+00, 2.1455905633214002e+03 },
    { 3.4200000000000002e-08, 4.8234786886499998e+00, 1.0283624415521999e+03 },
    { 3.4070000000000000e-08, 4.5091696123900000e+00, 9.1104257333199996e+02 },
    { 3.3920000000000002e-08, 2.3232954467500000e+00, 2.7825883401879997e+02 },
    { 3.3839999999999997e-08, 5.3922103007000004e+00, 1.5438263126452000e+03 },
    { 3.3820000000000001e-08, 3.2274417153999999e+00, 2.9226199388000001e+01 },
    { 3.3530000000000003e-08, 2.3031006481700000e+00, 1.0289551617486601e+04 },
    { 3.3530000000000003e-08, 1.6261435745199999e+00, 1.5763995198190000e+02 },
    { 3.3479999999999999e-08, 3.5516302953999999e+00, 6.2845603226997000e+03 },
    { 3.3470000000000001e-08, 4.1356069854099999e+00, 1.5104766984290001e+02 },
    { 3.3390000000000003e-08, 5.2671170725999996e+00, 4.4334240140160000e+02 },
    { 3.3209999999999998e-08, 3.8761625752000000e+00, 4.5024350008130000e+02 },
    { 3.3180000000000003e-08, 5.8557307232799998e+00, 5.6511568774670002e+02 },
    { 3.2959999999999999e-08, 3.0623844558900002e+00, 1.7665053250849999e+02 },
    { 3.2859999999999998e-08, 1.8262114868199999e+00, 4.3519621058510000e+02 },
    { 3.2460000000000001e-08, 4.0609611051399996e+00, 1.8324281464750001e+02 },
    { 3.2280000000000002e-08, 5.1369752825099999e+00, 5.2360129639400000e+01 },
    { 3.2159999999999998e-08, 2.8567208778199999e+00, 5.3653405886000002e+01 },
    { 3.2129999999999997e-08, 8.3729921110000005e-01, 5.6930847344070003e+02 },
    { 3.1879999999999998e-08, 2.6288614912299999e+00, 3.6137781986430002e+02 },
    { 3.1879999999999998e-08, 3.8503108961999999e-01, 4.8025159676200002e+02 },
    { 3.1769999999999999e-08, 5.6775044343800003e+00, 1.5580534066468001e+03 },
    { 3.1709999999999997e-08, 4.9718699743599997e+00, 1.0214770018919300e+04 },
    { 3.1559999999999999e-08, 6.2356497525999997e+00, 4.3208425281659999e+02 },
    { 3.1389999999999999e-08, 3.1000052291800002e+00, 1.5912442469019999e+02 },
    { 3.1380000000000000e-08, 6.3906502936999998e-01, 3.4736317418380003e+02 },
    { 3.1200000000000001e-08, 5.7986303258999996e+00, 7.3587651353180001e+02 },
    { 3.1200000000000001e-08, 1.2050457928800000e+00, 4.5660628755670001e+02 },
    { 3.1170000000000001e-08, 4.7425108763899999e+00, 4.5756949540319999e+02 },
    { 3.0990000000000002e-08, 2.5694384000000000e-03, 4.9447869076360001e+02 },
    { 3.0969999999999999e-08, 1.6713369708600001e+00, 3.9265794093220001e+02 },
    { 3.0969999999999999e-08, 4.4211901361099999e+00, 1.3954869557879999e+03 },
    { 3.0799999999999998e-08, 4.3586723393400000e+00, 4.5761767951299998e+02 },
    { 3.0720000000000000e-08, 2.3516080837400000e+00, 1.0400779795530001e+02 },
    { 3.0570000000000002e-08, 5.3956960156000000e-01, 2.7555052103309998e+02 },
    { 3.0510000000000000e-08, 6.2349624612500003e+00, 6.2815913772831000e+03 },
    { 3.0309999999999998e-08, 2.5594398597499999e+00, 4.0610313764109998e+02 },
    { 3.0279999999999997e-08, 4.0233691321000000e+00, 3.0927832265580003e+02 },
    { 3.0230000000000000e-08, 5.5173278520800002e+00, 1.0551891444952000e+03 },
    { 3.0209999999999997e-08, 6.1053267447700001e+00, 3.2835259365719998e+02 },
    { 3.0129999999999999e-08, 1.4952824265600000e+00, 1.2099138905240000e+02 },
    { 2.9819999999999998e-08, 7.7497441779999995e-02, 5.2575881183150000e+02 },
    { 2.9810000000000000e-08, 3.5226770667100000e+00, 2.0225339517410001e+02 },
    { 2.9679999999999998e-08, 1.2809286492400001e+00, 6.1307058337100000e+02 },
    { 2.9659999999999999e-08, 4.0851049482699997e+00, 2.1989137757699999e+02 },
    { 2.9480000000000000e-08, 1.1182641821299999e+00, 5.3362311835770004e+02 },
    { 2.9350000000000001e-08, 1.6748342708999999e+00, 4.0313419222450000e+02 },
    { 2.9340000000000000e-08, 5.9398799409900001e+00, 3.8549620982160002e+02 },
    { 2.9280000000000001e-08, 3.6162884968700002e+00, 1.1314552157707999e+03 },
    { 2.9150000000000000e-08, 5.4912301668200003e+00, 2.1187638603783998e+03 },
    { 2.9080000000000000e-08, 3.7150342544699999e+00, 5.7345307502490004e+02 },
    { 2.9040000000000001e-08, 2.1732862180799999e+00, 9.0684978763799995e+02 },
    { 2.9020000000000001e-08, 4.2293871594999999e+00, 5.5922598102330005e+02 },
    { 2.8990000000000000e-08, 1.3772238030799999e+00, 1.0211801073502700e+04 },
    { 2.8959999999999999e-08, 7.1994184395000005e-01, 1.6850521225016000e+03 },
    { 2.8889999999999999e-08, 4.3555572668400000e+00, 1.1731986822019999e+02 },
    { 2.8780000000000000e-08, 2.0117875829799998e+00, 9.4795176869240004e+02 },
    { 2.8699999999999999e-08, 2.3026349252100000e+00, 3.8846515523820000e+02 },
    { 2.8649999999999998e-08, 4.1693177285100003e+00, 8.3058371636239997e+02 },
    { 2.8600000000000001e-08, 3.6648863450000002e+00, 8.4481081036399996e+02 },
    { 2.8570000000000000e-08, 5.1399193245300001e+00, 8.3477650205639998e+02 },
    { 2.8530000000000001e-08, 5.1670868176400004e+00, 1.7071264167530001e+02 },
    { 2.8520000000000000e-08, 2.9061268529800000e+00, 1.8647239249500001e+02 },
    { 2.8450000000000000e-08, 5.4152028447199996e+00, 1.4510977900969999e+02 },
    { 2.8400000000000000e-08, 2.4234837922399999e+00, 3.8427236954419999e+02 },
    { 2.8369999999999999e-08, 6.0708324384800001e+00, 4.1974527630609998e+02 },
    { 2.8270000000000001e-08, 3.6861231243999998e-01, 4.2956713629260003e+02 },
    { 2.7899999999999998e-08, 5.7827063093000000e+00, 6.9774347789399997e+02 },
    { 2.7719999999999999e-08, 9.0661921369999998e-01, 1.3851749687070000e+02 },
    { 2.7649999999999999e-08, 5.7116268219900004e+00, 2.4846318565920001e+02 },
    { 2.7490000000000000e-08, 3.3097945934000000e-01, 2.1692243216040001e+02 },
    { 2.7339999999999999e-08, 7.9845484023000002e-01, 4.3222726516850003e+02 },
    { 2.7230000000000000e-08, 4.4483309596199998e+00, 2.4476805547999998e+00 },
    { 2.6950000000000000e-08, 4.2475081985000003e+00, 3.2642617796420001e+02 },
    { 2.6750000000000001e-08, 1.3133212443200000e+00, 5.2523754696970002e+02 },
    { 2.6740000000000000e-08, 2.2925635662700001e+00, 1.5837366516480000e+02 },
    { 2.6309999999999999e-08, 4.2106763564799996e+00, 2.0085575391590000e+03 },
    { 2.6140000000000001e-08, 3.3546501954800001e+00, 9.4646729598410002e+02 },
    { 2.5869999999999999e-08, 1.5930362230199999e+00, 4.0106476417959999e+02 },
    { 2.5810000000000001e-08, 4.0624420219899999e+00, 4.2537435059860002e+02 },
    { 2.5650000000000001e-08, 5.8686214322900003e+00, 3.4884764689209999e+02 },
    { 2.5550000000000000e-08, 3.1680200948999998e-01, 8.5827298831199997e+01 },
    { 2.5460000000000001e-08, 2.3206052693000001e-01, 1.0625633239269000e+03 },
    { 2.5349999999999999e-08, 4.1902325498600002e+00, 4.8147543703939999e+02 },
    { 2.5040000000000001e-08, 4.6935002909100003e+00, 6.5961044225620003e+02 },
    { 2.4590000000000000e-08, 3.6621648709499999e+00, 1.0197644218431000e+03 },
    { 2.4579999999999999e-08, 3.7421684269300002e+00, 4.5097721326419997e+02 },
    { 2.4540000000000000e-08, 3.2791342815900002e+00, 1.9577298761969999e+02 },
    { 2.4529999999999999e-08, 3.9357442204999998e-01, 5.2698265210889997e+02 },
    { 2.4430000000000001e-08, 2.1190446274400001e+00, 6.9355069219999996e+02 },
    { 2.4389999999999999e-08, 1.4571860864299999e+00, 4.6298612137459997e+02 },
    { 2.4350000000000000e-08, 9.9424789953000003e-01, 4.7733083545519997e+02 },
    { 2.4260000000000000e-08, 2.0713142797000001e+00, 1.0509963588011999e+03 },
    { 2.4120000000000000e-08, 2.5272937023000002e+00, 8.4608283475120004e+02 },
    { 2.3980000000000000e-08, 5.5917572816099996e+00, 2.3905941636200001e+01 },
    { 2.3940000000000001e-08, 1.3381278422000001e-01, 1.4711551657979999e+02 },
    { 2.3870000000000001e-08, 2.0419202402500001e+00, 6.5412438031559998e+02 },
    { 2.3849999999999998e-08, 3.7533677483000001e+00, 5.9884348936940000e+02 },
    { 2.3689999999999999e-08, 2.7584052705400000e+00, 1.0728555991259999e+02 },
    { 2.3619999999999999e-08, 3.6460504864999999e-01, 3.5181659230870002e+02 },
    { 2.3359999999999999e-08, 2.7365447214500001e+00, 2.1033015002139999e+02 },
    { 2.3339999999999999e-08, 1.6818425668100001e+00, 5.5503319532930004e+02 },
    { 2.3250000000000000e-08, 2.5393903828999997e-01, 4.4312995308050000e+02 },
    { 2.3000000000000001e-08, 2.1983519450800002e+00, 9.8629725265130003e+02 },
    { 2.2609999999999999e-08, 3.2498809573099998e+00, 6.4409007200799999e+02 },
    { 2.2580000000000001e-08, 5.2463625697699996e+00, 2.9747464249800000e+01 },
    { 2.2329999999999999e-08, 4.3734799426300004e+00, 7.0904981058880003e+02 },
    { 2.2189999999999999e-08, 2.4790256759800000e+00, 5.5800214074589996e+02 },
    { 2.1979999999999999e-08, 1.2130190302299999e+00, 6.0595703637019994e+02 },
};

static const VSOP87Term VSOP87A_NEPTUNE_Y2[97] = {
    { 4.5394210000000001e-05, 3.4561320792200001e+00, 3.6648562929500002e+01 },
    { 4.3479560000000000e-05, 8.8317230350999998e-01, 3.9617508346100003e+01 },
    { 3.5953939999999997e-05, 3.1415926535900001e+00, 0.0000000000000000e+00 },
    { 3.0586470000000001e-05, 5.3195661366499998e+00, 3.8133035637799999e+01 },
    { 2.1638090000000001e-05, 2.2086532213999999e-01, 7.6266071275599998e+01 },
    { 3.9463200000000004e-06, 4.1091546572600004e+00, 3.5164090221199999e+01 },
    { 3.0109400000000001e-06, 2.4659723217000001e-01, 4.1101981054399999e+01 },
    { 1.8662600000000000e-06, 8.5138234128000001e-01, 2.9689454166000000e+00 },
    { 1.6036500000000001e-06, 5.9944308104899999e+00, 2.0618554843720000e+02 },
    { 1.5689200000000000e-06, 2.0507361292400001e+00, 2.2041264243879999e+02 },
    { 1.1809100000000001e-06, 2.5655277957700000e+00, 5.2257741809380002e+02 },
    { 1.1447799999999999e-06, 4.7720919150999999e+00, 3.5685355082999997e+01 },
    { 1.1204900000000000e-06, 4.9319555973300000e+00, 5.3680451209540001e+02 },
    { 1.0594700000000000e-06, 5.8517480988799999e+00, 4.0580716192600001e+01 },
    { 1.0138100000000000e-06, 8.9282766843000005e-01, 2.1329909543799999e+02 },
    { 7.4114000000000003e-07, 3.4508663303499998e+00, 7.3297125859000005e+01 },
    { 7.3178000000000001e-07, 3.7603629082499999e+00, 5.2969096509459996e+02 },
    { 6.4249999999999999e-07, 9.1911470350000002e-01, 4.2659819087599999e+02 },
    { 5.5173999999999999e-07, 5.6655145215099996e+00, 1.4844727083000000e+00 },
    { 4.8439999999999997e-07, 4.9690712113400002e+00, 3.3679617512900002e+01 },
    { 4.0073999999999997e-07, 1.5979944641999999e-01, 4.4534181248999998e+00 },
    { 3.9838000000000000e-07, 3.5468881129300001e+00, 1.0593819301891999e+03 },
    { 3.8263999999999999e-07, 5.6111111749799996e+00, 4.2586453762700003e+01 },
    { 3.7331000000000000e-07, 1.6804115007600000e+00, 7.4781598567299994e+01 },
    { 3.5541999999999999e-07, 9.3237309218999997e-01, 3.1019488636999998e+01 },
    { 3.4068000000000000e-07, 1.3587685579100000e+00, 9.8899988524600005e+01 },
    { 3.3752999999999998e-07, 4.8939494379999999e+00, 1.1439910691340000e+02 },
    { 3.2194999999999999e-07, 3.4054458866599999e+00, 4.5246582638600003e+01 },
    { 3.1376999999999998e-07, 1.8253538171100001e+00, 2.8571808082200000e+01 },
    { 2.4148999999999997e-07, 2.5130969025100001e+00, 4.7694263193399998e+01 },
    { 1.7601000000000000e-07, 4.2357165561899999e+00, 7.1812653150700001e+01 },
    { 1.4424000000000000e-07, 1.1638035865400000e+00, 4.3371173787679999e+02 },
    { 1.3295999999999999e-07, 5.1159499103300003e+00, 4.1948464387519999e+02 },
    { 1.2772999999999999e-07, 1.3656678167900000e+00, 6.0766952886799999e+01 },
    { 1.0585000000000000e-07, 4.3774984757400004e+00, 3.3940249943799998e+01 },
    { 9.7370000000000004e-08, 9.9947107330000007e-02, 6.3989728631399998e+02 },
    { 9.6139999999999999e-08, 4.2184983860700003e+00, 6.9152524274800001e+01 },
    { 9.2550000000000005e-08, 6.1882675285400000e+00, 4.2325821331800000e+01 },
    { 7.4750000000000004e-08, 1.9243901324099999e+00, 2.2752618943959999e+02 },
    { 7.2530000000000002e-08, 4.8857408434399998e+00, 1.0975149658270000e+03 },
    { 7.1229999999999997e-08, 1.0365585986000000e-01, 7.1135470007999997e+00 },
    { 6.6819999999999999e-08, 3.2843814298799998e+00, 2.5854567807659998e+02 },
    { 6.5480000000000001e-08, 7.0267550536000001e-01, 3.0056280790500001e+01 },
    { 6.1830000000000005e-08, 5.5121789635200003e+00, 1.0664954771900000e+03 },
    { 6.0500000000000006e-08, 6.1160656324999998e+00, 3.2195144804599998e+01 },
    { 5.6249999999999997e-08, 1.4937727966500001e+00, 8.0767548473000002e+00 },
    { 5.4639999999999998e-08, 2.0583795131499998e+00, 1.0522683831884001e+03 },
    { 5.4359999999999998e-08, 3.3701145796300001e+00, 1.0601353552540000e+02 },
    { 5.3799999999999999e-08, 1.1510593257999999e+00, 2.2633917249000000e+01 },
    { 5.2789999999999998e-08, 2.7422567463299998e+00, 5.1546387109299997e+02 },
    { 5.2450000000000003e-08, 4.4853216253100001e+00, 4.4070926471000000e+01 },
    { 4.8330000000000003e-08, 5.9766936318499999e+00, 2.1478356814630001e+02 },
    { 4.7860000000000000e-08, 3.6902940065099998e+00, 4.6209790485100001e+01 },
    { 4.6880000000000000e-08, 9.5059652202000000e-01, 2.2189711514710001e+02 },
    { 4.6070000000000002e-08, 1.7860892585999999e-01, 5.4940412103680001e+02 },
    { 4.4969999999999998e-08, 5.7184702931200002e+00, 7.7750543983900002e+01 },
    { 4.2660000000000000e-08, 3.9428246462100001e+00, 2.0470107572890001e+02 },
    { 4.1379999999999997e-08, 2.7513803339999998e-01, 3.0508553696180002e+02 },
    { 4.0040000000000000e-08, 4.0473809997399997e+00, 5.3287235883230005e+02 },
    { 3.9799999999999999e-08, 3.4324621289800001e+00, 5.2650957135689998e+02 },
    { 3.9739999999999997e-08, 3.6198888625199999e+00, 3.3610502559880001e+02 },
    { 3.9319999999999997e-08, 5.2332282013200002e+00, 9.1786441523799994e+01 },
    { 3.9160000000000001e-08, 4.9265874898900002e+00, 2.0767002114549999e+02 },
    { 3.7720000000000003e-08, 4.0171520570999997e+00, 1.5890728952838001e+03 },
    { 3.7539999999999997e-08, 4.2351932505000001e+00, 9.4498282327580000e+02 },
    { 3.6920000000000002e-08, 5.8340853721499997e+00, 1.0309277421860000e+02 },
    { 3.6710000000000002e-08, 2.5894978695300002e+00, 3.1931263096340001e+02 },
    { 3.5670000000000001e-08, 5.0952995160000000e-02, 1.9907200143640000e+02 },
    { 3.3470000000000001e-08, 5.3710294787799997e+00, 5.3117543780289998e+02 },
    { 3.3460000000000003e-08, 3.0722502313000000e+00, 1.1624747044077999e+03 },
    { 3.2880000000000001e-08, 3.0203661124200001e+00, 4.0817831118039999e+02 },
    { 3.2679999999999999e-08, 4.8205241265499996e+00, 5.4391805909619995e+02 },
    { 3.1489999999999999e-08, 1.4828482766000000e+00, 3.4321857259960001e+02 },
    { 3.0290000000000002e-08, 4.4159869799899996e+00, 1.2991947716160001e+02 },
    { 2.9989999999999999e-08, 2.8495603005299999e+00, 9.3786927627499995e+02 },
    { 2.9670000000000000e-08, 5.7571856230999998e+00, 8.6030992875280003e+02 },
    { 2.9480000000000000e-08, 1.3901787411900000e+00, 1.4956319713459999e+02 },
    { 2.9250000000000001e-08, 3.1453916845299998e+00, 2.3593301268700000e+02 },
    { 2.9110000000000001e-08, 1.4349714175099999e+00, 3.1219908396260001e+02 },
    { 2.9060000000000000e-08, 2.3327331369500000e+00, 6.6672398925699997e+02 },
    { 2.8180000000000002e-08, 4.6076581786400004e+00, 1.0283624415521999e+03 },
    { 2.7150000000000002e-08, 3.3367529665400002e+00, 5.6363121503840000e+02 },
    { 2.6759999999999999e-08, 1.0396439129499999e+00, 4.5490936652729999e+02 },
    { 2.6000000000000001e-08, 2.6896794495999998e+00, 5.1838463239980001e+02 },
    { 2.5769999999999998e-08, 5.1965993386299996e+00, 1.0736090241908000e+03 },
    { 2.5760000000000000e-08, 1.4068834797600001e+00, 2.6826702943000001e+01 },
    { 2.5690000000000000e-08, 2.6554357295400002e+00, 4.6473122651379998e+02 },
    { 2.5419999999999999e-08, 2.6521906528499999e+00, 1.2655674786264001e+03 },
    { 2.5370000000000002e-08, 3.3191145577799999e+00, 4.5342489381899998e+02 },
    { 2.5209999999999999e-08, 1.5724066514999999e+00, 2.0936694217490000e+02 },
    { 2.5160000000000002e-08, 1.7461911782000000e-01, 2.1723124870109999e+02 },
    { 2.5130000000000001e-08, 3.1354070400800000e+00, 2.7406604832480002e+02 },
    { 2.5119999999999999e-08, 5.0529275962100000e+00, 4.2511371816769997e+02 },
    { 2.5010000000000000e-08, 3.2449079044800002e+00, 6.6964475056380002e+02 },
    { 2.4809999999999998e-08, 8.5513013918000003e-01, 6.5541765656220002e+02 },
    { 2.4690000000000001e-08, 5.3656453713900003e+00, 9.5209637027660006e+02 },
    { 2.3429999999999999e-08, 3.5068078327399999e+00, 9.8163138620530003e+02 },
};

static const VSOP87Term VSOP87A_NEPTUNE_Y3[34] = {
    { 1.9270299999999999e-06, 5.5508860101300002e+00, 3.6648562929500002e+01 },
    { 1.8221800000000000e-06, 5.0730641687400002e+00, 3.9617508346100003e+01 },
    { 1.7606200000000001e-06, 0.0000000000000000e+00, 0.0000000000000000e+00 },
    { 1.3073100000000000e-06, 2.1816916163600002e+00, 3.8133035637799999e+01 },
    { 8.5913000000000004e-07, 5.1393843643299997e+00, 7.6266071275599998e+01 },
    { 2.2391000000000000e-07, 1.4097462329999999e+00, 2.0618554843720000e+02 },
    { 2.1976000000000000e-07, 3.5990434774000002e-01, 3.5164090221199999e+01 },
    { 2.1616000000000000e-07, 3.1423526140000002e-01, 2.2041264243879999e+02 },
    { 1.9807000000000000e-07, 3.9806828601599999e+00, 4.1101981054399999e+01 },
    { 1.6259999999999999e-07, 4.2886847110600002e+00, 5.2257741809380002e+02 },
    { 1.5442999999999999e-07, 3.2154891999599999e+00, 5.3680451209540001e+02 },
    { 1.2954999999999999e-07, 4.8038998776900002e+00, 7.1135470007999997e+00 },
    { 6.7720000000000000e-08, 5.9091831155600003e+00, 6.9152524274800001e+01 },
    { 4.5240000000000000e-08, 2.6163026311699999e+00, 3.1019488636999998e+01 },
    { 4.5160000000000002e-08, 1.7181325049700000e+00, 4.5246582638600003e+01 },
    { 3.9150000000000003e-08, 5.3476032576300003e+00, 4.2659819087599999e+02 },
    { 3.6680000000000002e-08, 2.9757551896700001e+00, 9.8899988524600005e+01 },
    { 3.6559999999999998e-08, 2.1722527654600001e+00, 9.6888876491200006e+02 },
    { 3.6249999999999997e-08, 4.1984625185700004e+00, 4.8444438245600003e+02 },
    { 3.5040000000000001e-08, 3.0407412511700000e+00, 4.9867147645760002e+02 },
    { 3.4980000000000000e-08, 4.8455544273999998e+00, 4.0817831118039999e+02 },
    { 3.3039999999999997e-08, 3.8182443530499999e+00, 4.2240540518199998e+02 },
    { 3.1989999999999997e-08, 3.2440563299299998e+00, 1.5199203710090001e+03 },
    { 2.8760000000000001e-08, 3.6543482612200000e+00, 8.5319638175199998e+02 },
    { 2.6890000000000001e-08, 5.6117275977299999e+00, 4.4082528487759998e+02 },
    { 2.6619999999999999e-08, 2.0269081844299999e+00, 3.0508553696180002e+02 },
    { 2.6440000000000001e-08, 9.0420046362999995e-01, 4.6202291352809999e+02 },
    { 2.5780000000000000e-08, 5.3286642229299996e+00, 3.9557870223899999e+02 },
    { 2.5650000000000001e-08, 5.4670075776699996e+00, 4.3371173787679999e+02 },
    { 2.5419999999999999e-08, 1.0510715053699999e+00, 4.4482687410990002e+02 },
    { 2.4710000000000001e-08, 9.2256756337000001e-01, 3.1931263096340001e+02 },
    { 2.3540000000000001e-08, 2.0325895675100001e+00, 4.4779581952650000e+02 },
    { 2.2790000000000001e-08, 6.2078816295000001e+00, 4.5905396811150001e+02 },
    { 2.1470000000000000e-08, 1.0653257263600000e+00, 1.7516605980020000e+02 },
};

static const VSOP87Term VSOP87A_NEPTUNE_Y4[7] = {
    { 4.3340000000000003e-08, 3.8456950084499999e+00, 4.3371173787679999e+02 },
    { 4.2289999999999997e-08, 6.1448577486299998e+00, 5.1546387109299997e+02 },
    { 3.5469999999999999e-08, 1.0432124312200000e+00, 9.9022940591439999e+02 },
    { 3.1550000000000001e-08, 1.4082926066000001e-01, 4.6765198782060003e+02 },
    { 3.0169999999999998e-08, 4.7771936313100003e+00, 2.2752618943959999e+02 },
    { 2.3409999999999999e-08, 4.8374739560700002e+00, 2.2041264243879999e+02 },
    { 2.2950000000000000e-08, 3.1322120441900001e+00, 2.0618554843720000e+02 },
};

static const VSOP87Term VSOP87A_NEPTUNE_Y5[1] = {
    { 0.0, 0.0, 0.0 }
};

static const VSOP87Term VSOP87A_NEPTUNE_Z0[133] = {
    { 9.2866054404999998e-01, 1.4410393027799999e+00, 3.8133035637799999e+01 },
    { 1.2459784620000001e-02, 0.0000000000000000e+00, 0.0000000000000000e+00 },
    { 4.7433356700000004e-03, 2.5221877423799999e+00, 3.6648562929500002e+01 },
    { 4.5198793600000004e-03, 3.5094972054100002e+00, 3.9617508346100003e+01 },
    { 4.1755806799999998e-03, 5.9131069542099999e+00, 7.6266071275599998e+01 },
    { 8.4104329000000001e-04, 4.3892890009599999e+00, 1.4844727083000000e+00 },
    { 3.2704958000000001e-04, 1.5204869200100000e+00, 7.4781598567299994e+01 },
    { 3.0873335000000001e-04, 3.2901761145599999e+00, 3.5164090221199999e+01 },
    { 2.5812584000000000e-04, 3.1930312878199998e+00, 2.9689454166000000e+00 },
    { 1.6865319000000000e-04, 2.1325110442500002e+00, 4.1101981054399999e+01 },
    { 1.1789909000000000e-04, 3.6000187767499998e+00, 2.1329909543799999e+02 },
    { 1.1279680000000000e-04, 3.5581667633400000e+00, 5.2969096509459996e+02 },
    { 9.7701249999999996e-05, 2.8013397158600002e+00, 7.3297125859000005e+01 },
    { 4.1198729999999998e-05, 1.6793431683600000e+00, 7.7750543983900002e+01 },
    { 2.8686769999999999e-05, 4.2701152620300000e+00, 3.3679617512900002e+01 },
    { 2.8180340000000001e-05, 4.1066107779400003e+00, 1.1439910691340000e+02 },
    { 2.2134639999999999e-05, 1.9604513516800000e+00, 4.4534181248999998e+00 },
    { 1.8656500000000000e-05, 5.0554070957699997e+00, 7.1812653150700001e+01 },
    { 8.4017699999999994e-06, 9.4268885160000004e-01, 4.2586453762700003e+01 },
    { 5.3025200000000002e-06, 8.5800267792999996e-01, 1.1143016149680000e+02 },
    { 4.9085900000000000e-06, 6.0782730120900004e+00, 1.1291463420510000e+02 },
    { 4.5751599999999996e-06, 5.7165041207999998e+00, 1.0846121608020000e+02 },
    { 3.3125399999999999e-06, 2.9304964525999999e-01, 7.0328180442399997e+01 },
    { 3.3004499999999999e-06, 2.8383967621499999e+00, 4.2659819087599999e+02 },
    { 2.7758600000000001e-06, 1.4509201054500001e+00, 1.4807872442630000e+02 },
    { 2.7447400000000000e-06, 5.4265702243699998e+00, 3.2195144804599998e+01 },
    { 2.7358900000000001e-06, 3.9101368179399998e+00, 1.0593819301891999e+03 },
    { 2.0530600000000000e-06, 7.5818737084999999e-01, 5.9378908332000000e+00 },
    { 1.7351600000000001e-06, 5.8549803009900003e+00, 1.4510977900969999e+02 },
    { 1.4364699999999999e-06, 2.5162004781200000e+00, 3.7611770776000000e+01 },
    { 1.4127499999999999e-06, 1.7314759765700001e+00, 2.8571808082200000e+01 },
    { 1.4075800000000000e-06, 3.5094498969400001e+00, 3.8654300499599998e+01 },
    { 1.3909300000000001e-06, 1.6746670119100000e+00, 1.8472728735580000e+02 },
    { 1.3695500000000001e-06, 2.0339778663999999e-01, 7.9235016692200006e+01 },
    { 1.3350900000000001e-06, 4.7897710554700001e+00, 3.8084851528000002e+01 },
    { 1.3329199999999999e-06, 1.2338693592500001e+00, 3.8181219747599997e+01 },
    { 1.2629600000000000e-06, 4.4066138504000003e+00, 3.7169827791300001e+01 },
    { 1.2090599999999999e-06, 1.6176763660200000e+00, 3.9096243484299997e+01 },
    { 1.1176100000000000e-06, 6.2094823078500001e+00, 9.8899988524600005e+01 },
    { 1.1158899999999999e-06, 4.1856139557800001e+00, 4.7694263193399998e+01 },
    { 1.0262199999999999e-06, 8.1673762158999996e-01, 4.1927856940000003e+00 },
    { 9.8770999999999994e-07, 7.2335005781999995e-01, 1.0697674337190000e+02 },
    { 9.3918999999999998e-07, 5.6607810948000004e-01, 2.0618554843720000e+02 },
    { 8.1727000000000001e-07, 3.4786131525799999e+00, 2.2041264243879999e+02 },
    { 7.4559000000000005e-07, 2.3151888043900000e+00, 3.1219908396260001e+02 },
    { 7.4400999999999995e-07, 5.9993572716400001e+00, 1.8175834193919999e+02 },
    { 7.3140999999999996e-07, 1.8006995163399999e+00, 1.3703302416240001e+02 },
    { 6.6838000000000001e-07, 1.8718533090400000e+00, 2.2137585028530000e+02 },
    { 5.8302999999999995e-07, 5.6166256154800003e+00, 3.5685355082999997e+01 },
    { 5.5537000000000003e-07, 2.1197729605500002e+00, 1.0994568878849999e+02 },
    { 5.1928000000000003e-07, 4.0473854285999999e-01, 4.0580716192600001e+01 },
    { 5.1684999999999997e-07, 6.0283134764900002e+00, 4.4070926471000000e+01 },
    { 5.0936000000000000e-07, 1.9994012328999999e-01, 3.0056280790500001e+01 },
    { 4.9439000000000002e-07, 1.6617890571700000e+00, 6.8843707734099993e+01 },
    { 4.8182000000000001e-07, 2.9714199173700000e+00, 3.7872403206900003e+01 },
    { 4.7742000000000000e-07, 3.0526110537100002e+00, 3.8393668068700002e+01 },
    { 4.7361000000000002e-07, 3.5896454160400002e+00, 3.8021161053199997e+01 },
    { 4.7179999999999998e-07, 2.4340596710700000e+00, 3.8244910222400001e+01 },
    { 4.6428000000000003e-07, 2.6659673924199998e+00, 1.4659425171800001e+02 },
    { 4.4491999999999999e-07, 6.0061387860600002e+00, 1.4956319713459999e+02 },
    { 4.1357000000000002e-07, 8.6667380712999997e-01, 1.4362530630140000e+02 },
    { 4.1127000000000000e-07, 6.0523930382500000e+00, 1.1588357962169999e+02 },
    { 4.0753000000000000e-07, 3.5966875930400000e+00, 5.2257741809380002e+02 },
    { 3.8849000000000001e-07, 5.7929438175600003e+00, 4.6209790485100001e+01 },
    { 3.7856000000000002e-07, 5.1994579617700003e+00, 7.2073285581600004e+01 },
    { 3.7348999999999999e-07, 5.1506790304000001e+00, 3.3940249943799998e+01 },
    { 3.4690000000000002e-07, 1.4739876632600000e+00, 8.0767548473000002e+00 },
    { 3.4386000000000001e-07, 6.1524663092900003e+00, 2.1840690486870000e+02 },
    { 3.3382000000000001e-07, 8.8396990078000004e-01, 4.2325821331800000e+01 },
    { 3.3070999999999997e-07, 2.0257255059800001e+00, 2.5802441321480001e+02 },
    { 3.1716000000000002e-07, 3.0412624027000001e-01, 5.3680451209540001e+02 },
    { 2.7604999999999999e-07, 2.8154040594000000e+00, 1.8324281464750001e+02 },
    { 2.4615999999999998e-07, 4.0597272412000002e-01, 3.0710672096300001e+01 },
    { 2.1715999999999999e-07, 9.0792314747000002e-01, 1.7516605980020000e+02 },
    { 2.1633999999999999e-07, 3.2546922864700001e+00, 3.8846515523820000e+02 },
    { 2.1570999999999999e-07, 2.3933770675999999e+00, 1.5253214255120000e+02 },
    { 2.0384000000000000e-07, 5.8095441486499997e+00, 7.4223635415000002e+00 },
    { 1.8790000000000001e-07, 6.2408059806000005e-01, 3.5212274331000003e+01 },
    { 1.8640000000000000e-07, 1.0642464299300001e+00, 1.8027386923090000e+02 },
    { 1.7102000000000001e-07, 5.3996488979399997e+00, 4.1053796944600002e+01 },
    { 1.6990000000000001e-07, 2.1972652378999999e+00, 2.9467297614429998e+02 },
    { 1.6716000000000000e-07, 2.3325902589999999e-02, 2.5505546779820000e+02 },
    { 1.6082999999999999e-07, 4.2062909141500002e+00, 9.6320784650000002e-01 },
    { 1.5218000000000000e-07, 2.9318224877099999e+00, 2.1989137757699999e+02 },
    { 1.4756999999999999e-07, 2.0202952608300002e+00, 1.0549227066360000e+02 },
    { 1.4503000000000000e-07, 4.6661452379700004e+00, 1.1020632121940000e+02 },
    { 1.4303999999999999e-07, 5.1564493377699998e+00, 3.1540753498800001e+01 },
    { 1.3582000000000001e-07, 9.9222619680000002e-01, 4.4725317776799997e+01 },
    { 1.3302999999999999e-07, 2.0936209925000000e+00, 6.3989728631399998e+02 },
    { 1.2520999999999999e-07, 4.1449986025000002e-01, 1.8621176006409999e+02 },
    { 1.2363000000000000e-07, 3.0759947649699999e+00, 6.5922821389999999e+00 },
    { 1.1309000000000000e-07, 4.1525339270700004e+00, 4.8736514376280002e+02 },
    { 1.0825000000000001e-07, 4.1381747605300001e+00, 5.2126486179999998e-01 },
    { 1.0056000000000000e-07, 4.2707704974300000e+00, 1.5890728952838001e+03 },
    { 9.6130000000000001e-08, 2.8151591238000001e-01, 6.0766952886799999e+01 },
    { 9.4889999999999998e-08, 4.9420591967599998e+00, 1.5104766984290001e+02 },
    { 9.3550000000000001e-08, 1.2336036071100001e+00, 2.1692243216040001e+02 },
    { 9.1110000000000000e-08, 3.4902719166099998e+00, 4.5246582638600003e+01 },
    { 9.0050000000000002e-08, 7.2840747299999994e-02, 4.1948464387519999e+02 },
    { 8.9270000000000005e-08, 2.3686918724299999e+00, 3.3132153907380001e+02 },
    { 8.7740000000000003e-08, 5.7719568484300003e+00, 1.2530172972200001e+01 },
    { 8.5980000000000006e-08, 3.1028734600900001e+00, 2.5653994050649999e+02 },
    { 8.5640000000000005e-08, 2.1590462246200000e+00, 2.7406604832480002e+02 },
    { 8.4450000000000005e-08, 1.7584724643999999e-01, 2.9170403072770000e+02 },
    { 8.2790000000000002e-08, 5.7208452554500004e+00, 3.6760437514099998e+01 },
    { 8.1779999999999995e-08, 2.5637861075000001e-01, 3.9505633761500000e+01 },
    { 7.9440000000000002e-08, 3.8277699900000001e-01, 1.0213285546211000e+04 },
    { 7.7050000000000005e-08, 1.5415259515699999e+00, 8.4342826122899993e+01 },
    { 7.6420000000000005e-08, 5.5609700659700003e+00, 1.8769623277240001e+02 },
    { 7.4740000000000006e-08, 2.9327943600799999e+00, 2.7087335373900000e+01 },
    { 7.2820000000000000e-08, 5.8134816392699999e+00, 3.1639186965660002e+02 },
    { 7.1320000000000006e-08, 1.5097123433099999e+00, 9.4498282327580000e+02 },
    { 6.8960000000000003e-08, 4.7145332474000003e+00, 4.0610313764109998e+02 },
    { 6.3240000000000002e-08, 2.1840625446100002e+00, 1.4214083359310001e+02 },
    { 6.3160000000000004e-08, 5.9502514374000004e-01, 4.5342489381899998e+02 },
    { 6.2940000000000006e-08, 5.6936923701999997e-01, 3.1019488636999998e+01 },
    { 6.2690000000000001e-08, 1.9516279017700000e+00, 6.0595703637019994e+02 },
    { 6.2149999999999997e-08, 5.2796715353700003e+00, 7.1135470007999997e+00 },
    { 6.1770000000000003e-08, 5.2011533405100003e+00, 8.0719489400499995e+01 },
    { 6.1020000000000000e-08, 5.5876437872400002e+00, 6.6253120356299996e+02 },
    { 6.0899999999999996e-08, 4.4850681956100003e+00, 4.1529185818119998e+02 },
    { 5.6809999999999997e-08, 4.8079103996999999e+00, 6.4112112659139996e+02 },
    { 5.6459999999999997e-08, 7.6840537905999995e-01, 1.1045700263900001e+01 },
    { 5.5910000000000002e-08, 5.2063178829699996e+00, 1.4014645680499999e+01 },
    { 5.3340000000000000e-08, 4.5270353845800004e+00, 1.0309277421860000e+02 },
    { 4.9140000000000002e-08, 1.3990603811000000e+00, 2.5357099508990001e+02 },
    { 4.7580000000000000e-08, 2.5431225871200001e+00, 3.6797010200329998e+02 },
    { 4.6779999999999999e-08, 4.8754669629500000e+00, 4.4275170057060001e+02 },
    { 4.5190000000000003e-08, 2.0761059043099999e+00, 2.0424977891028000e+03 },
    { 4.4069999999999997e-08, 5.5811040201099997e+00, 2.5208652238159999e+02 },
    { 4.4059999999999999e-08, 3.3696669221999997e-01, 3.2835259365719998e+02 },
    { 4.3900000000000003e-08, 5.0894960485800000e+00, 2.8659622129700000e+02 },
    { 4.3049999999999998e-08, 1.3172414002799999e+00, 4.9304240216509999e+02 },
};

static const VSOP87Term VSOP87A_NEPTUNE_Z1[37] = {
    { 1.5488597100000001e-03, 2.1423903966400002e+00, 3.8133035637799999e+01 },
    { 9.4642760000000005e-05, 0.0000000000000000e+00, 0.0000000000000000e+00 },
    { 7.7837079999999999e-05, 4.4014690590500001e+00, 3.6648562929500002e+01 },
    { 6.8624139999999995e-05, 1.6593016061000001e+00, 3.9617508346100003e+01 },
    { 3.8918729999999999e-05, 5.4676113942700004e+00, 7.6266071275599998e+01 },
    { 7.9481100000000000e-06, 5.9863543088900002e+00, 3.5164090221199999e+01 },
    { 5.3940399999999998e-06, 4.9452133501400004e+00, 2.1329909543799999e+02 },
    { 5.0640500000000002e-06, 4.4976388513999999e-01, 4.1101981054399999e+01 },
    { 3.8805800000000000e-06, 1.5250951725399999e+00, 5.2969096509459996e+02 },
    { 3.7431499999999999e-06, 1.5622297163200001e+00, 2.9689454166000000e+00 },
    { 1.8312800000000000e-06, 5.4079462115299997e+00, 1.4844727083000000e+00 },
    { 1.7748500000000000e-06, 4.3716635876599996e+00, 7.3297125859000005e+01 },
    { 8.7975999999999998e-07, 2.1162689700500001e+00, 7.4781598567299994e+01 },
    { 7.9922999999999999e-07, 2.7163282734999999e-01, 7.7750543983900002e+01 },
    { 6.4916000000000002e-07, 6.5218136897000001e-01, 3.3679617512900002e+01 },
    { 6.4899999999999995e-07, 2.7452505078000000e-01, 4.4534181248999998e+00 },
    { 5.0480999999999998e-07, 3.5777472078699999e+00, 1.1439910691340000e+02 },
    { 4.0462999999999999e-07, 5.4712752230200001e+00, 4.2586453762700003e+01 },
    { 3.7061000000000001e-07, 2.2606407608399999e+00, 2.0618554843720000e+02 },
    { 3.3262999999999998e-07, 5.2528281902399998e+00, 7.1812653150700001e+01 },
    { 2.8943999999999998e-07, 1.7866832532600001e+00, 2.2041264243879999e+02 },
    { 2.6739000000000001e-07, 4.0780635966999998e+00, 3.7611770776000000e+01 },
    { 2.6252000000000000e-07, 1.9451333499300001e+00, 3.8654300499599998e+01 },
    { 2.5314000000000000e-07, 4.9306464841999997e-01, 4.2659819087599999e+02 },
    { 1.9382000000000000e-07, 2.5621066216699999e+00, 1.0593819301891999e+03 },
    { 1.5477000000000000e-07, 4.1258340495499999e+00, 2.8571808082200000e+01 },
    { 1.4126000000000001e-07, 2.2710540142200002e+00, 9.8899988524600005e+01 },
    { 1.4021000000000001e-07, 4.1268578050600002e+00, 3.5685355082999997e+01 },
    { 1.3798000000000000e-07, 4.8517497655000001e+00, 5.3680451209540001e+02 },
    { 1.2665000000000001e-07, 1.8949376113200000e+00, 4.0580716192600001e+01 },
    { 1.2100000000000001e-07, 3.1075206564200002e+00, 1.1143016149680000e+02 },
    { 1.2100000000000001e-07, 5.4632180394200001e+00, 5.2257741809380002e+02 },
    { 1.1332000000000000e-07, 1.8966939941400001e+00, 4.7694263193399998e+01 },
    { 1.0241000000000000e-07, 1.8889072613100000e+00, 1.3703302416240001e+02 },
    { 9.2640000000000001e-08, 5.3024587640299998e+00, 5.9378908332000000e+00 },
    { 7.4740000000000006e-08, 4.9693428418999996e+00, 1.1291463420510000e+02 },
    { 6.0080000000000006e-08, 9.2667374095999999e-01, 7.0328180442399997e+01 },
};

static const VSOP87Term VSOP87A_NEPTUNE_Z2[11] = {
    { 1.2648400000000001e-05, 1.9140149899200001e+00, 3.8133035637799999e+01 },
    { 1.3034600000000001e-06, 1.1272883339399999e+00, 3.6648562929500002e+01 },
    { 1.2799300000000000e-06, 4.7724113932799996e+00, 3.9617508346100003e+01 },
    { 6.3369999999999996e-07, 4.3731567151200004e+00, 7.6266071275599998e+01 },
    { 3.7203999999999999e-07, 8.9140628560000004e-02, 2.1329909543799999e+02 },
    { 3.3276000000000002e-07, 3.1415926535900001e+00, 0.0000000000000000e+00 },
    { 2.7192000000000002e-07, 3.0128070477000000e+00, 5.2969096509459996e+02 },
    { 1.2004000000000001e-07, 1.7095255297100000e+00, 3.5164090221199999e+01 },
    { 9.3040000000000005e-08, 4.2301080878199997e+00, 4.1101981054399999e+01 },
    { 6.5210000000000000e-08, 7.7866473080999998e-01, 2.2041264243879999e+02 },
    { 5.6529999999999997e-08, 4.3353068666699999e+00, 2.0618554843720000e+02 },
};

static const VSOP87Term VSOP87A_NEPTUNE_Z3[2] = {
    { 1.2422200000000001e-06, 3.0692891146200001e+00, 3.8133035637799999e+01 },
    { 4.4379999999999998e-08, 3.9800891303900001e+00, 3.6648562929500002e+01 },
};

static const VSOP87Term VSOP87A_NEPTUNE_Z4[1] = {
    { 0.0, 0.0, 0.0 }
};

static const VSOP87Term VSOP87A_NEPTUNE_Z5[1] = {
    { 0.0, 0.0, 0.0 }
};

static const VSOP87Block VSOP87A_NEPTUNE_X[6] = {
    { VSOP87A_NEPTUNE_X0, 772 },
    { VSOP87A_NEPTUNE_X1, 330 },
    { VSOP87A_NEPTUNE_X2, 102 },
    { VSOP87A_NEPTUNE_X3, 33 },
    { VSOP87A_NEPTUNE_X4, 7 },
    { VSOP87A_NEPTUNE_X5, 1 },
};

static const VSOP87Block VSOP87A_NEPTUNE_Y[6] = {
    { VSOP87A_NEPTUNE_Y0, 746 },
    { VSOP87A_NEPTUNE_Y1, 325 },
    { VSOP87A_NEPTUNE_Y2, 97 },
    { VSOP87A_NEPTUNE_Y3, 34 },
    { VSOP87A_NEPTUNE_Y4, 7 },
    { VSOP87A_NEPTUNE_Y5, 1 },
};

static const VSOP87Block VSOP87A_NEPTUNE_Z[6] = {
    { VSOP87A_NEPTUNE_Z0, 133 },
    { VSOP87A_NEPTUNE_Z1, 37 },
    { VSOP87A_NEPTUNE_Z2, 11 },
    { VSOP87A_NEPTUNE_Z3, 2 },
    { VSOP87A_NEPTUNE_Z4, 1 },
    { VSOP87A_NEPTUNE_Z5, 1 },
};

static const int32 VSOP87A_NEPTUNE_TRUNC_COUNT[11][3][6] = {
    {
        { 772, 330, 102, 33, 7, 0 },
        { 746, 325, 97, 34, 7, 0 },
        { 133, 37, 11, 2, 0, 0 },
    },
    {
        { 772, 330, 102, 33, 7, 0 },
        { 746, 325, 97, 34, 7, 0 },
        { 133, 37, 11, 2, 0, 0 },
    },
    {
        { 411, 158, 51, 13, 0, 0 },
        { 401, 158, 51, 13, 0, 0 },
        { 125, 37, 11, 1, 0, 0 },
    },
    {
        { 308, 117, 34, 12, 0, 0 },
        { 303, 110, 35, 12, 0, 0 },
        { 94, 34, 8, 1, 0, 0 },
    },
    {
        { 173, 52, 19, 5, 0, 0 },
        { 169, 50, 19, 5, 0, 0 },
        { 53, 17, 4, 1, 0, 0 },
    },
    {
        { 137, 45, 14, 4, 0, 0 },
        { 132, 42, 15, 4, 0, 0 },
        { 41, 12, 3, 1, 0, 0 },
    },
    {
        { 73, 26, 5, 0, 0, 0 },
        { 70, 26, 5, 0, 0, 0 },
        { 20, 8, 1, 0, 0, 0 },
    },
    {
        { 64, 15, 5, 0, 0, 0 },
        { 63, 15, 5, 0, 0, 0 },
        { 18, 5, 1, 0, 0, 0 },
    },
    {
        { 27, 8, 1, 0, 0, 0 },
        { 25, 7, 0, 0, 0, 0 },
        { 13, 4, 0, 0, 0, 0 },
    },
    {
        { 22, 7, 0, 0, 0, 0 },
        { 21, 7, 0, 0, 0, 0 },
        { 12, 1, 0, 0, 0, 0 },
    },
    {
        { 14, 3, 0, 0, 0, 0 },
        { 14, 4, 0, 0, 0, 0 },
        { 5, 1, 0, 0, 0, 0 },
    },
};

