// Copyright Epic Games, Inc. All Rights Reserved.

// Auto-generated VSOP87A header for SATURN (rectangular XYZ, J2000)
// Source: IMCCE/BDL VSOP87A.saturn file
// Generated by vsop87a_header_gen_imcce.py

#pragma once

#include "VSOP87A_Common.h"

static const VSOP87Term VSOP87A_SATURN_X0[1652] = {
    { 9.5163833579699997e+00, 8.7441380794000001e-01, 2.1329909543799999e+02 },
    { 2.6412374238000003e-01, 1.2390892620000001e-01, 4.2659819087599999e+02 },
    { 6.7604303389999995e-02, 4.1676714577800000e+00, 2.0618554843720000e+02 },
    { 6.6242601149999997e-02, 7.5094737779999998e-01, 2.2041264243879999e+02 },
    { 4.2447978169999998e-02, 0.0000000000000000e+00, 0.0000000000000000e+00 },
    { 2.3363404880000001e-02, 2.0222778467300002e+00, 7.1135470007999997e+00 },
    { 1.2553722470000001e-02, 2.1733891773099998e+00, 1.1020632121940000e+02 },
    { 1.1156844670000001e-02, 3.1568687837699998e+00, 4.1948464387519999e+02 },
    { 1.0976832319999999e-02, 5.6575333725599997e+00, 6.3989728631399998e+02 },
    { 7.1632848100000000e-03, 2.7114999370800001e+00, 3.1639186965660002e+02 },
    { 5.0931336499999997e-03, 4.9586562477999996e+00, 1.0309277421860000e+02 },
    { 4.3399443900000003e-03, 7.2012820974000002e-01, 5.2969096509459996e+02 },
    { 3.7289446099999998e-03, 1.3719549699999999e-03, 4.3371173787679999e+02 },
    { 9.7843523000000010e-04, 1.0148575041700001e+00, 3.2350541665740002e+02 },
    { 8.3782316000000003e-04, 6.2038893701999998e-01, 2.2752618943959999e+02 },
    { 8.0600536000000002e-04, 5.6210397979600000e+00, 1.1045700263900001e+01 },
    { 7.4150224000000002e-04, 2.3820606665500001e+00, 6.3278373931320004e+02 },
    { 7.0219382000000002e-04, 8.8789752415000001e-01, 2.0936694217490000e+02 },
    { 6.8855792000000002e-04, 4.0178809762699998e+00, 2.1723124870109999e+02 },
    { 6.5620467000000000e-04, 2.6972859333899999e+00, 2.0225339517410001e+02 },
    { 5.8297910999999996e-04, 2.1615525139899998e+00, 2.2434479570190001e+02 },
    { 5.4022836999999995e-04, 4.9092818437399997e+00, 8.5319638175199998e+02 },
    { 4.5550446000000003e-04, 1.8823503783000000e+00, 1.4227094001599999e+01 },
    { 4.4551703000000003e-04, 5.6076355353500000e+00, 6.3735898303399999e+01 },
    { 3.8345666999999999e-04, 4.3981550147800004e+00, 1.9907200143640000e+02 },
    { 3.1253048999999998e-04, 4.6297660183299998e+00, 7.3587651353180001e+02 },
    { 2.5491373999999997e-04, 1.6392242318100001e+00, 1.1731986822019999e+02 },
    { 2.5165185000000000e-04, 3.7800582256999998e-01, 2.1648048917570000e+02 },
    { 2.4805815000000000e-04, 5.5032767673300000e+00, 7.4781598567299994e+01 },
    { 2.4677050000000001e-04, 5.6038938242000000e+00, 4.1555249061209997e+02 },
    { 2.4673218999999999e-04, 5.9089157385000002e+00, 5.2257741809380002e+02 },
    { 2.4554498999999999e-04, 4.5315059809499996e+00, 2.1011770170029999e+02 },
    { 2.3372466999999999e-04, 5.5349198727599997e+00, 6.4701083331480004e+02 },
    { 2.3355467999999999e-04, 1.8791490123999999e-01, 1.4956319713459999e+02 },
    { 1.4731702999999999e-04, 4.6798190983800003e+00, 2.7703499374140000e+02 },
    { 1.2427525000000000e-04, 1.0299554574600001e+00, 1.0593819301891999e+03 },
    { 1.2393513999999999e-04, 4.1974762282100002e+00, 4.9033408917940000e+02 },
    { 1.2026472000000000e-04, 5.6637228283900001e+00, 3.5181659230870002e+02 },
    { 9.9433290000000002e-05, 8.4628387595999999e-01, 3.9321532631000000e+00 },
    { 9.0870930000000000e-05, 4.3350532676200002e+00, 1.0522683831884001e+03 },
    { 8.2220140000000005e-05, 2.4787530110399998e+00, 7.4299006053259995e+02 },
    { 7.1616709999999996e-05, 2.1815275173800002e+00, 9.5979227217800002e+01 },
    { 6.7177409999999999e-05, 5.5189746099699999e+00, 8.3896928775039999e+02 },
    { 6.3211010000000002e-05, 8.3915408770000000e-01, 3.0927832265580003e+02 },
    { 6.2329989999999994e-05, 2.4583775801500001e+00, 8.4608283475120004e+02 },
    { 6.0749579999999997e-05, 6.1590589733099996e+00, 4.4082528487759998e+02 },
    { 5.7758020000000000e-05, 5.3071769522899999e+00, 3.8133035637799999e+01 },
    { 5.3438939999999997e-05, 3.6004627359799999e+00, 4.1237109687440000e+02 },
    { 5.1941780000000003e-05, 4.5458446768599998e+00, 2.1085141488319999e+02 },
    { 5.1524740000000001e-05, 3.4669517150000001e-01, 2.1574677599280000e+02 },
    { 4.8605820000000003e-05, 2.6461045174999998e-01, 5.3680451209540001e+02 },
    { 4.1116950000000003e-05, 5.9615315304600003e+00, 1.3703302416240001e+02 },
    { 3.7925400000000001e-05, 5.9976656898299998e+00, 4.2266603761290003e+02 },
    { 3.7628340000000003e-05, 3.7211292022600002e+00, 2.1233588759150001e+02 },
    { 3.7474329999999997e-05, 1.1696513771399999e+00, 2.1426230328450001e+02 },
    { 3.3633229999999997e-05, 1.4208958668600000e+00, 4.3764389113990001e+02 },
    { 3.1693899999999998e-05, 5.7664040898800000e+00, 7.6266071275599998e+01 },
    { 3.1283250000000002e-05, 4.0548397655299997e+00, 1.3686602528450001e+03 },
    { 3.1145760000000000e-05, 8.4631897292000002e-01, 2.1325091132820000e+02 },
    { 3.1136410000000003e-05, 4.0441036719000003e+00, 2.1334727954780001e+02 },
    { 2.9904210000000001e-05, 4.1488068519999997e-02, 6.2567019231239999e+02 },
    { 2.9897779999999999e-05, 4.1567383660399999e+00, 1.0664954771900000e+03 },
    { 2.9664500000000000e-05, 5.3956882004600004e+00, 1.3851749687070000e+02 },
    { 2.9126760000000001e-05, 1.7689357710600000e+00, 3.1813937377000001e+00 },
    { 2.8865990000000000e-05, 1.1405792261900001e+00, 8.5827298831199997e+01 },
    { 2.8275270000000001e-05, 7.3252555641999995e-01, 3.3061896365820002e+02 },
    { 2.7133540000000000e-05, 9.6812639712000004e-01, 2.0373786788240000e+02 },
    { 2.6340750000000000e-05, 5.4064520152100002e+00, 2.8808069400530002e+02 },
    { 2.6186340000000000e-05, 5.4933483709799997e+00, 1.2747179660680000e+02 },
    { 2.5277460000000000e-05, 5.0975206838100000e+00, 6.2885158605009997e+02 },
    { 2.5074150000000001e-05, 1.4944713803800000e+00, 9.5612275556000004e+00 },
    { 2.4701809999999999e-05, 5.2743587005599997e+00, 1.1553611574070001e+03 },
    { 2.4276260000000001e-05, 3.9731121423100002e+00, 2.2286032299359999e+02 },
    { 2.3352700000000001e-05, 5.8679107251599998e+00, 3.8846515523820000e+02 },
    { 2.3090760000000000e-05, 3.6782143824700002e+00, 4.3053034413910001e+02 },
    { 2.1626289999999999e-05, 3.2695111990100001e+00, 3.4077089204480001e+02 },
    { 2.0953660000000000e-05, 3.5575908975599999e+00, 4.2341679713830001e+02 },
    { 1.9639469999999998e-05, 2.0508648718000000e+00, 1.4844727083000000e+00 },
    { 1.6359749999999999e-05, 2.1996886978000001e+00, 2.1277783057619999e+02 },
    { 1.6342620000000000e-05, 4.5435776753899999e+00, 1.2530172972200001e+01 },
    { 1.6327589999999999e-05, 2.6916482216500000e+00, 2.1382036029980000e+02 },
    { 1.5384250000000001e-05, 4.7854407708500002e+00, 8.6030992875280003e+02 },
    { 1.4986890000000000e-05, 3.6016805712900002e+00, 5.2690198039499997e+01 },
    { 1.4858559999999999e-05, 5.6550146340799996e+00, 9.4917560896980001e+02 },
    { 1.4612170000000000e-05, 5.9245674383600004e+00, 4.2977958461370002e+02 },
    { 1.4181160000000000e-05, 5.4141999359900002e+00, 3.5033211960040001e+02 },
    { 1.1734560000000000e-05, 3.7959168720799998e+00, 1.6850521225016000e+03 },
    { 1.1505950000000000e-05, 3.7470716001900000e+00, 2.0076892246579999e+02 },
    { 1.1476070000000000e-05, 1.9147238521000001e-01, 9.4206206196899996e+02 },
    { 1.1183690000000000e-05, 3.4662414958299999e+00, 7.0363318461740005e+02 },
    { 1.1117029999999999e-05, 4.7907488492000000e-01, 2.3463973644039999e+02 },
    { 1.0922440000000000e-05, 1.1356110774899999e+00, 1.7394221952280000e+02 },
    { 1.0894500000000000e-05, 5.8269067270999999e+00, 3.6286229257259998e+02 },
    { 1.0195660000000000e-05, 5.0070781102900002e+00, 1.4717530270636000e+03 },
    { 1.0091900000000001e-05, 1.3428948776100000e+00, 2.6598929347749998e+02 },
    { 1.0077610000000001e-05, 1.0896437132800001e+00, 2.2582926841020000e+02 },
    { 9.7707800000000000e-06, 6.0815669546500004e+00, 5.1546387109299997e+02 },
    { 9.1852499999999999e-06, 4.5769712521299999e+00, 5.6363121503840000e+02 },
    { 9.1843299999999999e-06, 4.8614949008500004e+00, 5.6511568774670002e+02 },
    { 8.8145899999999996e-06, 1.4722225262599999e+00, 2.0910630974399999e+02 },
    { 8.6798999999999997e-06, 3.2360340037799999e+00, 2.1037833413120001e+02 },
    { 8.6795200000000002e-06, 1.6516853661199999e+00, 2.1621985674480001e+02 },
    { 8.6506699999999996e-06, 3.4143225697299999e+00, 2.1749188113200000e+02 },
    { 8.5996899999999993e-06, 3.7093543814199998e+00, 2.5265597135319999e+02 },
    { 8.3314100000000002e-06, 2.9625645775399998e+00, 1.6060889739850001e+02 },
    { 8.2964099999999997e-06, 5.8167216895999996e-01, 2.1254833591260001e+02 },
    { 8.2503300000000000e-06, 4.3073433045599998e+00, 2.1404985496340001e+02 },
    { 7.9860499999999995e-06, 1.3539261002600000e+00, 2.2091400527800001e+01 },
    { 7.9840100000000008e-06, 2.0160925744000000e+00, 2.0788246946660001e+02 },
    { 7.8928199999999998e-06, 2.3960587226999999e-01, 2.2359403617650000e+02 },
    { 7.8794400000000006e-06, 2.1848404528100001e+00, 8.8865680217000005e+01 },
    { 7.8093500000000000e-06, 4.8385914845700002e+00, 7.2876296653099996e+02 },
    { 7.6843200000000000e-06, 2.8964807712899998e+00, 2.1871572140940000e+02 },
    { 7.6067000000000004e-06, 1.7254741859100000e+00, 2.1340641002400002e+01 },
    { 7.3135000000000004e-06, 5.9081539509000001e+00, 1.2655674786264001e+03 },
    { 7.2748500000000003e-06, 3.0850251851999999e-01, 1.1624747044077999e+03 },
    { 7.1889400000000002e-06, 1.9588301637100001e+00, 9.5628915597059995e+02 },
    { 7.1032699999999998e-06, 3.4170925641199998e+00, 4.1703696332039999e+02 },
    { 7.0853900000000001e-06, 9.1058417437000005e-01, 2.0767002114549999e+02 },
    { 6.9828800000000000e-06, 3.9606094967000000e-01, 6.2251425595100002e+01 },
    { 6.9675200000000003e-06, 5.7191090814000001e-01, 1.2443341522100000e+02 },
    { 6.9210399999999997e-06, 3.9771911483100002e+00, 2.1892816973050000e+02 },
    { 6.6558099999999997e-06, 2.8243444543699998e+00, 4.2511371816769997e+02 },
    { 6.5713099999999999e-06, 1.4019443291600000e+00, 6.5220371011699996e+01 },
    { 6.4707300000000001e-06, 2.9587294380400002e+00, 9.9160620955499994e+01 },
    { 6.3356699999999998e-06, 6.2324545017400004e+00, 7.0849445304200003e+01 },
    { 6.2451800000000002e-06, 2.7960342836700001e+00, 1.8159247264699999e+01 },
    { 6.1397500000000000e-06, 2.4566220856799998e+00, 1.4244965013379999e+02 },
    { 6.1322499999999996e-06, 4.4218708064600003e+00, 2.1796496188399999e+02 },
    { 5.9929200000000004e-06, 3.8568506214400000e+00, 5.5406998748280000e+02 },
    { 5.8115000000000002e-06, 2.0788043467800001e+00, 2.3145834270270001e+02 },
    { 5.4871200000000001e-06, 4.7707901323000002e-01, 2.0863322899200000e+02 },
    { 5.3043500000000003e-06, 2.5300524583300001e+00, 2.1478356814630001e+02 },
    { 5.1728400000000001e-06, 2.1633183808199998e+00, 2.1181462272970001e+02 },
    { 5.1608200000000003e-06, 2.5892913031400000e+00, 4.0843894361129998e+02 },
    { 5.1347400000000004e-06, 3.9337122032999999e-01, 4.1406801790380001e+02 },
    { 5.0394999999999998e-06, 6.2046055830800002e+00, 1.2584539316256000e+03 },
    { 5.0218500000000000e-06, 4.3700407152100000e+00, 2.5143213107579999e+02 },
    { 4.9397700000000001e-06, 3.4470358866000000e-01, 3.0216477565500003e+02 },
    { 4.6539200000000002e-06, 5.4075008393799999e+00, 6.5412438031559998e+02 },
    { 4.6486400000000001e-06, 3.5448085493299999e+00, 2.0014439921582000e+03 },
    { 4.4842500000000003e-06, 1.3567439226700000e+00, 1.5890728952838001e+03 },
    { 4.4747700000000004e-06, 4.7154244042800002e+00, 2.0300415469949999e+02 },
    { 4.4630500000000004e-06, 4.7440193434599998e+00, 1.7881448967202000e+03 },
    { 4.1609799999999999e-06, 5.9105145426599996e+00, 1.9120769491019999e+02 },
    { 4.1356300000000002e-06, 3.6358697430100002e+00, 1.4807872442630000e+02 },
    { 4.0246499999999997e-06, 4.2957119566999999e+00, 1.7787437278589999e+02 },
    { 4.0029000000000002e-06, 3.1587682493800000e+00, 4.0450679034820001e+02 },
    { 3.9960600000000003e-06, 3.2397325815500002e+00, 2.1318722085339999e+02 },
    { 3.9960600000000003e-06, 1.6495750800100000e+00, 2.1341097002260000e+02 },
    { 3.9512700000000002e-06, 8.8237892220000003e-02, 1.4788665740644001e+03 },
    { 3.8438399999999998e-06, 5.6554151877500001e+00, 1.5819593482830001e+03 },
    { 3.8279999999999996e-06, 2.5651791184900001e+00, 5.6622351302600002e+01 },
    { 3.8153100000000004e-06, 1.3961654463099999e+00, 3.1219908396260001e+02 },
    { 3.6977199999999999e-06, 5.3992464301599998e+00, 1.5104766984290001e+02 },
    { 3.6764300000000001e-06, 4.6605451886299996e+00, 1.1413847448250000e+02 },
    { 3.6730400000000001e-06, 5.8865028226499998e+00, 7.5010360753340001e+02 },
    { 3.5112099999999998e-06, 5.8640515554199997e+00, 9.8899988524600005e+01 },
    { 3.5029299999999999e-06, 5.0025570534100003e+00, 1.9832124191099999e+02 },
    { 3.4750199999999999e-06, 5.7686180345600002e+00, 1.9513984817330001e+02 },
    { 3.4734099999999999e-06, 6.1908132148999995e-01, 2.4872381809010000e+02 },
    { 3.3712000000000000e-06, 2.9624188349899998e+00, 4.7928838891549998e+02 },
    { 3.3578799999999999e-06, 1.0299330377100000e+00, 2.1351154375909999e+02 },
    { 3.3578799999999999e-06, 3.8593746238600000e+00, 2.1308664711690000e+02 },
    { 3.1940599999999998e-06, 7.2194326719000002e-01, 1.7516605980020000e+02 },
    { 3.1864100000000000e-06, 5.7007339841400002e+00, 8.3185574074960005e+02 },
    { 3.1726899999999999e-06, 6.1539994670700002e+00, 2.2827694896500000e+02 },
    { 3.1524099999999999e-06, 1.9823427881900000e+00, 1.0627416795630000e+02 },
    { 3.0483900000000000e-06, 1.7537900948500000e+00, 6.2830758499914000e+03 },
    { 3.0333200000000000e-06, 1.0898195645200000e+00, 4.8322054217860000e+02 },
    { 2.9635900000000000e-06, 1.5193332933100001e+00, 1.0294940738499999e+01 },
    { 2.9291800000000001e-06, 3.8651408098600002e+00, 4.2415051032119999e+02 },
    { 2.8082900000000000e-06, 3.7730992239600001e+00, 2.4476805547999998e+00 },
    { 2.7297200000000000e-06, 4.5556435217000004e+00, 1.0451548361876000e+03 },
    { 2.7029800000000001e-06, 5.1389267926500004e+00, 6.3596513305090002e+02 },
    { 2.6937999999999999e-06, 1.3030115979500001e+00, 2.7851946644970002e+02 },
    { 2.6901599999999998e-06, 2.1520392181500001e+00, 3.2743756992049998e+02 },
    { 2.6885400000000000e-06, 5.6698530612800004e+00, 2.0522234059069999e+02 },
    { 2.6838500000000001e-06, 4.7511831412400003e+00, 1.9195845443560000e+02 },
    { 2.6664099999999999e-06, 3.4964639410500000e+00, 1.2125202148330000e+02 },
    { 2.5247000000000002e-06, 1.6753959779100001e+00, 2.1355972786890001e+02 },
    { 2.5247000000000002e-06, 3.2139120877299998e+00, 2.1303846300710001e+02 },
    { 2.4120599999999999e-06, 5.7490529953099996e+00, 2.2137585028530000e+02 },
    { 2.3980000000000002e-06, 5.9273130806000003e+00, 1.5748458012822000e+03 },
    { 2.3693700000000002e-06, 4.9603279542500003e+00, 2.3539049596580000e+02 },
    { 2.3682399999999999e-06, 3.5739305282900000e+00, 2.1944943459230001e+02 },
    { 2.3409200000000000e-06, 5.7818629886900004e+00, 2.7555052103309998e+02 },
    { 2.3086199999999999e-06, 4.0381126908500002e+00, 4.9744763618019999e+02 },
    { 2.2458100000000000e-06, 1.9308820203599999e+00, 3.9356875915200000e+01 },
    { 2.1815900000000001e-06, 1.6490801684000000e+00, 2.6992144674060000e+02 },
    { 2.1716600000000001e-06, 6.1450126857300003e+00, 4.2904587143079999e+02 },
    { 2.1523199999999998e-06, 2.9787277254400002e+00, 4.2563498302950001e+02 },
    { 2.1398600000000002e-06, 2.4487178570400001e+00, 8.0767548473000002e+00 },
    { 2.1287699999999999e-06, 6.7438396717000004e-01, 6.5094298657790000e+02 },
    { 2.1144399999999999e-06, 2.1509635090000000e-02, 5.4391805909619995e+02 },
    { 2.0493099999999999e-06, 4.4634169984099996e+00, 8.4215068148809996e+02 },
    { 2.0351799999999999e-06, 4.4836100313099996e+00, 2.1045367663768002e+03 },
    { 2.0326400000000001e-06, 1.1594207911500001e+00, 1.3757737998458001e+03 },
    { 2.0264000000000002e-06, 2.9171527821400001e+00, 2.8414854074220000e+02 },
    { 1.9465200000000000e-06, 5.2673677853000003e+00, 4.8884961647109998e+02 },
    { 1.9323699999999999e-06, 4.1996861723699999e+00, 1.8892007304980001e+02 },
    { 1.9271100000000000e-06, 5.3362279089399998e+00, 1.8983512179396000e+03 },
    { 1.9128999999999999e-06, 3.2998417465599998e+00, 2.3178358618148000e+03 },
    { 1.8819600000000001e-06, 1.3410807460400000e+00, 2.0714875628370001e+02 },
    { 1.8802199999999999e-06, 5.0637735109000004e-01, 4.2756139872249997e+02 },
    { 1.8561600000000001e-06, 5.1236795211299997e+00, 6.0176425067620005e+02 },
    { 1.8254800000000000e-06, 1.2468511731900001e+00, 2.1059078245230000e+02 },
    { 1.8088200000000001e-06, 3.0808254678800000e+00, 8.4342826122899993e+01 },
    { 1.8079000000000001e-06, 4.9362087940000000e-02, 4.9181856188770001e+02 },
    { 1.7953000000000000e-06, 3.6427671918300000e+00, 2.1600740842370001e+02 },
    { 1.7719200000000001e-06, 3.1759679801199998e+00, 1.0213285546211000e+04 },
    { 1.7698899999999999e-06, 2.5944122678200001e+00, 3.4470304530790003e+02 },
    { 1.7636800000000000e-06, 5.0228212448999998e-01, 6.1855664531160005e+02 },
    { 1.7457399999999999e-06, 3.2907736666199998e+00, 4.2664637498579998e+02 },
    { 1.7456900000000000e-06, 9.2978841689999994e-02, 4.2655000676620000e+02 },
    { 1.7150399999999999e-06, 3.3200782816499999e+00, 7.3297125859000005e+01 },
    { 1.7003799999999999e-06, 6.2199202226299999e+00, 1.7952584437210000e+03 },
    { 1.6947099999999999e-06, 4.7802141355699996e+00, 5.0137978944330001e+02 },
    { 1.6697300000000001e-06, 4.3218577088399996e+00, 3.5574874557179999e+02 },
    { 1.6644900000000001e-06, 3.4089244005300001e+00, 1.2797945726280000e+03 },
    { 1.6351499999999999e-06, 7.6931838572000000e-01, 3.1245971639350000e+02 },
    { 1.5968500000000000e-06, 5.5337667819799998e+00, 3.5893013930950002e+02 },
    { 1.5803400000000000e-06, 3.4293687680799998e+00, 1.7935884549420001e+02 },
    { 1.5614100000000001e-06, 6.5756469164999998e-01, 2.3767811782620001e+02 },
    { 1.5578999999999999e-06, 4.9923774208299996e+00, 2.8956516671359998e+02 },
    { 1.5167600000000000e-06, 4.1375494273499998e+00, 2.1345915413239999e+02 },
    { 1.5167600000000000e-06, 7.5207673923999996e-01, 2.1313903674360000e+02 },
    { 1.5143200000000000e-06, 3.9051345673000002e+00, 2.2046082654860001e+02 },
    { 1.4905500000000001e-06, 5.6725783280100002e+00, 1.0702492748170000e+02 },
    { 1.4661799999999999e-06, 7.2929682139999996e-01, 2.2036445832900000e+02 },
    { 1.4633100000000001e-06, 6.1235904315600003e+00, 5.4166259714000002e+00 },
    { 1.3927900000000000e-06, 2.7124313033599998e+00, 6.3671589257630001e+02 },
    { 1.3814700000000000e-06, 3.1994094117300000e+00, 4.3615941843159999e+02 },
    { 1.3709000000000001e-06, 5.4265513033600001e+00, 1.1482476104062000e+03 },
    { 1.3422499999999999e-06, 5.6586986423000001e+00, 4.6658664459999999e+00 },
    { 1.3360100000000001e-06, 2.7500496741400000e+00, 3.5424722652100002e+01 },
    { 1.3179800000000000e-06, 2.5008353963399998e+00, 2.1202707105080000e+02 },
    { 1.3071100000000000e-06, 2.3885074674800002e+00, 2.1457111982519999e+02 },
    { 1.2903299999999999e-06, 1.4619897108900000e+00, 2.4723934538180001e+02 },
    { 1.2877300000000000e-06, 2.3342296404399998e+00, 7.8713751830400000e+01 },
    { 1.2765300000000000e-06, 5.1274916096099998e+00, 1.3458534360760001e+02 },
    { 1.2606900000000000e-06, 4.3718080635599996e+00, 1.8105576652360000e+02 },
    { 1.2596100000000000e-06, 3.3511796417899999e+00, 6.4382943957709995e+02 },
    { 1.2243099999999999e-06, 1.7060553556400000e+00, 6.2173803904930003e+02 },
    { 1.2232899999999999e-06, 1.9665057788600000e+00, 6.9152524274800001e+01 },
    { 1.1885999999999999e-06, 5.6503318872600001e+00, 1.8912376709388000e+03 },
    { 1.1824100000000000e-06, 2.1405704605799998e+00, 1.2726810256271999e+03 },
    { 1.1776500000000001e-06, 3.7585958449599999e+00, 3.2243328914400003e+01 },
    { 1.1681099999999999e-06, 1.6114583222100001e+00, 1.1338771495710000e+02 },
    { 1.1226600000000000e-06, 5.9751798782999999e-01, 1.6921656695024001e+03 },
    { 1.1103100000000000e-06, 1.0159846449400001e+00, 2.0613736432740001e+02 },
    { 1.1025300000000000e-06, 7.1569456241999996e-01, 2.4554242435239999e+02 },
    { 1.0910199999999999e-06, 5.4715285419399997e+00, 1.1439910691340000e+02 },
    { 1.0863699999999999e-06, 1.8162347538800001e+00, 7.6736908292080000e+02 },
    { 1.0604499999999999e-06, 4.1848328448599998e+00, 2.0623373254699999e+02 },
    { 1.0583000000000001e-06, 1.4564912819300000e+00, 2.1989137757699999e+02 },
    { 1.0531800000000000e-06, 5.2270655213000001e+00, 5.6290742925000004e+00 },
    { 1.0437099999999999e-06, 4.0331705466200001e+00, 1.0736090241908000e+03 },
    { 1.0374499999999999e-06, 4.3039373226200004e+00, 1.3615467058442000e+03 },
    { 1.0285300000000000e-06, 1.6581383922699999e+00, 6.7668051566499997e+01 },
    { 1.0040300000000000e-06, 3.5416658639700000e+00, 1.4000196957899999e+02 },
    { 9.9943999999999996e-07, 2.6599794101200001e+00, 1.4977853527000001e+01 },
    { 9.9527999999999990e-07, 1.4229807566499999e+00, 4.2607692601420001e+02 },
    { 9.8974999999999993e-07, 1.5563966093299999e+00, 4.1927856940000003e+00 },
    { 9.8219000000000004e-07, 4.4587854865999998e-01, 1.4414657116320001e+02 },
    { 9.7708000000000001e-07, 5.0283471776499997e+00, 2.2147430875962000e+03 },
    { 9.6530000000000010e-07, 6.0292361609400000e+00, 4.4793883187839998e+02 },
    { 9.5112999999999997e-07, 5.1700213224700002e+00, 6.4307868005169996e+02 },
    { 9.4974000000000003e-07, 4.2306025838399997e+00, 2.4209286360333999e+03 },
    { 9.1948999999999998e-07, 3.6046036442100000e+00, 2.0670681329900000e+02 },
    { 9.1320999999999996e-07, 1.9380776212299999e+00, 4.2711945573780002e+02 },
    { 9.1119999999999995e-07, 3.8188319798400001e+00, 7.7693031047640000e+02 },
    { 8.8219000000000004e-07, 5.6068693170000003e-02, 8.7311771539500000e+01 },
    { 8.6235999999999995e-07, 2.7267801209200000e+00, 9.1693228005540004e+02 },
    { 8.5499999999999997e-07, 1.6192461207400000e+00, 5.4174670747800000e+01 },
    { 8.3020000000000001e-07, 4.2350284882000002e+00, 1.5349535039770001e+02 },
    { 8.1885000000000000e-07, 3.2518196436200002e+00, 2.0606927819500001e+01 },
    { 8.1874000000000004e-07, 1.5977639495100000e+00, 3.5330106501699998e+02 },
    { 8.1777000000000005e-07, 4.5251993450799999e+00, 5.2549817940059995e+02 },
    { 8.1063999999999996e-07, 2.7454326222999997e-01, 1.2598732389849999e+02 },
    { 8.0268000000000004e-07, 3.0659556750100001e+00, 2.6342277314714001e+03 },
    { 7.9774000000000005e-07, 4.8351409101399998e+00, 1.5912442469019999e+02 },
    { 7.9482000000000002e-07, 2.7761066199400002e+00, 3.4788443904560000e+02 },
    { 7.8912000000000001e-07, 6.1870540963699998e+00, 2.9912639426919998e+02 },
    { 7.8548999999999997e-07, 3.2481508266399999e+00, 1.4014645680499999e+01 },
    { 7.7163999999999999e-07, 9.0106050330999998e-01, 3.2972519178089999e+02 },
    { 7.6087000000000001e-07, 3.4743859063999999e-01, 9.3494851496820002e+02 },
    { 7.5802000000000001e-07, 1.2158000594800000e+00, 4.6470422915999997e+01 },
    { 7.4886999999999996e-07, 6.1824409548299997e+00, 4.2096911658350001e+02 },
    { 7.4336000000000001e-07, 6.0723802886399998e+00, 2.1116503133776000e+03 },
    { 7.4135999999999996e-07, 4.0312500231500001e+00, 9.2047073954699997e+01 },
    { 7.3234000000000001e-07, 1.4203013690299999e+00, 2.8659622129700000e+02 },
    { 7.1694999999999997e-07, 2.4550052299899998e+00, 6.3033605875839999e+02 },
    { 7.1327999999999999e-07, 4.2572684530600000e+00, 9.6320784650000002e-01 },
    { 7.1095000000000005e-07, 6.1861658312000003e+00, 2.1160217440860001e+02 },
    { 7.0197999999999999e-07, 2.7291135399999999e-02, 3.3773251065900001e+02 },
    { 6.9650999999999998e-07, 1.4340046428200000e+00, 2.4379022388199999e+01 },
    { 6.9325999999999995e-07, 5.9775045066799999e+00, 2.1207525516059999e+02 },
    { 6.9325999999999995e-07, 5.1953069670899996e+00, 2.1452293571540000e+02 },
    { 6.8787000000000003e-07, 3.4130660458199999e+00, 1.1642609634290000e+02 },
    { 6.8652999999999999e-07, 5.9467909971999999e-01, 2.7310284047829998e+02 },
    { 6.6705999999999999e-07, 2.5609600473100000e+00, 2.2093390730060000e+02 },
    { 6.6535000000000001e-07, 1.6025744556999999e-01, 6.9651963761659999e+02 },
    { 6.6130999999999998e-07, 2.3596491451900001e+00, 4.2367742956920000e+02 },
    { 6.6015000000000004e-07, 4.0803642503699997e+00, 7.7841478318470001e+02 },
    { 6.5853000000000001e-07, 4.3513924080799997e+00, 3.3928641933649999e+02 },
    { 6.5292000000000005e-07, 5.6780134539800002e+00, 6.1780588578619995e+02 },
    { 6.4837000000000003e-07, 4.8294919015400000e+00, 6.1503391543000001e+00 },
    { 6.4557000000000003e-07, 2.8622924030000002e-01, 1.8498791978669999e+02 },
    { 6.4147000000000004e-07, 5.1569260693399999e+00, 1.4646394800628000e+03 },
    { 6.3715999999999997e-07, 5.7643181503900003e+00, 4.2193232442999999e+02 },
    { 6.3516999999999999e-07, 2.4008836824499999e+00, 3.9117346822389999e+02 },
    { 6.3336000000000000e-07, 5.0793425374699996e+00, 5.7616138801060004e+02 },
    { 6.2748000000000000e-07, 2.8205912075100001e+00, 2.9207613067999998e+00 },
    { 6.2412000000000001e-07, 6.0239719416900002e+00, 6.9258748435350003e+02 },
    { 6.1437000000000001e-07, 7.3813400987000000e-01, 1.8227960680100000e+02 },
    { 6.1091999999999997e-07, 1.5139777698800001e+00, 3.0534616939270001e+02 },
    { 6.0126000000000003e-07, 3.3113202745999998e-01, 4.3912836384820002e+02 },
    { 6.0050000000000001e-07, 5.3831268949600002e+00, 2.2076295405954002e+03 },
    { 5.9861000000000004e-07, 3.4528475193000001e+00, 4.8736514376280002e+02 },
    { 5.9577999999999995e-07, 4.6881984128500003e+00, 5.1205725331200000e+01 },
    { 5.9304999999999998e-07, 8.1375446389999995e-01, 2.9689454166000000e+00 },
    { 5.9022999999999996e-07, 4.9784916743699998e+00, 2.1499601646740001e+02 },
    { 5.8090000000000002e-07, 3.7800545827100001e+00, 4.1630325013750002e+02 },
    { 5.7131000000000000e-07, 3.9613436340299999e+00, 5.4302428721889999e+02 },
    { 5.6865000000000005e-07, 1.3079177224200000e+00, 4.4475743814070000e+02 },
    { 5.5947000000000002e-07, 2.0861528119299999e+00, 6.3841281360569997e+02 },
    { 5.5799999999999999e-07, 1.9287633553700001e+00, 3.4225536475310003e+02 },
    { 5.5588000000000003e-07, 1.5853991794300000e+00, 3.8724131496080003e+02 },
    { 5.5585000000000004e-07, 5.0024227947600002e+00, 2.2116340196420001e+02 },
    { 5.5380000000000000e-07, 2.9751819798699999e+00, 5.9803745040300001e+01 },
    { 5.5245000000000000e-07, 5.7808839527900000e+00, 4.3689313161450002e+02 },
    { 5.5031000000000001e-07, 2.6168405189700001e+00, 4.1852143602870001e+02 },
    { 5.4842000000000004e-07, 1.3848914610800001e+00, 5.1864526483069994e+02 },
    { 5.4700000000000001e-07, 4.5426023067200001e+00, 2.4161027108930000e+02 },
    { 5.4255000000000000e-07, 6.0993046758700000e+00, 2.6747376618580000e+02 },
    { 5.3989000000000005e-07, 6.1905057872900002e+00, 6.2736711334179995e+02 },
    { 5.3639000000000000e-07, 3.1963785921399999e+00, 1.2895626931510000e+02 },
    { 5.3435000000000002e-07, 4.4970458165200000e+00, 1.6682867252200001e+02 },
    { 5.3260999999999995e-07, 6.0429284298199999e+00, 2.0543478891180001e+02 },
    { 5.2274999999999999e-07, 1.9384784912099999e+00, 7.0653628910000004e+00 },
    { 5.2099999999999997e-07, 3.3510689831000001e-01, 2.0085575391590000e+03 },
    { 5.1908999999999997e-07, 4.3024047968900003e+00, 1.9269216761850001e+02 },
    { 5.1528000000000005e-07, 5.0271828029799996e+00, 1.9928444975750000e+02 },
    { 5.1282000000000001e-07, 1.4834622208899999e+00, 1.1695882514085999e+03 },
    { 5.0882999999999998e-07, 1.1877602458500001e+00, 3.1019488636999998e+01 },
    { 5.0427000000000000e-07, 5.1529812040799996e+00, 2.1186280683950000e+02 },
    { 5.0427000000000000e-07, 6.0198302696899999e+00, 2.1473538403649999e+02 },
    { 5.0144000000000002e-07, 6.2096093495400000e+00, 3.3406124266997999e+03 },
    { 4.9871000000000005e-07, 4.7307436359399997e+00, 2.5311349572528002e+03 },
    { 4.9622000000000003e-07, 2.6447781431099999e+00, 4.3079097657000000e+02 },
    { 4.8709999999999996e-07, 8.9401835269999996e-01, 4.2951895218279998e+02 },
    { 4.8658000000000001e-07, 5.1524025726599998e+00, 7.1617311105999999e+00 },
    { 4.8643999999999995e-07, 3.1662793557300000e+00, 9.6872999095099999e+01 },
    { 4.7988999999999999e-07, 2.7885843572000002e-01, 3.3365734504400001e+02 },
    { 4.7761000000000005e-07, 1.5155944660300000e+00, 5.5800214074589996e+02 },
    { 4.7716000000000002e-07, 3.9295661784000002e-01, 2.1966188291340001e+02 },
    { 4.7450000000000002e-07, 3.0841159502300002e+00, 3.2032402291969998e+02 },
    { 4.6921000000000000e-07, 5.7314145189400003e+00, 2.9201284726839998e+02 },
    { 4.6524000000000000e-07, 4.1495717033300004e+00, 2.8245161971279998e+02 },
    { 4.6354999999999999e-07, 3.1085338245199998e+00, 4.3222726516850003e+02 },
    { 4.6208999999999998e-07, 6.1116353577200000e+00, 4.2584743135060000e+02 },
    { 4.6113000000000000e-07, 4.0510462523399999e+00, 2.1419286731529999e+02 },
    { 4.6113000000000000e-07, 8.3857991425000000e-01, 2.1240532356070000e+02 },
    { 4.5774000000000002e-07, 2.3373362411000000e+00, 2.0566428357539999e+02 },
    { 4.5564000000000002e-07, 1.2521481781999999e+00, 2.7406604832480002e+02 },
    { 4.5109000000000000e-07, 6.0645913383999996e+00, 2.2731374111849999e+02 },
    { 4.5028999999999999e-07, 2.0321161584400000e+00, 4.2808266358430001e+02 },
    { 4.4695999999999997e-07, 3.3872299150999998e+00, 6.5922821389999999e+00 },
    { 4.4465000000000000e-07, 3.9665668180300000e+00, 2.7373205056900001e+03 },
    { 4.2678000000000001e-07, 3.8797222509700000e+00, 1.3554855145409999e+02 },
    { 4.2668000000000001e-07, 6.0521165065000004e-01, 4.2118156490460001e+02 },
    { 4.1662999999999999e-07, 2.5054594276700000e+00, 1.2035824960600000e+02 },
    { 4.0965999999999998e-07, 4.0990357898800003e+00, 1.6779385755008000e+03 },
    { 4.0436000000000000e-07, 4.4978856734000000e-01, 2.3390602325750001e+02 },
    { 3.9681999999999998e-07, 4.6310951820000001e-02, 7.3879727483859995e+02 },
    { 3.9377000000000001e-07, 2.1957897994600000e+00, 3.7390799283650000e+02 },
    { 3.9340999999999997e-07, 1.8890518828500000e+00, 4.5884151979040001e+02 },
    { 3.8934000000000002e-07, 2.4543134716699999e+00, 2.6450482076920002e+02 },
    { 3.8627999999999999e-07, 6.2390946427799996e+00, 1.4563104387150000e+02 },
    { 3.8453000000000002e-07, 5.9953003671399996e+00, 2.0915449385380001e+02 },
    { 3.8118999999999999e-07, 6.0302951024600002e+00, 3.1957326339430000e+02 },
    { 3.7614999999999999e-07, 1.0667439867000000e-01, 4.8077286162380000e+02 },
    { 3.7471000000000000e-07, 5.1776070548800002e+00, 2.1744369702220001e+02 },
    { 3.7352999999999998e-07, 6.6478489890000006e-02, 1.2513403846248000e+03 },
    { 3.7230000000000002e-07, 1.5725179308299999e+00, 4.1644497775600001e+01 },
    { 3.7132999999999998e-07, 4.5082899735800002e+00, 2.0693630796260001e+02 },
    { 3.6768000000000002e-07, 1.9158849897500001e+00, 3.3940249943799998e+01 },
    { 3.6698000000000002e-07, 3.6860410199300002e+00, 4.3126405732199999e+02 },
    { 3.6628000000000002e-07, 3.6678732406200001e+00, 4.2734895040139997e+02 },
    { 3.6077000000000002e-07, 4.6555496472800000e+00, 8.6742347575359997e+02 },
    { 3.5511000000000001e-07, 4.2588703310599998e+00, 2.4431858407499999e+02 },
    { 3.5400000000000002e-07, 2.8475002416500002e+00, 3.8135160823740000e+02 },
    { 3.5074999999999999e-07, 2.6666716100999999e-01, 2.3575873236100001e+01 },
    { 3.5021000000000001e-07, 1.1152454687900000e+00, 3.9617508346100003e+01 },
    { 3.4546000000000002e-07, 1.9005207025199999e+00, 7.7750543983900002e+01 },
    { 3.4121999999999998e-07, 4.0144235206000001e-01, 2.5976951835400001e+02 },
    { 3.3972999999999999e-07, 2.8278599595100000e+00, 2.9506196011279999e+03 },
    { 3.3780999999999998e-07, 1.3173955223700000e+00, 7.5632338265689998e+02 },
    { 3.3625000000000001e-07, 5.7783588175900000e+00, 3.9557870223899999e+02 },
    { 3.3612000000000002e-07, 1.6878990449000000e+00, 1.8180652604900001e+02 },
    { 3.3486000000000002e-07, 2.4098218011100001e+00, 2.8311175651300001e+01 },
    { 3.3480000000000001e-07, 2.3001465468600002e+00, 2.7087335373900000e+01 },
    { 3.3393000000000003e-07, 5.9096123005800001e+00, 2.4280421830342002e+03 },
    { 3.3248000000000002e-07, 1.8817859551300000e+00, 1.6209337010679999e+02 },
    { 3.2930999999999998e-07, 2.3144846356000000e+00, 8.1752133216199994e+01 },
    { 3.2066000000000001e-07, 3.3369551339900001e+00, 4.6595506679120001e+02 },
    { 3.1110999999999998e-07, 1.4024133608500000e+00, 7.5075952540000002e-01 },
    { 3.1091999999999998e-07, 4.8876351629499997e+00, 1.7810313497193999e+03 },
    { 3.1019000000000000e-07, 2.9605920958800001e+00, 1.2991947716160001e+02 },
    { 3.0492000000000001e-07, 1.1169530582400000e+00, 2.8096714700450002e+02 },
    { 3.0344999999999998e-07, 1.2603402641100001e+00, 4.5490936652729999e+02 },
    { 3.0270999999999999e-07, 2.8978975228300001e+00, 4.1162033734900001e+02 },
    { 3.0230000000000000e-07, 5.1028701022299998e+00, 2.5240214102519999e+03 },
    { 2.9988000000000001e-07, 3.3228292839600000e+00, 7.1074673161819999e+02 },
    { 2.9377000000000002e-07, 6.1872705349799997e+00, 1.3154696222179999e+02 },
    { 2.9306000000000001e-07, 3.5542081126900000e+00, 4.6473122651379998e+02 },
    { 2.8753999999999999e-07, 1.5838275010000000e-01, 1.9285222631290000e+02 },
    { 2.8444999999999998e-07, 2.4754383166999999e-01, 5.3362311835770004e+02 },
    { 2.8048999999999999e-07, 2.3056640750700002e+00, 1.3140394986990000e+02 },
    { 2.7836000000000001e-07, 1.1768168575500000e+00, 2.1626804085460000e+02 },
    { 2.7329999999999999e-07, 4.5639730840000003e-02, 3.6648562929500002e+01 },
    { 2.7324999999999999e-07, 8.5706476636999995e-01, 6.0766952886799999e+01 },
    { 2.7189999999999999e-07, 5.0401284868899996e+00, 2.1494362684070001e+02 },
    { 2.7189999999999999e-07, 6.1326829868799999e+00, 2.1165456403530001e+02 },
    { 2.7047000000000001e-07, 2.9254320832500000e+00, 3.6137781986430002e+02 },
    { 2.6934000000000000e-07, 5.5096001121200002e+00, 2.5494359321360000e+02 },
    { 2.6548000000000002e-07, 1.6041454871500000e+00, 4.0599126305649997e+02 },
    { 2.6346000000000000e-07, 3.7369238889099998e+00, 5.2126486179999998e-01 },
    { 2.6332000000000000e-07, 4.8243309132999999e+00, 1.6674774556399999e+01 },
    { 2.6319000000000002e-07, 3.5785518545200001e+00, 2.1033015002139999e+02 },
    { 2.6273999999999998e-07, 6.3012531276999995e-01, 4.5576651038700000e+01 },
    { 2.5983999999999998e-07, 4.4763976287000001e-01, 4.9426624244250002e+02 },
    { 2.5913000000000002e-07, 3.2469359723200002e+00, 2.4479166482700001e+02 },
    { 2.5645000000000000e-07, 3.8629260593499999e+00, 2.2997386999439999e+02 },
    { 2.5545999999999999e-07, 1.6881720125999999e+00, 4.1258354519549999e+02 },
    { 2.5541999999999999e-07, 5.7736289155799998e+00, 6.8154178408960001e+02 },
    { 2.5306000000000002e-07, 4.7312430250100004e+00, 2.3374596456309999e+02 },
    { 2.4828000000000000e-07, 2.3329135830000000e-02, 5.0835032409220003e+02 },
    { 2.4741000000000002e-07, 4.4451946330199998e+00, 2.8475268269093999e+03 },
    { 2.4674000000000000e-07, 4.2339266793399997e+00, 4.6209790485100001e+01 },
    { 2.4555000000000002e-07, 1.0872457906999999e-01, 2.3249494088155998e+03 },
    { 2.4389000000000000e-07, 1.4770118444400000e+00, 4.2240540518199998e+02 },
    { 2.4254000000000000e-07, 4.1596062297099996e+00, 1.4122580985639999e+02 },
    { 2.4191999999999999e-07, 3.4568002888999999e-01, 4.2044785172169998e+02 },
    { 2.4170000000000000e-07, 5.5881092550500000e+00, 1.8472728735580000e+02 },
    { 2.4128000000000000e-07, 4.6136087947000002e+00, 3.1492569389000000e+01 },
    { 2.4074999999999998e-07, 4.6793687760799996e+00, 2.5887574647669999e+02 },
    { 2.3952999999999998e-07, 2.5422728288099998e+00, 3.3455111692129998e+02 },
    { 2.3733000000000000e-07, 6.0769663750399996e+00, 4.6202291352809999e+02 },
    { 2.3601000000000001e-07, 1.5442544559000000e+00, 5.5651766803759995e+02 },
    { 2.3510999999999999e-07, 4.0578451477999999e-01, 2.1249594628590000e+02 },
    { 2.3510999999999999e-07, 4.4838416518099997e+00, 2.1410224459010001e+02 },
    { 2.3255000000000000e-07, 6.1207020086600004e+00, 5.9342686339800002e+02 },
    { 2.3225000000000001e-07, 1.3232439373900000e+00, 2.0707932031449999e+02 },
    { 2.3221000000000001e-07, 1.4116577781799999e+00, 7.3194436026870005e+02 },
    { 2.3031000000000000e-07, 2.9383931300000000e-03, 6.6704843720000000e+01 },
    { 2.2951000000000000e-07, 4.1549925382200001e+00, 9.2940845831999994e+01 },
    { 2.2709000000000000e-07, 1.4922934568999999e-01, 4.5097721326419997e+02 },
    { 2.2695999999999999e-07, 1.5297483602199999e+00, 2.2333340374560001e+02 },
    { 2.2399000000000001e-07, 3.5641946100299999e+00, 2.1951887056149999e+02 },
    { 2.2333999999999999e-07, 4.0624434477999998e+00, 4.2362924545940001e+02 },
    { 2.2198000000000001e-07, 2.4888390119900001e+00, 4.2648631629139999e+02 },
    { 2.2198000000000001e-07, 8.9868151046000000e-01, 4.2671006546059999e+02 },
    { 2.2193999999999999e-07, 3.1158561886900000e+00, 3.1321047591889999e+02 },
    { 2.2078000000000000e-07, 3.5391629793399999e+00, 1.8621176006409999e+02 },
    { 2.1570000000000001e-07, 3.2067177216600000e+00, 2.2460542813280000e+02 },
    { 2.1556999999999999e-07, 2.6033799906400001e+00, 4.3201481684740003e+02 },
    { 2.1493999999999999e-07, 3.3708996727099998e+00, 2.0639799675830000e+02 },
    { 2.1262999999999999e-07, 2.3821440110099998e+00, 3.4321857259960001e+02 },
    { 2.1218000000000001e-07, 5.3432040765600002e+00, 9.9911380480899993e+01 },
    { 2.1047000000000000e-07, 5.5780332634100001e+00, 2.4187090352019999e+02 },
    { 2.1022000000000001e-07, 3.7106405304800001e+00, 4.0221916848780000e+02 },
    { 2.0842000000000001e-07, 5.7371300643599996e+00, 1.0775864066459999e+02 },
    { 2.0830000000000001e-07, 3.2344417149400000e+00, 1.7542669223109999e+02 },
    { 2.0720999999999999e-07, 4.4261379845900004e+00, 8.4926422848890002e+02 },
    { 2.0697000000000001e-07, 3.6564193680699999e+00, 7.6348118626000003e+00 },
    { 2.0615000000000000e-07, 1.8389692907499999e+00, 2.1187638603783998e+03 },
    { 2.0562000000000001e-07, 5.2439952797500000e+00, 9.6340270297140000e+02 },
    { 2.0541000000000001e-07, 7.6737119207000004e-01, 2.8537238101960003e+02 },
    { 2.0538000000000000e-07, 3.1293088935000002e+00, 1.0601353552540000e+02 },
    { 2.0486000000000000e-07, 3.2996742294599999e+00, 1.5667674413540001e+02 },
    { 2.0403000000000001e-07, 5.2278523371799999e+00, 1.6289651925890001e+02 },
    { 2.0367000000000000e-07, 1.1582799753899999e+00, 1.1265400177420000e+02 },
    { 2.0123999999999999e-07, 3.7179230531099998e+00, 3.0537123753465999e+03 },
    { 2.0118999999999999e-07, 2.8596332308100001e+00, 3.5744566660120000e+02 },
    { 1.9924000000000000e-07, 2.9983759834899999e+00, 7.2073285581600004e+01 },
    { 1.9901999999999999e-07, 6.0767126868599997e+00, 1.5677322542813999e+03 },
    { 1.9787000000000001e-07, 4.9250564176699996e+00, 5.5258551477449998e+02 },
    { 1.9649999999999999e-07, 3.1485777431000002e+00, 5.6118353448360006e+02 },
    { 1.9469999999999999e-07, 1.8162267773300000e+00, 4.6824268865160002e+02 },
    { 1.9427999999999999e-07, 2.0697432324000000e+00, 3.9265794093220001e+02 },
    { 1.9076000000000000e-07, 3.3136377875099998e+00, 9.0588657979150003e+02 },
    { 1.8946999999999999e-07, 3.0728024733199999e+00, 6.3744960575920004e+02 },
    { 1.8918999999999999e-07, 5.0292398240100002e+00, 1.7165459766239999e+02 },
    { 1.8901000000000000e-07, 4.7080091290799997e+00, 5.7222923474749996e+02 },
    { 1.8705000000000000e-07, 4.2772330423399998e+00, 1.5253214255120000e+02 },
    { 1.8677000000000000e-07, 1.3390501885899999e+00, 8.3503713448730002e+02 },
    { 1.8652999999999999e-07, 2.7903946815000003e-01, 4.2681063919709999e+02 },
    { 1.8652999999999999e-07, 3.1084810542999999e+00, 4.2638574255489999e+02 },
    { 1.8381999999999999e-07, 1.7592760666800000e+00, 2.2020019411769999e+02 },
    { 1.8335000000000001e-07, 6.2478994797300000e+00, 1.7001006625970001e+02 },
    { 1.8316000000000001e-07, 5.5518217108799996e+00, 1.0038446123289999e+02 },
    { 1.8099000000000001e-07, 6.2507709368200004e+00, 2.3070758317729999e+02 },
    { 1.8078000000000001e-07, 3.6099820654500001e+00, 4.0525754987360000e+02 },
    { 1.8055000000000001e-07, 5.6501731997000004e-01, 1.8972322220190000e+02 },
    { 1.7985000000000001e-07, 1.3323055624500000e+00, 2.4038643081190000e+02 },
    { 1.7958000000000000e-07, 4.2186912579199998e+00, 4.0302231763990000e+02 },
    { 1.7882000000000001e-07, 4.3720442964400004e+00, 8.2858353414600003e+01 },
    { 1.7751999999999999e-07, 4.5899710886899996e+00, 7.0214871190910003e+02 },
    { 1.7749000000000001e-07, 6.2408714999999997e-02, 5.6904784100979998e+02 },
    { 1.7688999999999999e-07, 3.4857150254000002e-01, 2.9505122865419997e+02 },
    { 1.7651000000000001e-07, 6.2548250413200002e+00, 4.6053844081979997e+02 },
    { 1.7585000000000000e-07, 6.4170468840999995e-01, 5.4695644048199995e+02 },
    { 1.7354999999999999e-07, 2.4641865297700001e+00, 5.0402576179100002e+01 },
    { 1.7326000000000000e-07, 2.9167716727599999e+00, 1.9980571461930001e+02 },
    { 1.7268999999999999e-07, 5.2773884467699999e+00, 2.9519424100610001e+02 },
    { 1.7268000000000001e-07, 6.0963252180299996e+00, 1.9996577331370000e+02 },
    { 1.7119000000000001e-07, 4.6222683542000000e+00, 1.4659425171800001e+02 },
    { 1.6962000000000001e-07, 1.4572312738400000e+00, 5.5555446019110002e+02 },
    { 1.6934000000000001e-07, 4.9886288683600002e+00, 1.0924311337290000e+02 },
    { 1.6906999999999999e-07, 3.0836750140000001e-02, 7.7962992305000000e+01 },
    { 1.6807000000000000e-07, 2.1911908142000001e+00, 1.7245774681450001e+02 },
    { 1.6675999999999999e-07, 5.5784322337900001e+00, 7.0511765732569995e+02 },
    { 1.6595000000000000e-07, 4.7702049620500002e+00, 3.6434676528090000e+02 },
    { 1.6443000000000000e-07, 5.0589206430400004e+00, 2.2663241756229999e+02 },
    { 1.6425000000000001e-07, 5.6138026649399997e+00, 7.5721715453419995e+02 },
    { 1.6413000000000000e-07, 2.7537043049599998e+00, 2.2189711514710001e+02 },
    { 1.6402999999999999e-07, 3.2211702045299999e+00, 4.1953282798499998e+02 },
    { 1.6383000000000001e-07, 2.6738924100000001e+00, 2.0389792657679999e+02 },
    { 1.6374999999999999e-07, 3.9479995562300001e+00, 1.9943304451573999e+03 },
    { 1.6318000000000001e-07, 5.8601710582799997e+00, 2.6370167161709998e+02 },
    { 1.6222000000000001e-07, 5.7391312952500000e+00, 2.7444340526907999e+03 },
    { 1.6187000000000001e-07, 4.3883421950000001e-01, 4.0992341631959999e+02 },
    { 1.6152999999999999e-07, 6.0213499811000002e+00, 4.1053796944600002e+01 },
    { 1.6070999999999999e-07, 4.6438646964999997e+00, 2.0974232193759999e+03 },
    { 1.6053000000000000e-07, 1.0750930535500001e+00, 2.8571808082200000e+01 },
    { 1.5979000000000000e-07, 8.8810024478000005e-01, 8.3110498122419995e+02 },
    { 1.5970000000000001e-07, 2.3762331515100001e+00, 1.1116952906589999e+02 },
    { 1.5926000000000001e-07, 1.9728543233600000e+00, 2.2679247625670001e+02 },
    { 1.5886000000000001e-07, 4.8294361508800003e+00, 2.8404132799086001e+03 },
    { 1.5863000000000001e-07, 4.3635478466900004e+00, 5.7074476203920005e+02 },
    { 1.5800000000000001e-07, 5.6202312429999998e-02, 4.1943645976540000e+02 },
    { 1.5794000000000000e-07, 1.6643607405300000e+00, 2.5117149864490000e+02 },
    { 1.5755000000000000e-07, 2.9880085317399998e+00, 3.4884764689209999e+02 },
    { 1.5737000000000001e-07, 4.6442774751999999e-01, 5.2575881183150000e+02 },
    { 1.5689000000000000e-07, 7.5909996738999996e-01, 1.0064509366380000e+02 },
    { 1.5648000000000001e-07, 3.1376826869499999e+00, 2.1129335786790000e+02 },
    { 1.5648000000000001e-07, 1.7519434796400000e+00, 2.1530483300809999e+02 },
    { 1.5566999999999999e-07, 3.1139773458300000e+00, 2.2030076785419999e+02 },
    { 1.5461999999999999e-07, 2.2149431534700001e+00, 2.2270026429920000e+02 },
    { 1.5270000000000001e-07, 2.6650152364999999e+00, 8.1895145568100006e+01 },
    { 1.5174000000000001e-07, 4.9507863215799999e+00, 2.5658812461629998e+02 },
    { 1.5062999999999999e-07, 5.7450605175699998e+00, 4.4157604440300003e+02 },
    { 1.5012000000000000e-07, 1.5218864555400000e+00, 2.2052451702339999e+02 },
    { 1.4952000000000000e-07, 1.9032460128699999e+00, 4.2000590873700003e+02 },
    { 1.4822000000000001e-07, 1.6951755141500000e+00, 1.9577298761969999e+02 },
    { 1.4819000000000001e-07, 9.2519710715000003e-01, 9.8066817835879999e+02 },
    { 1.4810000000000001e-07, 1.6896026283300001e+00, 5.4073666535849998e+02 },
    { 1.4742000000000001e-07, 3.7777293163699999e+00, 1.0554497769261000e+03 },
    { 1.4697000000000000e-07, 2.8360306836900002e+00, 4.3274853003030000e+02 },
    { 1.4483999999999999e-07, 2.1414764159600002e+00, 1.1015813710960001e+02 },
    { 1.4446000000000001e-07, 6.0041684307700001e+00, 4.5761767951299998e+02 },
    { 1.4434000000000001e-07, 3.7766927622300002e+00, 3.8698068252989998e+02 },
    { 1.4392999999999999e-07, 5.3381372433400003e+00, 1.1025450532920000e+02 },
    { 1.4376000000000001e-07, 4.1654980970200004e+00, 4.4868959140380002e+02 },
    { 1.4368000000000000e-07, 2.6036310620499998e+00, 3.2670114707846001e+03 },
    { 1.4257000000000001e-07, 5.8436741640100003e+00, 4.2538269652899999e+01 },
    { 1.4079999999999999e-07, 7.7300407987999997e-01, 1.3333322124300000e+01 },
    { 1.4016000000000001e-07, 9.2450223186000002e-01, 4.2685882330689998e+02 },
    { 1.4016000000000001e-07, 2.4630183416799998e+00, 4.2633755844510000e+02 },
    { 1.3957000000000000e-07, 6.1756248373600000e+00, 8.6424208201590000e+02 },
    { 1.3829999999999999e-07, 5.0021440473399998e+00, 4.3467494572330003e+02 },
    { 1.3773999999999999e-07, 3.9824171579300001e+00, 4.7780391620720002e+02 },
    { 1.3740000000000000e-07, 4.3824323152900000e+00, 8.1506334611420004e+02 },
    { 1.3532000000000000e-07, 5.2332314437100003e+00, 6.6253120356299996e+02 },
    { 1.3428999999999999e-07, 6.6730550149000001e-01, 1.9365537546499999e+02 },
    { 1.3407000000000001e-07, 3.1896741077600002e+00, 2.3082520325630000e+02 },
    { 1.3350000000000000e-07, 2.2265510711799998e+00, 6.3893407846749994e+02 },
    { 1.3229000000000000e-07, 2.3757298766999999e+00, 1.4859801210651999e+03 },
    { 1.3220999999999999e-07, 2.1540304057999999e+00, 1.9054647649404001e+03 },
    { 1.3215999999999999e-07, 4.2367282439900000e+00, 1.8816931352440000e+02 },
    { 1.2977999999999999e-07, 2.9727985191000000e-01, 2.4175328344120001e+02 },
    { 1.2853000000000000e-07, 9.1490823924999998e-01, 1.9603362005060001e+02 },
    { 1.2802999999999999e-07, 5.5681963949100002e+00, 6.4234496686880004e+02 },
    { 1.2667000000000001e-07, 4.1993909747400000e+00, 3.1639186965660001e+03 },
    { 1.2578000000000001e-07, 1.8826591265599999e+00, 2.3857188970350001e+02 },
    { 1.2471999999999999e-07, 3.0838193922100001e+00, 8.9484087952760001e+02 },
    { 1.2407000000000000e-07, 4.0376106118799999e+00, 9.7676148247200004e+01 },
    { 1.2326000000000001e-07, 4.0103447110700001e+00, 2.3056457082540001e+02 },
    { 1.2270999999999999e-07, 4.8456399700000002e-01, 3.2668681039510000e+02 },
    { 1.2125000000000000e-07, 4.1150088727399998e+00, 3.8012776796000003e+02 },
    { 1.2006000000000000e-07, 4.2070374919800004e+00, 1.9662432088160000e+02 },
    { 1.2003000000000000e-07, 2.7246883529099999e+00, 2.5414044406150001e+02 },
    { 1.1891000000000000e-07, 3.3685007120099999e+00, 2.0326478713040001e+02 },
    { 1.1400000000000000e-07, 5.3292450417200001e+00, 1.1976754877499999e+02 },
    { 1.1396999999999999e-07, 3.4981755483999999e-01, 2.8000393915799998e+02 },
    { 1.1290000000000000e-07, 4.8181047169999998e+00, 3.8994962794650002e+02 },
    { 1.1233000000000000e-07, 6.0895177366500004e+00, 6.4086049416050002e+02 },
    { 1.1219000000000000e-07, 8.9544366061000003e-01, 4.5368552624990002e+02 },
    { 1.1206000000000000e-07, 4.1214116421600000e+00, 7.1467888488129995e+02 },
    { 1.1180000000000000e-07, 6.1784801713800004e+00, 2.6413412784722000e+03 },
    { 1.1049000000000001e-07, 1.5327881480000000e-01, 3.9739324334740002e+02 },
    { 1.1046000000000000e-07, 2.4713547849000000e-01, 5.2650957135689998e+02 },
    { 1.0975000000000000e-07, 5.5243138426000005e-01, 3.9510562148700001e+02 },
    { 1.0907000000000000e-07, 6.0747645811200002e+00, 1.8647239249500001e+02 },
    { 1.0842000000000000e-07, 2.5433958202800002e+00, 6.3994547042379997e+02 },
    { 1.0842000000000000e-07, 5.6287928214100003e+00, 6.3984910220419999e+02 },
    { 1.0793000000000000e-07, 5.7973745204200000e+00, 1.8841241239379999e+03 },
    { 1.0721000000000001e-07, 5.0085936036899996e+00, 1.7526107818300002e+01 },
    { 1.0607999999999999e-07, 4.6115569662900002e+00, 1.9122455111200001e+01 },
    { 1.0504000000000001e-07, 1.7759293709199999e+00, 2.0629742302180000e+02 },
    { 1.0470000000000000e-07, 5.0530075966799997e+00, 2.4012579838100001e+02 },
    { 1.0426000000000000e-07, 1.2140064557900001e+00, 8.2005928096030004e+02 },
    { 1.0384000000000000e-07, 4.0281851370000003e-02, 1.9683676920270000e+02 },
    { 1.0355000000000000e-07, 2.0204787038499998e+00, 2.0470107572890001e+02 },
    { 1.0335000000000000e-07, 1.8161116119000000e+00, 2.0199276274319999e+02 },
    { 1.0302000000000001e-07, 2.5986586426199998e+00, 1.0728555991259999e+02 },
    { 1.0104000000000000e-07, 9.7178466244999995e-01, 6.9364972595899999e+01 },
    { 1.0066000000000000e-07, 4.9580184665999999e-01, 4.2388987789030000e+02 },
    { 1.0066000000000000e-07, 2.8917187268800002e+00, 4.2930650386169998e+02 },
    { 1.0048000000000000e-07, 3.4620949247000001e+00, 3.3701042450032000e+03 },
    { 9.8850000000000002e-08, 2.7054189156900001e+00, 4.7694263193399998e+01 },
    { 9.8850000000000002e-08, 4.3667410428000002e+00, 2.9204947528600002e+01 },
    { 9.8580000000000000e-08, 3.3581251024899998e+00, 3.3137100791700000e+01 },
    { 9.8410000000000006e-08, 2.5884209382900001e+00, 1.3533610313299999e+02 },
    { 9.8200000000000006e-08, 1.9924044111999999e+00, 3.5452490529440001e+02 },
    { 9.7349999999999995e-08, 1.7334325150000000e-02, 4.3366355376700000e+02 },
    { 9.7310000000000002e-08, 1.9536884201200000e+00, 7.2333918012500007e+01 },
    { 9.7110000000000001e-08, 2.6182985586099998e+00, 1.4930936680660000e+03 },
    { 9.6820000000000003e-08, 6.1117610700000000e+00, 4.7217484191469998e+02 },
    { 9.6589999999999993e-08, 6.6282222486999998e-01, 3.8038840039090002e+02 },
    { 9.6530000000000005e-08, 3.8038993823100000e+00, 4.0695447090300001e+02 },
    { 9.6519999999999993e-08, 2.9403468103399999e+00, 1.9417664032680000e+02 },
    { 9.5939999999999998e-08, 3.2054528382599998e+00, 4.3375992198659998e+02 },
    { 9.5809999999999996e-08, 3.8196784927800000e+00, 6.0676018552230005e+02 },
    { 9.5459999999999996e-08, 2.3215658463100000e+00, 2.1914061805160000e+02 },
    { 9.5340000000000005e-08, 3.8618571132700001e+00, 2.3687496867409999e+02 },
    { 9.4689999999999996e-08, 5.9576329453900003e+00, 4.9575071515079998e+02 },
    { 9.4409999999999996e-08, 6.0927123183200003e+00, 4.1800017116689997e+02 },
    { 9.3940000000000006e-08, 1.8534201438100000e+00, 3.1931263096340001e+02 },
    { 9.3740000000000004e-08, 3.3684168945600002e+00, 2.0607367385260000e+02 },
    { 9.3699999999999999e-08, 4.8493440683399998e+00, 2.2976142167329999e+02 },
    { 9.3019999999999995e-08, 1.3743224973199999e+00, 1.4393412284210001e+02 },
    { 9.1780000000000005e-08, 2.4902164404199998e+00, 5.6214674233009998e+02 },
    { 9.1609999999999998e-08, 1.8060547188100000e+00, 5.9465070367540000e+02 },
    { 9.1189999999999998e-08, 6.1418290119999996e+00, 1.1291463420510000e+02 },
    { 9.1110000000000000e-08, 5.9205163082699999e+00, 1.6772244439930000e+02 },
    { 9.1059999999999996e-08, 7.7871223072999995e-01, 2.3842887735159999e+02 },
    { 8.9809999999999995e-08, 9.4573569494999998e-01, 1.8743560034149999e+02 },
    { 8.9640000000000001e-08, 4.6025924678700001e+00, 7.3980866679489998e+02 },
    { 8.9519999999999997e-08, 2.8590238267000001e-01, 1.4096517742550000e+02 },
    { 8.9109999999999995e-08, 1.3402635517000001e+00, 1.9643719973000000e+01 },
    { 8.9029999999999997e-08, 2.4370193561100000e+00, 6.4945851386959998e+02 },
    { 8.8800000000000001e-08, 3.0231611414900001e+00, 1.9040454575810000e+02 },
    { 8.7320000000000004e-08, 6.2411863650899999e+00, 1.5763995198190000e+02 },
    { 8.7130000000000000e-08, 7.7367091578000002e-01, 3.9346109008430000e+02 },
    { 8.6350000000000002e-08, 5.9376429851800001e+00, 8.2474219374879999e+02 },
    { 8.5319999999999999e-08, 4.5761524888400000e+00, 3.1568051495651998e+03 },
    { 8.4769999999999997e-08, 5.5402157593999997e+00, 3.0608259223474001e+03 },
    { 8.4390000000000003e-08, 4.5150039204099999e+00, 5.1153171782990000e+02 },
    { 8.4330000000000001e-08, 3.3864966308200000e+00, 4.2675824957039998e+02 },
    { 8.4330000000000001e-08, 1.0239427200000001e-03, 4.2643813218160000e+02 },
    { 8.4219999999999996e-08, 4.2254129430100003e+00, 5.0286426215159997e+02 },
    { 8.4110000000000003e-08, 5.2495950362999999e-01, 1.7224529849340001e+02 },
    { 8.3999999999999998e-08, 1.5169599877300000e+00, 2.8454188003199999e+01 },
    { 8.3890000000000005e-08, 1.8715582113300000e+00, 2.3242155054919999e+02 },
    { 8.3169999999999996e-08, 5.2791104649899996e+00, 6.6123792731640003e+02 },
    { 8.2459999999999999e-08, 2.2845114116900000e+00, 1.9438908864790000e+02 },
    { 8.0919999999999999e-08, 1.1253127817899999e+00, 6.7214061522839995e+02 },
    { 8.0830000000000003e-08, 1.5472653576900000e+00, 6.1929035849449997e+02 },
    { 8.0719999999999998e-08, 3.3252993392699999e+00, 7.2483081326790000e+02 },
    { 8.0319999999999994e-08, 4.5060372187100004e+00, 8.9759452094300002e+01 },
    { 8.0200000000000003e-08, 2.8376534266300002e+00, 1.8409414790939999e+02 },
    { 7.9990000000000004e-08, 5.7689238306600004e+00, 2.5583736509089999e+02 },
    { 7.9780000000000004e-08, 3.2527051219300001e+00, 3.9435486196160002e+02 },
    { 7.9510000000000002e-08, 1.4562829582300001e+00, 1.0554045477340000e+02 },
    { 7.9220000000000004e-08, 3.2197020342999999e-01, 1.7665053250849999e+02 },
    { 7.8829999999999998e-08, 1.8603221386200000e+00, 2.3619364511789999e+02 },
    { 7.8740000000000002e-08, 4.3811577768300003e+00, 2.4138150890326001e+03 },
    { 7.8009999999999995e-08, 1.8069706045300000e+00, 2.2218566345969998e+03 },
    { 7.7589999999999995e-08, 5.9545004290000003e-01, 3.4373983746139999e+02 },
    { 7.7420000000000001e-08, 3.4415356371000000e-01, 5.5659143456099997e+01 },
    { 7.7410000000000003e-08, 7.6022179542000001e-01, 8.4933526953900000e+01 },
    { 7.7000000000000001e-08, 2.7550159567499999e+00, 8.5712853501510006e+02 },
    { 7.6949999999999998e-08, 2.5807530855300000e+00, 2.0745757282439999e+02 },
    { 7.6899999999999994e-08, 3.9128367207600001e+00, 2.3916259053450000e+02 },
    { 7.5699999999999996e-08, 3.0287293245200000e+00, 9.9022940591439999e+02 },
    { 7.5219999999999995e-08, 5.2928493051299998e+00, 1.9589060769869999e+02 },
    { 7.4530000000000006e-08, 2.1464257846299999e+00, 1.2836556848410001e+02 },
    { 7.4439999999999997e-08, 3.0349436546900002e+00, 2.5272794265500000e+01 },
    { 7.4340000000000003e-08, 4.0072859407900001e+00, 2.5129781913599999e+01 },
    { 7.4099999999999995e-08, 2.8078568772299999e+00, 1.8023719907218001e+03 },
    { 7.3269999999999994e-08, 2.6051144165000002e+00, 2.3220910222809999e+02 },
    { 7.2989999999999994e-08, 1.2975881379600001e+00, 3.6679444583570000e+02 },
    { 7.2979999999999996e-08, 1.7499234667600001e+00, 4.2532616648880003e+02 },
    { 7.2979999999999996e-08, 1.6375971067799999e+00, 4.2787021526320001e+02 },
    { 7.2209999999999996e-08, 3.6587371101700001e+00, 2.5602862665600000e+01 },
    { 7.1770000000000000e-08, 5.8180699773700004e+00, 9.0981873305459999e+02 },
    { 7.1509999999999996e-08, 5.9787784790299998e+00, 3.1368355667089997e+02 },
    { 7.1190000000000004e-08, 4.2973121171199997e+00, 3.2595309721220002e+02 },
    { 7.0970000000000006e-08, 3.8663460387100002e+00, 2.3107223148140001e+03 },
    { 7.0539999999999995e-08, 6.1143554297499998e+00, 3.8405992122309999e+02 },
    { 7.0179999999999997e-08, 2.3741830428099999e+00, 2.9126208774300000e+02 },
    { 6.9349999999999996e-08, 3.2594662747699998e+00, 1.2869081196288000e+03 },
    { 6.9009999999999994e-08, 4.5103358933699997e+00, 2.4994765836750000e+02 },
    { 6.8410000000000002e-08, 2.3290441033899998e+00, 1.4148644228730001e+02 },
    { 6.8210000000000000e-08, 3.2217326476300001e+00, 4.4923210812500002e+02 },
    { 6.7630000000000004e-08, 1.1753477937500001e+00, 1.0483362299252999e+03 },
    { 6.7430000000000002e-08, 1.5803579022100001e+00, 4.1896337901340002e+02 },
    { 6.7029999999999999e-08, 8.0548964241999998e-01, 6.1144309831080000e+02 },
    { 6.6329999999999999e-08, 4.9423286830400004e+00, 2.6895823889410002e+02 },
    { 6.6209999999999995e-08, 5.0361061331199997e+00, 9.5388526386799995e+01 },
    { 6.5979999999999999e-08, 1.2349902615999999e+00, 8.8114920691600005e+01 },
    { 6.5970000000000001e-08, 4.3735256740899997e+00, 2.6087903141574199e+04 },
    { 6.5709999999999997e-08, 2.0329325023500000e+00, 1.1302313754934000e+03 },
    { 6.5079999999999998e-08, 8.0289146979000003e-01, 5.6660016045500004e+02 },
    { 6.4749999999999994e-08, 4.6437795557000001e+00, 7.1812653150700001e+01 },
    { 6.4620000000000006e-08, 5.6846790864700001e+00, 3.6997583957339998e+02 },
    { 6.4329999999999994e-08, 3.9700492783499999e+00, 3.4863519857099999e+02 },
    { 6.3489999999999995e-08, 4.2876397444299998e+00, 3.1394418910180002e+02 },
    { 6.3300000000000004e-08, 3.8853173947399999e+00, 1.1072758608120000e+02 },
    { 6.2989999999999997e-08, 3.5370734386199998e+00, 1.8016199464630000e+02 },
    { 6.2909999999999999e-08, 1.4972407050500001e+00, 1.1791056905120000e+02 },
    { 6.2620000000000001e-08, 1.6334232611199999e+00, 8.5001498801430000e+02 },
    { 6.1900000000000005e-08, 3.3872970067799999e+00, 1.0968505635760000e+02 },
    { 6.1879999999999996e-08, 4.9567067980000001e-01, 7.3905790726949999e+02 },
    { 6.1469999999999994e-08, 6.7579092413999997e-01, 6.3937602145220001e+02 },
    { 6.1230000000000000e-08, 1.0541653358300000e+00, 3.2892204262880000e+02 },
    { 6.0850000000000006e-08, 5.8940873359000001e-01, 4.3319047301500001e+02 },
    { 6.0819999999999998e-08, 3.9854742332200002e+00, 3.4803105662225998e+03 },
    { 6.0819999999999998e-08, 1.1928600578099999e+00, 6.4041855117579996e+02 },
    { 6.0650000000000004e-08, 3.6742825604500000e+00, 5.5137878594299998e+01 },
    { 6.0570000000000006e-08, 4.4290542189100002e+00, 8.5637777548969996e+02 },
    { 6.0390000000000000e-08, 1.0556534687999999e-01, 5.0531194270639998e+02 },
    { 6.0370000000000004e-08, 6.2560982720900000e+00, 5.9735901666109999e+02 },
    { 6.0360000000000006e-08, 3.9078322489000000e-01, 7.6981676347560006e+02 },
    { 6.0279999999999994e-08, 3.3268685930499999e+00, 1.1562294719080001e+02 },
    { 5.9919999999999997e-08, 1.9285412187000001e-01, 3.3683873878169999e+02 },
    { 5.9839999999999998e-08, 5.5125422937300002e+00, 2.2005159935945999e+03 },
    { 5.9580000000000001e-08, 4.9261487098699996e+00, 2.5863495096499999e+01 },
    { 5.9139999999999999e-08, 5.6275439394599998e+00, 3.1071461125430000e+02 },
    { 5.8999999999999999e-08, 2.2855010680699999e+00, 3.5834033404411998e+03 },
    { 5.8980000000000003e-08, 1.6235216239100001e+00, 8.4363515419639998e+02 },
    { 5.8939999999999997e-08, 6.0661151739600001e+00, 1.2099138905240000e+02 },
    { 5.8640000000000001e-08, 5.5332486768700004e+00, 3.8554439393140001e+02 },
    { 5.8190000000000000e-08, 9.1498157369999999e-02, 2.1595922431389999e+02 },
    { 5.8190000000000000e-08, 4.7981280092200000e+00, 2.1063896656209999e+02 },
    { 5.8089999999999999e-08, 4.8193249551499999e+00, 2.5435289238260000e+02 },
    { 5.7760000000000002e-08, 3.7782555533200002e+00, 9.7415515816300001e+01 },
    { 5.7709999999999999e-08, 5.3364449360000003e-02, 1.6454105066160000e+02 },
    { 5.7690000000000002e-08, 1.2847728244500001e+00, 3.2058465535059997e+02 },
    { 5.6939999999999999e-08, 5.4383585346899999e+00, 7.6025553592000006e+02 },
    { 5.6769999999999998e-08, 1.7954228727099999e+00, 2.1840690486870000e+02 },
    { 5.6769999999999998e-08, 3.0942032938800001e+00, 2.0819128600729999e+02 },
    { 5.6610000000000002e-08, 2.5826283364100000e+00, 1.2353964334370001e+02 },
    { 5.6360000000000003e-08, 7.1123925881000005e-01, 4.9989531673499999e+02 },
    { 5.5990000000000000e-08, 4.6857400371900004e+00, 5.8106824010899999e+01 },
    { 5.5990000000000000e-08, 5.5782585076600002e+00, 9.5209637027660006e+02 },
    { 5.5959999999999999e-08, 6.0016699785299998e+00, 1.0872184851110001e+02 },
    { 5.5910000000000002e-08, 2.5479584384399998e+00, 2.8511174858869998e+02 },
    { 5.5840000000000002e-08, 4.0959163091799997e+00, 2.0055647414469999e+02 },
    { 5.5560000000000003e-08, 1.4237004935999999e+00, 4.8640193591629998e+02 },
    { 5.5509999999999999e-08, 4.9446851035100003e+00, 4.9330303459599997e+02 },
    { 5.5150000000000001e-08, 5.6748809600600003e+00, 5.5103160609700001e+02 },
    { 5.5150000000000001e-08, 5.8700129851999998e+00, 4.5516999895819998e+02 },
    { 5.4300000000000003e-08, 2.2084383659900002e+00, 4.8173606947029998e+02 },
    { 5.3610000000000002e-08, 5.9494716254800002e+00, 2.9577331481288002e+03 },
    { 5.3449999999999999e-08, 4.7226799629100000e+00, 3.0508553696180002e+02 },
    { 5.3220000000000003e-08, 3.3047312443500001e+00, 4.8470501488690002e+02 },
    { 5.2479999999999997e-08, 1.6081273961800000e+00, 6.0447256366190004e+02 },
    { 5.2010000000000001e-08, 2.4561306500900000e+00, 7.4692221379570003e+02 },
    { 5.1319999999999999e-08, 4.9995518530499998e+00, 8.8796244247800004e+01 },
    { 5.1289999999999998e-08, 5.4456553379499999e+00, 2.0762183703570000e+02 },
    { 5.1289999999999998e-08, 5.7271561358199996e+00, 2.1897635384029999e+02 },
    { 5.0960000000000001e-08, 4.6933784843100002e+00, 2.6205714021440002e+02 },
    { 5.0629999999999997e-08, 1.6887567425100001e+00, 7.7448262992160005e+02 },
    { 5.0349999999999998e-08, 5.6276653095999996e-01, 6.5805653357870005e+02 },
    { 5.0290000000000002e-08, 9.9189626470000000e-02, 1.9725746032800001e+02 },
    { 5.0250000000000003e-08, 5.9534230434600000e+00, 7.2164941953020002e+02 },
    { 5.0239999999999999e-08, 1.8128124566000001e-01, 8.0672595883600002e+02 },
    { 5.0190000000000001e-08, 5.0774665232200000e+00, 1.0444040766621999e+03 },
    { 5.0160000000000000e-08, 2.0904659716700000e+00, 9.6962247809489998e+02 },
    { 5.0080000000000002e-08, 3.0071348383699998e+00, 1.7265475387399999e+01 },
    { 4.9770000000000002e-08, 4.7809861876299999e+00, 1.7808682110699999e+02 },
    { 4.9360000000000000e-08, 5.9618700033699996e+00, 4.3128970483899998e+01 },
    { 4.9089999999999998e-08, 5.2009569374800000e+00, 2.9823262239190001e+02 },
    { 4.9000000000000002e-08, 2.5528857817000001e+00, 3.7619561469690001e+02 },
    { 4.8900000000000001e-08, 5.5319148923299997e+00, 3.3120966448920001e+02 },
    { 4.8860000000000002e-08, 3.6316666795199999e+00, 2.0151968199120000e+02 },
    { 4.8800000000000000e-08, 6.0377262546999999e-01, 1.2050126195790000e+02 },
    { 4.8489999999999999e-08, 1.5447261656600000e+00, 1.2450285119020000e+02 },
    { 4.8450000000000000e-08, 3.8477541672200002e+00, 2.2604171673130000e+02 },
    { 4.8340000000000001e-08, 2.1003365592300001e+00, 1.0405598206510000e+02 },
    { 4.8109999999999999e-08, 4.6194261014500002e+00, 3.2769820235140003e+02 },
    { 4.7979999999999997e-08, 5.1398456832999995e-01, 1.7076082578510000e+02 },
    { 4.7810000000000003e-08, 3.8875770107099998e+00, 5.0456118318099999e+02 },
    { 4.7699999999999997e-08, 3.1940120974499999e+00, 9.9171387862270001e+02 },
    { 4.7659999999999998e-08, 5.3118094770499997e+00, 4.2490126984659997e+02 },
    { 4.7530000000000003e-08, 4.8235021952399997e+00, 1.0333583763982999e+03 },
    { 4.7370000000000000e-08, 1.8438585895699999e+00, 9.1396333463880001e+02 },
    { 4.7169999999999998e-08, 1.9909649202299999e+00, 1.0457724692690000e+02 },
    { 4.6989999999999999e-08, 5.6443894021099998e+00, 4.3993151300030001e+02 },
    { 4.6980000000000001e-08, 3.1883448021400000e+00, 5.3287235883230005e+02 },
    { 4.6779999999999999e-08, 2.1742631734800000e+00, 1.1143016149680000e+02 },
    { 4.6679999999999998e-08, 2.3117752816900001e+00, 6.0554504565700000e+01 },
    { 4.6499999999999999e-08, 4.4696728510800003e+00, 2.7140591944890002e+02 },
    { 4.6269999999999997e-08, 1.2563525054300000e+00, 7.7523338944700004e+02 },
    { 4.6220000000000000e-08, 3.2197836959299999e+00, 3.6864961146597998e+03 },
    { 4.6180000000000001e-08, 5.4746100133600004e+00, 6.3426821202149995e+02 },
    { 4.5709999999999997e-08, 2.0381543275000000e-01, 5.1939602435610004e+02 },
    { 4.5419999999999999e-08, 1.4582698244500001e+00, 4.9155792945680003e+02 },
    { 4.5359999999999997e-08, 2.7911754656799999e+00, 9.5857677783099996e+02 },
    { 4.5330000000000003e-08, 2.7036069920000001e+00, 1.8727554164710000e+02 },
    { 4.5130000000000001e-08, 2.8543857545600000e+00, 2.6826702943000001e+01 },
    { 4.5090000000000002e-08, 4.7840021453099997e+00, 2.2934073054800001e+02 },
    { 4.5069999999999999e-08, 5.2308164419100001e+00, 7.3269511979410004e+02 },
    { 4.4929999999999999e-08, 5.3598449432499997e+00, 3.3772177920039999e+03 },
    { 4.4589999999999998e-08, 5.5191033909699998e+00, 7.4527768239299996e+02 },
    { 4.4530000000000002e-08, 1.4911706573700001e+00, 1.9848130060540001e+02 },
    { 4.4419999999999997e-08, 1.6292996106500000e+00, 2.5382485042536000e+03 },
    { 4.4129999999999999e-08, 8.6251574445000001e-01, 4.3289029178299998e+01 },
    { 4.3840000000000001e-08, 2.8416811409700000e+00, 1.1312708252620000e+02 },
    { 4.3800000000000002e-08, 4.3114771573799997e+00, 5.2698265210889997e+02 },
    { 4.3719999999999997e-08, 4.2930841678800000e+00, 2.2507850888479999e+02 },
    { 4.3390000000000000e-08, 5.0544060743700001e+00, 6.0887779767699999e+02 },
    { 4.3320000000000000e-08, 1.5752727119200001e+00, 6.3697652500720005e+02 },
    { 4.2879999999999998e-08, 2.1314576299999999e+00, 1.8769623277240001e+02 },
    { 4.2860000000000001e-08, 4.4315601997499998e+00, 1.5961864422845999e+03 },
    { 4.2820000000000002e-08, 5.6536340041199997e+00, 8.4066620877979994e+02 },
    { 4.2510000000000001e-08, 7.0715248383999996e-01, 7.7130123618389996e+02 },
    { 4.2429999999999997e-08, 4.3161811765399998e+00, 3.4731970192218000e+03 },
    { 4.2200000000000001e-08, 4.9410482546500001e+00, 1.0212956637210000e+02 },
    { 4.2050000000000003e-08, 5.6635469868300001e+00, 1.1411340634054000e+03 },
    { 4.1729999999999997e-08, 1.8584322826299999e+00, 2.1101147357759999e+02 },
    { 4.1729999999999997e-08, 3.0311938839599999e+00, 2.1558671729840000e+02 },
    { 4.1540000000000000e-08, 4.9828782912099996e+00, 3.1787634236489998e+02 },
    { 4.1330000000000000e-08, 1.7675113917900001e+00, 1.0304459010879999e+02 },
    { 4.1320000000000002e-08, 1.7227641077900000e+00, 4.3423300273860002e+02 },
    { 4.0770000000000000e-08, 5.0379532151199999e+00, 6.5019222705250002e+02 },
    { 4.0670000000000000e-08, 4.9580102075200001e+00, 1.0314095832840000e+02 },
    { 4.0600000000000000e-08, 3.3984552964199999e+00, 2.2811689027060001e+02 },
    { 4.0550000000000002e-08, 5.8820420846999999e-01, 9.2307706385600000e+01 },
    { 4.0490000000000001e-08, 4.1192418248999996e+00, 3.7242352012819998e+02 },
    { 4.0429999999999999e-08, 6.1258515581499999e+00, 3.1735507750310001e+02 },
    { 4.0180000000000000e-08, 1.4481415560399999e+00, 6.3182053146670000e+02 },
    { 3.9920000000000003e-08, 1.0863881475000001e-01, 2.4851136976900000e+02 },
    { 3.9750000000000002e-08, 4.5566317490400001e+00, 1.0186893394120000e+02 },
    { 3.9680000000000002e-08, 5.5698367172700003e+00, 3.9828701522469999e+02 },
    { 3.9610000000000002e-08, 6.0203153824199998e+00, 4.9721252622900003e+01 },
    { 3.9449999999999999e-08, 5.6953993768300002e+00, 3.3780194662820003e+02 },
    { 3.9190000000000002e-08, 4.8085298204800004e+00, 1.4169889060840001e+02 },
    { 3.9060000000000000e-08, 2.2642211938300001e+00, 5.6193429400900004e+02 },
    { 3.9039999999999998e-08, 2.2136130106300000e+00, 3.7890392768260000e+02 },
    { 3.8980000000000002e-08, 5.5128708169499996e+00, 3.1883955021140002e+02 },
    { 3.8600000000000002e-08, 4.8568362556300002e+00, 5.1242548970719997e+02 },
    { 3.8579999999999999e-08, 4.9075123195700003e+00, 6.3523141986799999e+02 },
    { 3.8549999999999998e-08, 5.2264517101600001e+00, 4.2537435059860002e+02 },
    { 3.8549999999999998e-08, 4.4442541705699998e+00, 4.2782203115340002e+02 },
    { 3.8199999999999998e-08, 1.9827061609200001e+00, 3.2824071907259997e+02 },
    { 3.8099999999999997e-08, 2.5166177580800002e+00, 3.1644005376640001e+02 },
    { 3.8099999999999997e-08, 4.5146439347199996e+00, 1.1494162363460001e+02 },
    { 3.7879999999999999e-08, 4.1363663604900003e+00, 2.5854567807659998e+02 },
    { 3.7870000000000001e-08, 5.8567975302099997e+00, 6.5242745928620002e+02 },
    { 3.7749999999999997e-08, 5.6022199347199999e+00, 3.1634368554679997e+02 },
    { 3.7700000000000000e-08, 4.7995556392500003e+00, 2.1467169356170001e+02 },
    { 3.7700000000000000e-08, 9.0070527339999998e-02, 2.1192649731430001e+02 },
    { 3.7510000000000003e-08, 2.6117809173400000e+00, 1.1191856752295000e+03 },
    { 3.7490000000000000e-08, 5.4874289366899998e+00, 1.4541859555040000e+02 },
    { 3.7350000000000000e-08, 2.2813106562100001e+00, 3.1542866181009998e+02 },
    { 3.7289999999999998e-08, 3.4904169036999999e-01, 4.0747573576479999e+02 },
    { 3.7100000000000001e-08, 3.6075229658199999e+00, 2.2894549679899999e+01 },
    { 3.6930000000000000e-08, 2.7409530432899998e+00, 2.3890195810360001e+02 },
    { 3.6909999999999998e-08, 4.8718733607999998e-01, 1.9428851491140000e+02 },
    { 3.6769999999999998e-08, 4.8544682585299999e+00, 4.1873388434980001e+02 },
    { 3.6729999999999999e-08, 1.3920083899400000e+00, 6.6820846196529999e+02 },
    { 3.6690000000000000e-08, 1.2660810893700001e+00, 8.0198224538700003e+01 },
    { 3.6489999999999998e-08, 3.5382664664000001e+00, 4.4412429869430002e+02 },
    { 3.6379999999999999e-08, 6.2256304561400002e+00, 6.3448066034260000e+02 },
    { 3.5719999999999998e-08, 1.3711343751400000e+00, 8.5171190904369996e+02 },
    { 3.5710000000000000e-08, 4.1690817085500003e+00, 2.7302069586891998e+03 },
    { 3.5670000000000001e-08, 5.4625767717700002e+00, 6.2322251175760005e+02 },
    { 3.5670000000000001e-08, 2.2693346941000000e-01, 4.8758044776399998e+01 },
    { 3.5619999999999997e-08, 2.8991457103000001e-01, 7.0066423920080001e+02 },
    { 3.5420000000000002e-08, 2.1043192828000001e+00, 2.3932264922889999e+02 },
    { 3.5390000000000001e-08, 9.9888072541999995e-01, 2.3294281541100000e+02 },
    { 3.5259999999999999e-08, 5.1615881800599999e+00, 4.9596316347189997e+02 },
    { 3.5220000000000000e-08, 2.6413366227799999e+00, 3.1516802937919999e+02 },
    { 3.5189999999999999e-08, 5.4145048664299997e+00, 7.0756533788050001e+02 },
    { 3.5000000000000002e-08, 4.3844046260100002e+00, 7.8946048344860003e+02 },
    { 3.4959999999999997e-08, 4.0428886684999998e-01, 6.1632141307790005e+02 },
    { 3.4669999999999999e-08, 4.3235178050599998e+00, 4.2829511190540001e+02 },
    { 3.4620000000000002e-08, 5.2887439804499996e+00, 6.2759623029905997e+03 },
    { 3.4620000000000002e-08, 1.6427456264000001e-01, 1.8254023923189999e+02 },
    { 3.4520000000000001e-08, 3.1927324342899999e+00, 1.8802630117250001e+02 },
    { 3.4380000000000001e-08, 1.7095449456500000e+00, 4.0841348623499997e+01 },
    { 3.4219999999999998e-08, 2.7755459258399999e+00, 2.6843697403229999e+02 },
    { 3.4109999999999999e-08, 5.2529385987600001e+00, 2.5169078632512001e+03 },
    { 3.4060000000000002e-08, 6.1485174208400002e+00, 4.9893210888850001e+02 },
    { 3.3960000000000001e-08, 3.4591089695999999e-01, 3.7872403206900003e+01 },
    { 3.3640000000000002e-08, 6.0729958314700001e+00, 1.3310087089929999e+02 },
    { 3.3589999999999998e-08, 5.4496938958600003e+00, 8.4290144101349995e+02 },
    { 3.3519999999999998e-08, 7.2463251624000002e-01, 6.9110301164520001e+02 },
    { 3.3290000000000002e-08, 2.9400048628800000e+00, 7.6588461021249998e+02 },
    { 3.3269999999999999e-08, 1.8070427355700001e+00, 1.0633140834523001e+03 },
    { 3.3200000000000000e-08, 2.1951041016000001e+00, 1.2273649419160000e+02 },
    { 3.2819999999999999e-08, 1.3427384132400000e+00, 2.1176223310300000e+02 },
    { 3.2819999999999999e-08, 3.5468877533500001e+00, 2.1483595777299999e+02 },
    { 3.2800000000000003e-08, 4.4836167722999998e-01, 1.0206171999210201e+04 },
    { 3.2780000000000000e-08, 4.4024385170400002e+00, 2.3230967596459999e+02 },
    { 3.2700000000000002e-08, 2.7057576533100001e+00, 9.2404582705619998e+02 },
    { 3.2420000000000002e-08, 4.0990714724500004e+00, 2.8266406803389998e+02 },
    { 3.2380000000000003e-08, 4.1531824242399997e+00, 3.3848327018440000e+02 },
    { 3.1960000000000003e-08, 4.5574947603900000e+00, 8.2792358748649997e+02 },
    { 3.1760000000000001e-08, 5.3140330110400003e+00, 2.8117959532560002e+02 },
    { 3.1310000000000000e-08, 5.9410289395199998e+00, 3.0683064210100002e+02 },
    { 3.1100000000000001e-08, 4.5347786117000002e+00, 1.6560483224460000e+02 },
    { 3.0990000000000002e-08, 5.4603338834599997e+00, 6.9970103135429997e+02 },
    { 3.0759999999999999e-08, 2.3267749075399999e+00, 1.3044074202339999e+02 },
    { 3.0729999999999998e-08, 5.8947969275699998e+00, 3.2741250177853999e+03 },
    { 3.0659999999999998e-08, 3.7497266216799998e+00, 1.6708930495289999e+02 },
    { 3.0650000000000000e-08, 2.7789677025400001e+00, 7.1600204829600003e+01 },
    { 3.0559999999999997e-08, 2.2512482610100002e+00, 3.5426427286350003e+02 },
    { 3.0559999999999997e-08, 5.4133908411900000e+00, 1.7957129381530001e+02 },
    { 3.0519999999999998e-08, 5.0492306291300002e+00, 5.4153981451059997e+02 },
    { 3.0449999999999998e-08, 1.6599385604100001e+00, 2.8546403739102002e+03 },
    { 3.0449999999999998e-08, 1.3395620724299999e+00, 4.8588067105450000e+02 },
    { 3.0199999999999999e-08, 3.9123754073999999e+00, 3.7967024358792000e+03 },
    { 3.0080000000000002e-08, 4.1019718721000000e+00, 4.3446249740219997e+02 },
    { 2.9950000000000000e-08, 2.8166475112499998e+00, 9.1786441523799994e+01 },
    { 2.9900000000000003e-08, 5.5775109232300002e+00, 7.8234693644779998e+02 },
    { 2.9859999999999997e-08, 2.5685039094100000e+00, 9.2797833480099996e+01 },
    { 2.9519999999999999e-08, 2.3137845566299999e+00, 6.4552636060650002e+02 },
    { 2.9449999999999999e-08, 3.6417998972499999e+00, 3.4639996633729999e+02 },
    { 2.9440000000000001e-08, 1.5794112246700001e+00, 4.2325821331800000e+01 },
    { 2.9200000000000000e-08, 1.2361886203400001e+00, 6.2588264063350005e+02 },
    { 2.9180000000000001e-08, 3.4340917063699998e+00, 3.5723321828010000e+02 },
    { 2.9130000000000000e-08, 4.6529686047699998e+00, 7.7229279122099996e+01 },
    { 2.9020000000000001e-08, 1.0541154846600000e+00, 5.1471311156759998e+02 },
    { 2.8870000000000000e-08, 2.0939474137400000e+00, 9.3531546663000000e+01 },
    { 2.8629999999999999e-08, 4.7190431831100002e+00, 2.4405795164410000e+02 },
    { 2.8559999999999999e-08, 4.5182720699000001e-01, 5.4450875992720000e+02 },
    { 2.8450000000000000e-08, 2.0520173939599999e+00, 5.3197858695499997e+02 },
    { 2.8279999999999999e-08, 3.7558634776600002e+00, 2.5642806592189999e+02 },
    { 2.8180000000000002e-08, 7.4477457405000003e-01, 8.3769726336320002e+02 },
    { 2.8110000000000002e-08, 1.3898015358500000e+00, 6.4138175902230000e+02 },
    { 2.8040000000000002e-08, 4.4019284075599998e+00, 4.2516190227750002e+02 },
    { 2.8040000000000002e-08, 5.2687774731700001e+00, 4.2803447947450002e+02 },
    { 2.7800000000000001e-08, 4.3737083603700002e+00, 1.7368158709190001e+02 },
    { 2.7689999999999998e-08, 4.2067241737999996e+00, 1.0562005364515001e+03 },
    { 2.7570000000000001e-08, 2.2729374140000001e-02, 5.2677020378780003e+02 },
    { 2.7540000000000000e-08, 3.2087635426299999e+00, 6.2025356634100001e+02 },
    { 2.7520000000000001e-08, 3.5371501347100001e+00, 3.2202094394910000e+02 },
    { 2.7330000000000001e-08, 4.3544249936400004e+00, 3.5164090221199999e+01 },
    { 2.7220000000000002e-08, 1.9091293279699999e+00, 6.4409007200799999e+02 },
    { 2.7070000000000000e-08, 2.1119324990299999e+00, 1.7638990007760000e+02 },
    { 2.7010000000000002e-08, 5.3101072665600002e+00, 3.0061086697750000e+02 },
    { 2.6930000000000000e-08, 5.9288627683200001e+00, 4.3296097835140000e+02 },
    { 2.6860000000000000e-08, 1.5925446852000000e-01, 6.4281804762080003e+02 },
    { 2.6659999999999999e-08, 3.9197951613400002e+00, 1.0807225711916001e+03 },
    { 2.6600000000000000e-08, 3.1532549270399999e+00, 4.4327296543239999e+02 },
    { 2.6550000000000000e-08, 5.3066245789500002e+00, 4.4061283655649999e+02 },
    { 2.6510000000000000e-08, 9.4022335035000004e-01, 4.5342489381899998e+02 },
    { 2.6400000000000001e-08, 5.3888675860199999e+00, 1.1095708074480000e+02 },
    { 2.6029999999999999e-08, 4.7197103392899997e+00, 5.0849333644410001e+02 },
    { 2.5939999999999999e-08, 5.6662129691400001e+00, 5.2012973753899996e+02 },
    { 2.5900000000000000e-08, 6.3795679785000003e-01, 1.7017012495410000e+02 },
    { 2.5810000000000001e-08, 2.6302033756999998e+00, 1.5896436599579999e+02 },
    { 2.5810000000000001e-08, 2.7868173292699998e+00, 1.0412226829245001e+03 },
    { 2.5790000000000001e-08, 6.1799056820100002e+00, 9.5310776223289997e+02 },
    { 2.5790000000000001e-08, 3.0164766736500002e+00, 7.6885355562910001e+02 },
    { 2.5780000000000000e-08, 5.2871655660199997e+00, 3.0560680182359999e+02 },
    { 2.5640000000000000e-08, 3.2999934558200001e+00, 4.2749196275330002e+02 },
    { 2.5640000000000000e-08, 8.7527117730000006e-02, 4.2570441899870002e+02 },
    { 2.5590000000000000e-08, 5.3644772097200004e+00, 6.3914652678860000e+02 },
    { 2.5580000000000001e-08, 1.5271417549499999e+00, 3.1097524368519998e+02 },
    { 2.5419999999999999e-08, 2.8050831901000001e-01, 3.5478553772530000e+02 },
    { 2.5410000000000001e-08, 3.9525669836300001e+00, 2.6271141844705999e+03 },
    { 2.5310000000000000e-08, 5.1351244778199998e+00, 9.0562601246400007e+01 },
    { 2.5209999999999999e-08, 2.3119758357300002e+00, 4.5172797278960002e+02 },
    { 2.5209999999999999e-08, 5.4835814180999998e-01, 9.2783496796739996e+02 },
    { 2.5200000000000001e-08, 3.3603347891599999e+00, 6.8547393735269998e+02 },
    { 2.5189999999999999e-08, 1.9778799798300000e+00, 2.9764192156090002e+02 },
    { 2.5180000000000001e-08, 6.0172332983999999e+00, 1.0380412891868000e+03 },
    { 2.5130000000000001e-08, 4.7684495903300004e+00, 1.2518417474640000e+02 },
    { 2.4900000000000001e-08, 5.9760455983199998e+00, 2.4874160449477999e+03 },
    { 2.4789999999999999e-08, 5.2788372749499999e+00, 6.3129926660490003e+02 },
    { 2.4730000000000000e-08, 1.6959548387400001e+00, 1.0945556169400000e+02 },
    { 2.4669999999999999e-08, 1.1614087147400001e+00, 4.2956713629260003e+02 },
    { 2.4570000000000001e-08, 5.7533660117199998e+00, 2.4702689706070001e+02 },
    { 2.4550000000000001e-08, 4.8731262365000001e+00, 1.0490869894507000e+03 },
    { 2.4410000000000002e-08, 7.1752432784999998e-01, 5.8631331639719997e+02 },
    { 2.4340000000000002e-08, 5.5328719145000005e-01, 1.8324281464750001e+02 },
    { 2.4310000000000001e-08, 1.2643935632900001e+00, 2.3593301268700000e+02 },
    { 2.4299999999999999e-08, 3.1692082686199998e+00, 9.5228467692400002e+01 },
    { 2.4170000000000001e-08, 1.7478609885200000e+00, 1.2566151699982800e+04 },
    { 2.4089999999999999e-08, 8.8586446483000003e-01, 3.0609692891809999e+02 },
    { 2.4030000000000001e-08, 2.2699249748399999e+00, 4.5220105354160000e+02 },
    { 2.3969999999999999e-08, 9.9397031563000005e-01, 1.3647280995819001e+03 },
    { 2.3930000000000000e-08, 2.8135916460899999e+00, 2.4351557300350000e+03 },
    { 2.3930000000000000e-08, 5.9161252911600002e+00, 6.1462449204849997e+02 },
    { 2.3860000000000000e-08, 4.3725786742499997e+00, 4.3074279246020001e+02 },
    { 2.3860000000000000e-08, 4.1598378269199996e+00, 2.5234706759822191e+04 },
    { 2.3840000000000000e-08, 2.0653821192400001e+00, 3.8997952100978000e+03 },
    { 2.3800000000000001e-08, 8.8280891663000005e-01, 1.5126011816400000e+02 },
    { 2.3689999999999999e-08, 8.4466299230999997e-01, 2.0597310011610000e+02 },
    { 2.3689999999999999e-08, 9.0337052069000001e-01, 2.2062509075989999e+02 },
    { 2.3460000000000000e-08, 5.9695888662899996e+00, 4.4720511869550000e+02 },
    { 2.3310000000000002e-08, 1.2806305677200001e+00, 5.1838463239980001e+02 },
    { 2.3240000000000002e-08, 3.9803095300400000e+00, 3.0465988283458000e+03 },
    { 2.3149999999999999e-08, 3.0208014686000001e-01, 6.6812248533995999e+03 },
    { 2.3079999999999999e-08, 3.0997119484600000e+00, 1.5497982310600000e+02 },
    { 2.3009999999999999e-08, 3.6555883586200002e+00, 2.6302034806090001e+02 },
    { 2.2989999999999999e-08, 6.2462745873000003e+00, 4.0918970313670002e+02 },
    { 2.2959999999999998e-08, 5.3180144884000002e+00, 1.8084331820250000e+02 },
    { 2.2919999999999999e-08, 6.1868804109599997e+00, 9.1247886193049999e+02 },
    { 2.2849999999999999e-08, 5.2806576345399998e+00, 3.6936096616606001e+03 },
    { 2.2710000000000000e-08, 5.0954214106200002e+00, 1.9280404220310001e+02 },
    { 2.2710000000000000e-08, 6.0773900631500002e+00, 2.3379414867290001e+02 },
    { 2.2679999999999999e-08, 6.7365451500000006e-02, 1.1718758732690001e+03 },
    { 2.2640000000000000e-08, 2.7011064148699999e+00, 1.0478969524800000e+02 },
    { 2.2580000000000001e-08, 5.1480613013800003e+00, 1.0089793540101000e+03 },
    { 2.2530000000000001e-08, 2.2268398919200001e+00, 3.1017209453309999e+02 },
    { 2.2500000000000000e-08, 5.3779331645599999e+00, 3.6604368631030002e+02 },
    { 2.2139999999999998e-08, 2.2252527408299998e+00, 6.4959738580800007e+01 },
    { 2.2069999999999998e-08, 1.5112534298000000e+00, 6.3570450061999998e+02 },
    { 2.2049999999999999e-08, 3.3962361761400000e+00, 1.7097327410619999e+02 },
    { 2.2029999999999999e-08, 2.7461143818100000e+00, 4.9508804301799998e+01 },
    { 2.1990000000000000e-08, 3.6740732459199998e+00, 7.8404385747720005e+02 },
    { 2.1979999999999999e-08, 1.5324597000000000e+00, 9.4524345570670005e+02 },
    { 2.1920000000000000e-08, 5.3821404119700000e+00, 1.4575259330619999e+03 },
    { 2.1850000000000000e-08, 3.8672456296900002e+00, 1.0994568878849999e+02 },
    { 2.1789999999999999e-08, 2.8655950015999998e-01, 6.6672398925699997e+02 },
    { 2.1780000000000000e-08, 3.2971729361000000e+00, 1.1807062774560001e+02 },
    { 2.1769999999999999e-08, 2.6043470944100000e+00, 6.7880499887599996e+01 },
    { 2.1360000000000001e-08, 3.5404441689700001e+00, 1.1081399749656000e+03 },
    { 2.1310000000000000e-08, 6.2573449229999994e-02, 1.3828873468465999e+03 },
    { 2.1310000000000000e-08, 3.6720018533299998e+00, 1.0361403908040000e+02 },
    { 2.1249999999999998e-08, 1.8280430197799999e+00, 4.8715269544170002e+02 },
    { 2.1209999999999999e-08, 5.6583519793699999e+00, 7.9961241183519996e+02 },
    { 2.1109999999999998e-08, 5.2441995989999999e+00, 4.2245358929179997e+02 },
    { 2.1109999999999998e-08, 4.9517173148299998e+00, 6.7091677495099998e+02 },
    { 2.1089999999999999e-08, 3.4816653824300001e+00, 7.5780785536520000e+02 },
    { 2.1019999999999999e-08, 4.3504036164400004e+00, 9.6022130923370003e+02 },
    { 2.0980000000000000e-08, 8.7690013529999999e-02, 2.5291660378410000e+02 },
    { 2.0940000000000001e-08, 6.2168735510699999e+00, 8.3578789401270001e+02 },
    { 2.0920000000000001e-08, 5.6018140474000004e+00, 6.2960234557549995e+02 },
    { 2.0910000000000000e-08, 1.3563652981300001e+00, 3.1710322435667999e+03 },
    { 2.0829999999999999e-08, 4.0478699025399996e+00, 2.7533807271199998e+02 },
    { 2.0829999999999999e-08, 2.2449466647600000e+00, 5.2165803728013998e+03 },
    { 2.0809999999999999e-08, 1.4570708970499999e+00, 5.8720708827450005e+02 },
    { 2.0720000000000000e-08, 2.0569430519499998e+00, 4.5246582638600003e+01 },
    { 2.0710000000000001e-08, 2.1719452059500002e+00, 3.5499798604640000e+02 },
    { 2.0630000000000000e-08, 5.9937130018299998e+00, 1.7712361326050001e+02 },
    { 2.0529999999999999e-08, 2.0559189179900001e+00, 9.0823233677299996e+01 },
    { 2.0210000000000000e-08, 5.8962871572799997e+00, 9.1375088631769995e+02 },
    { 2.0179999999999999e-08, 2.8022211351699999e+00, 2.5020829079839999e+02 },
    { 2.0170000000000001e-08, 4.3363381173300004e+00, 2.4335537622850001e+02 },
    { 2.0160000000000000e-08, 1.0855717693900000e+00, 8.8379517926369999e+02 },
    { 2.0090000000000000e-08, 6.1518299905999996e-01, 1.0335340664950000e+02 },
    { 2.0090000000000000e-08, 3.8925765133299999e+00, 1.3725924061081000e+03 },
    { 2.0079999999999998e-08, 3.4646862568299999e+00, 5.2894020556919997e+02 },
    { 1.9950000000000000e-08, 5.6422239744500002e+00, 4.0933271548860000e+02 },
    { 1.9939999999999999e-08, 2.3060158862999999e-01, 5.8493641121146002e+03 },
    { 1.9930000000000000e-08, 7.3100346778000003e-01, 6.2491943278700001e+02 },
    { 1.9690000000000000e-08, 8.7879585646000002e-01, 5.3044172461999995e+02 },
    { 1.9639999999999999e-08, 2.7251736886900000e+00, 4.0160025067299998e+01 },
    { 1.9620000000000000e-08, 1.6638148011500000e+00, 9.7795738083342003e+03 },
    { 1.9519999999999999e-08, 4.3477704845399998e+00, 1.1583539551190000e+02 },
    { 1.9420000000000001e-08, 6.2508676865000001e-01, 2.4643619622969999e+02 },
    { 1.9420000000000001e-08, 3.6757207736700002e+00, 9.1467900690210008e+03 },
    { 1.9320000000000000e-08, 3.9525383542100001e+00, 1.0625633239269000e+03 },
    { 1.9300000000000001e-08, 1.6801103490000000e+00, 2.7161836777000002e+02 },
    { 1.9259999999999998e-08, 4.6041126371700001e+00, 4.1656388256840000e+02 },
    { 1.9239999999999999e-08, 5.3471331050000001e-02, 5.3388375078859997e+02 },
    { 1.9160000000000001e-08, 3.2209758183099999e+00, 1.0257150935679999e+02 },
    { 1.8909999999999999e-08, 5.0079493459300002e+00, 1.1653954657146000e+03 },
    { 1.8889999999999999e-08, 3.6487222979200000e+00, 7.6440013750419996e+02 },
    { 1.8790000000000001e-08, 5.2729059130899998e+00, 1.3406407874580000e+02 },
    { 1.8780000000000000e-08, 3.1955038406299998e+00, 1.5578297225809999e+02 },
    { 1.8730000000000000e-08, 5.3971366612700002e+00, 3.4484605765980001e+02 },
    { 1.8590000000000000e-08, 1.4544717317500000e+00, 1.0837609525774001e+03 },
    { 1.8559999999999999e-08, 4.1741176812100003e+00, 3.2261164478009999e+02 },
    { 1.8559999999999999e-08, 3.5785816470100000e+00, 6.7925416222920001e+02 },
    { 1.8480000000000001e-08, 2.5536576016699999e+00, 4.1969709219629999e+02 },
    { 1.8360000000000000e-08, 2.9269424716799999e+00, 6.4456315275999998e+02 },
    { 1.8349999999999999e-08, 5.8541360059200001e+00, 2.4575487267349999e+02 },
    { 1.8329999999999999e-08, 2.2594199633700001e+00, 2.6763382488020000e+02 },
    { 1.8320000000000001e-08, 2.6490248303200001e+00, 1.0704276304530999e+03 },
    { 1.8299999999999998e-08, 6.3474599911999996e-01, 2.5950888592310002e+02 },
    { 1.8229999999999998e-08, 5.2127937826900004e+00, 6.5716276170139997e+02 },
    { 1.8069999999999999e-08, 6.7694533190000003e-02, 1.1939672737968001e+03 },
    { 1.8050000000000000e-08, 2.7450747745999999e+00, 1.3891071219701000e+03 },
    { 1.7920000000000001e-08, 4.1515758898500001e+00, 3.7895888888784002e+03 },
    { 1.7859999999999999e-08, 4.9788700988399999e+00, 3.4618751801619999e+02 },
    { 1.7850000000000001e-08, 4.7044691673300001e+00, 1.0970942747017000e+03 },
    { 1.7850000000000001e-08, 1.8384890319000000e+00, 7.3683972137830006e+02 },
    { 1.7809999999999999e-08, 1.9920025414000000e-01, 2.0592491600630001e+02 },
    { 1.7809999999999999e-08, 1.5488332588600000e+00, 2.2067327486970001e+02 },
    { 1.7800000000000001e-08, 1.6820288873000000e+00, 9.7724602613333991e+03 },
    { 1.7750000000000000e-08, 2.3895003948900002e+00, 1.9810879358989999e+02 },
    { 1.7750000000000000e-08, 5.6417184252899997e+00, 2.2848939728610000e+02 },
    { 1.7739999999999999e-08, 4.9082677311199996e+00, 2.0644618086809999e+02 },
    { 1.7739999999999999e-08, 3.1229510890599999e+00, 2.2015201000790000e+02 },
    { 1.7700000000000000e-08, 2.7699706874499999e+00, 6.1069233878540001e+02 },
    { 1.7640000000000001e-08, 6.1514977982800003e+00, 5.6071045373159996e+02 },
    { 1.7630000000000000e-08, 5.6185224503100004e+00, 2.8489930026759998e+02 },
    { 1.7599999999999999e-08, 1.0580996044600000e+00, 1.0916252591036000e+03 },
    { 1.7599999999999999e-08, 4.3200749709000004e+00, 7.3491330568529997e+02 },
    { 1.7590000000000001e-08, 2.2682776738800001e+00, 2.7515475996916002e+03 },
    { 1.7590000000000001e-08, 4.5858259743499996e+00, 2.0003520928290001e+02 },
    { 1.7590000000000001e-08, 3.4453928458299998e+00, 2.2656298159310001e+02 },
    { 1.7579999999999999e-08, 3.5313864123399998e+00, 4.5246168597249999e+02 },
    { 1.7560000000000000e-08, 2.1751509956200001e+00, 2.7078286873866000e+03 },
    { 1.7439999999999999e-08, 5.5481186571699999e+00, 1.1664068576709001e+03 },
    { 1.7420000000000000e-08, 3.5247757902000001e+00, 5.8319272331999997e+01 },
    { 1.7360000000000001e-08, 2.0883485095399998e+00, 1.3411226285559999e+02 },
    { 1.7249999999999999e-08, 1.3264862091800000e+00, 6.2901893969922003e+03 },
    { 1.7150000000000001e-08, 5.0504247328699998e+00, 7.0904981058880003e+02 },
    { 1.7140000000000000e-08, 6.0830092630400001e+00, 7.9186832582400001e+01 },
    { 1.7100000000000001e-08, 4.0474047543399996e+00, 1.1767017984094000e+03 },
    { 1.7080000000000001e-08, 3.2075617740800002e+00, 3.1201997842609999e+03 },
    { 1.7070000000000000e-08, 2.0521338265800000e+00, 9.8357471803400003e+01 },
    { 1.7030000000000001e-08, 1.0072503184999999e+00, 3.9814400287280000e+02 },
    { 1.6910000000000001e-08, 4.0243525275799996e+00, 1.2687488723641000e+03 },
    { 1.6870000000000002e-08, 2.7711040964000002e+00, 1.0220399093211799e+04 },
    { 1.6840000000000001e-08, 3.8024837185300000e+00, 3.3629906980024002e+03 },
    { 1.6840000000000001e-08, 3.6525554724400000e+00, 2.7333883524268999e+03 },
    { 1.6829999999999999e-08, 4.0162182512699998e+00, 6.5034884924301996e+03 },
    { 1.6820000000000001e-08, 5.7177268795399998e+00, 3.5905168874420001e+03 },
    { 1.6650000000000000e-08, 6.2988746649000005e-01, 1.6357784281510001e+02 },
    { 1.6639999999999999e-08, 4.9529002299400000e+00, 1.0298089963400000e+02 },
    { 1.6619999999999999e-08, 3.0105699928799998e+00, 9.3419775544280003e+02 },
    { 1.6619999999999999e-08, 2.3387675102199998e+00, 1.3576145525811000e+03 },
    { 1.6530000000000000e-08, 4.8996208472299996e+00, 3.1867949151699997e+02 },
    { 1.6449999999999998e-08, 5.1790972234300003e+00, 2.4947457761550001e+02 },
    { 1.6440000000000000e-08, 2.7057503022999998e+00, 5.2502509864859996e+02 },
    { 1.6420000000000001e-08, 2.8188039494299999e+00, 5.9282480178500002e+01 },
    { 1.6210000000000001e-08, 2.7051123463999999e-01, 2.5455119402261000e+04 },
    { 1.6169999999999999e-08, 2.3918627743199998e+00, 8.3379618276399995e+01 },
    { 1.6070000000000001e-08, 1.3040153368700000e+00, 1.6456952465863999e+03 },
    { 1.6009999999999999e-08, 5.4389143334099996e+00, 1.0433698188649800e+04 },
    { 1.5939999999999999e-08, 5.1313116082299999e+00, 6.2439816792520003e+02 },
    { 1.5939999999999999e-08, 4.4695442949999997e-01, 1.0931254934210000e+02 },
    { 1.5900000000000000e-08, 8.8427818488999999e-01, 7.0193626358799997e+02 },
    { 1.5869999999999999e-08, 5.4994187032199999e+00, 1.4833935685719999e+02 },
    { 1.5850000000000000e-08, 2.9467615263400000e+00, 4.0028879843164000e+03 },
    { 1.5840000000000002e-08, 5.2834354257799996e+00, 3.7672106175757999e+03 },
    { 1.5810000000000001e-08, 5.5342432827500003e+00, 1.1592933106701000e+03 },
    { 1.5790000000000001e-08, 4.8754332813600003e+00, 1.4775945496771999e+03 },
    { 1.5770000000000002e-08, 4.2198852596399998e+00, 9.7938009023358009e+03 },
    { 1.5749999999999999e-08, 5.9855315484000005e-01, 7.0045179087969996e+02 },
    { 1.5720000000000001e-08, 5.1160487868600004e+00, 2.0903096723752001e+03 },
    { 1.5679999999999999e-08, 4.5367174065100002e+00, 1.2247586176070000e+02 },
    { 1.5630000000000002e-08, 1.4514904854599999e+00, 3.3914448860056000e+03 },
    { 1.5620000000000000e-08, 5.3887481710599996e+00, 1.1160042814917999e+03 },
    { 1.5449999999999999e-08, 5.1761788988099999e+00, 4.6972716135989998e+02 },
    { 1.5399999999999999e-08, 2.6418213545999998e+00, 2.8109214616052000e+03 },
    { 1.5379999999999999e-08, 2.0532282834300002e+00, 2.6484548254729998e+03 },
    { 1.5220000000000000e-08, 5.5260909059000003e+00, 4.0615132175090002e+02 },
    { 1.5220000000000000e-08, 1.8915606181000000e-01, 3.4741135829360002e+02 },
    { 1.5189999999999999e-08, 3.8296889281199999e+00, 1.1443154571431000e+03 },
    { 1.5189999999999999e-08, 1.7760203423600001e+00, 1.1514290041439001e+03 },
    { 1.5170000000000000e-08, 9.6214801531000005e-01, 5.3591074021810005e+02 },
    { 1.5130000000000000e-08, 3.9925684800000000e-01, 1.1474968508807999e+03 },
    { 1.5119999999999999e-08, 4.2890756903599998e+00, 4.2824272227870000e+02 },
    { 1.5110000000000001e-08, 9.1634243174999996e-01, 1.7480372612796000e+03 },
    { 1.5100000000000000e-08, 2.5366421375499999e+00, 7.5483890994859996e+02 },
    { 1.5069999999999999e-08, 3.0324009671600001e+00, 7.2801220700559998e+02 },
    { 1.5049999999999999e-08, 4.0035826514100004e+00, 4.4704506000110001e+02 },
    { 1.5030000000000000e-08, 6.4126765917999995e-01, 3.1587060479479999e+02 },
    { 1.5030000000000000e-08, 2.5980164304800000e+00, 4.1385556958270001e+02 },
    { 1.5030000000000000e-08, 3.8161437738599999e+00, 4.1130943055358002e+03 },
    { 1.4999999999999999e-08, 6.1578972111399999e+00, 2.5345912050530001e+02 },
    { 1.4979999999999999e-08, 1.9860461975399999e+00, 1.7469297904819999e+02 },
    { 1.4950000000000002e-08, 3.2273093680499998e+00, 7.7623812138408583e+04 },
    { 1.4920000000000001e-08, 2.0618532370000001e-02, 7.4591082183940000e+02 },
    { 1.4909999999999999e-08, 2.7128928150500000e+00, 5.8635912061161998e+03 },
    { 1.4909999999999999e-08, 6.1979751410099997e+00, 5.7462713378959997e+03 },
    { 1.4810000000000000e-08, 1.3226608923900001e+00, 9.6764810341156008e+03 },
    { 1.4780000000000001e-08, 6.7769884370000005e-02, 3.4936891175390002e+02 },
    { 1.4730000000000000e-08, 6.0389635214600004e+00, 1.3544331588434000e+03 },
    { 1.4699999999999999e-08, 5.3225874675600000e+00, 2.6936015933849999e+03 },
    { 1.4690000000000000e-08, 7.0970063562999997e-01, 1.4528430202735001e+03 },
    { 1.4640000000000001e-08, 4.4130969981400003e+00, 3.5989334715600000e+02 },
    { 1.4620000000000000e-08, 6.2796173187299997e+00, 3.0141401612959999e+02 },
    { 1.4580000000000001e-08, 2.8057048508000002e+00, 4.1529185818119998e+02 },
    { 1.4580000000000001e-08, 3.0495437565299999e+00, 9.3529756174581998e+03 },
    { 1.4580000000000001e-08, 5.5914360226900000e+00, 1.1285344544640000e+03 },
    { 1.4550000000000000e-08, 5.1725252980800001e+00, 3.3514181775230003e+02 },
    { 1.4530000000000000e-08, 1.7620050960400000e+00, 6.0840471692500000e+02 },
    { 1.4520000000000000e-08, 9.5391083866000004e-01, 3.1665250208750001e+02 },
    { 1.4500000000000001e-08, 8.2068686253000001e-01, 2.5558212176479599e+04 },
    { 1.4489999999999999e-08, 3.1730096020000000e+00, 1.0200250542740000e+03 },
    { 1.4450000000000000e-08, 1.3634281083499999e+00, 1.0538039607899999e+02 },
    { 1.4450000000000000e-08, 5.5477635281800000e+00, 1.5465346256309001e+03 },
    { 1.4450000000000000e-08, 4.8723253569300002e+00, 3.1496916025644000e+03 },
    { 1.4429999999999999e-08, 2.0592469764799999e+00, 4.1481877742920000e+02 },
    { 1.4380000000000000e-08, 3.1437225374000000e+00, 5.5013783421970004e+02 },
    { 1.4349999999999999e-08, 5.6607566935199998e+00, 1.1120721282287000e+03 },
    { 1.4330000000000000e-08, 5.9603131234099997e+00, 1.8484490743480001e+02 },
    { 1.4330000000000000e-08, 4.1491919406599997e+00, 3.8872578766909999e+02 },
    { 1.4310000000000000e-08, 3.5324697894399999e+00, 1.4670701142751000e+03 },
    { 1.4279999999999999e-08, 5.3719414729799997e+00, 2.8332997329077998e+03 },
    { 1.4170000000000000e-08, 2.1680579284700001e+00, 1.0330522253970000e+02 },
    { 1.4130000000000000e-08, 4.9567965898999997e+00, 1.0288032589750000e+02 },
    { 1.4090000000000001e-08, 4.7532835768800004e+00, 4.0100015313171998e+03 },
    { 1.4090000000000001e-08, 3.7368285483300001e+00, 3.2105773610260002e+02 },
    { 1.4079999999999999e-08, 1.5866256688400000e+00, 7.6343692965770003e+02 },
    { 1.4050000000000000e-08, 5.5145938220600002e+00, 9.5920991727740000e+02 },
    { 1.4020000000000001e-08, 2.7389279784999998e-01, 3.1076279536409999e+02 },
    { 1.4000000000000000e-08, 2.6014578064999999e-01, 1.9502906564537000e+03 },
    { 1.3960000000000001e-08, 5.4548210324799999e+00, 1.4614410399825999e+03 },
    { 1.3920000000000000e-08, 2.2529345900000001e-02, 5.8604098123784997e+03 },
    { 1.3920000000000000e-08, 5.3903932241800003e+00, 3.8330916169770001e+02 },
    { 1.3910000000000000e-08, 4.5113620452700003e+00, 8.1294573395949999e+02 },
    { 1.3850000000000000e-08, 1.2042383324899999e+00, 1.1272624300768000e+03 },
    { 1.3840000000000000e-08, 5.1883875318999995e-01, 7.1861103814440003e+02 },
    { 1.3780000000000000e-08, 2.6543076515999999e+00, 1.2481419445445999e+03 },
    { 1.3770000000000000e-08, 4.0253409903900002e+00, 1.8355163118819999e+02 },
    { 1.3770000000000000e-08, 8.6428517620000000e-01, 2.4304655968780000e+02 },
    { 1.3760000000000000e-08, 6.1103736579000001e-01, 5.3532003938710000e+02 },
    { 1.3730000000000000e-08, 4.4796986657400000e+00, 8.8002308469499997e+02 },
    { 1.3710000000000000e-08, 5.9009282940699999e+00, 1.2301427559742999e+03 },
    { 1.3710000000000000e-08, 3.1219950399299998e+00, 5.8525455058523003e+03 },
    { 1.3690000000000000e-08, 2.9764289494999998e+00, 2.0156710861598001e+03 },
    { 1.3660000000000000e-08, 5.2272439415500003e+00, 6.7383753625780002e+02 },
    { 1.3620000000000001e-08, 5.1136503273500002e+00, 7.3342883297699996e+02 },
    { 1.3610000000000001e-08, 2.5316006676000002e-01, 1.4395096981491999e+03 },
    { 1.3599999999999999e-08, 2.9410663727499999e+00, 7.3168372783780001e+02 },
    { 1.3599999999999999e-08, 3.6975918129199998e+00, 6.7049608382570000e+02 },
    { 1.3580000000000000e-08, 5.7435588616200004e+00, 8.7710232338820003e+02 },
    { 1.3570000000000000e-08, 6.2775402921500003e+00, 2.6381598847344999e+03 },
    { 1.3570000000000000e-08, 3.3559883897299998e+00, 1.0160830151030000e+02 },
    { 1.3560000000000000e-08, 3.9868011686200000e+00, 1.0850940018999999e+02 },
    { 1.3529999999999999e-08, 1.6753327984999999e-01, 5.9884348936940000e+02 },
    { 1.3520000000000000e-08, 3.7596645030700002e+00, 5.2980283967920002e+02 },
    { 1.3500000000000000e-08, 3.5265562488500000e+00, 2.5440892308259390e+04 },
    { 1.3430000000000000e-08, 4.0501950329799996e+00, 9.7355463135800005e+02 },
    { 1.3380000000000000e-08, 3.2385668589000000e-01, 5.5110104206619997e+02 },
    { 1.3380000000000000e-08, 2.8846575268000002e-01, 1.2442268376239999e+03 },
    { 1.3290000000000000e-08, 2.6204366258500000e+00, 1.3214390704036000e+03 },
    { 1.3290000000000000e-08, 3.7829091972800000e+00, 1.2262106027112000e+03 },
    { 1.3290000000000000e-08, 5.3441781539000000e-01, 6.8005731138130000e+02 },
    { 1.3290000000000000e-08, 5.0378679526100001e+00, 2.7405018994277002e+03 },
    { 1.3280000000000000e-08, 4.2661323444399999e+00, 1.6805251279940001e+02 },
    { 1.3230000000000000e-08, 5.7163734440000002e-01, 1.1829215735329001e+03 },
    { 1.3210000000000000e-08, 4.4367048081700000e+00, 3.2324478422649997e+02 },
    { 1.3189999999999999e-08, 4.6840664364600002e+00, 5.4369930152402003e+03 },
    { 1.3170000000000000e-08, 3.8385855877599999e+00, 2.6036021918500001e+02 },
    { 1.3140000000000001e-08, 6.2656438584999998e+00, 1.5606187072805999e+03 },
    { 1.3060000000000001e-08, 3.4604960247199998e+00, 1.1887377689770000e+02 },
    { 1.3020000000000000e-08, 1.4459835388900000e+00, 9.7906195085980999e+03 },
    { 1.3020000000000000e-08, 3.6950825875400000e+00, 1.0962429414398000e+03 },
    { 1.2989999999999999e-08, 3.7372300741800002e+00, 1.0017201291180000e+02 },
    { 1.2989999999999999e-08, 5.4385007802200001e+00, 2.5097943162503998e+03 },
    { 1.2930000000000001e-08, 7.3720210492000005e-01, 4.4386366626339998e+02 },
    { 1.2919999999999999e-08, 2.5848212811600000e+00, 9.5016019371300004e+01 },
    { 1.2890000000000000e-08, 5.8875867943199998e+00, 1.4033342159716999e+03 },
    { 1.2880000000000000e-08, 4.2379455453999998e-01, 1.3405615255148000e+03 },
    { 1.2870000000000000e-08, 5.1164381891400001e+00, 1.3780614217062000e+03 },
    { 1.2860000000000001e-08, 7.9929117404000005e-01, 1.0820158474381999e+03 },
    { 1.2849999999999999e-08, 3.9350444173299999e+00, 4.1059807585349999e+03 },
    { 1.2810000000000000e-08, 4.5454485993600002e+00, 9.7827552020718995e+03 },
    { 1.2790000000000001e-08, 6.2790063933999996e-01, 2.0889386142289999e+02 },
    { 1.2790000000000001e-08, 1.1201328736600000e+00, 2.1770432945310000e+02 },
    { 1.2790000000000001e-08, 3.5160497538700000e+00, 2.2312095542450001e+02 },
    { 1.2790000000000001e-08, 4.5151690663100004e+00, 2.0347723545150001e+02 },
    { 1.2730000000000001e-08, 5.5887711000999996e-01, 1.6534419981369999e+02 },
    { 1.2709999999999999e-08, 4.6342889473400000e+00, 2.5771511271917600e+04 },
    { 1.2680000000000000e-08, 6.1901128561999998e+00, 9.4599421523210003e+02 },
    { 1.2670000000000000e-08, 5.3258201888399999e+00, 1.1487218766540001e+02 },
    { 1.2650000000000001e-08, 1.7453995125599999e+00, 3.1347110834979998e+02 },
    { 1.2639999999999999e-08, 6.8934669983999997e-01, 1.6811199692385001e+03 },
    { 1.2630000000000000e-08, 1.7255629686300000e+00, 1.6960978227655000e+03 },
    { 1.2590000000000001e-08, 2.5697853487300000e+00, 5.3339002410215999e+03 },
    { 1.2550000000000000e-08, 4.6903146451100000e+00, 1.3654788591073000e+03 },
    { 1.2550000000000000e-08, 1.9819985504800000e+00, 8.7757540414020002e+02 },
    { 1.2540000000000000e-08, 1.3141247639999999e-02, 1.6708250284999999e+03 },
    { 1.2530000000000000e-08, 5.0443478954599996e+00, 4.8225733433210002e+02 },
    { 1.2530000000000000e-08, 4.4639270439000001e+00, 5.8675233593793000e+03 },
    { 1.2490000000000000e-08, 4.4846745610700003e+00, 1.2577031721001999e+03 },
    { 1.2490000000000000e-08, 1.3061347599299999e+00, 1.5347381658416000e+03 },
    { 1.2460000000000000e-08, 5.6890395085499996e+00, 7.5877106321170004e+02 },
    { 1.2450000000000001e-08, 4.6550732712800000e+00, 2.9349731997670000e+02 },
    { 1.2439999999999999e-08, 1.2577765006199999e+00, 3.4874241132234001e+03 },
    { 1.2369999999999999e-08, 4.4515743057000003e+00, 5.5969906177530004e+02 },
    { 1.2360000000000000e-08, 4.3074340674000000e+00, 5.3909213395580002e+02 },
    { 1.2360000000000000e-08, 3.0370190103000000e-01, 7.6174000862829996e+02 },
    { 1.2360000000000000e-08, 4.0292296872100000e+00, 2.9435060541272001e+03 },
    { 1.2290000000000000e-08, 2.7544175146500001e+00, 2.3210172555525000e+03 },
    { 1.2280000000000000e-08, 6.1489504277700000e+00, 9.1456373123700004e+01 },
    { 1.2260000000000000e-08, 3.9449658609500000e+00, 1.9731328165069001e+03 },
    { 1.2240000000000001e-08, 2.0953732601600001e+00, 3.0679394693482000e+03 },
    { 1.2240000000000001e-08, 2.8205362438499999e+00, 2.5344913081041599e+04 },
    { 1.2210000000000000e-08, 3.2108567037100002e+00, 7.7596710262989995e+02 },
    { 1.2210000000000000e-08, 6.0780976578599999e+00, 4.7306861379200001e+02 },
    { 1.2200000000000000e-08, 4.6327578682699997e+00, 1.4584720945659999e+03 },
    { 1.2159999999999999e-08, 6.1522130634799996e+00, 9.3672027114598004e+03 },
    { 1.2140000000000000e-08, 6.0814661420000000e+00, 1.1907858800591000e+03 },
    { 1.2120000000000000e-08, 1.8260829286300000e+00, 8.7832616366560001e+02 },
    { 1.2120000000000000e-08, 5.8388892031699999e+00, 9.4131130244359997e+02 },
    { 1.2110000000000001e-08, 2.2983411005600001e+00, 2.7081521861789997e+02 },
    { 1.2110000000000001e-08, 7.1050825969999995e-02, 8.7439401040250004e+02 },
    { 1.2080000000000000e-08, 1.3271448227100000e+00, 1.1588357962169999e+02 },
    { 1.2070000000000000e-08, 4.6107689966100001e+00, 2.5976223661672002e+03 },
    { 1.2040000000000001e-08, 1.1466552152500000e+00, 1.9620871162430001e+03 },
    { 1.2000000000000000e-08, 3.0983695347000002e+00, 5.2846712481719999e+02 },
    { 1.1949999999999999e-08, 4.7774896653000001e+00, 4.7061123747000003e+01 },
    { 1.1949999999999999e-08, 6.1587789193300004e+00, 1.1521797636693000e+03 },
    { 1.1910000000000000e-08, 2.8850995152999999e+00, 4.6675821594330000e+02 },
    { 1.1870000000000000e-08, 1.0383506453100000e+00, 1.3947361962626001e+03 },
    { 1.1850000000000000e-08, 2.3098444964099998e+00, 7.7299815721330003e+02 },
    { 1.1850000000000000e-08, 5.6803378882100004e+00, 1.0846069285524200e+04 },
    { 1.1800000000000000e-08, 5.7125568519299996e+00, 7.7400954916959995e+02 },
    { 1.1800000000000000e-08, 5.6758237281500001e+00, 2.7258157561650000e+02 },
    { 1.1800000000000000e-08, 1.3721646424000000e+00, 2.0124896924221000e+03 },
    { 1.1790000000000000e-08, 5.5698346194999999e+00, 2.6099335863139999e+02 },
    { 1.1780000000000000e-08, 1.4351169578300000e+00, 1.3922885157077999e+03 },
    { 1.1770000000000001e-08, 1.8281518939200001e+00, 3.0305854753230000e+02 },
    { 1.1760000000000001e-08, 2.9035785333000002e+00, 2.5190521182780000e+02 },
    { 1.1760000000000001e-08, 6.4353107930999998e-01, 4.7529915918498000e+03 },
    { 1.1730000000000000e-08, 2.8254534306300001e+00, 1.3836904959987000e+03 },
    { 1.1720000000000000e-08, 5.8873631563900002e+00, 9.7977330555988992e+03 },
    { 1.1710000000000000e-08, 5.5309986687899997e+00, 1.4756851803267000e+03 },
    { 1.1700000000000001e-08, 2.1269876401299999e+00, 7.0485702489480002e+02 },
    { 1.1660000000000000e-08, 5.0589055416999997e+00, 2.4673990589494001e+03 },
    { 1.1640000000000000e-08, 3.5232029240900000e+00, 9.1300012679229997e+02 },
    { 1.1609999999999999e-08, 6.4813941639999995e-01, 9.8778172535960005e+02 },
    { 1.1600000000000000e-08, 6.2371876044499999e+00, 1.5274459087229999e+02 },
    { 1.1570000000000000e-08, 1.3290117292500001e+00, 1.3741292684431000e+03 },
    { 1.1549999999999999e-08, 1.1380500770899999e+00, 6.5994677196479997e+03 },
    { 1.1549999999999999e-08, 1.8006798044800001e+00, 3.0852756313039998e+02 },
    { 1.1539999999999999e-08, 4.0787574863999998e-01, 2.0408008680733999e+03 },
    { 1.1530000000000000e-08, 5.8361836959800000e+00, 4.7631944349890000e+02 },
    { 1.1530000000000000e-08, 5.1750506368600000e+00, 1.7739178027185999e+03 },
    { 1.1510000000000000e-08, 1.9215782223800000e+00, 7.7544583776809998e+02 },
    { 1.1479999999999999e-08, 1.8326287753999999e+00, 9.7918370565049997e+02 },
    { 1.1469999999999999e-08, 5.2326105523999997e+00, 2.0479144150742000e+03 },
    { 1.1460000000000000e-08, 2.6043829437200001e+00, 9.8215265106710001e+02 },
    { 1.1420000000000001e-08, 1.4819030385000001e-01, 1.9872168981566001e+03 },
    { 1.1420000000000001e-08, 5.2062504792000004e+00, 9.0684978763799995e+02 },
    { 1.1390000000000000e-08, 4.2551543568500003e+00, 1.9902381732660001e+02 },
    { 1.1390000000000000e-08, 3.7760644633300000e+00, 2.2757437354940001e+02 },
    { 1.1380000000000000e-08, 5.0006884072000002e+00, 5.8454319588515000e+03 },
    { 1.1310000000000000e-08, 3.9627299667100000e+00, 5.5400857894587998e+03 },
    { 1.1310000000000000e-08, 3.0820481577100001e+00, 1.2222784494481000e+03 },
    { 1.1290000000000001e-08, 5.0016173428600004e+00, 1.2742621293299999e+01 },
    { 1.1280000000000001e-08, 1.7176922971899999e+00, 9.7673602509570003e+02 },
    { 1.1280000000000001e-08, 4.7677116531100001e+00, 1.4599565672742999e+03 },
    { 1.1269999999999999e-08, 3.0971437723000000e-01, 7.8477008520160583e+04 },
    { 1.1260000000000000e-08, 3.7966031453300002e+00, 8.4853051530599998e+02 },
    { 1.1160000000000000e-08, 5.9859230587100001e+00, 1.9015326116773001e+03 },
    { 1.1160000000000000e-08, 9.1462209636000003e-01, 9.4753107756710006e+02 },
    { 1.1150000000000001e-08, 1.3500547892999999e+00, 1.0798287993143001e+03 },
    { 1.1129999999999999e-08, 3.0232570801600001e+00, 5.2354062594029995e+02 },
    { 1.1090000000000000e-08, 1.6657509665900001e+00, 1.3977051416792001e+03 },
    { 1.1080000000000001e-08, 6.0959865701100000e+00, 1.2591876448085000e+03 },
    { 1.1069999999999999e-08, 5.3790120876399996e+00, 2.4422692770357999e+03 },
    { 1.1050000000000000e-08, 1.9819387271900000e+00, 3.6908206769610001e+02 },
    { 1.1050000000000000e-08, 3.8728165518000002e-01, 6.6109491496450005e+02 },
    { 1.1050000000000000e-08, 1.1491974138100001e+00, 1.9912018554619999e+02 },
    { 1.1050000000000000e-08, 5.9883609919000003e-01, 2.2747800532980000e+02 },
    { 1.1020000000000000e-08, 4.6306643877999996e+00, 5.3169670266469996e+02 },
    { 1.0999999999999999e-08, 7.7366362491999996e-01, 1.7244089984167999e+03 },
    { 1.0989999999999999e-08, 5.9965140558399996e+00, 5.2768522752449996e+02 },
    { 1.0960000000000000e-08, 1.2679484299099999e+00, 3.7301422095919997e+02 },
    { 1.0960000000000000e-08, 4.3294794470399998e+00, 1.9982625984204999e+03 },
    { 1.0940000000000001e-08, 9.1660450584999997e-01, 3.8038159828799999e+03 },
    { 1.0940000000000001e-08, 4.6515939742199999e+00, 2.6231820312075001e+03 },
    { 1.0929999999999999e-08, 9.5182537662000000e-01, 2.6404295011230799e+04 },
    { 1.0920000000000000e-08, 5.1760698110000003e-01, 9.3888066823129998e+02 },
    { 1.0900000000000000e-08, 9.5231319439999995e-01, 7.3832419408659996e+02 },
    { 1.0880000000000000e-08, 2.6761901854199999e+00, 2.6302955782083000e+03 },
    { 1.0880000000000000e-08, 3.1069254613499999e+00, 2.3320629558164001e+03 },
    { 1.0850000000000000e-08, 2.5953704247900000e+00, 5.9220302312060005e+02 },
    { 1.0820000000000000e-08, 5.2499654970199998e+00, 2.1510071892927999e+03 },
    { 1.0810000000000001e-08, 1.0545160077600000e+00, 3.1691313451840000e+02 },
    { 1.0780000000000000e-08, 5.2941331725999996e+00, 9.0291763437489999e+02 },
    { 1.0760000000000000e-08, 7.6185840399000004e-01, 8.9877303279069997e+02 },
    { 1.0750000000000000e-08, 3.7909834335500001e+00, 6.4531391228539997e+02 },
    { 1.0719999999999999e-08, 4.0108276578200002e+00, 2.2057270113320001e+02 },
    { 1.0719999999999999e-08, 4.0203911623600002e+00, 2.0602548974280000e+02 },
    { 1.0719999999999999e-08, 1.1226785432799999e+00, 2.0634560713159999e+02 },
    { 1.0719999999999999e-08, 6.2535496972000004e-01, 2.2025258374440000e+02 },
    { 1.0719999999999999e-08, 5.1413504792599998e+00, 5.8238116313410001e+02 },
    { 1.0710000000000000e-08, 1.1107315197400001e+00, 7.8159617692239999e+02 },
    { 1.0700000000000000e-08, 3.5451348252599999e+00, 1.4710022675381999e+03 },
    { 1.0690000000000000e-08, 4.6791389291299996e+00, 6.2176310719126004e+03 },
    { 1.0690000000000000e-08, 3.2159703449500001e+00, 3.8427236954419999e+02 },
    { 1.0670000000000001e-08, 5.0016227172500001e+00, 9.0440210708320001e+02 },
    { 1.0670000000000001e-08, 6.0365878835600002e+00, 1.0551891444952000e+03 },
    { 1.0640000000000000e-08, 1.4095547310000001e-01, 9.7756416550711001e+03 },
    { 1.0630000000000000e-08, 1.8074245950600001e+00, 9.2107688163959995e+02 },
    { 1.0600000000000001e-08, 1.3225950604600001e+00, 2.5200892569889002e+03 },
    { 1.0579999999999999e-08, 3.2994628163300002e+00, 6.1387373252309999e+02 },
    { 1.0560000000000000e-08, 2.4142161851899999e+00, 6.2038977273999997e+01 },
    { 1.0560000000000000e-08, 1.3180410154100000e+00, 3.7078367556622002e+03 },
    { 1.0550000000000000e-08, 4.0398238014199999e+00, 7.2322514589612001e+03 },
    { 1.0540000000000000e-08, 2.7235895799500001e+00, 5.9666839803348003e+03 },
    { 1.0540000000000000e-08, 1.8348909645000000e-01, 1.3049243545416000e+03 },
    { 1.0500000000000000e-08, 6.3919198125999999e-01, 3.3217287233569999e+02 },
    { 1.0480000000000000e-08, 1.7899078516700000e+00, 5.3261172640140001e+02 },
    { 1.0480000000000000e-08, 3.7316880502799998e+00, 4.4294861751924000e+03 },
    { 1.0449999999999999e-08, 4.6972579484799999e+00, 1.2474082313617000e+03 },
    { 1.0449999999999999e-08, 1.7193481027099999e+00, 4.4779581952650000e+02 },
    { 1.0390000000000001e-08, 5.0935261236800002e+00, 6.8189316428300003e+01 },
    { 1.0350000000000000e-08, 1.5097414650400001e+00, 6.2617402095243997e+03 },
    { 1.0280000000000000e-08, 6.8506380971000003e-01, 6.2248879857470001e+02 },
    { 1.0270000000000000e-08, 3.0921018798600000e+00, 1.6992792165031999e+03 },
    { 1.0270000000000000e-08, 2.7348581790400002e+00, 3.6570042963564001e+03 },
    { 1.0260000000000001e-08, 1.0332844985800000e+00, 5.9964663852149999e+02 },
    { 1.0230000000000000e-08, 1.3083038567900001e+00, 6.0824465823059995e+02 },
    { 1.0230000000000000e-08, 2.7256649546500000e+00, 5.2252923398400003e+02 },
    { 1.0230000000000000e-08, 3.5477763189900000e+00, 1.0547160637432000e+03 },
    { 1.0220000000000000e-08, 5.8195370042899999e+00, 8.1214258480740000e+02 },
    { 1.0210000000000000e-08, 2.8324905786299999e+00, 5.2406189080210004e+02 },
    { 1.0200000000000000e-08, 8.1327138026000001e-01, 6.6401567627129998e+02 },
    { 1.0200000000000000e-08, 3.9136288098300001e+00, 1.0320464880320000e+02 },
    { 1.0179999999999999e-08, 2.9159622144499999e+00, 6.4064804583939997e+02 },
    { 1.0179999999999999e-08, 4.9171441065700003e+00, 6.2475131155228000e+03 },
    { 1.0179999999999999e-08, 2.3111644040699999e+00, 3.7007232086613999e+03 },
    { 1.0169999999999999e-08, 2.2994344133300002e+00, 1.9392449561898000e+03 },
    { 1.0160000000000000e-08, 1.8127666609999999e+00, 1.3324847706675000e+03 },
    { 1.0150000000000000e-08, 3.0958383013900002e+00, 4.1109907248719998e+02 },
    { 1.0150000000000000e-08, 1.6402612258700000e+00, 7.1453587252939997e+02 },
    { 1.0140000000000000e-08, 2.5220686379999999e-02, 1.2184272231430000e+02 },
    { 1.0130000000000000e-08, 5.4339656007799997e+00, 8.1065811209909998e+02 },
    { 1.0099999999999999e-08, 2.7031104317799999e+00, 3.9140223040994001e+03 },
    { 1.0099999999999999e-08, 3.1932257327300002e+00, 1.1868537267960000e+03 },
    { 1.0090000000000000e-08, 2.3031683878200000e+00, 1.1262992222303001e+03 },
    { 1.0080000000000000e-08, 4.3074500918399998e+00, 1.0283214178770000e+02 },
    { 1.0080000000000000e-08, 4.3311135772999998e-01, 3.0412232911529998e+02 },
    { 1.0020000000000000e-08, 4.1836971864599999e+00, 5.2182919830149200e+04 },
    { 9.9800000000000007e-09, 5.4453598614000001e+00, 1.1779413446800000e+01 },
    { 9.9599999999999995e-09, 1.1383388497700000e+00, 1.3293033769297999e+03 },
    { 9.9499999999999998e-09, 4.2857396444000004e+00, 1.1201488830759999e+03 },
    { 9.9300000000000002e-09, 6.0130691628899999e+00, 1.3962206689709001e+03 },
    { 9.9200000000000005e-09, 1.5283396696300000e+00, 7.2631528597620002e+02 },
    { 9.9100000000000007e-09, 1.5211363691199999e+00, 3.3136972318360000e+02 },
    { 9.8899999999999996e-09, 8.7301593505999997e-01, 5.5703893289940004e+02 },
    { 9.8899999999999996e-09, 5.4627313055100002e+00, 1.1162461155180799e+04 },
    { 9.8700000000000000e-09, 2.6481011108599999e+00, 9.6813800538659996e+02 },
    { 9.8500000000000005e-09, 5.3732857243399996e+00, 1.0129115072732000e+03 },
    { 9.8199999999999996e-09, 6.0374279490699996e+00, 1.2694996318895001e+03 },
    { 9.7700000000000008e-09, 1.8951621433400001e+00, 2.7385360000370002e+02 },
    { 9.7599999999999994e-09, 3.2505679367700000e+00, 8.7046185713939997e+02 },
    { 9.7499999999999996e-09, 5.2036116497200000e+00, 2.2257887878601000e+03 },
    { 9.7399999999999999e-09, 5.0255279261999997e-01, 1.9975118388951000e+03 },
    { 9.7300000000000001e-09, 3.9327362570000002e+00, 1.7313907037070001e+02 },
    { 9.7300000000000001e-09, 1.8291286905000001e-01, 1.3332355301928999e+03 },
    { 9.7300000000000001e-09, 5.4877873612899997e+00, 5.1890589726159999e+02 },
    { 9.7200000000000003e-09, 3.3978177528400000e+00, 1.8589943420244001e+03 },
    { 9.7200000000000003e-09, 3.1561200546800001e+00, 8.9255325766720000e+02 },
    { 9.7100000000000006e-09, 2.8861179455000001e+00, 3.2812385647862002e+03 },
    { 9.6999999999999992e-09, 4.5938249660099997e+00, 8.6652970387630000e+02 },
    { 9.6999999999999992e-09, 9.2586021520999995e-01, 5.1322609901396390e+04 },
    { 9.6999999999999992e-09, 4.4448954122500002e+00, 5.2724328453980002e+02 },
    { 9.6799999999999997e-09, 4.7098256208700002e+00, 3.9494556279260001e+02 },
    { 9.6500000000000004e-09, 2.4000659213700000e+00, 1.6740064222377000e+03 },
    { 9.6199999999999995e-09, 4.9168712191999999e+00, 1.7598337210689001e+03 },
    { 9.6199999999999995e-09, 1.7803388727200000e+00, 7.7033802833740003e+02 },
    { 9.6199999999999995e-09, 6.2986061526000003e-01, 9.8853248488500003e+02 },
    { 9.5999999999999999e-09, 2.4741659067600001e+00, 1.1978994270599001e+03 },
    { 9.5999999999999999e-09, 2.4584342276999999e-01, 5.8422505651137999e+03 },
    { 9.5900000000000002e-09, 2.9470098907000000e+00, 2.4968702593660001e+02 },
    { 9.5900000000000002e-09, 1.9426162758900001e+00, 1.7691116493940001e+02 },
    { 9.5800000000000004e-09, 1.9739215768000000e+00, 1.7913262904579001e+03 },
    { 9.5800000000000004e-09, 4.1374580380800001e+00, 7.1149749114359997e+02 },
    { 9.5700000000000007e-09, 2.6383202160799999e+00, 1.0529677415867600e+04 },
    { 9.5200000000000002e-09, 4.7469515638699997e+00, 3.5691762464396002e+03 },
    { 9.5200000000000002e-09, 4.4952768725099999e+00, 1.5276246188407999e+03 },
    { 9.5200000000000002e-09, 1.2492294016100001e+00, 5.2168692736147590e+04 },
    { 9.5000000000000007e-09, 2.9138457875800001e+00, 5.6829708148440000e+02 },
    { 9.5000000000000007e-09, 4.1969824428600004e+00, 5.0759956456679998e+02 },
    { 9.4899999999999993e-09, 8.6426366309000002e-01, 1.0007099997773799e+04 },
    { 9.4500000000000002e-09, 9.1239433600000003e-02, 8.3821852822500000e+02 },
    { 9.4500000000000002e-09, 2.2179897412699998e+00, 2.1077181601144998e+03 },
    { 9.4500000000000002e-09, 1.4974345329400001e+00, 8.2962050851590004e+02 },
    { 9.4400000000000005e-09, 1.6146590623099999e+00, 2.7483662059539001e+03 },
    { 9.4300000000000007e-09, 2.6942789422200000e+00, 1.8755090578863999e+03 },
    { 9.4099999999999996e-09, 2.8278999788100001e+00, 9.4498282327580000e+02 },
    { 9.3999999999999998e-09, 1.9113947399800000e+00, 4.2161870797543997e+03 },
    { 9.3999999999999998e-09, 4.0461452564000000e-01, 2.3139037085517002e+03 },
    { 9.3999999999999998e-09, 2.9428389905299999e+00, 4.6176228109720000e+02 },
    { 9.3900000000000000e-09, 4.7216065342500002e+00, 1.2573265246983599e+04 },
    { 9.3800000000000003e-09, 3.0415708561999999e-01, 6.0054041039879996e+02 },
    { 9.3600000000000008e-09, 2.1736384240300000e+00, 3.2180849562800000e+02 },
    { 9.3600000000000008e-09, 3.6703047309999998e-01, 4.8025159676200002e+02 },
    { 9.3499999999999994e-09, 2.0374669968100001e+00, 8.4189004905720003e+02 },
    { 9.3399999999999996e-09, 1.5983441898699999e+00, 1.3009922012785000e+03 },
    { 9.3100000000000003e-09, 6.0460668903899997e+00, 3.9069087570985998e+03 },
    { 9.3100000000000003e-09, 4.5649664468699997e+00, 5.2957909051000001e+02 },
    { 9.2900000000000008e-09, 3.2420344455499999e+00, 5.1716079212240004e+02 },
    { 9.2799999999999994e-09, 1.9803291714100000e+00, 6.8961853893689999e+02 },
    { 9.2799999999999994e-09, 2.0380447250899998e+00, 9.7463699926100006e+01 },
    { 9.2699999999999996e-09, 6.2584504063799997e+00, 6.3278373931320002e+03 },
    { 9.2699999999999996e-09, 5.7692906863999998e+00, 2.0491352405000001e+02 },
    { 9.2699999999999996e-09, 2.2619281337800001e+00, 2.2168466682600001e+02 },
    { 9.2599999999999999e-09, 4.8740777830599997e+00, 1.0254416802454000e+03 },
    { 9.2599999999999999e-09, 3.3867514258600000e+00, 9.8968936765544004e+03 },
    { 9.2400000000000004e-09, 4.6907436853000002e+00, 1.0727152523135001e+03 },
    { 9.2199999999999992e-09, 4.8508211001800001e+00, 4.0153784493159998e+02 },
    { 9.1999999999999997e-09, 3.6098860875900001e+00, 6.3203297978779995e+02 },
    { 9.1999999999999997e-09, 4.6007028667799998e+00, 8.0279380557290006e+02 },
    { 9.1999999999999997e-09, 3.2919480529300000e+00, 1.4607073267997000e+03 },
    { 9.1800000000000001e-09, 2.6384440923399999e+00, 3.8575684225250001e+02 },
    { 9.1700000000000004e-09, 1.7875919850899999e+00, 1.2559038152982001e+04 },
    { 9.1199999999999999e-09, 1.3941684568199999e+00, 8.3258945393249996e+02 },
    { 9.1000000000000004e-09, 4.9370013513500002e+00, 2.4067015420317998e+03 },
    { 9.0900000000000007e-09, 4.5106267233499997e+00, 3.1172600321060003e+02 },
    { 9.0500000000000000e-09, 5.0833748142799999e+00, 7.9568025857210000e+02 },
    { 9.0200000000000007e-09, 6.0083296362600001e+00, 1.4072663692348001e+03 },
    { 9.0099999999999993e-09, 4.1117850141899996e+00, 1.1270499817556999e+03 },
    { 8.9999999999999995e-09, 2.8526148461099998e+00, 1.0223126761344000e+03 },
    { 8.9999999999999995e-09, 1.8073842941100000e+00, 5.9012392022559998e+03 },
    { 8.9899999999999998e-09, 9.5311796469999999e-01, 6.0595703637019994e+02 },
    { 8.9899999999999998e-09, 5.1636119183900000e+00, 4.2023540340059998e+02 },
    { 8.9899999999999998e-09, 5.3812305145800003e+00, 9.3101636170510005e+02 },
    { 8.9600000000000005e-09, 2.5072435868900000e+00, 9.8141893788419998e+02 },
    { 8.9299999999999996e-09, 3.8292031811500000e+00, 9.5709230512270005e+02 },
    { 8.9199999999999998e-09, 5.1409333559000003e-01, 6.6380322795020004e+02 },
    { 8.9100000000000000e-09, 5.1346937561899999e+00, 6.6964475056380002e+02 },
    { 8.8900000000000005e-09, 4.1841066475200002e+00, 2.3067901615508999e+03 },
    { 8.8900000000000005e-09, 2.7885580655200002e+00, 5.1582491417233996e+03 },
    { 8.8800000000000008e-09, 3.2713558632100002e+00, 2.6514501332450200e+04 },
    { 8.8400000000000001e-09, 9.0715778767999999e-01, 1.3364169239306000e+03 },
    { 8.8400000000000001e-09, 2.1888854014499999e+00, 6.5541765656220002e+02 },
    { 8.8300000000000003e-09, 1.1492592605000000e-01, 3.7864329525170001e+02 },
    { 8.8100000000000008e-09, 5.4619701764500004e+00, 2.6917068721520002e+02 },
    { 8.8100000000000008e-09, 5.7108412973200000e+00, 1.5742750366080000e+02 },
    { 8.7999999999999994e-09, 3.0973539794899998e+00, 3.0386169668439999e+02 },
    { 8.7799999999999999e-09, 1.5352906197200000e+00, 1.6623797169100001e+02 },
    { 8.7399999999999992e-09, 1.0402653909399999e+00, 4.9351548291709997e+02 },
    { 8.7199999999999997e-09, 2.5155312979400000e+00, 6.3624023718799997e+01 },
    { 8.7099999999999999e-09, 1.9275562336400001e+00, 1.0293271552420001e+02 },
    { 8.7000000000000001e-09, 5.1997485786700004e+00, 1.0325283291300001e+02 },
    { 8.7000000000000001e-09, 1.9730906752299999e+00, 1.4678208738005001e+03 },
    { 8.7000000000000001e-09, 6.2028576670600000e+00, 4.0132539661049998e+02 },
    { 8.7000000000000001e-09, 4.7180276081899999e+00, 7.4920983565610004e+02 },
    { 8.6800000000000006e-09, 5.2167239477200003e+00, 4.5396924964117998e+03 },
    { 8.6800000000000006e-09, 1.3582660748000000e-01, 1.0168436605363000e+03 },
    { 8.6699999999999992e-09, 2.0196147647700000e+00, 4.4949274055590001e+02 },
    { 8.6699999999999992e-09, 7.5858082011000005e-01, 1.5519227142710000e+02 },
    { 8.6599999999999995e-09, 5.2033868790600000e+00, 1.7849635029824999e+03 },
    { 8.6599999999999995e-09, 3.5515363423099999e+00, 1.2938786542777000e+03 },
    { 8.6499999999999997e-09, 2.7096032409999999e+00, 1.5354889253670001e+03 },
    { 8.6100000000000007e-09, 2.8456860137300000e+00, 1.0956098019934000e+03 },
    { 8.5999999999999993e-09, 3.4047176646000001e+00, 4.7365931462300000e+02 },
    { 8.5899999999999995e-09, 2.6262763356400001e+00, 1.8865547581502999e+03 },
    { 8.5799999999999997e-09, 4.1063008749000002e-01, 1.0350029078010000e+03 },
    { 8.5700000000000000e-09, 5.4967378177999997e+00, 6.3847772888000002e+01 },
    { 8.5299999999999993e-09, 4.4609162483300002e+00, 1.2411884562381999e+03 },
    { 8.5199999999999995e-09, 6.1645847308199997e+00, 4.7601051388506003e+03 },
    { 8.5099999999999998e-09, 4.5666793607100002e+00, 3.4660834722210002e+03 },
    { 8.4499999999999996e-09, 1.5220918210400001e+00, 8.0354456509830004e+02 },
    { 8.4399999999999998e-09, 4.2670556387499996e+00, 3.4945376602242000e+03 },
    { 8.4300000000000000e-09, 4.6967953891600001e+00, 2.4846318565920001e+02 },
    { 8.4300000000000000e-09, 1.9283077742999999e-01, 1.7813500521680001e+02 },
    { 8.4200000000000003e-09, 5.6360660310300004e+00, 4.6427852706304002e+03 },
    { 8.4000000000000008e-09, 8.8655149234999997e-01, 1.0513051753418999e+03 },
    { 8.3899999999999994e-09, 9.3530391093999998e-01, 1.3362449706799200e+04 },
    { 8.3699999999999998e-09, 5.4229092879099996e+00, 5.7053231371810000e+02 },
    { 8.3500000000000003e-09, 5.2742603618799997e+00, 9.6006125053929998e+02 },
    { 8.3300000000000008e-09, 4.7819086329899996e+00, 1.6818707287638999e+03 },
    { 8.3099999999999996e-09, 7.7280313249999999e-01, 4.3663249918359998e+02 },
    { 8.2999999999999999e-09, 4.0225285677100002e+00, 8.2154375366859995e+02 },
    { 8.2900000000000001e-09, 1.9448430402700001e+00, 5.9571448525840003e+02 },
    { 8.2700000000000006e-09, 3.5437534165300000e+00, 3.6793825676590000e+03 },
    { 8.2399999999999997e-09, 1.6590490394299999e+00, 5.4785021235930003e+02 },
    { 8.2399999999999997e-09, 3.5270933491000001e-01, 1.1634379122543000e+03 },
    { 8.2100000000000004e-09, 5.3781466068099997e+00, 9.4702954856783999e+03 },
    { 8.1899999999999992e-09, 2.3785501776800002e+00, 6.6502706822760001e+02 },
    { 8.1500000000000002e-09, 4.1137634079699996e+00, 6.9917976649249999e+02 },
    { 8.1500000000000002e-09, 1.5329230695400000e+00, 3.9197661737599998e+02 },
    { 8.1199999999999993e-09, 5.3059895898800002e+00, 2.0007086448546801e+04 },
    { 8.0900000000000000e-09, 6.2792420745799999e+00, 1.3718416465826999e+03 },
    { 8.0700000000000005e-09, 5.1543311714500000e+00, 6.8865533109039995e+02 },
    { 8.0499999999999993e-09, 1.9170003488600000e+00, 5.8009354127370000e+02 },
    { 8.0299999999999998e-09, 2.4301768672600002e+00, 1.9992859354545191e+04 },
    { 8.0200000000000000e-09, 4.2421162480900003e+00, 1.3866594414153001e+03 },
    { 8.0100000000000003e-09, 2.7112028532999999e-01, 4.7535623565240002e+02 },
    { 8.0000000000000005e-09, 2.8363303164999998e-01, 1.4711551657979999e+02 },
    { 8.0000000000000005e-09, 4.6059931349400003e+00, 2.7948267429620000e+02 },
    { 7.9900000000000007e-09, 4.4849247365200000e+00, 2.8435946736463002e+03 },
    { 7.9799999999999993e-09, 2.4689388926100002e+00, 4.3790452357080000e+02 },
    { 7.9500000000000001e-09, 8.7874818130999999e-01, 5.2973914920439995e+02 },
    { 7.9300000000000005e-09, 4.4341701998999999e-01, 9.5162328952459995e+02 },
    { 7.9200000000000008e-09, 4.7071021042899996e+00, 8.2399143422340001e+02 },
    { 7.9200000000000008e-09, 5.8302997825300000e+00, 7.4617145427030005e+02 },
    { 7.9200000000000008e-09, 1.6230636194600001e+00, 5.4227659212385997e+03 },
    { 7.9099999999999994e-09, 3.9660071362500000e+00, 5.2964278098479997e+02 },
    { 7.8899999999999998e-09, 2.8040530899200000e+00, 6.7803032195180003e+02 },
    { 7.8800000000000001e-09, 1.8302600186100000e+00, 6.8302625679790003e+02 },
    { 7.8800000000000001e-09, 4.8967906821499998e+00, 9.3812990870589999e+02 },
    { 7.8800000000000001e-09, 8.6378191467999998e-01, 1.4080171287602000e+03 },
    { 7.8600000000000006e-09, 4.3993920329399998e+00, 1.0532315910349000e+03 },
    { 7.8500000000000008e-09, 5.3885406604200003e+00, 1.3175069171405000e+03 },
    { 7.8299999999999996e-09, 5.1861646293000003e+00, 8.4141696830520004e+02 },
    { 7.8299999999999996e-09, 5.0462434975199999e+00, 9.6250893109410003e+02 },
    { 7.8299999999999996e-09, 1.3440325561799999e+00, 2.2700492457780001e+02 },
    { 7.8299999999999996e-09, 4.0400095681999998e-01, 1.9959326629820001e+02 },
    { 7.8100000000000001e-09, 1.9614042578199999e+00, 4.5468060434126000e+03 },
    { 7.7699999999999994e-09, 4.9595178922000001e+00, 1.0318739036899999e+03 },
    { 7.7599999999999997e-09, 2.7720456728700000e+00, 4.9447869076360001e+02 },
    { 7.7599999999999997e-09, 4.3568598891099999e+00, 2.0140265547571000e+03 },
    { 7.7499999999999999e-09, 1.6863594938099999e+00, 1.7032113697663001e+03 },
    { 7.6899999999999997e-09, 4.3000341059600000e+00, 1.0522201990786000e+03 },
    { 7.6799999999999999e-09, 3.1984147608400000e+00, 8.4533207522580005e+02 },
    { 7.6799999999999999e-09, 1.5432387450200000e+00, 6.0587310542895002e+03 },
    { 7.6799999999999999e-09, 2.2879214821899998e+00, 1.3103933701397000e+03 },
    { 7.6700000000000002e-09, 2.9661764677700000e+00, 2.0934910661129002e+03 },
    { 7.6600000000000004e-09, 6.0383210737699997e+00, 2.4390878832981002e+03 },
    { 7.6600000000000004e-09, 1.6804010490000000e-02, 1.2356117715723999e+03 },
    { 7.5699999999999993e-09, 4.2307819990800004e+00, 5.4959816523824002e+03 },
    { 7.5499999999999998e-09, 1.3377444978199999e+00, 1.8823874949360001e+02 },
    { 7.5400000000000000e-09, 4.2685738242899998e+00, 7.3592469764160001e+02 },
    { 7.5200000000000005e-09, 2.1363587411000000e+00, 4.1310481005730003e+02 },
    { 7.4899999999999996e-09, 3.8672899961299998e+00, 7.8552833018549995e+02 },
    { 7.4799999999999998e-09, 5.9436991076099996e+00, 1.0632770190086199e+04 },
    { 7.4400000000000000e-09, 2.8747509376000002e-01, 4.5719698838770000e+02 },
    { 7.4300000000000002e-09, 5.0254955032700002e+00, 1.0876931058405000e+03 },
    { 7.4199999999999996e-09, 4.7177502403300000e+00, 7.4028174754689996e+02 },
    { 7.4000000000000001e-09, 3.7888345203900000e+00, 1.0160929010109001e+03 },
    { 7.3700000000000000e-09, 2.0850106015600001e+00, 5.2161421024729998e+02 },
    { 7.3600000000000002e-09, 1.2002859341500001e+00, 4.0370364119610002e+02 },
    { 7.2900000000000003e-09, 3.0885585864900000e+00, 1.6496273998495001e+03 },
    { 7.2900000000000003e-09, 6.1310410622299996e+00, 9.5146323083020002e+02 },
    { 7.2799999999999997e-09, 9.9359722954999996e-01, 1.0523165672982000e+03 },
    { 7.2799999999999997e-09, 2.3099264835400000e+00, 8.3918173607150004e+02 },
    { 7.2600000000000002e-09, 2.3223851200899999e+00, 5.4547196777370004e+02 },
    { 7.2500000000000004e-09, 4.1815140904800003e+00, 9.9564603188579997e+02 },
    { 7.2000000000000000e-09, 1.8091115697900000e+00, 2.5562491676980000e+02 },
    { 7.1900000000000002e-09, 1.1946731953100000e+00, 1.7369915610157000e+03 },
    { 7.1900000000000002e-09, 6.2771208912500001e+00, 1.0371171527998279e+05 },
    { 7.1900000000000002e-09, 1.0814415575300000e+00, 4.6743953949949997e+02 },
    { 7.1799999999999996e-09, 2.9666900494699999e+00, 9.9889407505091003e+03 },
    { 7.1699999999999998e-09, 4.5523787918099998e+00, 6.7025604938666002e+03 },
    { 7.1399999999999997e-09, 3.8705431194600002e+00, 1.1758080265321000e+03 },
    { 7.0800000000000004e-09, 4.7211834024300003e+00, 2.4312235767718998e+03 },
    { 7.0299999999999999e-09, 1.5546032188900001e+00, 5.6542242639412998e+03 },
    { 7.0100000000000004e-09, 4.5827319905300001e+00, 4.3263934009738005e+03 },
    { 6.9900000000000001e-09, 5.3407768075200002e+00, 7.2558157279329998e+02 },
    { 6.9800000000000003e-09, 3.2560359173200002e+00, 4.9819839570559998e+02 },
    { 6.9600000000000000e-09, 3.1715511287600000e+00, 2.2540999635113999e+03 },
    { 6.8699999999999996e-09, 5.4336062892200001e+00, 1.0384353374400000e+02 },
    { 6.8699999999999996e-09, 4.4096222515900001e+00, 1.5638001010183000e+03 },
    { 6.8500000000000001e-09, 5.8608979755300004e+00, 4.5505237887919998e+02 },
    { 6.8400000000000004e-09, 4.8175858891500001e+00, 6.0626632075526004e+03 },
    { 6.8100000000000003e-09, 4.9708906934400003e+00, 6.0527571281400003e+02 },
    { 6.8100000000000003e-09, 2.4040047291600000e+00, 4.5809076026499997e+02 },
    { 6.7999999999999997e-09, 4.1419088303900002e+00, 2.1438936422920001e+03 },
};

static const VSOP87Term VSOP87A_SATURN_X1[892] = {
    { 7.5751039620000002e-02, 0.0000000000000000e+00, 0.0000000000000000e+00 },
    { 3.0850417160000001e-02, 4.2756574912799996e+00, 4.2659819087599999e+02 },
    { 2.7149183989999999e-02, 5.8522941239700002e+00, 2.0618554843720000e+02 },
    { 2.6431009089999999e-02, 5.3329195058399996e+00, 2.2041264243879999e+02 },
    { 6.2710452000000003e-03, 3.2898307969000001e-01, 7.1135470007999997e+00 },
    { 3.1235651200000000e-03, 4.8300172494100000e+00, 4.1948464387519999e+02 },
    { 2.5656095300000000e-03, 3.5247893434300002e+00, 6.3989728631399998e+02 },
    { 2.0364657000000001e-03, 1.1099868178200001e+00, 2.1329909543799999e+02 },
    { 1.8919627399999999e-03, 4.4864245355200003e+00, 4.3371173787679999e+02 },
    { 1.1953114500000001e-03, 1.1473509607800001e+00, 1.1020632121940000e+02 },
    { 6.6901224999999995e-04, 5.2025750037999998e+00, 2.2752618943959999e+02 },
    { 6.6764237999999997e-04, 3.7234659692799998e+00, 3.1639186965660002e+02 },
    { 3.1000839999999999e-04, 6.0606791943699996e+00, 1.9907200143640000e+02 },
    { 3.0418099999999999e-04, 1.8746903350999999e-01, 1.4227094001599999e+01 },
    { 2.2275210000000000e-04, 6.1953087801400004e+00, 1.0309277421860000e+02 },
    { 1.8939377000000001e-04, 2.7761830672499999e+00, 8.5319638175199998e+02 },
    { 1.8093009000000001e-04, 5.0916272386500001e+00, 2.0936694217490000e+02 },
    { 1.7777854000000000e-04, 6.1038159335100000e+00, 2.1723124870109999e+02 },
    { 1.7120250000000000e-04, 4.5961166418800001e+00, 6.3278373931320004e+02 },
    { 1.6296200999999999e-04, 4.8694568143700003e+00, 2.1648048917570000e+02 },
    { 1.6192653000000000e-04, 5.6079801444999999e+00, 3.2350541665740002e+02 },
    { 1.5894491000000000e-04, 3.6535023040000003e-02, 2.1011770170029999e+02 },
    { 1.4466009999999999e-04, 3.6744938009000001e+00, 6.4701083331480004e+02 },
    { 1.1061528000000000e-04, 3.1630714609999999e-02, 1.1731986822019999e+02 },
    { 9.8731830000000004e-05, 5.2006530735699998e+00, 2.0225339517410001e+02 },
    { 8.7076079999999997e-05, 6.0351173163700000e+00, 2.2434479570190001e+02 },
    { 5.5122220000000000e-05, 2.6055664234800000e+00, 1.1045700263900001e+01 },
    { 5.4991090000000002e-05, 4.4035060341500003e+00, 4.4082528487759998e+02 },
    { 4.0082569999999999e-05, 1.4894296680700001e+00, 5.2257741809380002e+02 },
    { 3.5711959999999997e-05, 5.1082190837899999e+00, 4.1237109687440000e+02 },
    { 2.7637859999999999e-05, 3.9625359020900000e+00, 9.5979227217800002e+01 },
    { 2.7313810000000000e-05, 4.1089222365999998e+00, 1.4956319713459999e+02 },
    { 1.8758620000000002e-05, 2.5238408058599999e+00, 3.9321532631000000e+00 },
    { 1.7658160000000000e-05, 7.5684544353000005e-01, 2.7703499374140000e+02 },
    { 1.6889570000000000e-05, 3.9827095073100001e+00, 4.2266603761290003e+02 },
    { 1.5447870000000000e-05, 5.3028392383599998e+00, 3.3061896365820002e+02 },
    { 1.5116480000000000e-05, 6.2327459877700004e+00, 3.1813937377000001e+00 },
    { 1.4491430000000000e-05, 6.2650717986100002e+00, 5.2969096509459996e+02 },
    { 1.3955509999999999e-05, 2.0353364254100001e+00, 1.0664954771900000e+03 },
    { 1.3422769999999999e-05, 2.9892955787500002e+00, 2.1085141488319999e+02 },
    { 1.3310450000000001e-05, 1.9089952687700000e+00, 2.1574677599280000e+02 },
    { 1.3304130000000001e-05, 5.0631220321199999e+00, 2.3463973644039999e+02 },
    { 1.2733150000000000e-05, 9.5724293959999998e-02, 7.4299006053259995e+02 },
    { 1.2710500000000000e-05, 2.4033846867499999e+00, 4.1555249061209997e+02 },
    { 1.1716799999999999e-05, 4.5934141212700004e+00, 5.3680451209540001e+02 },
    { 1.1290560000000001e-05, 5.4579452929499999e+00, 4.2341679713830001e+02 },
    { 1.1263999999999999e-05, 2.8817321373400002e+00, 8.6030992875280003e+02 },
    { 1.1012450000000000e-05, 4.0769810882400002e+00, 4.2977958461370002e+02 },
    { 9.6156800000000004e-06, 4.4893645774099999e+00, 8.4608283475120004e+02 },
    { 9.5984900000000007e-06, 9.2675530268999995e-01, 8.3896928775039999e+02 },
    { 9.2641000000000000e-06, 1.3585731558400000e+00, 6.2567019231239999e+02 },
    { 8.8260399999999993e-06, 5.4146430443200000e+00, 4.3764389113990001e+02 },
    { 8.2049699999999998e-06, 4.7455508421900001e+00, 2.2359403617650000e+02 },
    { 8.1109599999999997e-06, 3.2138637050000002e-02, 2.1340641002400002e+01 },
    { 7.6715600000000001e-06, 6.2309026596599999e+00, 1.0593819301891999e+03 },
    { 7.1857200000000002e-06, 6.1272767562299997e+00, 4.3053034413910001e+02 },
    { 6.2584199999999998e-06, 3.8826585190600000e+00, 8.8865680217000005e+01 },
    { 6.0838899999999996e-06, 2.5087066094499999e+00, 3.0927832265580003e+02 },
    { 5.9616900000000002e-06, 5.0997516382299999e+00, 1.2443341522100000e+02 },
    { 5.5139400000000001e-06, 1.5526195683499999e+00, 5.1546387109299997e+02 },
    { 4.7220099999999997e-06, 3.6091498548800001e+00, 6.5412438031559998e+02 },
    { 4.6573399999999997e-06, 2.1394106677999999e-01, 2.0300415469949999e+02 },
    { 4.5606500000000001e-06, 1.1195602461800001e+00, 7.3587651353180001e+02 },
    { 4.0172899999999998e-06, 4.4267933451900001e+00, 8.5827298831199997e+01 },
    { 3.7692200000000000e-06, 3.7124680270499999e+00, 7.6266071275599998e+01 },
    { 3.4038500000000000e-06, 1.7861164851800000e+00, 3.0216477565500003e+02 },
    { 3.3964900000000002e-06, 2.7382306255000000e+00, 2.1796496188399999e+02 },
    { 3.3936000000000000e-06, 7.4274349867000000e-01, 1.1553611574070001e+03 },
    { 3.3430300000000000e-06, 3.0781870291800000e+00, 2.0373786788240000e+02 },
    { 3.2337600000000000e-06, 9.8449125890000003e-02, 1.9195845443560000e+02 },
    { 3.1493800000000002e-06, 2.8470065321999999e-01, 7.2876296653099996e+02 },
    { 3.1151099999999999e-06, 2.3648434986400000e+00, 6.2885158605009997e+02 },
    { 3.1115400000000000e-06, 1.9111392773200000e+00, 9.4206206196899996e+02 },
    { 3.0496800000000001e-06, 1.8416896544000000e-01, 2.3145834270270001e+02 },
    { 2.9822699999999998e-06, 2.1636437464000000e+00, 2.0863322899200000e+02 },
    { 2.9060700000000000e-06, 1.8117953441700001e+00, 2.2286032299359999e+02 },
    { 2.7065799999999999e-06, 2.7026551936100001e+00, 2.8808069400530002e+02 },
    { 2.6417300000000000e-06, 3.5530712615000000e+00, 3.6286229257259998e+02 },
    { 2.5989400000000001e-06, 8.7228358124000005e-01, 1.8159247264699999e+01 },
    { 2.5951500000000000e-06, 6.0166001848599997e+00, 1.0294940738499999e+01 },
    { 2.1485900000000002e-06, 6.0319696118200001e+00, 2.0788246946660001e+02 },
    { 2.1249900000000001e-06, 5.3903466404799998e+00, 9.4917560896980001e+02 },
    { 2.1117000000000000e-06, 2.1269877508200001e+00, 1.3851749687070000e+02 },
    { 2.0899600000000001e-06, 5.1644212039199999e+00, 2.1871572140940000e+02 },
    { 2.0826400000000000e-06, 4.4098129791599998e+00, 4.0843894361129998e+02 },
    { 1.9580900000000001e-06, 3.5649768849000002e-01, 5.2690198039499997e+01 },
    { 1.9068699999999999e-06, 6.2200015250799998e+00, 2.0076892246579999e+02 },
    { 1.8852899999999999e-06, 4.5283523313000001e-01, 3.4077089204480001e+02 },
    { 1.7768800000000000e-06, 6.2209753072599998e+00, 2.1621985674480001e+02 },
    { 1.7750900000000001e-06, 4.9408587982399998e+00, 2.1037833413120001e+02 },
    { 1.7715100000000000e-06, 6.2696260816300002e+00, 1.6060889739850001e+02 },
    { 1.7173300000000000e-06, 1.1289784890100001e+00, 8.3185574074960005e+02 },
    { 1.7164600000000000e-06, 2.3045716357099999e+00, 1.2747179660680000e+02 },
    { 1.7083799999999999e-06, 1.1743920517799999e+00, 3.5033211960040001e+02 },
    { 1.7034700000000001e-06, 2.6369850510799999e+00, 2.0767002114549999e+02 },
    { 1.6720300000000000e-06, 5.0029204528999998e+00, 2.2582926841020000e+02 },
    { 1.6496799999999999e-06, 4.6205948267699997e+00, 2.2091400527800001e+01 },
    { 1.6388900000000000e-06, 2.2392037969400000e+00, 2.1892816973050000e+02 },
    { 1.6333900000000000e-06, 1.4882763710800000e+00, 1.9513984817330001e+02 },
    { 1.6256599999999999e-06, 6.1252463472900001e+00, 9.5628915597059995e+02 },
    { 1.5276600000000000e-06, 4.2693590215599997e+00, 5.6622351302600002e+01 },
    { 1.5081199999999999e-06, 3.1387811175999998e+00, 6.3596513305090002e+02 },
    { 1.4956999999999999e-06, 3.1719520948000002e-01, 9.5612275556000004e+00 },
    { 1.4619400000000000e-06, 4.2027539012800004e+00, 7.0849445304200003e+01 },
    { 1.4546900000000000e-06, 4.6371394258500001e+00, 2.6598929347749998e+02 },
    { 1.4071600000000000e-06, 4.4083369403399999e+00, 1.4244965013379999e+02 },
    { 1.4065400000000000e-06, 4.8104293652000002e-01, 1.4717530270636000e+03 },
    { 1.3750700000000000e-06, 1.6588073054700001e+00, 1.2584539316256000e+03 },
    { 1.3664599999999999e-06, 4.5163023683699999e+00, 5.4391805909619995e+02 },
    { 1.3475499999999999e-06, 3.9246165477699999e+00, 7.5010360753340001e+02 },
    { 1.3455800000000000e-06, 8.4258328590999998e-01, 4.9033408917940000e+02 },
    { 1.3375399999999999e-06, 1.2592820972200001e+00, 7.0363318461740005e+02 },
    { 1.2554099999999999e-06, 4.2907790850899996e+00, 4.4793883187839998e+02 },
    { 1.2309599999999999e-06, 2.8486968643399999e+00, 4.8322054217860000e+02 },
    { 1.2266400000000001e-06, 1.7364795415000001e+00, 1.2530172972200001e+01 },
    { 1.1795399999999999e-06, 2.1522771952599999e+00, 6.1855664531160005e+02 },
    { 1.1156800000000000e-06, 3.5050063862299998e+00, 2.6992144674060000e+02 },
    { 1.0804799999999999e-06, 6.2510448474200002e+00, 1.0451548361876000e+03 },
    { 1.0752999999999999e-06, 2.0478165492299998e+00, 4.2415051032119999e+02 },
    { 1.0679000000000000e-06, 2.7458772012599999e+00, 5.6511568774670002e+02 },
    { 1.0545600000000000e-06, 3.5845817117599998e+00, 2.0910630974399999e+02 },
    { 1.0421599999999999e-06, 1.3036705757400000e+00, 2.1749188113200000e+02 },
    { 1.0368000000000000e-06, 1.3827521003000001e-01, 1.0627416795630000e+02 },
    { 9.7995000000000004e-07, 1.2723099159200000e+00, 1.2797945726280000e+03 },
    { 9.5815000000000009e-07, 2.8074923031300001e+00, 1.9120769491019999e+02 },
    { 9.1966999999999998e-07, 2.3104306992699999e+00, 4.9744763618019999e+02 },
    { 8.9612000000000001e-07, 4.3880786475700004e+00, 1.2655674786264001e+03 },
    { 8.9154000000000000e-07, 7.7963675709999999e-02, 6.3735898303399999e+01 },
    { 8.8400000000000003e-07, 2.0984351215800001e+00, 1.0736090241908000e+03 },
    { 8.4412999999999998e-07, 6.1721123565099996e+00, 1.1338771495710000e+02 },
    { 8.4109000000000002e-07, 1.7716914652700000e+00, 2.1944943459230001e+02 },
    { 8.2302999999999998e-07, 3.2854878254500002e+00, 6.4307868005169996e+02 },
    { 8.2126999999999999e-07, 2.0504163990999999e-01, 1.1624747044077999e+03 },
    { 8.1345999999999998e-07, 4.7063511823599997e+00, 6.5094298657790000e+02 },
    { 7.8375000000000001e-07, 1.2805764386400000e+00, 1.0702492748170000e+02 },
    { 7.4954999999999999e-07, 1.4137437842300000e+00, 2.8414854074220000e+02 },
    { 7.3213999999999999e-07, 4.3114971671999998e+00, 3.4470304530790003e+02 },
    { 7.3134999999999999e-07, 8.5149756693000000e-01, 1.1482476104062000e+03 },
    { 7.1289999999999998e-07, 5.5445341457500001e+00, 1.3757737998458001e+03 },
    { 7.0686000000000002e-07, 2.4084637803000000e+00, 2.5143213107579999e+02 },
    { 6.6418000000000001e-07, 4.5313106829500001e+00, 3.3773251065900001e+02 },
    { 6.5942999999999997e-07, 3.1580687131400000e+00, 2.0714875628370001e+02 },
    { 6.5942000000000001e-07, 4.8387279362099997e+00, 6.3671589257630001e+02 },
    { 6.4926000000000003e-07, 1.3837898547700000e+00, 1.5748458012822000e+03 },
    { 6.3883000000000001e-07, 1.9893452814100001e+00, 2.3539049596580000e+02 },
    { 6.3076999999999998e-07, 3.8264964353000002e+00, 3.5893013930950002e+02 },
    { 6.3038999999999997e-07, 3.9791318494399999e+00, 4.3689313161450002e+02 },
    { 6.3005000000000001e-07, 5.1022913320200001e+00, 4.7928838891549998e+02 },
    { 6.1592999999999998e-07, 1.8111349610999999e-01, 4.0450679034820001e+02 },
    { 6.1070999999999999e-07, 2.1868225550000001e-01, 1.7881448967202000e+03 },
    { 6.0282999999999995e-07, 4.2435516368400004e+00, 4.1703696332039999e+02 },
    { 5.9724000000000002e-07, 6.1566726562699996e+00, 9.9160620955499994e+01 },
    { 5.9411000000000002e-07, 3.3326775598300000e+00, 7.4781598567299994e+01 },
    { 5.8706000000000002e-07, 7.4631691498999997e-01, 1.1413847448250000e+02 },
    { 5.7459000000000001e-07, 1.3001355087699999e+00, 9.8899988524600005e+01 },
    { 5.5973999999999996e-07, 5.3749535499000001e-01, 5.5406998748280000e+02 },
    { 5.4283000000000000e-07, 4.7402761164000001e-01, 1.5890728952838001e+03 },
    { 5.4191999999999997e-07, 2.4219176913300000e+00, 1.7516605980020000e+02 },
    { 5.2615000000000004e-07, 4.0206359198900001e+00, 4.6658664459999999e+00 },
    { 5.2017999999999999e-07, 2.0114535584900000e+00, 2.2046082654860001e+02 },
    { 5.0842000000000000e-07, 2.6605555890599999e+00, 1.8105576652360000e+02 },
    { 5.0833000000000005e-07, 2.0533098403999999e+00, 9.3494851496820002e+02 },
    { 5.0403000000000005e-07, 1.5673694340900000e+00, 1.2125202148330000e+02 },
    { 5.0399999999999996e-07, 1.5209542425000000e+00, 4.2904587143079999e+02 },
    { 5.0350000000000003e-07, 5.6172354023400004e+00, 4.1630325013750002e+02 },
    { 4.8508999999999996e-07, 3.0866405070599998e+00, 1.5104766984290001e+02 },
    { 4.8167000000000000e-07, 1.8913965564199999e+00, 3.5181659230870002e+02 },
    { 4.7119999999999998e-07, 1.2179552442599999e+00, 1.0522683831884001e+03 },
    { 4.5466000000000002e-07, 3.9512211319800001e+00, 1.5819593482830001e+03 },
    { 4.4915000000000002e-07, 2.3072052628400002e+00, 2.4554242435239999e+02 },
    { 4.4554000000000000e-07, 1.9573241140800000e+00, 8.4215068148809996e+02 },
    { 4.4007000000000000e-07, 2.2618765467399999e+00, 2.0522234059069999e+02 },
    { 4.3682999999999997e-07, 2.4941622469200002e+00, 1.9832124191099999e+02 },
    { 4.1544000000000002e-07, 5.9029849061700004e+00, 6.4382943957709995e+02 },
    { 4.1101000000000003e-07, 5.9798216701399998e+00, 1.3615467058442000e+03 },
    { 4.0793999999999999e-07, 3.7745361058200002e+00, 6.2173803904930003e+02 },
    { 4.0579999999999999e-07, 2.8230226241000000e+00, 8.6742347575359997e+02 },
    { 4.0198000000000000e-07, 4.0030518433399997e+00, 8.1752133216199994e+01 },
    { 3.9033999999999999e-07, 4.4650963373200003e+00, 1.3154696222179999e+02 },
    { 3.7520000000000002e-07, 2.3835306768800000e+00, 2.2827694896500000e+02 },
    { 3.6819999999999998e-07, 2.9782633630299999e+00, 2.0613736432740001e+02 },
    { 3.6724000000000000e-07, 1.8905337994300000e+00, 1.4807872442630000e+02 },
    { 3.6297999999999999e-07, 5.6645962876500002e+00, 4.4475743814070000e+02 },
    { 3.6203999999999998e-07, 2.7500199748699998e+00, 2.1404985496340001e+02 },
    { 3.5352000000000001e-07, 2.3793320123499999e+00, 5.0137978944330001e+02 },
    { 3.5204999999999998e-07, 6.1348791110000001e+00, 1.4788665740644001e+03 },
    { 3.4395000000000001e-07, 5.5991266624000002e-01, 1.4646394800628000e+03 },
    { 3.4251000000000001e-07, 2.8792205467400001e+00, 2.8956516671359998e+02 },
    { 3.3504000000000001e-07, 1.0451441450300001e+00, 4.3615941843159999e+02 },
    { 3.3234000000000002e-07, 5.2597298517700004e+00, 1.7787437278589999e+02 },
    { 3.2822000000000001e-07, 2.1947545824899999e+00, 2.1254833591260001e+02 },
    { 3.1852000000000002e-07, 3.2147940814900000e+00, 4.1406801790380001e+02 },
    { 3.1833999999999998e-07, 1.7956617937399999e+00, 2.7555052103309998e+02 },
    { 3.1244000000000001e-07, 4.4284320507999997e+00, 7.8713751830400000e+01 },
    { 3.1198000000000002e-07, 1.1075698988300000e+00, 1.8912376709388000e+03 },
    { 3.0914999999999999e-07, 5.2423918533800000e+00, 1.6921656695024001e+03 },
    { 2.9956999999999998e-07, 5.4048768185700000e+00, 3.2243328914400003e+01 },
    { 2.9442000000000001e-07, 5.7660878613900000e+00, 3.5574874557179999e+02 },
    { 2.9366000000000000e-07, 5.7086986114300000e+00, 2.1478356814630001e+02 },
    { 2.8965000000000000e-07, 5.9260316232300001e+00, 2.4872381809010000e+02 },
    { 2.7676000000000000e-07, 8.9718596794000005e-01, 3.4788443904560000e+02 },
    { 2.7617999999999998e-07, 6.2254675643799997e+00, 2.1045367663768002e+03 },
    { 2.7482000000000002e-07, 3.8599706759400001e+00, 1.1439910691340000e+02 },
    { 2.5563000000000002e-07, 4.0292891604200003e+00, 6.9152524274800001e+01 },
    { 2.5281000000000000e-07, 1.2506591459500001e+00, 4.2193232442999999e+02 },
    { 2.4810000000000000e-07, 8.3325589326000005e-01, 4.2563498302950001e+02 },
    { 2.4663999999999999e-07, 4.5616854888400002e+00, 4.2756139872249997e+02 },
    { 2.4476999999999999e-07, 1.7574124279000001e+00, 1.2513403846248000e+03 },
    { 2.4261000000000003e-07, 2.4600555997000001e-01, 1.2726810256271999e+03 },
    { 2.4182999999999999e-07, 1.7668226820199999e+00, 5.0835032409220003e+02 },
    { 2.3974000000000000e-07, 3.1370364878000001e+00, 1.3458534360760001e+02 },
    { 2.3967999999999999e-07, 1.9375665678600000e+00, 4.3126405732199999e+02 },
    { 2.3454999999999999e-07, 2.9606076932500001e+00, 2.7851946644970002e+02 },
    { 2.3120999999999999e-07, 4.4741693130600000e+00, 2.3070758317729999e+02 },
    { 2.3061000000000000e-07, 3.7224615774699998e+00, 1.8983512179396000e+03 },
    { 2.2410000000000000e-07, 4.1701470822999998e+00, 3.9356875915200000e+01 },
    { 2.2384000000000000e-07, 2.8538915908900000e+00, 5.7616138801060004e+02 },
    { 2.2254000000000001e-07, 2.9747319118600002e+00, 6.0176425067620005e+02 },
    { 2.1876999999999999e-07, 3.7409486360200002e+00, 2.9912639426919998e+02 },
    { 2.1820000000000001e-07, 5.1766844928399998e+00, 4.0525754987360000e+02 },
    { 2.1731999999999999e-07, 5.2200571077699998e+00, 2.1181462272970001e+02 },
    { 2.1278000000000000e-07, 3.9974771937800000e+00, 3.1957326339430000e+02 },
    { 2.1271999999999999e-07, 5.5451958674000001e-01, 9.1693228005540004e+02 },
    { 2.1171000000000001e-07, 2.7761883641999998e+00, 2.2137585028530000e+02 },
    { 2.1066999999999999e-07, 4.8938198404000000e+00, 4.2118156490460001e+02 },
    { 2.0790999999999999e-07, 4.8633753092200003e+00, 2.4175328344120001e+02 },
    { 2.0377999999999999e-07, 4.2434049831899996e+00, 4.2655000676620000e+02 },
    { 2.0375000000000001e-07, 1.1587530685900000e+00, 4.2664637498579998e+02 },
    { 1.9942000000000000e-07, 1.8712028803100000e+00, 2.9505122865419997e+02 },
    { 1.9879000000000000e-07, 4.9510549929899996e+00, 3.1245971639350000e+02 },
    { 1.9761999999999999e-07, 2.0352395509200001e+00, 6.9651963761659999e+02 },
    { 1.9761000000000001e-07, 1.4480757519900000e+00, 1.8892007304980001e+02 },
    { 1.9556000000000001e-07, 5.5700539266900000e+00, 3.2743756992049998e+02 },
    { 1.9224000000000001e-07, 2.1842700921099998e+00, 3.3365734504400001e+02 },
    { 1.8507000000000000e-07, 4.7083436463300004e+00, 1.4414657116320001e+02 },
    { 1.8385000000000000e-07, 3.7551557521099999e+00, 6.7668051566499997e+01 },
    { 1.8206999999999999e-07, 4.8659303659000003e-01, 6.2759623029905997e+03 },
    { 1.8160000000000001e-07, 5.3733651077199998e+00, 2.0543478891180001e+02 },
    { 1.8071000000000001e-07, 6.2175954015699997e+00, 1.6682867252200001e+02 },
    { 1.7797000000000000e-07, 6.1760691228000004e+00, 6.2901893969922003e+03 },
    { 1.7765000000000001e-07, 1.1521843957400000e+00, 5.4166259714000002e+00 },
    { 1.7662000000000000e-07, 5.8327214570599999e+00, 1.7952584437210000e+03 },
    { 1.7576000000000001e-07, 4.8887220803100000e+00, 3.1321047591889999e+02 },
    { 1.7303000000000001e-07, 5.1348019813799999e+00, 3.2032402291969998e+02 },
    { 1.7270000000000000e-07, 4.3461363289900001e+00, 1.2035824960600000e+02 },
    { 1.7154000000000001e-07, 3.4193282006899999e+00, 2.3767811782620001e+02 },
    { 1.7109999999999999e-07, 1.2914621427099999e+00, 4.3222726516850003e+02 },
    { 1.7090000000000000e-07, 1.9098181250099999e+00, 1.0206171999210201e+04 },
    { 1.6705999999999999e-07, 1.3172826359700001e+00, 1.0220399093211799e+04 },
    { 1.6689999999999999e-07, 2.9893752523000000e-01, 1.7810313497193999e+03 },
    { 1.6597000000000000e-07, 8.5397214019000001e-01, 2.2076295405954002e+03 },
    { 1.6458000000000001e-07, 4.3080335169400001e+00, 4.3912836384820002e+02 },
    { 1.6166000000000001e-07, 4.6922740021499996e+00, 1.4977853527000001e+01 },
    { 1.6075000000000001e-07, 1.9242040565400000e+00, 7.7841478318470001e+02 },
    { 1.5935000000000001e-07, 5.7604144894799996e+00, 1.6779385755008000e+03 },
    { 1.5239000000000000e-07, 5.3233825861800002e+00, 4.2951895218279998e+02 },
    { 1.5127999999999999e-07, 1.5158617004399999e+00, 7.1074673161819999e+02 },
    { 1.4849000000000000e-07, 5.2670450998099998e+00, 1.8227960680100000e+02 },
    { 1.4807000000000000e-07, 5.0145816514500003e+00, 2.7310284047829998e+02 },
    { 1.4583000000000000e-07, 6.8816508751000005e-01, 9.9911380480899993e+01 },
    { 1.4560999999999999e-07, 4.5500197472700004e+00, 3.8135160823740000e+02 },
    { 1.4469999999999999e-07, 6.9520749578999996e-01, 5.9803745040300001e+01 },
    { 1.4448999999999999e-07, 6.1826112071299999e+00, 5.2549817940059995e+02 },
    { 1.4172000000000000e-07, 1.9960468334000001e-01, 1.1642609634290000e+02 },
    { 1.4072000000000001e-07, 4.0634492862100000e+00, 3.9557870223899999e+02 },
    { 1.3920999999999999e-07, 3.3345745877900002e+00, 5.5800214074589996e+02 },
    { 1.3624999999999999e-07, 1.0643419563300001e+00, 3.1219908396260001e+02 },
    { 1.3601000000000001e-07, 1.7868852981000000e-01, 9.6872999095099999e+01 },
    { 1.3493000000000000e-07, 2.4321526516300001e+00, 8.4926422848890002e+02 },
    { 1.3339000000000001e-07, 5.0299203774399999e+00, 2.0085575391590000e+03 },
    { 1.3246000000000001e-07, 4.6548380338499999e+00, 1.2991947716160001e+02 },
    { 1.3178000000000001e-07, 1.4767572218200000e+00, 1.5677322542813999e+03 },
    { 1.3052999999999999e-07, 4.9088633539900002e+00, 1.9603362005060001e+02 },
    { 1.2965000000000000e-07, 3.4512426136299998e+00, 8.7311771539500000e+01 },
    { 1.2694000000000000e-07, 5.4472126758000003e+00, 1.1695882514085999e+03 },
    { 1.2655000000000000e-07, 5.9528382794099999e+00, 2.4209286360333999e+03 },
    { 1.2602000000000001e-07, 6.0991765622800003e+00, 2.8454188003199999e+01 },
    { 1.2510000000000000e-07, 3.7818398791800001e+00, 7.5721715453419995e+02 },
    { 1.2505000000000000e-07, 6.0685488576299997e+00, 1.9662432088160000e+02 },
    { 1.2398999999999999e-07, 4.1674425474000003e+00, 3.2972519178089999e+02 },
    { 1.2205000000000001e-07, 3.5075290789400002e+00, 6.6123792731640003e+02 },
    { 1.2156000000000001e-07, 5.8378252244200004e+00, 3.7390799283650000e+02 },
    { 1.2081000000000001e-07, 2.1788912729000001e+00, 6.2251425595100002e+01 },
    { 1.1992000000000000e-07, 1.4624723663100001e+00, 5.6363121503840000e+02 },
    { 1.1964000000000000e-07, 6.2336829188899996e+00, 2.3056457082540001e+02 },
    { 1.1819000000000000e-07, 5.4037105134999996e-01, 4.3079097657000000e+02 },
    { 1.1742000000000000e-07, 2.5772520045800000e+00, 4.5576651038700000e+01 },
    { 1.1552000000000000e-07, 1.8455441206900001e+00, 5.8493641121146002e+03 },
    { 1.1528000000000000e-07, 4.8193012536399999e+00, 4.3201481684740003e+02 },
    { 1.1494999999999999e-07, 3.7585889113999998e-01, 9.2047073954699997e+01 },
    { 1.1422000000000000e-07, 1.3933692639000000e-01, 6.1780588578619995e+02 },
    { 1.1380000000000000e-07, 1.4423768879200001e+00, 1.5667674413540001e+02 },
    { 1.1378000000000000e-07, 3.4372716133400001e+00, 2.2147430875962000e+03 },
    { 1.1336000000000000e-07, 4.2186204640899998e+00, 1.3140394986990000e+02 },
    { 1.1322000000000000e-07, 1.6508827712400000e+00, 7.7693031047640000e+02 },
    { 1.1105000000000001e-07, 5.6027295752499997e+00, 4.6470422915999997e+01 },
    { 1.0930000000000001e-07, 4.1961944044599999e+00, 1.4563104387150000e+02 },
    { 1.0914999999999999e-07, 2.7944549962299998e+00, 1.9996577331370000e+02 },
    { 1.0876000000000000e-07, 3.3433301484300002e+00, 3.0534616939270001e+02 },
    { 1.0814000000000000e-07, 3.2721893582699999e+00, 9.7795738083342003e+03 },
    { 1.0778000000000000e-07, 4.1803381518400000e+00, 3.4321857259960001e+02 },
    { 1.0712000000000000e-07, 6.0878565230300001e+00, 4.2711945573780002e+02 },
    { 1.0673000000000000e-07, 5.8203845343399996e+00, 2.2116340196420001e+02 },
    { 1.0670999999999999e-07, 5.5962913664600000e+00, 4.2607692601420001e+02 },
    { 1.0639000000000000e-07, 6.1839689269999998e-01, 8.4342826122899993e+01 },
    { 1.0536000000000000e-07, 3.5258181880000003e-02, 1.5349535039770001e+02 },
    { 1.0519000000000000e-07, 2.1710656525299998e+00, 9.2940845831999994e+01 },
    { 1.0417000000000000e-07, 4.5736801747199998e+00, 4.1644497775600001e+01 },
    { 1.0377000000000000e-07, 2.4373455232299999e+00, 2.1240532356070000e+02 },
    { 1.0377000000000000e-07, 2.4522806433600000e+00, 2.1419286731529999e+02 },
    { 1.0288000000000001e-07, 1.2521722865700000e+00, 5.8635912061161998e+03 },
    { 1.0237999999999999e-07, 5.8523085907899999e+00, 2.4431858407499999e+02 },
    { 1.0229000000000000e-07, 1.1692448632300001e+00, 1.9928444975750000e+02 },
    { 1.0216999999999999e-07, 5.7190622511100004e+00, 6.5220371011699996e+01 },
    { 1.0094000000000000e-07, 4.8768131605600002e+00, 2.5976951835400001e+02 },
    { 1.0068000000000000e-07, 1.3635627578800000e+00, 2.0670681329900000e+02 },
    { 1.0057000000000001e-07, 4.5621087058999998e-01, 2.5887574647669999e+02 },
    { 1.0051000000000000e-07, 6.2803050508000002e-01, 7.6736908292080000e+02 },
    { 1.0047000000000000e-07, 4.6188126694999998e-01, 2.0606927819500001e+01 },
    { 1.0043000000000001e-07, 2.0951710014699998e+00, 2.2663241756229999e+02 },
    { 9.8589999999999998e-08, 2.8811472610600002e+00, 5.7222923474749996e+02 },
    { 9.7790000000000004e-08, 2.4554089795900000e+00, 1.8816931352440000e+02 },
    { 9.7510000000000004e-08, 1.0049505546199999e+00, 1.9269216761850001e+02 },
    { 9.7450000000000002e-08, 4.9564995237799998e+00, 1.8498791978669999e+02 },
    { 9.7250000000000001e-08, 3.3002356349599999e+00, 1.8180652604900001e+02 },
    { 9.6719999999999995e-08, 6.2802960702700004e+00, 2.3857188970350001e+02 },
    { 9.6120000000000003e-08, 2.6794965763000000e+00, 9.7938009023358009e+03 },
    { 9.5900000000000005e-08, 4.9481047241200002e+00, 3.2668681039510000e+02 },
    { 9.5570000000000001e-08, 8.9168093040999996e-01, 4.1162033734900001e+02 },
    { 9.3709999999999997e-08, 3.7204308734899998e+00, 2.2731374111849999e+02 },
    { 9.2369999999999999e-08, 3.9664350736600000e+00, 2.8096714700450002e+02 },
    { 9.2099999999999998e-08, 4.5089787702999997e+00, 7.7962992305000000e+01 },
    { 9.2029999999999998e-08, 4.3015304866999999e+00, 2.1457111982519999e+02 },
    { 9.2029999999999998e-08, 5.8809567989000000e-01, 2.1202707105080000e+02 },
    { 9.1870000000000001e-08, 3.2625026111700000e+00, 5.5651766803759995e+02 },
    { 9.0610000000000002e-08, 2.7298648830599999e+00, 2.0623373254699999e+02 },
    { 9.0079999999999997e-08, 1.1855739891899999e+00, 6.3744960575920004e+02 },
    { 8.9260000000000007e-08, 3.8000835846799998e+00, 2.3390602325750001e+02 },
    { 8.9019999999999999e-08, 9.2333738768999996e-01, 8.1895145568100006e+01 },
    { 8.8839999999999993e-08, 2.0359171569500001e+00, 1.5912442469019999e+02 },
    { 8.8730000000000001e-08, 3.2632922925700001e+00, 9.6340270297140000e+02 },
    { 8.7870000000000005e-08, 1.2969549837700001e+00, 5.1205725331200000e+01 },
    { 8.7699999999999998e-08, 5.5056717354200000e+00, 2.1116503133776000e+03 },
    { 8.7100000000000006e-08, 2.5187009489499999e+00, 2.3842887735159999e+02 },
    { 8.6850000000000000e-08, 5.3011248887500004e+00, 2.2036445832900000e+02 },
    { 8.6760000000000004e-08, 1.3679562372100000e+00, 8.2474219374879999e+02 },
    { 8.3579999999999998e-08, 5.7656413895000003e-01, 2.5240214102519999e+03 },
    { 8.2119999999999997e-08, 5.3579301150000001e-02, 2.0974232193759999e+03 },
    { 8.1890000000000001e-08, 9.3497045776999999e-01, 1.9589060769869999e+02 },
    { 8.1839999999999997e-08, 6.1834195707199999e+00, 2.4161027108930000e+02 },
    { 8.1600000000000003e-08, 1.2827027820000001e+00, 6.0766952886799999e+01 },
    { 8.0400000000000005e-08, 2.6867651352799999e+00, 5.4695644048199995e+02 },
    { 8.0299999999999998e-08, 1.5723017324799999e+00, 2.4479166482700001e+02 },
    { 7.9360000000000004e-08, 2.5733152730000000e+00, 1.7394221952280000e+02 },
    { 7.9150000000000004e-08, 2.6339083100899998e+00, 5.2650957135689998e+02 },
    { 7.8260000000000001e-08, 3.5255286566900002e+00, 2.3575873236100001e+01 },
    { 7.6290000000000004e-08, 3.3825162836499998e+00, 2.9519424100610001e+02 },
    { 7.5969999999999998e-08, 4.5629194014900003e+00, 1.8647239249500001e+02 },
    { 7.3889999999999995e-08, 2.4969774747600000e+00, 2.5494359321360000e+02 },
    { 7.3029999999999999e-08, 1.2222156417800001e+00, 1.8841241239379999e+03 },
    { 7.2950000000000001e-08, 4.6947569621000002e-01, 1.4930936680660000e+03 },
    { 7.2769999999999996e-08, 2.4749756879599998e+00, 6.1144309831080000e+02 },
    { 7.1869999999999994e-08, 3.9083860390999998e+00, 8.6424208201590000e+02 },
    { 7.1079999999999999e-08, 1.5633328969000000e+00, 4.2734895040139997e+02 },
    { 7.1060000000000002e-08, 2.6861530499800002e+00, 5.7074476203920005e+02 },
    { 7.0760000000000006e-08, 1.5116351888899999e+00, 4.6473122651379998e+02 },
    { 7.0350000000000005e-08, 3.9558609231399999e+00, 4.5884151979040001e+02 },
    { 7.0140000000000005e-08, 3.2303171071999999e+00, 6.2736711334179995e+02 },
    { 6.9409999999999998e-08, 1.4762964781500001e+00, 5.6118353448360006e+02 },
    { 6.8939999999999994e-08, 5.9864810143599998e+00, 2.8245161971279998e+02 },
    { 6.7620000000000006e-08, 2.2742402667500001e+00, 4.2096911658350001e+02 },
    { 6.6650000000000005e-08, 1.2589314042499999e+00, 3.5744566660120000e+02 },
    { 6.6479999999999997e-08, 1.3147501654200000e+00, 1.2869081196288000e+03 },
    { 6.5180000000000005e-08, 6.2180373693800002e+00, 6.3841281360569997e+02 },
    { 6.4700000000000004e-08, 5.2548508131100000e+00, 1.6209337010679999e+02 },
    { 6.4469999999999994e-08, 4.2515572453299999e+00, 1.0775864066459999e+02 },
    { 6.4360000000000002e-08, 5.6478343260699999e+00, 1.9943304451573999e+03 },
    { 6.3150000000000006e-08, 4.8620482315500002e+00, 2.3249494088155998e+03 },
    { 6.2909999999999999e-08, 4.7971599362399999e+00, 4.2367742956920000e+02 },
    { 6.2859999999999995e-08, 9.4028611989999999e-01, 4.3274853003030000e+02 },
    { 6.2299999999999995e-08, 2.9356251623200000e+00, 2.7406604832480002e+02 },
    { 6.2159999999999995e-08, 2.0233932655400002e+00, 1.7165459766239999e+02 },
    { 6.1879999999999996e-08, 1.4824147165899999e+00, 3.3928641933649999e+02 },
    { 6.1360000000000002e-08, 3.0841905991599998e+00, 1.2598732389849999e+02 },
    { 6.1290000000000002e-08, 4.9662896419999996e+00, 2.1101147357759999e+02 },
    { 6.1290000000000002e-08, 6.2065218317699999e+00, 2.1558671729840000e+02 },
    { 6.0349999999999994e-08, 2.9385492129199999e+00, 3.1019488636999998e+01 },
    { 6.0320000000000000e-08, 1.3156992156300000e+00, 5.3287235883230005e+02 },
    { 5.9979999999999998e-08, 4.4195423176999998e-01, 3.3455111692129998e+02 },
    { 5.9789999999999995e-08, 4.7283810205999999e-01, 2.4012579838100001e+02 },
    { 5.9629999999999999e-08, 2.8774973311699998e+00, 1.6674774556399999e+01 },
    { 5.9179999999999998e-08, 3.8583008322799999e+00, 4.2240540518199998e+02 },
    { 5.9050000000000003e-08, 2.5085044218300001e+00, 8.5637777548969996e+02 },
    { 5.8680000000000000e-08, 5.7059847609299998e+00, 2.6450482076920002e+02 },
    { 5.8649999999999999e-08, 3.8858625792399999e+00, 5.5103160609700001e+02 },
    { 5.8129999999999998e-08, 2.2794974535199999e+00, 3.8133035637799999e+01 },
    { 5.7410000000000003e-08, 4.4291629686900000e+00, 1.2895626931510000e+02 },
    { 5.7280000000000001e-08, 5.7149811867600002e+00, 2.7373205056900001e+03 },
    { 5.5959999999999999e-08, 2.6115950504100001e+00, 1.9365537546499999e+02 },
    { 5.5890000000000000e-08, 3.9068513254299999e+00, 5.4174670747800000e+01 },
    { 5.5789999999999999e-08, 2.4186252394099998e+00, 2.5265597135319999e+02 },
    { 5.5479999999999998e-08, 3.1487585972200001e+00, 2.5311349572528002e+03 },
    { 5.4620000000000002e-08, 4.8649532469000001e-01, 1.9054647649404001e+03 },
    { 5.4410000000000002e-08, 2.0641715662600002e+00, 2.2997386999439999e+02 },
    { 5.3220000000000003e-08, 4.7699903249200002e+00, 2.4379022388199999e+01 },
    { 5.2880000000000001e-08, 2.8605665817900001e+00, 2.6747376618580000e+02 },
    { 5.2520000000000003e-08, 4.3271542051800003e+00, 3.4225536475310003e+02 },
    { 5.2070000000000002e-08, 2.7160799036499998e+00, 5.3362311835770004e+02 },
    { 5.1469999999999997e-08, 1.2145256013400001e+00, 4.8884961647109998e+02 },
    { 5.1020000000000003e-08, 1.7114292323100000e+00, 4.7217484191469998e+02 },
    { 5.0039999999999997e-08, 3.2104742195300000e+00, 6.5019222705250002e+02 },
    { 4.9990000000000000e-08, 1.4332503905700000e+00, 4.4868959140380002e+02 },
    { 4.8730000000000000e-08, 1.3870898998200001e+00, 7.2164941953020002e+02 },
    { 4.8569999999999998e-08, 2.2195245341400001e+00, 2.9204947528600002e+01 },
    { 4.8529999999999998e-08, 3.8218702842500001e+00, 1.8972322220190000e+02 },
    { 4.7790000000000000e-08, 2.2856656694900002e+00, 3.9739324334740002e+02 },
    { 4.7740000000000003e-08, 1.4563135653999999e+00, 3.9435486196160002e+02 },
    { 4.7239999999999998e-08, 8.8455881434000005e-01, 2.0566428357539999e+02 },
    { 4.6569999999999999e-08, 5.2191653957400002e+00, 2.4280421830342002e+03 },
    { 4.5890000000000003e-08, 1.0930402228200000e+00, 1.1411340634054000e+03 },
    { 4.5590000000000000e-08, 1.7260413775000001e+00, 4.1852143602870001e+02 },
    { 4.5590000000000000e-08, 3.5279848709800001e+00, 8.3503713448730002e+02 },
    { 4.5499999999999997e-08, 8.6759003557000003e-01, 2.9126208774300000e+02 },
    { 4.5270000000000001e-08, 8.6326298235999999e-01, 2.2093390730060000e+02 },
    { 4.5020000000000002e-08, 3.8117153981600000e+00, 1.6772244439930000e+02 },
    { 4.4910000000000003e-08, 4.3564271967700003e+00, 8.5001498801430000e+02 },
    { 4.4890000000000000e-08, 2.1303113160199998e+00, 4.0992341631959999e+02 },
    { 4.4710000000000001e-08, 8.6761100667000002e-01, 3.6137781986430002e+02 },
    { 4.4689999999999998e-08, 5.6488199746200003e+00, 4.2808266358430001e+02 },
    { 4.4069999999999997e-08, 3.0585517060000000e-01, 2.8404132799086001e+03 },
    { 4.3910000000000001e-08, 4.2759484774500001e+00, 4.9181856188770001e+02 },
    { 4.2690000000000000e-08, 5.4282615264200000e+00, 2.0151968199120000e+02 },
    { 4.2690000000000000e-08, 2.6029572937599998e+00, 2.2507850888479999e+02 },
    { 4.2410000000000001e-08, 3.8119767339999999e+00, 3.6997583957339998e+02 },
    { 4.2400000000000002e-08, 9.3998618693000002e-01, 2.1187638603783998e+03 },
    { 4.2179999999999998e-08, 4.2982470687000003e+00, 1.3533610313299999e+02 },
    { 4.2100000000000000e-08, 9.6298353438999995e-01, 2.2005159935945999e+03 },
    { 4.2030000000000000e-08, 2.4208699264600000e+00, 3.6434676528090000e+02 },
    { 4.0849999999999999e-08, 2.0459559600700001e+00, 5.1939602435610004e+02 },
    { 4.0119999999999998e-08, 5.2733972250300001e+00, 4.2511371816769997e+02 },
    { 4.0119999999999998e-08, 2.5891283562399998e+00, 4.9426624244250002e+02 },
    { 3.9669999999999997e-08, 5.1820213897600000e+00, 1.2050126195790000e+02 },
    { 3.9599999999999997e-08, 1.0194129431800001e+00, 2.9201284726839998e+02 },
    { 3.9410000000000000e-08, 1.1997013253700000e+00, 4.3375992198659998e+02 },
    { 3.9349999999999998e-08, 3.7102720620000000e+00, 4.2584743135060000e+02 },
    { 3.9130000000000000e-08, 2.2069611831499998e+00, 5.0456118318099999e+02 },
    { 3.8050000000000000e-08, 4.2503958811600002e+00, 6.9258748435350003e+02 },
    { 3.7849999999999998e-08, 5.0400395426599998e+00, 2.1494362684070001e+02 },
    { 3.7849999999999998e-08, 6.1327719311099997e+00, 2.1165456403530001e+02 },
    { 3.7819999999999997e-08, 6.1795864500100004e+00, 2.4138150890326001e+03 },
    { 3.7809999999999999e-08, 1.0374868110000000e+00, 1.0601353552540000e+02 },
    { 3.7399999999999997e-08, 4.8871640014600004e+00, 6.5805653357870005e+02 },
    { 3.7350000000000000e-08, 5.8788814415299999e+00, 3.3940249943799998e+01 },
    { 3.7130000000000002e-08, 1.9256281253200001e+00, 2.0326478713040001e+02 },
    { 3.7130000000000002e-08, 6.1055906948600001e+00, 2.2333340374560001e+02 },
    { 3.7030000000000001e-08, 1.3986812165800000e+00, 1.0554497769261000e+03 },
    { 3.7000000000000000e-08, 4.8957664408100001e+00, 1.7001006625970001e+02 },
    { 3.6939999999999999e-08, 1.2543008214100000e+00, 4.6595506679120001e+02 },
    { 3.6820000000000001e-08, 8.0307954179999996e-01, 7.3269511979410004e+02 },
    { 3.6780000000000002e-08, 1.4573130916299999e+00, 4.1800017116689997e+02 },
    { 3.6099999999999999e-08, 1.8378229624600000e+00, 4.0918970313670002e+02 },
    { 3.5840000000000002e-08, 5.7917123838300002e+00, 7.7750543983900002e+01 },
    { 3.5710000000000000e-08, 5.6587079674999996e-01, 5.1153171782990000e+02 },
    { 3.5660000000000003e-08, 1.7979478889500000e+00, 2.1160217440860001e+02 },
    { 3.5660000000000003e-08, 6.2332709312299999e+00, 2.1499601646740001e+02 },
    { 3.5549999999999997e-08, 5.0905788408100001e+00, 7.3297125859000005e+01 },
    { 3.5180000000000001e-08, 2.6426270051000000e-01, 3.3137100791700000e+01 },
    { 3.4770000000000000e-08, 2.2806246853000001e-01, 2.2218566345969998e+03 },
    { 3.3799999999999998e-08, 7.6892084558999996e-01, 2.3687496867409999e+02 },
    { 3.3360000000000002e-08, 1.2942181360999999e-01, 1.2836556848410001e+02 },
    { 3.2999999999999998e-08, 6.2016765708100001e+00, 5.4302428721889999e+02 },
    { 3.2870000000000003e-08, 4.8839283087599998e+00, 1.8409414790939999e+02 },
    { 3.2700000000000002e-08, 1.8201409467300000e+00, 7.1467888488129995e+02 },
    { 3.2350000000000002e-08, 2.7224047055599998e+00, 1.1265400177420000e+02 },
    { 3.2290000000000000e-08, 1.2824732243900001e+00, 1.3703302416240001e+02 },
    { 3.1690000000000001e-08, 5.3118119353899997e+00, 5.5137878594299998e+01 },
    { 3.1669999999999998e-08, 2.5257995186999999e-01, 2.0199276274319999e+02 },
    { 3.1669999999999998e-08, 1.4954535611299999e+00, 2.2460542813280000e+02 },
    { 3.1650000000000002e-08, 5.8906847800200000e+00, 2.0055647414469999e+02 },
    { 3.1650000000000002e-08, 2.1405340401599999e+00, 2.2604171673130000e+02 },
    { 3.1009999999999998e-08, 5.7069682804599999e+00, 4.0695447090300001e+02 },
    { 3.0670000000000003e-08, 2.4688255631099998e+00, 4.4157604440300003e+02 },
    { 3.0479999999999999e-08, 6.9309647906000005e-01, 6.0626632075526004e+03 },
    { 3.0029999999999998e-08, 3.4764754350799998e+00, 4.5097721326419997e+02 },
    { 3.0029999999999998e-08, 4.7820182641500004e+00, 2.6413412784722000e+03 },
    { 2.9980000000000001e-08, 2.3410010241100001e+00, 3.8846515523820000e+02 },
    { 2.9929999999999997e-08, 4.4725728242400002e+00, 4.0599126305649997e+02 },
    { 2.9929999999999997e-08, 4.3808954225800001e+00, 4.6202291352809999e+02 },
    { 2.9790000000000001e-08, 1.3234598145200001e+00, 1.4859801210651999e+03 },
    { 2.9429999999999999e-08, 4.3027637690700002e+00, 2.5583736509089999e+02 },
    { 2.9370000000000001e-08, 4.9620362520899999e+00, 7.3905790726949999e+02 },
    { 2.9060000000000000e-08, 1.4931501414699999e+00, 4.0302231763990000e+02 },
    { 2.8979999999999999e-08, 2.0779669754699999e+00, 1.0807225711916001e+03 },
    { 2.8810000000000001e-08, 4.2782837083000000e+00, 9.2797833480099996e+01 },
    { 2.8649999999999998e-08, 7.2730984230000004e-02, 5.8106824010899999e+01 },
    { 2.8579999999999999e-08, 5.5456942318499998e+00, 5.4073666535849998e+02 },
    { 2.8500000000000000e-08, 5.3905343906400001e+00, 2.2052451702339999e+02 },
    { 2.8470000000000000e-08, 5.7602516320500001e+00, 3.4863519857099999e+02 },
    { 2.8470000000000000e-08, 2.2120835705999999e+00, 9.2783496796739996e+02 },
    { 2.8430000000000000e-08, 3.9095638889000001e-01, 6.3033605875839999e+02 },
    { 2.8310000000000000e-08, 2.6404730704400001e+00, 2.8475268269093999e+03 },
    { 2.8260000000000000e-08, 2.7017592939799999e+00, 8.4933526953900000e+01 },
    { 2.8200000000000001e-08, 6.2770432690100000e+00, 2.5658812461629998e+02 },
    { 2.7820000000000000e-08, 1.0347092925000000e-01, 6.3893407846749994e+02 },
    { 2.7630000000000000e-08, 2.6730278017900000e+00, 5.6360650166766000e+03 },
    { 2.7549999999999999e-08, 5.3632309571499999e+00, 7.3194436026870005e+02 },
    { 2.7510000000000000e-08, 3.0778637800799999e+00, 5.1864526483069994e+02 },
    { 2.7100000000000001e-08, 1.3624335172499999e+00, 1.8023719907218001e+03 },
    { 2.7050000000000001e-08, 4.4893395967299998e+00, 6.0554504565700000e+01 },
    { 2.7030000000000001e-08, 3.6710563952599999e+00, 3.4484605765980001e+02 },
    { 2.6899999999999999e-08, 5.2169450418399999e+00, 9.8066817835879999e+02 },
    { 2.6689999999999999e-08, 2.2346120274099999e+00, 8.1506334611420004e+02 },
    { 2.6589999999999999e-08, 4.9874948503300001e+00, 2.7444340526907999e+03 },
    { 2.6499999999999999e-08, 1.8352207706400001e+00, 1.7224529849340001e+02 },
    { 2.6359999999999999e-08, 3.0028112725000000e-01, 6.4945851386959998e+02 },
    { 2.6350000000000001e-08, 2.0911262492899998e+00, 9.9928729037722005e+03 },
    { 2.6130000000000000e-08, 6.4725972976000001e-01, 7.3980866679489998e+02 },
    { 2.6050000000000002e-08, 2.5883311313699999e+00, 1.5961864422845999e+03 },
    { 2.5979999999999998e-08, 3.2290656510900000e+00, 1.3686602528450001e+03 },
    { 2.5970000000000000e-08, 3.5423361547999999e-01, 4.2648631629139999e+02 },
    { 2.5970000000000000e-08, 5.0472614211200000e+00, 4.2671006546059999e+02 },
    { 2.5510000000000001e-08, 1.2844510495799999e+00, 1.8484490743480001e+02 },
    { 2.5450000000000000e-08, 4.0988325921500000e+00, 4.5505237887919998e+02 },
    { 2.5340000000000001e-08, 6.0516725448799997e+00, 2.7302069586891998e+03 },
    { 2.5259999999999999e-08, 5.4408747778700004e+00, 8.5712853501510006e+02 },
    { 2.5189999999999999e-08, 5.6520793824700002e+00, 6.4892613984285999e+03 },
    { 2.5100000000000000e-08, 3.8938687801100000e+00, 6.4086049416050002e+02 },
    { 2.4809999999999998e-08, 2.9664200605700000e+00, 1.0554045477340000e+02 },
    { 2.4739999999999999e-08, 1.5008327520300000e+00, 4.1953282798499998e+02 },
    { 2.4649999999999999e-08, 2.8315037252600002e+00, 3.0609692891809999e+02 },
    { 2.4640000000000001e-08, 5.6503023412599997e+00, 3.0537123753465999e+03 },
    { 2.4630000000000000e-08, 7.2737478636999997e-01, 6.2960234557549995e+02 },
    { 2.4599999999999999e-08, 2.0822225276699999e+00, 5.6502921106781996e+03 },
    { 2.4590000000000000e-08, 4.2181278805400000e+00, 4.6824268865160002e+02 },
    { 2.4470000000000000e-08, 3.1172091443599999e+00, 2.0693630796260001e+02 },
    { 2.4470000000000000e-08, 4.9140096758200000e+00, 2.1966188291340001e+02 },
    { 2.4439999999999999e-08, 3.2574010662199999e+00, 4.2044785172169998e+02 },
    { 2.4120000000000000e-08, 3.1227662729299999e+00, 1.6850521225016000e+03 },
    { 2.4039999999999999e-08, 3.5156588092500001e+00, 9.5805018068978006e+03 },
    { 2.3790000000000000e-08, 8.6162227364999999e-01, 1.4575259330619999e+03 },
    { 2.3790000000000000e-08, 3.5753552419000001e+00, 5.9465070367540000e+02 },
    { 2.3530000000000000e-08, 2.5796772768700000e+00, 3.9814400287280000e+02 },
    { 2.3470000000000001e-08, 7.9433458212999997e-01, 1.0419471094648199e+04 },
    { 2.3440000000000000e-08, 1.0371065615899999e+00, 4.0221916848780000e+02 },
    { 2.3289999999999999e-08, 2.1275626873500002e+00, 4.5490936652729999e+02 },
    { 2.3120000000000001e-08, 5.8119293371499996e+00, 7.2483081326790000e+02 },
    { 2.2950000000000000e-08, 2.1434120187899999e+00, 5.0286426215159997e+02 },
    { 2.2749999999999999e-08, 4.1090733309600003e+00, 3.9346109008430000e+02 },
    { 2.2600000000000001e-08, 4.1112669084900002e+00, 4.2538269652899999e+01 },
    { 2.2420000000000002e-08, 2.1960170559999999e-02, 7.7448262992160005e+02 },
    { 2.2200000000000000e-08, 5.9921573289200003e+00, 5.2575881183150000e+02 },
    { 2.2169999999999999e-08, 5.6021624748300001e+00, 2.3107223148140001e+03 },
    { 2.2010000000000000e-08, 2.6968984595099998e+00, 4.3467494572330003e+02 },
    { 2.1950000000000001e-08, 9.5099493253000000e-01, 5.9342686339800002e+02 },
    { 2.1820000000000000e-08, 4.4276193788200002e+00, 4.2681063919709999e+02 },
    { 2.1820000000000000e-08, 9.7387565778999996e-01, 4.2638574255489999e+02 },
    { 2.1780000000000000e-08, 3.0487250257200000e+00, 2.5435289238260000e+02 },
    { 2.1699999999999999e-08, 1.3234757327300000e+00, 1.0380412891868000e+03 },
    { 2.1530000000000002e-08, 5.4486333965700000e+00, 2.3294281541100000e+02 },
    { 2.1410000000000001e-08, 2.4475211469000002e+00, 4.8077286162380000e+02 },
    { 2.1360000000000001e-08, 1.0957725207300000e+00, 1.8621176006409999e+02 },
    { 2.1330000000000000e-08, 1.7410444431700001e+00, 5.5258551477449998e+02 },
    { 2.1159999999999999e-08, 3.2885849735199999e+00, 1.3828873468465999e+03 },
    { 2.0570000000000002e-08, 8.1369675900000005e-01, 2.9823262239190001e+02 },
    { 2.0450000000000001e-08, 5.9441462493200001e+00, 4.7780391620720002e+02 },
    { 2.0420000000000000e-08, 2.2222750144000000e-01, 3.8012776796000003e+02 },
    { 2.0420000000000000e-08, 7.2463541688999999e-01, 6.0697767545533998e+03 },
    { 2.0350000000000000e-08, 3.9559480372100002e+00, 4.8173606947029998e+02 },
    { 2.0179999999999999e-08, 2.2012848792600002e+00, 9.9999864507729999e+03 },
    { 1.9989999999999999e-08, 1.1871678970599999e+00, 6.0768903015542001e+03 },
    { 1.9799999999999999e-08, 6.2695950952699997e+00, 1.8802630117250001e+02 },
    { 1.9799999999999999e-08, 6.0847581168199998e+00, 2.0629742302180000e+02 },
    { 1.9770000000000001e-08, 4.7860484819700000e+00, 1.0483362299252999e+03 },
    { 1.9740000000000000e-08, 4.0863452127000000e-01, 6.3994547042379997e+02 },
    { 1.9580000000000001e-08, 3.3483271564399999e+00, 1.4393412284210001e+02 },
    { 1.9480000000000000e-08, 4.6093387648100004e+00, 2.6843697403229999e+02 },
    { 1.9470000000000002e-08, 4.9884525961900001e+00, 1.4122580985639999e+02 },
    { 1.9390000000000000e-08, 4.1112362777199998e+00, 1.0064509366380000e+02 },
    { 1.9119999999999998e-08, 2.1503829946300002e+00, 1.6289651925890001e+02 },
    { 1.9099999999999999e-08, 4.6155859568500004e+00, 1.7542669223109999e+02 },
    { 1.9009999999999999e-08, 6.2144676780400001e+00, 2.7140591944890002e+02 },
    { 1.8990000000000000e-08, 4.7442297068000000e-01, 1.0490869894507000e+03 },
    { 1.8950000000000001e-08, 1.7135137048900000e+00, 8.3578789401270001e+02 },
    { 1.8880000000000001e-08, 9.7996534938000002e-01, 6.4234496686880004e+02 },
    { 1.8740000000000001e-08, 2.3410695922100002e+00, 7.6981676347560006e+02 },
    { 1.8720000000000002e-08, 2.2510745252600000e+00, 9.0588657979150003e+02 },
    { 1.8710000000000000e-08, 4.0711466948000002e-01, 2.0607367385260000e+02 },
    { 1.8699999999999999e-08, 4.4853046848500000e+00, 4.9508804301799998e+01 },
    { 1.8629999999999999e-08, 2.6113282112700000e+00, 1.0007099997773799e+04 },
    { 1.8629999999999999e-08, 3.4330142103000001e+00, 6.9970103135429997e+02 },
    { 1.8609999999999999e-08, 3.5494419732400000e+00, 5.1653626887241999e+03 },
    { 1.8570000000000000e-08, 3.0912032561200000e+00, 1.4169889060840001e+02 },
    { 1.8550000000000001e-08, 7.0172463076000002e-01, 7.6025553592000006e+02 },
    { 1.8390000000000001e-08, 5.5520837055400003e+00, 4.8640193591629998e+02 },
    { 1.8259999999999999e-08, 4.9912647379999999e+00, 4.8470501488690002e+02 },
    { 1.8240000000000000e-08, 4.3158734549000000e-01, 4.2000590873700003e+02 },
    { 1.8229999999999998e-08, 6.2173348083600004e+00, 2.5382485042536000e+03 },
    { 1.8110000000000001e-08, 9.5890385522999999e-01, 8.4290144101349995e+02 },
    { 1.8060000000000001e-08, 4.2818266182000002e+00, 1.1939672737968001e+03 },
    { 1.8040000000000001e-08, 1.1880647530000001e-01, 3.5330106501699998e+02 },
    { 1.8019999999999999e-08, 4.0910229672199998e+00, 9.5662747128962001e+03 },
    { 1.7960000000000000e-08, 3.7938534275500002e+00, 2.4038643081190000e+02 },
    { 1.7929999999999999e-08, 3.4304502105500001e+00, 4.1943645976540000e+02 },
    { 1.7879999999999999e-08, 2.0340874860599998e+00, 1.7076082578510000e+02 },
    { 1.7830000000000002e-08, 3.9343085442499999e+00, 2.1233588759150001e+02 },
    { 1.7820000000000000e-08, 4.6410292062299998e+00, 7.1600204829600003e+01 },
    { 1.7780000000000001e-08, 2.3043527178100001e+00, 7.0214871190910003e+02 },
    { 1.7730000000000001e-08, 9.1546662134000001e-01, 9.9022940591439999e+02 },
    { 1.7710000000000001e-08, 3.5765740536699999e+00, 5.5555446019110002e+02 },
    { 1.7599999999999999e-08, 6.0642782037799998e+00, 3.1931263096340001e+02 },
    { 1.7529999999999999e-08, 4.9198493353400004e+00, 3.3334988796990001e+03 },
    { 1.7459999999999999e-08, 2.5293483580000001e-02, 3.1568051495651998e+03 },
    { 1.7450000000000001e-08, 5.8733628110000002e-01, 2.8332997329077998e+03 },
    { 1.7369999999999999e-08, 2.5605087140000000e+00, 6.2248879857470001e+02 },
    { 1.7330000000000000e-08, 3.3972875641200000e+00, 7.0511765732569995e+02 },
    { 1.7319999999999999e-08, 6.0064433018600001e+00, 1.3324847706675000e+03 },
    { 1.7310000000000001e-08, 6.4743919781000003e-01, 1.1262992222303001e+03 },
    { 1.7129999999999999e-08, 1.3018219165000000e+00, 4.5761767951299998e+02 },
    { 1.7120000000000000e-08, 3.1211342041800001e+00, 2.6080789594573391e+04 },
    { 1.7059999999999999e-08, 7.5846006307000002e-01, 9.7676148247200004e+01 },
    { 1.6910000000000001e-08, 2.0785465296900001e+00, 7.8946048344860003e+02 },
    { 1.6870000000000002e-08, 5.7913650979000000e+00, 3.0465988283458000e+03 },
    { 1.6860000000000000e-08, 4.3656921937900002e+00, 3.3477259737006002e+03 },
    { 1.6740000000000000e-08, 2.0458859875999999e-01, 3.0508553696180002e+02 },
    { 1.6639999999999999e-08, 5.3772423908000002e+00, 1.1272624300768000e+03 },
    { 1.6639999999999999e-08, 6.0687828243700004e+00, 4.8736514376280002e+02 },
    { 1.6610000000000001e-08, 2.5696619132600000e+00, 2.6095016688575000e+04 },
    { 1.6540000000000001e-08, 4.6737922013300004e+00, 2.9577331481288002e+03 },
    { 1.6510000000000000e-08, 2.9515134297799999e+00, 1.5253214255120000e+02 },
    { 1.6429999999999999e-08, 1.1693166873200000e+00, 1.1120721282287000e+03 },
    { 1.6390000000000000e-08, 5.0730830202700004e+00, 4.2685882330689998e+02 },
    { 1.6190000000000001e-08, 1.8786807456600001e+00, 7.8404385747720005e+02 },
    { 1.6140000000000001e-08, 4.7266108724200002e+00, 3.0608259223474001e+03 },
    { 1.5830000000000000e-08, 2.1330050712000001e-01, 3.7301422095919997e+02 },
    { 1.5780000000000000e-08, 5.9854919647199996e+00, 1.9040454575810000e+02 },
    { 1.5780000000000000e-08, 5.1873195090499999e+00, 2.3619364511789999e+02 },
    { 1.5729999999999999e-08, 9.7760246716999999e-01, 7.4692221379570003e+02 },
    { 1.5720000000000001e-08, 4.5245461806999998e+00, 9.1786441523799994e+01 },
    { 1.5650000000000001e-08, 6.0408479571999996e-01, 3.9265794093220001e+02 },
    { 1.5620000000000000e-08, 2.7511257954100001e+00, 2.0014439921582000e+03 },
    { 1.5620000000000000e-08, 6.4998564498000000e-01, 2.5169078632512001e+03 },
    { 1.5600000000000001e-08, 1.2055866217200000e+00, 6.1462449204849997e+02 },
    { 1.5530000000000001e-08, 3.4607064895400002e+00, 1.1010264279648000e+03 },
    { 1.5460000000000001e-08, 2.4225447887199998e+00, 2.5854567807659998e+02 },
    { 1.5360000000000000e-08, 1.4668055603300001e+00, 2.1325091132820000e+02 },
    { 1.5360000000000000e-08, 1.6557470606300000e+00, 1.5606187072805999e+03 },
    { 1.5309999999999999e-08, 4.3913198381500003e+00, 5.2165803728013998e+03 },
    { 1.5279999999999999e-08, 6.5096988362999997e-01, 9.0684978763799995e+02 },
    { 1.5259999999999999e-08, 6.2640509496699996e+00, 1.8016199464630000e+02 },
    { 1.5230000000000001e-08, 1.5241954556000001e-01, 9.2107688163959995e+02 },
    { 1.5060000000000001e-08, 2.7403889741999998e-01, 2.5117149864490000e+02 },
    { 1.5010000000000000e-08, 1.9603224678000000e+00, 1.2442268376239999e+03 },
    { 1.4950000000000002e-08, 3.0472676604400002e+00, 2.6370167161709998e+02 },
    { 1.4810000000000000e-08, 2.7619229039600000e+00, 3.1492569389000000e+01 },
    { 1.4769999999999999e-08, 4.2597965987000003e-01, 4.0132539661049998e+02 },
    { 1.4670000000000000e-08, 3.9384158920000002e-02, 8.2792358748649997e+02 },
    { 1.4520000000000000e-08, 6.2712978949300000e+00, 1.1302313754934000e+03 },
    { 1.4489999999999999e-08, 2.4022242693100000e+00, 1.6992792165031999e+03 },
    { 1.4460000000000000e-08, 6.4453355125000000e-01, 2.2189711514710001e+02 },
    { 1.4460000000000000e-08, 1.1034999617500001e+00, 2.0470107572890001e+02 },
    { 1.4460000000000000e-08, 1.5218892040900001e+00, 4.6285581766287996e+03 },
    { 1.4370000000000001e-08, 3.7011424491299998e+00, 1.0444040766621999e+03 },
    { 1.4279999999999999e-08, 5.4185843464000003e+00, 1.9022833712027000e+03 },
    { 1.4240000000000000e-08, 4.5301356121699996e+00, 1.4000196957899999e+02 },
    { 1.4200000000000000e-08, 1.1411347981000000e+00, 2.5272794265500000e+01 },
    { 1.4170000000000000e-08, 6.2054061618600000e+00, 3.4731970192218000e+03 },
    { 1.4079999999999999e-08, 4.3168910886900003e+00, 3.3683873878169999e+02 },
    { 1.4060000000000000e-08, 9.3043359633000000e-01, 7.0066423920080001e+02 },
    { 1.4009999999999999e-08, 3.6474313628999999e-01, 2.0903096723752001e+03 },
    { 1.3949999999999999e-08, 5.8147570827399999e+00, 9.1467900690210008e+03 },
    { 1.3869999999999999e-08, 5.7391338473199998e+00, 5.2724328453980002e+02 },
    { 1.3850000000000000e-08, 4.2613004324599997e+00, 4.1053796944600002e+01 },
    { 1.3780000000000000e-08, 2.6389752039599998e+00, 3.1639186965660001e+03 },
    { 1.3510000000000000e-08, 3.0397265528199999e+00, 1.7935884549420001e+02 },
    { 1.3410000000000001e-08, 4.9773029997500000e+00, 4.3121663069721999e+03 },
    { 1.3350000000000000e-08, 2.3400249656700001e+00, 2.3178358618148000e+03 },
    { 1.3340000000000001e-08, 2.4422118703100000e+00, 2.8266406803389998e+02 },
    { 1.3300000000000000e-08, 4.9089887550000003e-02, 2.0003520928290001e+02 },
    { 1.3300000000000000e-08, 1.6989436254500001e+00, 2.2656298159310001e+02 },
    { 1.3220000000000000e-08, 1.9154122345800000e+00, 2.4476805547999998e+00 },
    { 1.3220000000000000e-08, 4.9138095874300003e+00, 6.9364972595899999e+01 },
    { 1.3210000000000000e-08, 6.6259853003000002e-01, 1.7739178027185999e+03 },
    { 1.3160000000000000e-08, 4.8841170791100001e+00, 4.6053844081979997e+02 },
    { 1.3150000000000000e-08, 2.7674322750400000e+00, 1.7913262904579001e+03 },
    { 1.2940000000000000e-08, 4.6284242708700001e+00, 1.1900351205336999e+03 },
    { 1.2909999999999999e-08, 4.7541550526799998e+00, 6.9407195706180005e+02 },
    { 1.2880000000000000e-08, 1.8599354157300001e+00, 2.4723934538180001e+02 },
    { 1.2870000000000000e-08, 2.6504708129999999e-01, 2.8659622129700000e+02 },
    { 1.2870000000000000e-08, 5.9633599505300001e+00, 7.2333918012500007e+01 },
    { 1.2840000000000000e-08, 1.7125454549200001e+00, 3.1394418910180002e+02 },
    { 1.2790000000000001e-08, 5.0934500213699998e+00, 4.4923210812500002e+02 },
    { 1.2770000000000000e-08, 5.8844748549799997e+00, 3.2261164478009999e+02 },
    { 1.2700000000000000e-08, 2.1909229980900000e+00, 1.1767017984094000e+03 },
    { 1.2550000000000000e-08, 3.4166869155800002e+00, 6.9110301164520001e+02 },
    { 1.2509999999999999e-08, 2.0450714362500002e+00, 9.0981873305459999e+02 },
    { 1.2470000000000000e-08, 4.6466416770199999e+00, 1.8743560034149999e+02 },
    { 1.2470000000000000e-08, 2.4298448956999999e-01, 2.3916259053450000e+02 },
    { 1.2460000000000000e-08, 1.7951908524100000e+00, 2.4874160449477999e+03 },
    { 1.2350000000000000e-08, 3.3986225763200002e+00, 1.1926254413668799e+04 },
    { 1.2310000000000001e-08, 4.0759299068499999e+00, 5.6193429400900004e+02 },
    { 1.2290000000000000e-08, 2.4457902032500001e+00, 5.1535908996834391e+04 },
    { 1.2260000000000000e-08, 5.2437758047200003e+00, 4.1273213995373999e+03 },
    { 1.2220000000000000e-08, 3.8365380851999997e-01, 3.3514181775230003e+02 },
    { 1.2170000000000001e-08, 2.0160949706200002e+00, 2.0053761454213000e+03 },
    { 1.2159999999999999e-08, 5.5200994565699997e+00, 3.4884764689209999e+02 },
    { 1.2019999999999999e-08, 3.7747151183500001e+00, 4.1873388434980001e+02 },
    { 1.1970000000000001e-08, 2.0534395973100001e+00, 1.0625633239269000e+03 },
    { 1.1949999999999999e-08, 3.1716986437700001e+00, 1.4599565672742999e+03 },
    { 1.1930000000000000e-08, 9.8399514661999998e-01, 1.4685716333258999e+03 },
    { 1.1910000000000000e-08, 3.6857795003499998e+00, 9.5952390892233998e+03 },
    { 1.1770000000000001e-08, 1.6340430340000001e-01, 1.3285526174044001e+03 },
    { 1.1749999999999999e-08, 4.7719745425599998e+00, 1.6811199692385001e+03 },
    { 1.1749999999999999e-08, 1.7558540843799999e+00, 1.6740064222377000e+03 },
    { 1.1730000000000000e-08, 8.9010724985000000e-01, 1.1592933106701000e+03 },
    { 1.1710000000000000e-08, 3.9474305222099999e+00, 6.7214061522839995e+02 },
    { 1.1679999999999999e-08, 1.0824841429300001e+00, 7.4638586215399997e+01 },
    { 1.1670000000000000e-08, 1.6177693149800001e+00, 1.7842127434571000e+03 },
    { 1.1660000000000000e-08, 4.3214728445299997e+00, 6.3448066034260000e+02 },
    { 1.1640000000000000e-08, 6.0363221106499996e+00, 3.3914448860056000e+03 },
    { 1.1539999999999999e-08, 5.7445117891199997e+00, 2.9211277828246002e+03 },
    { 1.1460000000000000e-08, 3.3774172887999998e-01, 2.2030076785419999e+02 },
    { 1.1430000000000000e-08, 3.1721576299999998e-01, 3.1496916025644000e+03 },
    { 1.1330000000000000e-08, 5.8698978085999998e-01, 3.1883955021140002e+02 },
    { 1.1310000000000000e-08, 3.4920733891000002e+00, 6.3984910220419999e+02 },
    { 1.1300000000000000e-08, 3.3735105774499998e+00, 5.4817545583807996e+03 },
    { 1.1290000000000001e-08, 2.0592339207400001e+00, 4.8025159676200002e+02 },
    { 1.1280000000000001e-08, 3.9230383855500000e+00, 3.9828701522469999e+02 },
    { 1.1240000000000000e-08, 3.2059958250400000e+00, 2.1077181601144998e+03 },
    { 1.1220000000000001e-08, 5.4686103789800002e+00, 6.4679257579615996e+03 },
    { 1.1210000000000001e-08, 3.7039207707899999e+00, 3.6604368631030002e+02 },
    { 1.1190000000000000e-08, 4.9500532661000003e+00, 6.2830758499914000e+03 },
    { 1.1180000000000000e-08, 5.4137266220500004e+00, 1.5347381658416000e+03 },
    { 1.1140000000000001e-08, 6.5973607800000004e-03, 2.5234706759822191e+04 },
    { 1.1120000000000000e-08, 5.8644581432600003e+00, 1.1443154571431000e+03 },
    { 1.1110000000000000e-08, 3.9428704248300002e+00, 2.5668418497699000e+04 },
    { 1.1100000000000000e-08, 5.0848652800099998e+00, 7.3168372783780001e+02 },
    { 1.1080000000000001e-08, 4.9231660441500003e+00, 1.3647280995819001e+03 },
    { 1.1050000000000000e-08, 4.9369732828200004e+00, 1.6805251279940001e+02 },
    { 1.0950000000000000e-08, 4.5881521259899998e+00, 6.7442823708879996e+02 },
    { 1.0950000000000000e-08, 4.5287318184299998e+00, 7.4591082183940000e+02 },
    { 1.0940000000000001e-08, 6.2570970141900002e+00, 7.3879727483859995e+02 },
    { 1.0910000000000000e-08, 4.7545979908099998e+00, 1.6882335162392999e+03 },
    { 1.0900000000000000e-08, 5.4953140818000001e-01, 1.3143255234027999e+03 },
    { 1.0840000000000000e-08, 2.2284675920599999e+00, 1.4096517742550000e+02 },
    { 1.0840000000000000e-08, 7.6684360231000004e-01, 2.4351557300350000e+03 },
    { 1.0820000000000000e-08, 2.3363344553899998e+00, 7.5632338265689998e+02 },
    { 1.0800000000000001e-08, 2.6139308086000002e+00, 6.8154178408960001e+02 },
    { 1.0789999999999999e-08, 6.2795377697700001e+00, 2.9069006888230001e+03 },
    { 1.0730000000000001e-08, 1.5423961867300000e+00, 4.9447869076360001e+02 },
    { 1.0730000000000001e-08, 4.3430894503399999e+00, 8.7583029900099996e+02 },
    { 1.0700000000000000e-08, 1.1480463577900000e+00, 3.5976304344427999e+03 },
    { 1.0690000000000000e-08, 5.3453372639300003e+00, 1.2720243872000001e+00 },
    { 1.0610000000000000e-08, 3.6938797107500001e+00, 6.8865533109039995e+02 },
    { 1.0589999999999999e-08, 4.4774385951099998e+00, 2.5654191403697390e+04 },
    { 1.0540000000000000e-08, 4.1992116364400003e+00, 9.3531546663000000e+01 },
    { 1.0509999999999999e-08, 8.7183660869000001e-01, 5.2684554629428003e+03 },
    { 1.0440000000000000e-08, 2.1293319928100001e+00, 2.6342277314714001e+03 },
    { 1.0400000000000000e-08, 2.4536151315299999e+00, 9.8853248488500003e+02 },
    { 1.0400000000000000e-08, 1.2901227200000001e+00, 6.5922821389999999e+00 },
    { 1.0350000000000000e-08, 3.0216738211699998e+00, 2.3593301268700000e+02 },
    { 1.0340000000000000e-08, 1.1192371087599999e+00, 1.1521797636693000e+03 },
    { 1.0320000000000001e-08, 2.5628671407899999e+00, 2.4187090352019999e+02 },
    { 1.0320000000000001e-08, 2.3267590257999999e+00, 1.8472728735580000e+02 },
    { 1.0260000000000001e-08, 3.0765745070300001e+00, 5.3786617841622001e+03 },
    { 1.0230000000000000e-08, 1.9081577235100000e+00, 3.8405992122309999e+02 },
    { 1.0200000000000000e-08, 5.6664434399900001e+00, 2.6271141844705999e+03 },
    { 1.0160000000000000e-08, 2.3744468916899999e+00, 1.2648167191010000e+03 },
    { 1.0140000000000000e-08, 7.2711076946999997e-01, 6.0676018552230005e+02 },
    { 1.0140000000000000e-08, 3.8449957151100000e+00, 2.4643619622969999e+02 },
    { 1.0109999999999999e-08, 3.9808314635599999e+00, 1.3866050922260001e+02 },
    { 1.0090000000000000e-08, 1.9004615681100001e+00, 1.9872168981566001e+03 },
    { 1.0080000000000000e-08, 4.0781393082499999e+00, 3.9069087570985998e+03 },
    { 1.0050000000000001e-08, 4.2737279850199998e+00, 1.5851407420206999e+03 },
    { 9.9800000000000007e-09, 2.2444893805300001e+00, 9.3888066823129998e+02 },
    { 9.9699999999999993e-09, 5.1021071898599999e+00, 9.8778172535960005e+02 },
    { 9.9599999999999995e-09, 3.0687219144000002e+00, 4.8631979130691998e+03 },
    { 9.8600000000000003e-09, 1.9554948387400000e+00, 5.8422505651137999e+03 },
    { 9.8099999999999998e-09, 1.4613308346700000e+00, 1.4710022675381999e+03 },
    { 9.7900000000000003e-09, 4.7406023937000002e-01, 2.4067015420317998e+03 },
    { 9.7700000000000008e-09, 1.7301310760099999e+00, 2.6484548254729998e+03 },
    { 9.7599999999999994e-09, 2.0272104266600000e+00, 3.9510562148700001e+02 },
    { 9.7200000000000003e-09, 2.2008556171400002e+00, 1.6708250284999999e+03 },
    { 9.6699999999999999e-09, 5.8667667461799997e+00, 1.5126011816400000e+02 },
    { 9.6500000000000004e-09, 5.7669478173800002e+00, 1.7408487739300000e+01 },
    { 9.5800000000000004e-09, 4.9863217458099998e+00, 1.2466574718362999e+03 },
    { 9.5700000000000007e-09, 4.3412231753999997e+00, 4.3366355376700000e+02 },
    { 9.5599999999999992e-09, 4.6117920790899998e+00, 1.5709136480191000e+03 },
    { 9.5300000000000000e-09, 5.9117119093000003e-01, 2.8261861859070000e+03 },
    { 9.4899999999999993e-09, 1.9832165436300000e+00, 9.4599421523210003e+02 },
    { 9.4500000000000002e-09, 2.5454436435300001e+00, 2.0597310011610000e+02 },
    { 9.4500000000000002e-09, 5.4857751766499998e+00, 2.2062509075989999e+02 },
    { 9.4199999999999993e-09, 3.5013612499700000e+00, 1.1487218766540001e+02 },
    { 9.4099999999999996e-09, 3.6724416049199999e+00, 1.1976754877499999e+02 },
    { 9.4099999999999996e-09, 5.5230230639700002e+00, 9.7415515816300001e+01 },
    { 9.4099999999999996e-09, 1.6511976329400000e+00, 3.5834033404411998e+03 },
    { 9.3999999999999998e-09, 2.3847078147300000e+00, 2.8793768165339998e+02 },
    { 9.3000000000000006e-09, 1.3286103671500000e+00, 1.8770105769372001e+03 },
    { 9.2500000000000001e-09, 1.9129473176000000e-01, 1.3654788591073000e+03 },
    { 9.2199999999999992e-09, 5.2767414677800000e+00, 3.6864961146597998e+03 },
    { 9.2099999999999994e-09, 3.3789855899100001e+00, 9.7724602613333991e+03 },
    { 9.1399999999999995e-09, 9.4055736317000005e-01, 3.0683064210100002e+02 },
    { 9.1399999999999995e-09, 6.0681222844700002e+00, 2.8537238101960003e+02 },
    { 9.1199999999999999e-09, 5.2940126247399997e+00, 1.0412226829245001e+03 },
    { 9.1199999999999999e-09, 3.4512878282799999e+00, 9.5209637027660006e+02 },
    { 9.0900000000000007e-09, 2.9205965063399999e+00, 1.2505896250994001e+03 },
    { 9.0599999999999997e-09, 2.0114580419100001e+00, 1.2779450795420800e+04 },
    { 9.0400000000000002e-09, 3.0405197068400001e+00, 4.7700076705510003e+02 },
    { 9.0300000000000005e-09, 2.3029877348000000e-01, 1.2353964334370001e+02 },
    { 9.0099999999999993e-09, 3.7591483072999998e+00, 4.4400667861530002e+02 },
    { 8.9500000000000007e-09, 5.4186785098000001e-01, 1.1191856752295000e+03 },
    { 8.9100000000000000e-09, 4.9599869370800000e+00, 4.4437132691939996e+03 },
    { 8.8599999999999996e-09, 5.7330274283399998e+00, 3.8038159828799999e+03 },
    { 8.8499999999999998e-09, 3.2146834174899999e+00, 6.0887779767699999e+02 },
    { 8.8400000000000001e-09, 5.1367459080800000e+00, 2.2186752408593002e+03 },
    { 8.8200000000000006e-09, 3.5567433178000001e+00, 2.4422692770357999e+03 },
    { 8.7700000000000001e-09, 5.8029868770500004e+00, 3.7895888888784002e+03 },
    { 8.7399999999999992e-09, 9.3951630481000004e-01, 1.4756851803267000e+03 },
    { 8.7199999999999997e-09, 5.5642811175100002e+00, 3.3629906980024002e+03 },
    { 8.7099999999999999e-09, 6.0181465514400001e+00, 2.9435060541272001e+03 },
    { 8.6800000000000006e-09, 4.9585913212000001e+00, 1.3718416465826999e+03 },
    { 8.6399999999999999e-09, 5.3697705468899999e+00, 3.3701042450032000e+03 },
    { 8.6300000000000002e-09, 6.2521486651600000e+00, 3.5691762464396002e+03 },
    { 8.6100000000000007e-09, 3.8510097597300001e+00, 7.8234693644779998e+02 },
    { 8.5999999999999993e-09, 1.2315858863700000e+00, 1.2146667056107601e+04 },
    { 8.5799999999999997e-09, 5.6954765871999999e+00, 7.8159617692239999e+02 },
    { 8.5799999999999997e-09, 5.6258367287300004e+00, 9.7355463135800005e+02 },
    { 8.5299999999999993e-09, 1.2163501806599999e+00, 5.3998590583309999e+02 },
    { 8.5299999999999993e-09, 3.6055399803500001e+00, 1.8944190646765001e+03 },
    { 8.5099999999999998e-09, 5.7676611068500003e+00, 2.8546403739102002e+03 },
    { 8.4599999999999993e-09, 3.1755368625200000e+00, 2.1148317071153001e+03 },
    { 8.4599999999999993e-09, 6.0407127363199997e+00, 1.9902381732660001e+02 },
    { 8.4599999999999993e-09, 1.9905060838599999e+00, 2.2757437354940001e+02 },
    { 8.4300000000000000e-09, 5.0005881651599999e+00, 1.2395439248355001e+03 },
    { 8.4200000000000003e-09, 4.8241153271100004e+00, 8.3110498122419995e+02 },
    { 8.4100000000000005e-09, 3.8887305668500001e+00, 4.7306861379200001e+02 },
    { 8.3899999999999994e-09, 2.0198735243299999e+00, 2.2020019411769999e+02 },
    { 8.3899999999999994e-09, 6.0113452958500000e+00, 2.0639799675830000e+02 },
    { 8.3699999999999998e-09, 1.2680241221899999e+00, 3.1172600321060003e+02 },
    { 8.3600000000000001e-09, 5.8710060471699999e+00, 1.3554855145409999e+02 },
    { 8.3400000000000006e-09, 5.8507028015299998e+00, 7.6343692965770003e+02 },
    { 8.2900000000000001e-09, 6.1752762792900002e+00, 1.3725924061081000e+03 },
    { 8.2900000000000001e-09, 7.7508627124999996e-01, 1.7265475387399999e+01 },
    { 8.2900000000000001e-09, 5.3211003903899998e+00, 4.4152590811908003e+03 },
    { 8.2800000000000004e-09, 1.2515019779000001e+00, 1.1494162363460001e+02 },
    { 8.2700000000000006e-09, 4.3141814592500003e+00, 1.1656560981455000e+03 },
    { 8.2599999999999992e-09, 3.6748522151800000e+00, 2.1989137757699999e+02 },
    { 8.2599999999999992e-09, 1.2718870557999999e+00, 2.8489930026759998e+02 },
    { 8.2399999999999997e-09, 4.4577657502300001e+00, 4.9893210888850001e+02 },
    { 8.2200000000000002e-09, 2.6402033272100001e+00, 5.4369930152402003e+03 },
    { 8.1799999999999995e-09, 4.2407842014900003e+00, 5.0551563675048001e+03 },
    { 8.1699999999999997e-09, 8.2100313330999997e-01, 6.3523141986799999e+02 },
    { 8.1699999999999997e-09, 2.1002131761600000e+00, 1.2577031721001999e+03 },
    { 8.1500000000000002e-09, 6.1360086762999999e-01, 1.7920770499833000e+03 },
    { 8.1300000000000007e-09, 8.6230265040999998e-01, 1.5780271950199001e+03 },
    { 8.0800000000000002e-09, 1.5570706489099999e+00, 8.1762864674800005e+02 },
    { 8.0000000000000005e-09, 4.1345976498599999e+00, 9.3672027114598004e+03 },
    { 7.9799999999999993e-09, 3.4758778171400002e+00, 4.9596316347189997e+02 },
    { 7.9500000000000001e-09, 5.5641591797699999e+00, 1.9786673806107989e+04 },
    { 7.9300000000000005e-09, 3.9999568942000002e-01, 4.0171150783180001e+03 },
    { 7.8299999999999996e-09, 2.2753490508300001e+00, 3.4945376602242000e+03 },
    { 7.7799999999999992e-09, 6.0324742954200001e+00, 4.1896337901340002e+02 },
    { 7.7300000000000004e-09, 5.7499749349600000e+00, 6.2025356634100001e+02 },
    { 7.6799999999999999e-09, 4.7689689460100002e+00, 4.4365997221932003e+03 },
    { 7.6700000000000002e-09, 4.7248344081000004e+00, 8.3821852822500000e+02 },
    { 7.6299999999999995e-09, 4.2558193305999996e+00, 1.9810879358989999e+02 },
    { 7.6299999999999995e-09, 3.7753994895799998e+00, 2.2848939728610000e+02 },
    { 7.6199999999999997e-09, 3.2156951381800001e+00, 5.1582491417233996e+03 },
    { 7.5800000000000007e-09, 1.4318512631900000e+00, 1.9975118388951000e+03 },
    { 7.5200000000000005e-09, 3.5093627571699999e+00, 3.1347110834979998e+02 },
    { 7.5200000000000005e-09, 1.3802634094199999e+00, 1.1312708252620000e+02 },
    { 7.4999999999999993e-09, 2.1513662178600002e+00, 5.2826825569443999e+03 },
    { 7.4300000000000002e-09, 3.4420540238799999e+00, 5.4227659212385997e+03 },
    { 7.3799999999999997e-09, 2.5256710141999998e+00, 1.0931254934210000e+02 },
    { 7.3600000000000002e-09, 2.5465652013700000e+00, 1.4607073267997000e+03 },
    { 7.2799999999999997e-09, 1.2453545674099999e+00, 5.1742094545271590e+04 },
    { 7.2200000000000003e-09, 8.4796954175000006e-01, 8.7832616366560001e+02 },
    { 7.2099999999999997e-09, 4.5827565468599998e+00, 9.5235700270749999e+02 },
    { 7.1900000000000002e-09, 1.0899940468800000e+00, 5.0123677709139997e+02 },
    { 7.1699999999999998e-09, 3.9928376876399998e+00, 1.0392501437542078e+05 },
    { 7.1399999999999997e-09, 4.4611851028000000e+00, 2.1155824666407002e+03 },
    { 7.1099999999999996e-09, 1.8999809053700001e+00, 2.0592491600630001e+02 },
    { 7.1099999999999996e-09, 6.1312379148099998e+00, 2.2067327486970001e+02 },
    { 7.0600000000000000e-09, 1.1232856604999999e+00, 6.4456315275999998e+02 },
    { 7.0500000000000003e-09, 6.0701492856400003e+00, 2.7533807271199998e+02 },
    { 6.9699999999999997e-09, 4.7022489122800000e+00, 3.2741250177853999e+03 },
    { 6.9500000000000002e-09, 4.5829683623299999e+00, 3.1710322435667999e+03 },
    { 6.9399999999999996e-09, 1.6152296286700001e+00, 6.6427630870220003e+02 },
    { 6.9200000000000001e-09, 6.2893428225000003e-01, 7.5403576079649997e+02 },
    { 6.9200000000000001e-09, 5.5731877263999996e+00, 2.5448005855260191e+04 },
    { 6.9200000000000001e-09, 2.8188174805699999e+00, 1.1653954657146000e+03 },
    { 6.8900000000000000e-09, 3.8224101094199998e+00, 1.0372905296613999e+03 },
    { 6.8100000000000003e-09, 2.6530171784000001e-01, 1.0038446123289999e+02 },
};

static const VSOP87Term VSOP87A_SATURN_X2[481] = {
    { 5.6074633400000004e-03, 1.2640163228200001e+00, 2.0618554843720000e+02 },
    { 5.4583451800000000e-03, 3.6234370965700000e+00, 2.2041264243879999e+02 },
    { 4.4334218599999999e-03, 3.1415926535900001e+00, 0.0000000000000000e+00 },
    { 3.3610971299999999e-03, 2.4254743246000001e+00, 2.1329909543799999e+02 },
    { 2.2430226900000002e-03, 2.4915120351900000e+00, 4.2659819087599999e+02 },
    { 8.7170924000000004e-04, 4.8904895169099998e+00, 7.1135470007999997e+00 },
    { 5.0028094000000001e-04, 2.7011904608099999e+00, 4.3371173787679999e+02 },
    { 4.5122590000000001e-04, 3.6735068942999999e-01, 4.1948464387519999e+02 },
    { 3.2847824000000001e-04, 1.5921015366900000e+00, 6.3989728631399998e+02 },
    { 2.7153555000000002e-04, 3.4980400221800001e+00, 2.2752618943959999e+02 },
    { 1.2676167000000001e-04, 1.4546572953000001e+00, 1.9907200143640000e+02 },
    { 1.0330738000000000e-04, 4.7694953129000002e+00, 1.4227094001599999e+01 },
    { 7.2491490000000001e-05, 5.7026455324700001e+00, 1.1020632121940000e+02 },
    { 4.9235849999999997e-05, 3.0846303904200001e+00, 2.1648048917570000e+02 },
    { 4.7773579999999998e-05, 1.8169515534900000e+00, 2.1011770170029999e+02 },
    { 4.6532139999999997e-05, 1.8371004821300001e+00, 6.4701083331480004e+02 },
    { 4.1666330000000000e-05, 5.3288787422599997e+00, 3.1639186965660002e+02 },
    { 3.5083850000000001e-05, 7.8251653368999996e-01, 8.5319638175199998e+02 },
    { 2.6604700000000000e-05, 3.1673139321199999e+00, 2.0936694217490000e+02 },
    { 2.5681140000000000e-05, 1.7402422857199999e+00, 2.1723124870109999e+02 },
    { 2.5380270000000000e-05, 2.6509761240700001e+00, 4.4082528487759998e+02 },
    { 2.5069859999999999e-05, 1.7478181770100001e+00, 1.0309277421860000e+02 },
    { 2.5032770000000000e-05, 4.6945036891100003e+00, 1.1731986822019999e+02 },
    { 2.1292560000000001e-05, 2.8453141367000001e-01, 6.3278373931320004e+02 },
    { 1.8419889999999999e-05, 3.9926987289400002e+00, 3.2350541665740002e+02 },
    { 1.2285109999999999e-05, 4.2906039519000000e-01, 4.1237109687440000e+02 },
    { 8.0473199999999997e-06, 3.3574070604899999e+00, 2.3463973644039999e+02 },
    { 7.9640700000000001e-06, 1.0948382316300000e+00, 2.0225339517410001e+02 },
    { 7.3252499999999999e-06, 3.8615026092200000e+00, 2.2434479570190001e+02 },
    { 5.4542600000000002e-06, 5.7879371314099997e+00, 9.5979227217800002e+01 },
    { 4.5658600000000004e-06, 5.3123795203200004e+00, 5.2969096509459996e+02 },
    { 4.5565700000000002e-06, 3.4986648051799998e+00, 5.2257741809380002e+02 },
    { 4.4035799999999999e-06, 3.5984029026500002e+00, 3.3061896365820002e+02 },
    { 4.4021700000000004e-06, 4.6242969526700000e+00, 2.1340641002400002e+01 },
    { 4.2679100000000001e-06, 1.0048323152200000e+00, 8.6030992875280003e+02 },
    { 4.0371900000000002e-06, 2.9538647140399998e+00, 2.2359403617650000e+02 },
    { 3.9932100000000000e-06, 2.2196228796300002e+00, 4.2977958461370002e+02 },
    { 3.9197400000000002e-06, 6.9006482494999999e-01, 1.1045700263900001e+01 },
    { 3.5382700000000001e-06, 2.0857483814700002e+00, 4.2266603761290003e+02 },
    { 3.3793099999999998e-06, 1.7425274150000000e-02, 1.0664954771900000e+03 },
    { 3.0647999999999999e-06, 4.4423583549499996e+00, 3.1813937377000001e+00 },
    { 2.7776599999999998e-06, 3.9926341476400000e+00, 3.9321532631000000e+00 },
    { 2.6608699999999998e-06, 1.1060500595700000e+00, 4.2341679713830001e+02 },
    { 2.6346100000000000e-06, 3.3294562212600001e+00, 1.2443341522100000e+02 },
    { 2.5275700000000002e-06, 5.5863331406299999e+00, 8.8865680217000005e+01 },
    { 2.4455199999999998e-06, 1.8185517221100000e+00, 6.5412438031559998e+02 },
    { 2.2387900000000002e-06, 2.0308132129700001e+00, 2.0300415469949999e+02 },
    { 1.9816600000000002e-06, 1.7539178258200001e+00, 1.9195845443560000e+02 },
    { 1.9461700000000001e-06, 2.6459755873500002e+00, 6.2567019231239999e+02 },
    { 1.9321300000000001e-06, 2.3551275052000000e+00, 1.4956319713459999e+02 },
    { 1.7195700000000000e-06, 1.2686525665899999e+00, 3.0927832265580003e+02 },
    { 1.6813600000000001e-06, 2.7717165437800002e+00, 5.3680451209540001e+02 },
    { 1.6509299999999999e-06, 4.4599643617800000e+00, 7.4299006053259995e+02 },
    { 1.6332399999999999e-06, 3.3330006674899999e+00, 5.1546387109299997e+02 },
    { 1.3268700000000000e-06, 3.1896322891300000e+00, 3.0216477565500003e+02 },
    { 1.2514100000000001e-06, 2.9519133847700001e+00, 7.3587651353180001e+02 },
    { 1.2322700000000001e-06, 3.2822718900700001e+00, 4.3764389113990001e+02 },
    { 1.1707000000000001e-06, 2.5073325374700000e+00, 2.7703499374140000e+02 },
    { 1.0512199999999999e-06, 4.2158496999899997e+00, 1.0294940738499999e+01 },
    { 9.9291999999999997e-07, 1.9488585950000001e-01, 8.4608283475120004e+02 },
    { 9.4722000000000003e-07, 1.9165937353800000e+00, 4.3053034413910001e+02 },
    { 9.3821999999999999e-07, 3.9601906006099998e+00, 4.1555249061209997e+02 },
    { 8.3689000000000002e-07, 2.7191414014999999e+00, 8.3896928775039999e+02 },
    { 8.2638000000000002e-07, 2.5544586120499999e+00, 4.4793883187839998e+02 },
    { 8.0263000000000003e-07, 4.5965670204000002e+00, 2.3145834270270001e+02 },
    { 6.8110999999999999e-07, 2.0646767639600001e+00, 7.2876296653099996e+02 },
    { 6.0854999999999998e-07, 4.7237880266800003e+00, 1.0593819301891999e+03 },
    { 5.8324999999999998e-07, 5.1630225421600002e+00, 1.8159247264699999e+01 },
    { 4.7762999999999997e-07, 2.8613608914099999e+00, 8.3185574074960005e+02 },
    { 4.6269000000000002e-07, 7.2126710600000001e-03, 4.0843894361129998e+02 },
    { 4.6260999999999998e-07, 2.7373179601099999e+00, 5.4391805909619995e+02 },
    { 4.4406999999999998e-07, 3.4948903174799999e+00, 1.9513984817330001e+02 },
    { 4.4323999999999999e-07, 3.6600009660500001e+00, 9.4206206196899996e+02 },
    { 4.0116999999999999e-07, 3.7888677376300000e+00, 6.1855664531160005e+02 },
    { 3.9962999999999999e-07, 1.1923143992500000e+00, 6.3596513305090002e+02 },
    { 3.9947000000000002e-07, 1.9404411224000001e-01, 1.0736090241908000e+03 },
    { 3.7114999999999998e-07, 2.1461011067000002e+00, 8.5827298831199997e+01 },
    { 3.6125999999999999e-07, 4.6822790972000004e+00, 2.0373786788240000e+02 },
    { 3.6033000000000000e-07, 1.7719563898999999e-01, 2.2286032299359999e+02 },
    { 3.4387000000000002e-07, 1.3865021839900000e+00, 6.4307868005169996e+02 },
    { 3.4250000000000000e-07, 1.5374903200900001e+00, 3.6286229257259998e+02 },
    { 3.3921999999999999e-07, 2.1663814988999999e+00, 4.3689313161450002e+02 },
    { 3.1940000000000001e-07, 2.7205759185899998e+00, 3.3773251065900001e+02 },
    { 3.0471999999999999e-07, 5.9824733575300000e+00, 5.6622351302600002e+01 },
    { 3.0068999999999997e-07, 5.3379297414300000e+00, 2.6992144674060000e+02 },
    { 2.9973000000000000e-07, 2.8091458433600001e+00, 1.0522683831884001e+03 },
    { 2.9852000000000000e-07, 5.5224529875900004e+00, 1.2797945726280000e+03 },
    { 2.9256000000000002e-07, 4.4526816711099997e+00, 1.1338771495710000e+02 },
    { 2.7530999999999999e-07, 2.7000228870599998e+00, 1.1553611574070001e+03 },
    { 2.7368000000000000e-07, 2.0459873220399998e+00, 7.5010360753340001e+02 },
    { 2.6971000000000000e-07, 2.7261203245900001e+00, 1.3154696222179999e+02 },
    { 2.5348000000000002e-07, 4.5924815835399997e+00, 4.8322054217860000e+02 },
    { 2.4974000000000001e-07, 2.0150907220800001e+00, 7.6266071275599998e+01 },
    { 2.4053000000000000e-07, 2.1377697387399999e+00, 7.0849445304200003e+01 },
    { 2.3977999999999999e-07, 5.7265825229400003e+00, 8.1752133216199994e+01 },
    { 2.3209000000000001e-07, 1.0005452240799999e+00, 8.6742347575359997e+02 },
    { 2.2242999999999999e-07, 1.7144673971300000e+00, 1.0451548361876000e+03 },
    { 2.1017999999999999e-07, 3.7976092866200002e+00, 1.6060889739850001e+02 },
    { 2.0692999999999999e-07, 8.6918818397999997e-01, 2.8808069400530002e+02 },
    { 2.0650000000000000e-07, 2.9774033468200001e+00, 1.0702492748170000e+02 },
    { 2.0405999999999999e-07, 1.2703810265900000e+00, 4.1630325013750002e+02 },
    { 2.0372000000000000e-07, 2.5789406957400001e+00, 1.1482476104062000e+03 },
    { 2.0293999999999999e-07, 2.1513519429599999e+00, 2.2091400527800001e+01 },
    { 2.0261000000000001e-07, 4.2921832791299996e+00, 9.5628915597059995e+02 },
    { 1.9907000000000000e-07, 4.5946667773599996e+00, 1.0627416795630000e+02 },
    { 1.9672999999999999e-07, 2.0858432964300002e+00, 2.0076892246579999e+02 },
    { 1.9289000000000000e-07, 5.7571935129999996e-01, 4.9744763618019999e+02 },
    { 1.9228000000000000e-07, 3.4647084552399998e+00, 1.2584539316256000e+03 },
    { 1.8610000000000001e-07, 6.1735658028699998e+00, 1.4244965013379999e+02 },
    { 1.8339000000000000e-07, 2.8469340644200001e+00, 9.4917560896980001e+02 },
    { 1.8185000000000000e-07, 2.8041473993700001e+00, 2.2582926841020000e+02 },
    { 1.8157000000000000e-07, 2.6242305019700001e+00, 5.2690198039499997e+01 },
    { 1.7671000000000000e-07, 3.7415377752899999e+00, 9.3494851496820002e+02 },
    { 1.7400000000000000e-07, 2.8088877414900000e+00, 3.4077089204480001e+02 },
    { 1.7172999999999999e-07, 1.1979641047500000e+00, 6.3735898303399999e+01 },
    { 1.6208000000000001e-07, 6.0483551748600002e+00, 3.4470304530790003e+02 },
    { 1.5869000000000000e-07, 4.7111865672999997e-01, 6.2885158605009997e+02 },
    { 1.5755999999999999e-07, 2.5916823574599999e+00, 6.5094298657790000e+02 },
    { 1.5417000000000001e-07, 3.1662353719900000e+00, 2.4175328344120001e+02 },
    { 1.5274000000000000e-07, 9.8168978226000003e-01, 2.6598929347749998e+02 },
    { 1.4885000000000001e-07, 4.0207593474600003e+00, 1.3851749687070000e+02 },
    { 1.4609000000000001e-07, 5.5839768690499998e+00, 4.9033408917940000e+02 },
    { 1.3724000000000000e-07, 2.1165879369899998e+00, 3.5893013930950002e+02 },
    { 1.3693000000000000e-07, 8.2029906057000002e-01, 6.3671589257630001e+02 },
    { 1.3535999999999999e-07, 6.0305617854499998e+00, 2.8414854074220000e+02 },
    { 1.3432000000000000e-07, 4.5032997677999997e+00, 6.2901893969922003e+03 },
    { 1.3150999999999999e-07, 3.7528566421799998e+00, 1.3757737998458001e+03 },
    { 1.3121000000000000e-07, 2.1448162532800001e+00, 6.2759623029905997e+03 },
    { 1.3012000000000000e-07, 5.3833010739900002e+00, 1.9120769491019999e+02 },
    { 1.2937000000000000e-07, 8.5990781123000004e-01, 1.8105576652360000e+02 },
    { 1.2620000000000001e-07, 5.9266831302799998e+00, 1.0220399093211799e+04 },
    { 1.2424000000000001e-07, 3.4035854413300002e+00, 2.9505122865419997e+02 },
    { 1.2370000000000000e-07, 4.5725518259999998e-01, 4.0525754987360000e+02 },
    { 1.2326999999999999e-07, 3.5681813300199998e+00, 1.0206171999210201e+04 },
    { 1.2303000000000001e-07, 3.7614586950500000e+00, 4.4475743814070000e+02 },
    { 1.1897000000000000e-07, 3.5283957670900001e+00, 5.0835032409220003e+02 },
    { 1.1807000000000000e-07, 6.2455780740000001e-02, 4.2415051032119999e+02 },
    { 1.1403000000000000e-07, 2.5045034636399999e+00, 1.4717530270636000e+03 },
    { 1.1298999999999999e-07, 2.8799180349700002e+00, 3.5033211960040001e+02 },
    { 1.0780000000000000e-07, 4.2519129555299999e+00, 1.2747179660680000e+02 },
    { 1.0714000000000001e-07, 4.0166693681199996e+00, 2.4554242435239999e+02 },
    { 1.0680000000000000e-07, 4.7813242342300004e+00, 1.1624747044077999e+03 },
    { 1.0230999999999999e-07, 5.6839507028199998e+00, 7.0363318461740005e+02 },
    { 1.0108000000000000e-07, 5.5854319299600004e+00, 2.3539049596580000e+02 },
    { 1.0079000000000000e-07, 7.1634516077999999e-01, 3.5181659230870002e+02 },
    { 9.8280000000000004e-08, 6.0169425106099999e+00, 1.2530172972200001e+01 },
    { 9.3019999999999995e-08, 4.7856258625899999e+00, 9.5612275556000004e+00 },
    { 9.0880000000000004e-08, 2.2734813699799998e+00, 1.4646394800628000e+03 },
    { 8.7940000000000005e-08, 3.4677820892600000e+00, 1.2513403846248000e+03 },
    { 8.7289999999999996e-08, 1.4816646565000000e+00, 1.3615467058442000e+03 },
    { 8.7079999999999996e-08, 2.4613675099800001e+00, 1.3686602528450001e+03 },
    { 8.6519999999999997e-08, 3.1849508225399998e+00, 1.5748458012822000e+03 },
    { 8.4219999999999996e-08, 1.7486822592300000e+00, 6.6123792731640003e+02 },
    { 8.3990000000000000e-08, 3.3072860026900002e+00, 6.2830758499914000e+03 },
    { 8.2240000000000001e-08, 3.5392546820000002e+00, 5.8493641121146002e+03 },
    { 7.9070000000000006e-08, 5.8970402736900001e+00, 5.8635912061161998e+03 },
    { 7.8870000000000004e-08, 9.0663919996999998e-01, 5.6511568774670002e+02 },
    { 7.8030000000000005e-08, 4.7188646546299999e+00, 1.0213285546211000e+04 },
    { 7.7589999999999995e-08, 5.8060844121899997e+00, 6.2173803904930003e+02 },
    { 7.6899999999999994e-08, 4.9628344409200000e+00, 9.7795738083342003e+03 },
    { 7.3939999999999999e-08, 1.0374392325499999e+00, 9.7938009023358009e+03 },
    { 7.2779999999999994e-08, 2.7059550293000001e+00, 4.0450679034820001e+02 },
    { 6.9380000000000003e-08, 6.8217212931000004e-01, 2.0863322899200000e+02 },
    { 6.8299999999999996e-08, 3.2415911601100000e+00, 1.2655674786264001e+03 },
    { 6.7780000000000002e-08, 1.0635680061899999e+00, 2.1796496188399999e+02 },
    { 6.6979999999999995e-08, 5.8186582824000004e+00, 1.2125202148330000e+02 },
    { 6.5400000000000003e-08, 5.5348049797599996e+00, 1.5890728952838001e+03 },
    { 6.5239999999999994e-08, 5.3433091742199998e+00, 4.2904587143079999e+02 },
    { 6.4020000000000000e-08, 3.6178333361999998e-01, 3.1321047591889999e+02 },
    { 6.1959999999999994e-08, 4.5112795377500001e+00, 1.4788665740644001e+03 },
    { 6.1799999999999998e-08, 4.6582314526599999e+00, 5.8564776591153995e+03 },
    { 6.1469999999999994e-08, 2.2419977487299998e+00, 3.1957326339430000e+02 },
    { 6.1090000000000000e-08, 5.1859147098999996e+00, 1.9589060769869999e+02 },
    { 6.0530000000000000e-08, 2.8727511275800000e+00, 2.3070758317729999e+02 },
    { 5.8640000000000001e-08, 2.4823825643899999e+00, 1.1413847448250000e+02 },
    { 5.8250000000000002e-08, 3.5751997006499998e+00, 1.6921656695024001e+03 },
    { 5.7749999999999998e-08, 6.0821300617600000e+00, 9.7866873553349997e+03 },
    { 5.5850000000000000e-08, 1.1724827784600000e+00, 4.7928838891549998e+02 },
    { 5.4900000000000002e-08, 5.3782033222700001e+00, 3.4788443904560000e+02 },
    { 5.3300000000000001e-08, 2.8767950010000001e-02, 8.4215068148809996e+02 },
    { 5.1760000000000002e-08, 6.2752352776800002e+00, 7.8713751830400000e+01 },
    { 4.8860000000000002e-08, 2.0492529546400000e+00, 6.4382943957709995e+02 },
    { 4.8109999999999999e-08, 3.2214483579300000e+00, 1.5677322542813999e+03 },
    { 4.7710000000000002e-08, 2.2351081506199999e+00, 9.9911380480899993e+01 },
    { 4.7180000000000003e-08, 2.2920833072200000e+00, 1.7881448967202000e+03 },
    { 4.6080000000000000e-08, 1.9843329843999999e+00, 7.5721715453419995e+02 },
    { 4.5739999999999998e-08, 5.2306212447400000e+00, 4.3615941843159999e+02 },
    { 4.5639999999999997e-08, 3.0546434480600002e+00, 8.2474219374879999e+02 },
    { 4.4360000000000001e-08, 5.6011764000199999e+00, 9.9999864507729999e+03 },
    { 4.3870000000000002e-08, 2.9927747884899998e+00, 1.8912376709388000e+03 },
    { 4.3690000000000003e-08, 4.1865051485900002e+00, 6.0697767545533998e+03 },
    { 4.2710000000000003e-08, 4.0208643043499999e+00, 1.7516605980020000e+02 },
    { 4.2410000000000001e-08, 1.0218510130300000e+00, 2.8956516671359998e+02 },
    { 4.2330000000000003e-08, 5.5239262990800002e+00, 1.2726810256271999e+03 },
    { 4.2200000000000001e-08, 2.0784113744999999e+00, 1.7810313497193999e+03 },
    { 4.2039999999999998e-08, 8.0177985688999998e-01, 5.7616138801060004e+02 },
    { 4.1950000000000002e-08, 4.3521675032299996e+00, 3.3365734504400001e+02 },
    { 4.1570000000000001e-08, 3.7790671297500000e+00, 3.9814400287280000e+02 },
    { 4.0630000000000001e-08, 3.2087389071499999e+00, 3.2668681039510000e+02 },
    { 4.0600000000000000e-08, 6.1459213125299996e+00, 1.2035824960600000e+02 },
    { 4.0259999999999998e-08, 6.9077662766000003e-01, 2.5143213107579999e+02 },
    { 4.0259999999999998e-08, 6.0298944881600001e+00, 7.1074673161819999e+02 },
    { 3.9780000000000003e-08, 1.2320245662000000e+00, 1.3458534360760001e+02 },
    { 3.9090000000000001e-08, 2.1316821802399999e+00, 1.6850521225016000e+03 },
    { 3.8490000000000003e-08, 3.0000049458000002e+00, 9.8899988524600005e+01 },
    { 3.8159999999999999e-08, 4.1941896219199997e+00, 6.1144309831080000e+02 },
    { 3.7679999999999997e-08, 2.7912645685999998e-01, 5.0137978944330001e+02 },
    { 3.7410000000000002e-08, 6.2287950834000005e-01, 8.4926422848890002e+02 },
    { 3.6720000000000001e-08, 3.3493721639300000e+00, 1.5819593482830001e+03 },
    { 3.6360000000000003e-08, 4.6151072600600003e+00, 5.2650957135689998e+02 },
    { 3.5740000000000001e-08, 5.9477981390599997e+00, 1.5667674413540001e+02 },
    { 3.5420000000000002e-08, 6.3053476413999998e-01, 3.2032402291969998e+02 },
    { 3.5329999999999999e-08, 2.1464546512700000e+00, 1.1439910691340000e+02 },
    { 3.3939999999999998e-08, 3.0992498888500002e+00, 1.9603362005060001e+02 },
    { 3.3769999999999997e-08, 1.6071192993700001e+00, 2.9912639426919998e+02 },
    { 3.3759999999999999e-08, 4.3532465426800000e+00, 1.7952584437210000e+03 },
    { 3.3200000000000000e-08, 1.3034428988000000e+00, 1.6779385755008000e+03 },
    { 3.2749999999999999e-08, 1.9241447676900001e+00, 1.6682867252200001e+02 },
    { 3.0470000000000001e-08, 1.7903760001900000e+00, 2.3056457082540001e+02 },
    { 2.9790000000000001e-08, 1.2301422525200001e+00, 1.5104766984290001e+02 },
    { 2.9449999999999999e-08, 2.1506415813299999e+00, 5.5103160609700001e+02 },
    { 2.9280000000000001e-08, 2.3276706779300000e+00, 3.9557870223899999e+02 },
    { 2.8979999999999999e-08, 4.4925473180199997e+00, 7.4781598567299994e+01 },
    { 2.8940000000000000e-08, 6.2524255980200003e+00, 3.8135160823740000e+02 },
    { 2.8480000000000001e-08, 3.3216407625600000e+00, 7.2164941953020002e+02 },
    { 2.8190000000000000e-08, 6.1957651865800001e+00, 4.3126405732199999e+02 },
    { 2.8150000000000001e-08, 5.2791190347599999e+00, 4.1406801790380001e+02 },
    { 2.8130000000000001e-08, 5.6678057655599998e+00, 4.2193232442999999e+02 },
    { 2.8050000000000000e-08, 3.1184951994999999e+00, 2.5976951835400001e+02 },
    { 2.7980000000000000e-08, 7.3382998899999996e-02, 1.2991947716160001e+02 },
    { 2.7829999999999998e-08, 5.6923915356899997e+00, 1.2869081196288000e+03 },
    { 2.6230000000000001e-08, 3.2367621996799998e+00, 5.5406998748280000e+02 },
    { 2.5950000000000001e-08, 3.4030572886599999e+00, 2.0085575391590000e+03 },
    { 2.5810000000000001e-08, 2.2066203646400000e+00, 2.0974232193759999e+03 },
    { 2.5799999999999999e-08, 4.9867611485900003e+00, 1.4930936680660000e+03 },
    { 2.5550000000000000e-08, 2.5549086562000001e-01, 2.2997386999439999e+02 },
    { 2.5550000000000000e-08, 1.4925426473800001e+00, 1.9662432088160000e+02 },
    { 2.5349999999999999e-08, 5.3026783035399996e+00, 3.0534616939270001e+02 },
    { 2.5300000000000002e-08, 3.9118724705300001e+00, 6.9651963761659999e+02 },
    { 2.5300000000000002e-08, 2.6618168816300001e+00, 2.2076295405954002e+03 },
    { 2.5130000000000001e-08, 5.9225442461500002e+00, 3.4321857259960001e+02 },
    { 2.4649999999999999e-08, 1.1030422575500001e+00, 5.7222923474749996e+02 },
    { 2.4610000000000000e-08, 3.9549476958700001e+00, 4.0918970313670002e+02 },
    { 2.4150000000000001e-08, 4.5796621344500004e+00, 4.5576651038700000e+01 },
    { 2.4039999999999999e-08, 3.8525372839699998e+00, 9.2783496796739996e+02 },
    { 2.3899999999999999e-08, 2.5694798932500000e+00, 6.0626632075526004e+03 },
    { 2.3899999999999999e-08, 5.7829722826999996e-01, 8.5637777548969996e+02 },
    { 2.3849999999999998e-08, 5.7041058028300000e+00, 6.9152524274800001e+01 },
    { 2.3670000000000000e-08, 5.1285237177700003e+00, 5.9803745040300001e+01 },
    { 2.3530000000000000e-08, 1.3892939330600000e+00, 6.5019222705250002e+02 },
    { 2.3289999999999999e-08, 2.4255608470499999e+00, 1.4414657116320001e+02 },
    { 2.3199999999999999e-08, 4.0496728499800003e+00, 5.1939602435610004e+02 },
    { 2.2819999999999999e-08, 2.0437230349500002e+00, 4.3912836384820002e+02 },
    { 2.2720000000000001e-08, 6.0707832492799998e+00, 1.3533610313299999e+02 },
    { 2.2609999999999999e-08, 3.1106246232200001e+00, 4.2118156490460001e+02 },
    { 2.2510000000000001e-08, 1.8341144976299999e+00, 1.4563104387150000e+02 },
    { 2.2259999999999999e-08, 2.8456499981399999e+00, 1.1411340634054000e+03 },
    { 2.2189999999999999e-08, 3.4242811609000001e+00, 1.8802630117250001e+02 },
    { 2.2110000000000001e-08, 3.6624762674900002e+00, 2.7555052103309998e+02 },
    { 2.1970000000000001e-08, 5.9120004337600003e+00, 5.3287235883230005e+02 },
    { 2.1929999999999999e-08, 4.1115371575499999e+00, 2.0522234059069999e+02 },
    { 2.1929999999999999e-08, 3.9196816626300000e+00, 2.2137585028530000e+02 },
    { 2.1929999999999999e-08, 1.5600905654899999e+00, 2.0714875628370001e+02 },
    { 2.1929999999999999e-08, 1.8794294751000001e-01, 2.1944943459230001e+02 },
    { 2.1750000000000000e-08, 5.6300233252499998e+00, 4.3222726516850003e+02 },
    { 2.1670000000000001e-08, 2.7114244494099999e+00, 1.1695882514085999e+03 },
    { 2.1620000000000001e-08, 4.6003285839999997e+00, 2.3857188970350001e+02 },
    { 2.1489999999999999e-08, 4.9971558962599998e+00, 9.1693228005540004e+02 },
    { 2.1340000000000001e-08, 2.9546815638299999e+00, 1.7394221952280000e+02 },
    { 2.1220000000000001e-08, 4.1044386699999998e-02, 9.2940845831999994e+01 },
    { 2.0809999999999999e-08, 4.9315774016800002e+00, 1.8892007304980001e+02 },
    { 2.0660000000000001e-08, 3.3299460983300002e+00, 2.6271141844705999e+03 },
    { 2.0380000000000001e-08, 1.7469083063299999e+00, 2.4138150890326001e+03 },
    { 2.0310000000000001e-08, 2.2950829382300002e+00, 9.2047073954699997e+01 },
    { 2.0000000000000000e-08, 4.6322454228099996e+00, 3.2243328914400003e+01 },
    { 1.9840000000000001e-08, 4.7306386046500002e+00, 5.6431785636774002e+03 },
    { 1.9709999999999999e-08, 2.5856012501599999e+00, 1.4575259330619999e+03 },
    { 1.9680000000000002e-08, 5.8873380882299999e+00, 1.3140394986990000e+02 },
    { 1.9589999999999999e-08, 2.6890120145799998e+00, 7.3269511979410004e+02 },
    { 1.9460000000000000e-08, 2.1517689193199998e+00, 2.4209286360333999e+03 },
    { 1.9410000000000000e-08, 3.3760292399399998e+00, 3.7390799283650000e+02 },
    { 1.9130000000000000e-08, 1.6459670822200001e+00, 2.9519424100610001e+02 },
    { 1.8950000000000001e-08, 5.0473045264999996e+00, 2.9126208774300000e+02 },
    { 1.8940000000000000e-08, 2.9913863015700000e+00, 1.8983512179396000e+03 },
    { 1.8870000000000000e-08, 2.9444366599700000e+00, 1.8841241239379999e+03 },
    { 1.8839999999999999e-08, 4.9515406704200000e+00, 5.5651766803759995e+02 },
    { 1.8690000000000001e-08, 2.3895834684800001e+00, 3.5976304344427999e+03 },
    { 1.8629999999999999e-08, 2.2092771517199998e+00, 2.1045367663768002e+03 },
    { 1.8550000000000001e-08, 4.3689486349199997e+00, 2.1116503133776000e+03 },
    { 1.8340000000000001e-08, 6.0904518568499997e+00, 5.6118353448360006e+02 },
    { 1.8259999999999999e-08, 1.8644600663499999e+00, 3.4484605765980001e+02 },
    { 1.8189999999999999e-08, 5.1111183004000005e-01, 2.2046082654860001e+02 },
    { 1.8189999999999999e-08, 1.2369216829600000e+00, 2.0613736432740001e+02 },
    { 1.8189999999999999e-08, 4.4347060400800000e+00, 2.0623373254699999e+02 },
    { 1.8189999999999999e-08, 3.5965127800999999e+00, 2.2036445832900000e+02 },
    { 1.8180000000000001e-08, 3.0394957130400000e+00, 1.2050126195790000e+02 },
    { 1.8110000000000001e-08, 1.0303505050599999e+00, 5.7074476203920005e+02 },
    { 1.7940000000000001e-08, 6.1194307844900004e+00, 9.5733882598969994e+03 },
    { 1.7929999999999999e-08, 5.8214628333500000e+00, 9.1300012679229997e+02 },
    { 1.7870000000000001e-08, 6.2412335069499996e+00, 2.3767811782620001e+02 },
    { 1.7809999999999999e-08, 2.6367479015700002e+00, 1.1642609634290000e+02 },
    { 1.7750000000000000e-08, 6.0837738398700001e+00, 3.2598979237837998e+03 },
    { 1.7640000000000001e-08, 2.7987192014300000e+00, 1.8647239249500001e+02 },
    { 1.7490000000000000e-08, 6.0966504114599998e+00, 4.1703696332039999e+02 },
    { 1.7459999999999999e-08, 3.3611641691899998e+00, 8.9877303279069997e+02 },
    { 1.7420000000000000e-08, 5.3746206917599997e+00, 4.2563498302950001e+02 },
    { 1.7369999999999999e-08, 1.7457062795300000e+00, 1.6772244439930000e+02 },
    { 1.7090000000000000e-08, 5.2872606403000004e+00, 8.1895145568100006e+01 },
    { 1.7030000000000001e-08, 2.8271487383600000e+00, 4.2756139872249997e+02 },
    { 1.7000000000000000e-08, 6.2180054394499997e+00, 5.6363121503840000e+02 },
    { 1.6910000000000001e-08, 2.4399558965399999e+00, 4.5505237887919998e+02 },
    { 1.6740000000000000e-08, 2.0001575255900002e+00, 2.0014439921582000e+03 },
    { 1.6639999999999999e-08, 1.9323756058999999e+00, 2.5265597135319999e+02 },
    { 1.6510000000000000e-08, 4.5823580440900002e+00, 5.4695644048199995e+02 },
    { 1.6390000000000000e-08, 3.5201320193900001e+00, 2.6413412784722000e+03 },
    { 1.5970000000000000e-08, 5.7293043189899997e+00, 2.8096714700450002e+02 },
    { 1.5850000000000000e-08, 1.2884001343400000e+00, 1.5961864422845999e+03 },
    { 1.5810000000000001e-08, 2.3150050877299999e+00, 2.5887574647669999e+02 },
    { 1.5790000000000001e-08, 3.3473637072800000e+00, 2.3249494088155998e+03 },
    { 1.5530000000000001e-08, 2.3610855160999999e-01, 6.8547393735269998e+02 },
    { 1.5460000000000001e-08, 2.4705704317400001e+00, 7.3980866679489998e+02 },
    { 1.5440000000000001e-08, 2.4437978702800001e+00, 1.0490869894507000e+03 },
    { 1.5270000000000000e-08, 3.0471730336600000e+00, 2.7310284047829998e+02 },
    { 1.4909999999999999e-08, 2.0909063036200002e+00, 2.4012579838100001e+02 },
    { 1.4629999999999999e-08, 4.3556587140000003e-01, 2.9823262239190001e+02 },
    { 1.4600000000000000e-08, 1.9820531038200000e+00, 6.9970103135429997e+02 },
    { 1.4310000000000000e-08, 3.5719678798199999e+00, 1.2442268376239999e+03 },
    { 1.4230000000000001e-08, 8.4203295561000002e-01, 2.0151968199120000e+02 },
    { 1.4230000000000001e-08, 9.0600055738999996e-01, 2.2507850888479999e+02 },
    { 1.4140000000000000e-08, 6.2136867461299996e+00, 1.3718416465826999e+03 },
    { 1.4070000000000000e-08, 3.9976288449699999e+00, 9.9928729037722005e+03 },
    { 1.4040000000000000e-08, 2.8117867914699999e+00, 3.1245971639350000e+02 },
    { 1.3910000000000000e-08, 3.3558992486700001e+00, 6.1780588578619995e+02 },
    { 1.3900000000000000e-08, 3.7350418957199998e+00, 7.3194436026870005e+02 },
    { 1.3840000000000000e-08, 2.2144196201900002e+00, 2.5240214102519999e+03 },
    { 1.3790000000000000e-08, 2.1205257209499999e+00, 2.7373205056900001e+03 },
    { 1.3770000000000000e-08, 2.5374187884400001e+00, 1.7739178027185999e+03 },
    { 1.3590000000000000e-08, 1.5696249112700000e+00, 9.6340270297140000e+02 },
    { 1.3520000000000000e-08, 5.5324692043499999e+00, 5.4298794682394000e+03 },
    { 1.3459999999999999e-08, 1.9015880644100001e+00, 2.4067015420317998e+03 },
    { 1.3360000000000000e-08, 4.8746582352700001e+00, 1.9054647649404001e+03 },
    { 1.3300000000000000e-08, 6.1617928598900003e+00, 3.5762897934404000e+03 },
    { 1.3280000000000000e-08, 2.5895620640499999e+00, 2.1426230328450001e+02 },
    { 1.3280000000000000e-08, 5.4416567561300004e+00, 2.1233588759150001e+02 },
    { 1.3230000000000000e-08, 2.9888200852500000e+00, 9.8066817835879999e+02 },
    { 1.3220000000000000e-08, 6.1423556107900001e+00, 4.2161870797543997e+03 },
    { 1.3210000000000000e-08, 4.3765899911500004e+00, 2.8454188003199999e+01 },
    { 1.3210000000000000e-08, 9.8022217091999997e-01, 3.5574874557179999e+02 },
    { 1.3200000000000001e-08, 5.9409795562100003e+00, 2.9648466951296000e+03 },
    { 1.3049999999999999e-08, 1.3712099118800001e+00, 1.9943304451573999e+03 },
    { 1.3049999999999999e-08, 3.8234231283200000e+00, 9.6872999095099999e+01 },
    { 1.3000000000000001e-08, 3.2649522413300001e+00, 8.3578789401270001e+02 },
    { 1.2989999999999999e-08, 3.5336325630899998e+00, 7.6736908292080000e+02 },
    { 1.2880000000000000e-08, 2.2477634976399998e+00, 6.2960234557549995e+02 },
    { 1.2800000000000000e-08, 8.7484258070999998e-01, 4.1202078525365996e+03 },
    { 1.2690000000000000e-08, 3.2255206014700000e+00, 3.2743756992049998e+02 },
    { 1.2680000000000000e-08, 3.8970234968100002e+00, 4.7217484191469998e+02 },
    { 1.2590000000000001e-08, 9.6011598153000000e-01, 8.5001498801430000e+02 },
    { 1.2490000000000000e-08, 2.5178604466499999e+00, 2.5169078632512001e+03 },
    { 1.2460000000000000e-08, 6.7281854121999995e-01, 9.3600891644589992e+03 },
    { 1.2369999999999999e-08, 2.8249881308100000e+00, 2.1934024465938001e+03 },
    { 1.2310000000000001e-08, 4.5000939130199997e+00, 3.1781457905676002e+03 },
    { 1.2220000000000000e-08, 5.7117501886299999e+00, 2.1334727954780001e+02 },
    { 1.2220000000000000e-08, 2.3194686315499999e+00, 2.1325091132820000e+02 },
    { 1.2190000000000000e-08, 1.2931124020700000e+00, 9.4599421523210003e+02 },
    { 1.1959999999999999e-08, 1.8011248866700000e+00, 2.3178358618148000e+03 },
    { 1.1930000000000000e-08, 5.6723525306200004e+00, 4.1130943055358002e+03 },
    { 1.1910000000000000e-08, 2.1150080597000001e+00, 1.5349535039770001e+02 },
    { 1.1860000000000000e-08, 1.6724602956000001e-01, 3.8997952100978000e+03 },
    { 1.1749999999999999e-08, 1.5368724230899999e+00, 2.7302069586891998e+03 },
    { 1.1670000000000000e-08, 4.5935837211099999e+00, 1.9365537546499999e+02 },
    { 1.1670000000000000e-08, 3.4376350990700000e+00, 2.3294281541100000e+02 },
    { 1.1630000000000001e-08, 2.3385034682000000e-01, 1.4807872442630000e+02 },
    { 1.1549999999999999e-08, 5.9051839654400000e+00, 6.7668051566499997e+01 },
    { 1.1539999999999999e-08, 3.5601447215099999e+00, 1.8484490743480001e+02 },
    { 1.1339999999999999e-08, 1.5304480894100001e+00, 5.2549817940059995e+02 },
    { 1.1240000000000000e-08, 2.1905220660100002e+00, 3.9140223040994001e+03 },
    { 1.1220000000000001e-08, 4.0293851920200003e+00, 1.2836556848410001e+02 },
    { 1.1100000000000000e-08, 1.1783076374799999e+00, 4.0171150783180001e+03 },
    { 1.1100000000000000e-08, 3.1983239545800002e+00, 7.3905790726949999e+02 },
    { 1.1050000000000000e-08, 3.1958419480700000e+00, 1.0380412891868000e+03 },
    { 1.1030000000000000e-08, 1.6692676681300000e+00, 1.2623860848887000e+03 },
    { 1.0999999999999999e-08, 5.4137802847399996e+00, 3.2670114707846001e+03 },
    { 1.0880000000000000e-08, 3.9035789259000002e+00, 2.4280421830342002e+03 },
    { 1.0850000000000000e-08, 7.3411935110000007e-02, 1.1767017984094000e+03 },
    { 1.0820000000000000e-08, 1.1430630721599999e+00, 1.3828873468465999e+03 },
    { 1.0760000000000000e-08, 1.3903287901700001e+00, 3.7895888888784002e+03 },
    { 1.0680000000000000e-08, 5.9039886533099999e+00, 3.8926816630970002e+03 },
    { 1.0660000000000001e-08, 7.3008144676999998e-01, 3.6864961146597998e+03 },
    { 1.0620000000000000e-08, 5.9882737605200003e+00, 4.2090735327536004e+03 },
    { 1.0480000000000000e-08, 1.7289508333800001e+00, 3.0537123753465999e+03 },
    { 1.0410000000000000e-08, 3.2828420118100001e+00, 1.5606187072805999e+03 },
    { 1.0340000000000000e-08, 2.0021394509700001e+00, 2.8404132799086001e+03 },
    { 1.0230000000000000e-08, 4.4653650083600001e+00, 5.6360650166766000e+03 },
    { 1.0220000000000000e-08, 1.5232062072999999e-01, 3.5834033404411998e+03 },
    { 1.0200000000000000e-08, 3.0270239670999999e+00, 2.2005159935945999e+03 },
    { 1.0190000000000001e-08, 5.1665722492099997e+00, 5.5800214074589996e+02 },
    { 1.0120000000000001e-08, 3.5956986890699998e+00, 2.1160217440860001e+02 },
    { 1.0120000000000001e-08, 4.4355201311099997e+00, 2.1499601646740001e+02 },
    { 1.0080000000000000e-08, 3.6300105686699999e+00, 2.0326478713040001e+02 },
    { 1.0080000000000000e-08, 4.4012082515099999e+00, 2.2333340374560001e+02 },
    { 1.0029999999999999e-08, 2.8931647710199999e+00, 6.0766952886799999e+01 },
    { 9.9900000000000005e-09, 5.6375637740100002e+00, 1.7165459766239999e+02 },
    { 9.9699999999999993e-09, 6.1686764037400001e+00, 7.7693031047640000e+02 },
    { 9.8600000000000003e-09, 3.7085659898899999e+00, 3.4874241132234001e+03 },
    { 9.7300000000000001e-09, 1.5610746565600000e+00, 1.8180652604900001e+02 },
    { 9.7300000000000001e-09, 3.3285515100300000e+00, 2.4479166482700001e+02 },
    { 9.7200000000000003e-09, 5.4763715748199999e+00, 2.5494359321360000e+02 },
    { 9.6199999999999995e-09, 5.3347231772299999e+00, 2.1404985496340001e+02 },
    { 9.6199999999999995e-09, 2.6964956429500000e+00, 2.1254833591260001e+02 },
    { 9.5599999999999992e-09, 2.5659424531499999e+00, 3.1219908396260001e+02 },
    { 9.5399999999999997e-09, 3.0818851669499998e+00, 2.0670681329900000e+02 },
    { 9.5399999999999997e-09, 4.9493336532300001e+00, 2.1989137757699999e+02 },
    { 9.5399999999999997e-09, 2.5897425560900000e+00, 2.0566428357539999e+02 },
    { 9.5399999999999997e-09, 5.4414762640900003e+00, 2.2093390730060000e+02 },
    { 9.5300000000000000e-09, 4.3629887292999996e+00, 3.4945376602242000e+03 },
    { 9.5200000000000002e-09, 2.2614314684900001e+00, 1.9125783119411999e+03 },
    { 9.4799999999999995e-09, 5.3988174048999999e+00, 1.8094855377225999e+03 },
    { 9.4799999999999995e-09, 1.6835045563700000e+00, 2.1574677599280000e+02 },
    { 9.4500000000000002e-09, 1.1221137656300000e+00, 3.8846515523820000e+02 },
    { 9.4400000000000005e-09, 1.2731988531800000e+00, 1.0515176236630000e+03 },
    { 9.4199999999999993e-09, 2.0956326066300002e+00, 9.5310776223289997e+02 },
    { 9.3700000000000005e-09, 4.3913095311900001e+00, 5.2575881183150000e+02 },
    { 9.3399999999999996e-09, 2.0185562713100000e+00, 2.8332997329077998e+03 },
    { 9.2900000000000008e-09, 4.5989104164000000e-01, 1.4756851803267000e+03 },
    { 9.2300000000000006e-09, 5.0147958718999996e-01, 7.3168372783780001e+02 },
    { 9.1999999999999997e-09, 3.2681671118800000e+00, 4.5884151979040001e+02 },
    { 9.1700000000000004e-09, 1.2557119512599999e+00, 6.0176425067620005e+02 },
    { 9.1100000000000002e-09, 1.3268874746399999e+00, 3.3629906980024002e+03 },
    { 9.0900000000000007e-09, 1.1053209521700000e+00, 3.1496916025644000e+03 },
    { 9.0799999999999993e-09, 2.8747488802699999e+00, 7.4591082183940000e+02 },
    { 9.0699999999999995e-09, 2.9120277668000001e+00, 2.2147430875962000e+03 },
    { 9.0599999999999997e-09, 4.8971520567000004e+00, 9.0588657979150003e+02 },
    { 9.0099999999999993e-09, 1.7538186422499999e+00, 8.2399143422340001e+02 },
    { 8.9800000000000000e-09, 5.8749674419900000e+00, 9.5662747128962001e+03 },
    { 8.9600000000000005e-09, 8.1358229051999997e-01, 1.6992792165031999e+03 },
    { 8.9299999999999996e-09, 6.0479939373800002e+00, 1.8023719907218001e+03 },
    { 8.8499999999999998e-09, 5.0403809648299998e+00, 1.0372905296613999e+03 },
    { 8.8499999999999998e-09, 5.4030047828000001e+00, 4.3263934009738005e+03 },
    { 8.8300000000000003e-09, 1.9514825592400000e+00, 2.9363925071263998e+03 },
    { 8.7799999999999999e-09, 4.7990575515599998e+00, 4.4223726281915997e+03 },
    { 8.7399999999999992e-09, 2.3928737558700002e+00, 2.0903096723752001e+03 },
    { 8.7099999999999999e-09, 2.3719135396100000e+00, 1.3703302416240001e+02 },
    { 8.7000000000000001e-09, 5.8353928664600003e+00, 2.6484548254729998e+03 },
    { 8.7000000000000001e-09, 2.3818371296100000e+00, 1.0562005364515001e+03 },
    { 8.6900000000000004e-09, 5.8544979397899999e+00, 9.2797833480099996e+01 },
    { 8.6499999999999997e-09, 2.5555085710699998e+00, 2.5097943162503998e+03 },
    { 8.6200000000000004e-09, 3.1578254688100000e+00, 3.9739324334740002e+02 },
    { 8.6100000000000007e-09, 3.1780761010000003e-01, 2.9435060541272001e+03 },
    { 8.5199999999999995e-09, 6.0220091408599998e+00, 8.3503713448730002e+02 },
    { 8.5099999999999998e-09, 3.3934524410200000e+00, 2.9577331481288002e+03 },
    { 8.4300000000000000e-09, 4.8186096668099996e+00, 4.1162033734900001e+02 },
    { 8.3199999999999994e-09, 5.9473000653600003e+00, 4.2233006267552000e+03 },
    { 8.2900000000000001e-09, 4.7608668629400004e+00, 7.1453587252939997e+02 },
    { 8.2800000000000004e-09, 6.1507332181300001e+00, 6.0554504565700000e+01 },
    { 8.2499999999999994e-09, 3.0365469898900002e+00, 9.9160620955499994e+01 },
    { 8.2299999999999999e-09, 4.3659438865300002e+00, 2.5382485042536000e+03 },
    { 8.0700000000000005e-09, 5.3653595036599997e+00, 3.0679394693482000e+03 },
    { 8.0700000000000005e-09, 4.6131866193600004e+00, 2.2218566345969998e+03 },
    { 7.9099999999999994e-09, 2.0127901752300001e+00, 2.0199276274319999e+02 },
    { 7.9099999999999994e-09, 6.0184286449500002e+00, 2.2460542813280000e+02 },
    { 7.8500000000000008e-09, 6.3645876991999994e-01, 4.4365997221932003e+03 },
    { 7.7099999999999992e-09, 5.8044254313600003e+00, 8.3110498122419995e+02 },
    { 7.7099999999999992e-09, 1.9374330208299999e+00, 3.0508553696180002e+02 },
    { 7.6399999999999993e-09, 1.3958968235400000e+00, 8.4933526953900000e+01 },
    { 7.5900000000000005e-09, 8.6897209239999995e-02, 7.7841478318470001e+02 },
    { 7.5100000000000007e-09, 6.1361746688699998e+00, 3.5691762464396002e+03 },
    { 7.4600000000000003e-09, 2.3181387280999999e-01, 1.0807225711916001e+03 },
    { 7.4400000000000000e-09, 4.6572951588400002e+00, 2.7851946644970002e+02 },
    { 7.4400000000000000e-09, 5.7331346787000004e-01, 5.0456118318099999e+02 },
    { 7.4199999999999996e-09, 4.7096864514799996e+00, 1.1191856752295000e+03 },
    { 7.3200000000000004e-09, 4.6520796438000001e-01, 1.7224529849340001e+02 },
    { 7.3200000000000004e-09, 4.4244182022100000e+00, 2.5435289238260000e+02 },
    { 7.2600000000000002e-09, 4.6190372831099999e+00, 2.8617539209110000e+03 },
    { 7.1600000000000001e-09, 3.1353354367000003e-01, 3.7967024358792000e+03 },
    { 7.1500000000000003e-09, 1.8907010122900001e+00, 7.6025553592000006e+02 },
    { 7.1399999999999997e-09, 4.0303714556900001e+00, 3.1710322435667999e+03 },
    { 7.0399999999999997e-09, 5.1363608921699999e+00, 3.0609692891809999e+02 },
    { 6.9900000000000001e-09, 8.9798477774999996e-01, 3.4660834722210002e+03 },
    { 6.9299999999999999e-09, 5.5131906871599998e+00, 6.3744960575920004e+02 },
    { 6.8599999999999999e-09, 4.4967843205399998e+00, 2.8546403739102002e+03 },
};

static const VSOP87Term VSOP87A_SATURN_X3[215] = {
    { 7.7115951999999998e-04, 2.9771438536199999e+00, 2.0618554843720000e+02 },
    { 7.5340436000000002e-04, 1.8920800524800001e+00, 2.2041264243879999e+02 },
    { 1.8450895000000000e-04, 3.1415926535900001e+00, 0.0000000000000000e+00 },
    { 1.0527243999999999e-04, 6.6368256890999999e-01, 4.2659819087599999e+02 },
    { 8.9949459999999994e-05, 9.1696559754999996e-01, 4.3371173787679999e+02 },
    { 8.0451599999999994e-05, 3.1286441288700000e+00, 7.1135470007999997e+00 },
    { 7.4035940000000003e-05, 1.7862738587000000e+00, 2.2752618943959999e+02 },
    { 4.5051490000000000e-05, 2.2453131918699998e+00, 4.1948464387519999e+02 },
    { 3.4680100000000000e-05, 3.1459054444599999e+00, 1.9907200143640000e+02 },
    { 2.9746010000000000e-05, 6.0003064155499999e+00, 6.3989728631399998e+02 },
    { 2.3420890000000001e-05, 3.0609177164300001e+00, 1.4227094001599999e+01 },
    { 1.2307150000000000e-05, 4.3819613006899996e+00, 2.1329909543799999e+02 },
    { 1.0238880000000000e-05, 1.1386558690000000e-02, 6.4701083331480004e+02 },
    { 8.0634600000000006e-06, 1.2820583104300001e+00, 2.1648048917570000e+02 },
    { 7.8852899999999999e-06, 8.9850292553000000e-01, 4.4082528487759998e+02 },
    { 7.6873100000000008e-06, 3.6130927590800002e+00, 2.1011770170029999e+02 },
    { 4.5615800000000000e-06, 5.1423539114799999e+00, 8.5319638175199998e+02 },
    { 4.3162399999999997e-06, 3.7790955566100002e+00, 1.1020632121940000e+02 },
    { 3.8424900000000002e-06, 3.0623355820299998e+00, 1.1731986822019999e+02 },
    { 3.2374900000000001e-06, 1.6513598609000000e+00, 2.3463973644039999e+02 },
    { 2.7931200000000001e-06, 2.0919473937599999e+00, 4.1237109687440000e+02 },
    { 2.4232500000000000e-06, 3.3726329182799999e+00, 1.0309277421860000e+02 },
    { 1.9747299999999999e-06, 2.2950275056699998e+00, 6.3278373931320004e+02 },
    { 1.9481999999999998e-06, 7.9561075452999996e-01, 3.1639186965660002e+02 },
    { 1.5389000000000000e-06, 2.8817019349100002e+00, 2.1340641002400002e+01 },
    { 1.5353900000000001e-06, 2.3205802363500001e+00, 3.2350541665740002e+02 },
    { 1.2521000000000000e-06, 1.4555747906400001e+00, 2.0936694217490000e+02 },
    { 1.1454000000000001e-06, 1.1649044510300000e+00, 2.2359403617650000e+02 },
    { 1.1276000000000000e-06, 3.4067900001500000e+00, 2.1723124870109999e+02 },
    { 1.1124299999999999e-06, 5.4174572273599999e+00, 8.6030992875280003e+02 },
    { 8.7336000000000003e-07, 3.6956268262999997e-01, 4.2977958461370002e+02 },
    { 8.5308999999999997e-07, 1.8928041172700001e+00, 3.3061896365820002e+02 },
    { 8.5135000000000001e-07, 3.3758165930000000e-02, 6.5412438031559998e+02 },
    { 7.8230999999999997e-07, 3.4291459701300000e+00, 1.9195845443560000e+02 },
    { 7.7939000000000005e-07, 1.5601059801999999e+00, 1.2443341522100000e+02 },
    { 7.4130000000000000e-07, 1.4239275047700000e+00, 9.5979227217800002e+01 },
    { 6.8141999999999997e-07, 9.8387434300999999e-01, 8.8865680217000005e+01 },
    { 5.7492000000000002e-07, 3.8478236410300002e+00, 2.0300415469949999e+02 },
    { 5.7443000000000004e-07, 3.0304344400400001e+00, 3.0927832265580003e+02 },
    { 5.6662000000000003e-07, 4.3422995619900000e+00, 1.0664954771900000e+03 },
    { 5.5815000000000001e-07, 3.1440643267200001e+00, 2.0225339517410001e+02 },
    { 5.2827000000000000e-07, 1.7340899604600000e+00, 2.2434479570190001e+02 },
    { 4.6987000000000001e-07, 5.4464376340499996e+00, 5.2257741809380002e+02 },
    { 4.3368000000000000e-07, 2.1492345016000000e-01, 4.2266603761290003e+02 },
    { 3.8392000000000001e-07, 4.6179398160999998e+00, 3.0216477565500003e+02 },
    { 3.6710000000000000e-07, 8.0403065387999995e-01, 4.4793883187839998e+02 },
    { 3.4539000000000000e-07, 4.0671116258400000e+00, 6.2567019231239999e+02 },
    { 3.3564000000000001e-07, 5.1404601176800000e+00, 5.1546387109299997e+02 },
    { 2.7097000000000000e-07, 3.5289853066800001e+00, 4.2341679713830001e+02 },
    { 2.6301000000000002e-07, 5.3362829454499998e+00, 3.9321532631000000e+00 },
    { 1.8598000000000000e-07, 9.6593938431000004e-01, 5.3680451209540001e+02 },
    { 1.8273000000000000e-07, 5.5200962661200004e+00, 1.1045700263900001e+01 },
    { 1.7384000000000000e-07, 2.6048738920000001e+00, 7.4299006053259995e+02 },
    { 1.6213999999999999e-07, 5.3389519194500004e+00, 1.9513984817330001e+02 },
    { 1.2459999999999999e-07, 1.1391723475200000e+00, 4.3764389113990001e+02 },
    { 1.1913000000000000e-07, 3.2742179509000002e-01, 4.3689313161450002e+02 },
    { 1.1351000000000000e-07, 4.5921497252199996e+00, 1.0736090241908000e+03 },
    { 1.0736000000000000e-07, 9.5136288745999997e-01, 5.4391805909619995e+02 },
    { 1.0400000000000000e-07, 8.6968337094000003e-01, 3.3773251065900001e+02 },
    { 9.8830000000000006e-08, 9.6970461885000003e-01, 1.3154696222179999e+02 },
    { 9.8420000000000004e-08, 3.1252531210000001e+00, 5.2969096509459996e+02 },
    { 9.5630000000000003e-08, 5.3744437447499998e+00, 6.1855664531160005e+02 },
    { 9.5599999999999996e-08, 3.9095062040499999e+00, 7.2876296653099996e+02 },
    { 9.3450000000000006e-08, 2.6038136240599998e+00, 8.4608283475120004e+02 },
    { 8.8850000000000005e-08, 1.2477534632600000e+00, 8.1752133216199994e+01 },
    { 8.8010000000000005e-08, 4.6231735475900004e+00, 8.3185574074960005e+02 },
    { 8.6319999999999995e-08, 5.7985945748400001e+00, 6.4307868005169996e+02 },
    { 8.3290000000000000e-08, 5.4734319683499999e+00, 8.6742347575359997e+02 },
    { 7.5569999999999994e-08, 6.7492634194000001e-01, 1.4956319713459999e+02 },
    { 7.1079999999999999e-08, 5.6457517740899998e+00, 6.3596513305090002e+02 },
    { 6.9689999999999997e-08, 2.9683989423499999e+00, 1.8159247264699999e+01 },
    { 6.9329999999999999e-08, 2.6505798182500002e+00, 2.3145834270270001e+02 },
    { 6.5509999999999996e-08, 2.0659265572500001e+00, 4.0843894361129998e+02 },
    { 6.3510000000000004e-08, 4.6567068533900002e+00, 7.3587651353180001e+02 },
    { 6.2639999999999997e-08, 4.4109354755999997e+00, 8.3896928775039999e+02 },
    { 6.0720000000000004e-08, 4.8505245402200003e+00, 4.3053034413910001e+02 },
    { 6.0250000000000000e-08, 8.8008570150999998e-01, 2.6992144674060000e+02 },
    { 5.8220000000000001e-08, 3.5259188618500001e+00, 1.2797945726280000e+03 },
    { 5.7110000000000000e-08, 2.6728279927999998e+00, 1.0593819301891999e+03 },
    { 5.4259999999999998e-08, 1.5176454823700001e+00, 2.4175328344120001e+02 },
    { 5.0920000000000002e-08, 5.4229585040200003e+00, 4.1555249061209997e+02 },
    { 4.9800000000000003e-08, 3.9967647114700000e+00, 2.7703499374140000e+02 },
    { 4.8529999999999998e-08, 1.4148495343200000e+00, 5.6622351302600002e+01 },
    { 4.7880000000000003e-08, 5.2345791697399999e+00, 4.1630325013750002e+02 },
    { 4.5020000000000002e-08, 5.1807269530799998e+00, 2.9505122865419997e+02 },
    { 4.3560000000000001e-08, 5.2854265070600004e+00, 5.0835032409220003e+02 },
    { 4.2669999999999998e-08, 1.5673436135999999e-01, 7.5010360753340001e+02 },
    { 4.1619999999999998e-08, 2.5789755800899998e+00, 4.0525754987360000e+02 },
    { 4.0469999999999998e-08, 5.5860728940000000e-02, 4.8322054217860000e+02 },
    { 4.0200000000000003e-08, 5.3926350780999996e+00, 9.4206206196899996e+02 },
    { 3.9629999999999998e-08, 4.6589468556200000e+00, 1.1482476104062000e+03 },
    { 3.7609999999999997e-08, 2.0298402131000001e-01, 1.8484490743480001e+02 },
    { 3.7580000000000003e-08, 6.2681842486199999e+00, 6.6123792731640003e+02 },
    { 3.4959999999999997e-08, 5.4679615881399997e+00, 9.3494851496820002e+02 },
    { 3.4440000000000003e-08, 4.2308482121000002e-01, 7.0849445304200003e+01 },
    { 3.1410000000000001e-08, 2.3725496265600001e+00, 1.1338771495710000e+02 },
    { 3.0549999999999999e-08, 3.5499307146399999e+00, 1.0451548361876000e+03 },
    { 2.9849999999999999e-08, 4.2427137011099996e+00, 1.4646394800628000e+03 },
    { 2.8190000000000000e-08, 2.3752685563400000e+00, 1.0627416795630000e+02 },
    { 2.7770000000000000e-08, 5.3622025096600003e+00, 3.9814400287280000e+02 },
    { 2.6700000000000001e-08, 6.3080955086000001e-01, 1.9589060769869999e+02 },
    { 2.6700000000000001e-08, 1.1172239621400000e+00, 2.3070758317729999e+02 },
    { 2.6320000000000000e-08, 5.2566866726799999e+00, 4.9744763618019999e+02 },
    { 2.6239999999999999e-08, 5.9277470087599999e+00, 3.6286229257259998e+02 },
    { 2.3899999999999999e-08, 5.3524924424400000e+00, 2.1116503133776000e+03 },
    { 2.2790000000000001e-08, 5.9206683145400003e+00, 6.1144309831080000e+02 },
    { 2.1820000000000000e-08, 2.7596282470000000e-02, 8.5827298831199997e+01 },
    { 2.1740000000000001e-08, 3.6085690252800000e+00, 2.0014439921582000e+03 },
    { 2.1719999999999999e-08, 4.7248223597600001e+00, 8.2474219374879999e+02 },
    { 2.1089999999999999e-08, 2.1875431825199998e+00, 1.3757737998458001e+03 },
    { 2.0860000000000000e-08, 1.6383066216900000e+00, 4.4475743814070000e+02 },
    { 2.0640000000000002e-08, 3.7341369056999998e+00, 6.3671589257630001e+02 },
    { 2.0120000000000001e-08, 5.6125818366200004e+00, 9.2783496796739996e+02 },
    { 1.9750000000000001e-08, 2.3828678583300000e+00, 1.0294940738499999e+01 },
    { 1.9470000000000002e-08, 2.9299091741000000e-01, 6.5094298657790000e+02 },
    { 1.8940000000000000e-08, 3.1774753456200000e+00, 2.6342277314714001e+03 },
    { 1.8810000000000001e-08, 1.5168814780900000e+00, 3.4470304530790003e+02 },
    { 1.8259999999999999e-08, 5.1875736109200004e+00, 1.2584539316256000e+03 },
    { 1.8030000000000000e-08, 1.8513923646399999e+00, 1.9120769491019999e+02 },
    { 1.7999999999999999e-08, 2.3340265789500001e+00, 1.6921656695024001e+03 },
    { 1.7840000000000000e-08, 4.5042367073099996e+00, 2.6413412784722000e+03 },
    { 1.7310000000000001e-08, 4.2563655624900001e+00, 1.0702492748170000e+02 },
    { 1.7120000000000000e-08, 3.3761547942700001e+00, 2.3178358618148000e+03 },
    { 1.7100000000000001e-08, 9.1014674347000002e-01, 2.1187638603783998e+03 },
    { 1.7030000000000001e-08, 5.3120489674900000e+00, 1.2513403846248000e+03 },
    { 1.7030000000000001e-08, 4.0324841928600001e+00, 1.7810313497193999e+03 },
    { 1.6660000000000002e-08, 1.7012224307499999e+00, 1.8770105769372001e+03 },
    { 1.6230000000000000e-08, 5.8570946884900001e+00, 1.5961864422845999e+03 },
    { 1.6199999999999999e-08, 3.5926017972400000e+00, 1.3544331588434000e+03 },
    { 1.6110000000000000e-08, 4.9878003597599996e+00, 1.1553611574070001e+03 },
    { 1.5810000000000001e-08, 3.3392712286999999e+00, 1.6708250284999999e+03 },
    { 1.5679999999999999e-08, 3.0396245073100001e+00, 2.3539049596580000e+02 },
    { 1.5340000000000000e-08, 2.9643815267500000e+00, 2.5311349572528002e+03 },
    { 1.4890000000000000e-08, 4.9035295497100000e+00, 9.9911380480899993e+01 },
    { 1.4860000000000000e-08, 2.7942223021100001e+00, 9.5628915597059995e+02 },
    { 1.4790000000000000e-08, 2.7178242259299998e+00, 2.9435060541272001e+03 },
    { 1.4780000000000001e-08, 2.7757104161199999e+00, 6.2901893969922003e+03 },
    { 1.4780000000000001e-08, 5.2663730278800003e+00, 5.8493641121146002e+03 },
    { 1.4640000000000001e-08, 5.1430971064900000e+00, 1.5677322542813999e+03 },
    { 1.4629999999999999e-08, 4.1796129127599997e+00, 5.8635912061161998e+03 },
    { 1.4629999999999999e-08, 3.8624705127399999e+00, 6.2759623029905997e+03 },
    { 1.4610000000000000e-08, 4.4994580379700002e+00, 2.8414854074220000e+02 },
    { 1.4580000000000001e-08, 5.0055787328000001e+00, 1.4717530270636000e+03 },
    { 1.4480000000000000e-08, 4.7638115806499997e+00, 1.5748458012822000e+03 },
    { 1.4230000000000001e-08, 2.0348614524000000e-01, 2.7515475996916002e+03 },
    { 1.4219999999999999e-08, 1.4544368775600001e+00, 3.1568051495651998e+03 },
    { 1.4180000000000000e-08, 4.8963513185099998e+00, 1.8912376709388000e+03 },
    { 1.4040000000000000e-08, 4.6028000205100001e+00, 1.1411340634054000e+03 },
    { 1.4009999999999999e-08, 1.8864278120400000e+00, 1.4244965013379999e+02 },
    { 1.3780000000000000e-08, 5.3331751583699996e+00, 1.2442268376239999e+03 },
    { 1.3660000000000000e-08, 1.5072001251600000e+00, 1.6060889739850001e+02 },
    { 1.3529999999999999e-08, 1.6397132859600001e+00, 2.3320629558164001e+03 },
    { 1.3500000000000000e-08, 4.3290857318300002e+00, 2.1045367663768002e+03 },
    { 1.3430000000000000e-08, 9.2628967168999998e-01, 7.2164941953020002e+02 },
    { 1.2909999999999999e-08, 3.3788835017399999e+00, 2.2147430875962000e+03 },
    { 1.2870000000000000e-08, 2.8703303676199998e+00, 2.9506196011279999e+03 },
    { 1.2709999999999999e-08, 3.8497801541899999e+00, 2.5169078632512001e+03 },
    { 1.2670000000000000e-08, 3.2390335639700001e+00, 2.7302069586891998e+03 },
    { 1.2580000000000001e-08, 3.1337619183599998e+00, 1.9872168981566001e+03 },
    { 1.2180000000000001e-08, 4.6413710080400001e+00, 1.8023719907218001e+03 },
    { 1.2170000000000001e-08, 6.1048663699000005e-01, 2.4351557300350000e+03 },
    { 1.1930000000000000e-08, 2.3532804375400000e+00, 2.0863322899200000e+02 },
    { 1.1930000000000000e-08, 5.6779383826399998e+00, 2.1796496188399999e+02 },
    { 1.1710000000000000e-08, 1.3291209646100000e+00, 6.7668051566499997e+01 },
    { 1.1690000000000001e-08, 3.5184420856599998e+00, 1.0380412891868000e+03 },
    { 1.1590000000000000e-08, 4.0581051903900001e+00, 1.7739178027185999e+03 },
    { 1.1490000000000001e-08, 5.2166564689700001e+00, 1.0522683831884001e+03 },
    { 1.1409999999999999e-08, 4.3653117629400002e+00, 2.2005159935945999e+03 },
    { 1.1199999999999999e-08, 2.2023739497400001e+00, 2.2218566345969998e+03 },
    { 1.1069999999999999e-08, 4.5225000207099999e+00, 1.7881448967202000e+03 },
    { 1.0760000000000000e-08, 4.6554979092600002e+00, 2.3107223148140001e+03 },
    { 1.0649999999999999e-08, 3.1166846264800001e+00, 2.6271141844705999e+03 },
    { 1.0480000000000000e-08, 5.4234178049399997e+00, 1.6992792165031999e+03 },
    { 1.0430000000000000e-08, 3.9488844324399999e+00, 1.5667674413540001e+02 },
    { 1.0370000000000000e-08, 5.0577325587099997e+00, 1.5606187072805999e+03 },
    { 1.0360000000000000e-08, 1.9215063261200001e+00, 2.0085575391590000e+03 },
    { 1.0300000000000000e-08, 3.1417956609500002e+00, 1.2726810256271999e+03 },
    { 1.0250000000000001e-08, 1.4103142503799999e+00, 2.6484548254729998e+03 },
    { 1.0230000000000000e-08, 3.1635416575000002e+00, 1.1624747044077999e+03 },
    { 1.0220000000000000e-08, 2.7387914225599999e+00, 1.7952584437210000e+03 },
    { 1.0080000000000000e-08, 4.2559639302200001e+00, 1.5890728952838001e+03 },
    { 1.0039999999999999e-08, 3.8251973772399999e+00, 2.0974232193759999e+03 },
    { 9.8299999999999993e-09, 3.5787392365800001e+00, 2.5382485042536000e+03 },
    { 9.8000000000000001e-09, 3.3824234512400002e+00, 2.6598929347749998e+02 },
    { 9.6999999999999992e-09, 2.4213725505700001e+00, 3.2598979237837998e+03 },
    { 9.6899999999999994e-09, 4.1189105727299999e+00, 2.5240214102519999e+03 },
    { 9.5900000000000002e-09, 2.8108082137800001e+00, 3.1813937377000001e+00 },
    { 9.4699999999999998e-09, 1.1060239893499999e+00, 2.7373205056900001e+03 },
    { 9.4300000000000007e-09, 1.4305928163599999e+00, 1.2655674786264001e+03 },
    { 9.4300000000000007e-09, 4.3786582438000003e+00, 1.4575259330619999e+03 },
    { 9.3399999999999996e-09, 2.8506181663499999e+00, 2.8475268269093999e+03 },
    { 9.3100000000000003e-09, 4.5813830075400004e+00, 2.2076295405954002e+03 },
    { 9.1499999999999992e-09, 3.9515079166499998e+00, 2.4138150890326001e+03 },
    { 9.1100000000000002e-09, 4.6317239681800002e+00, 5.2690198039499997e+01 },
    { 8.9199999999999998e-09, 4.8707294154999996e+00, 3.4077089204480001e+02 },
    { 8.9000000000000003e-09, 4.2201287053999997e-01, 3.5893013930950002e+02 },
    { 8.8599999999999996e-09, 4.9816254105200004e+00, 1.8841241239379999e+03 },
    { 8.7399999999999992e-09, 3.2003061521100000e+00, 3.1321047591889999e+02 },
    { 8.6900000000000004e-09, 1.7435685832000001e+00, 6.2173803904930003e+02 },
    { 8.5899999999999995e-09, 6.0974319551100002e+00, 1.1695882514085999e+03 },
    { 8.2900000000000001e-09, 1.2300192504500000e+00, 3.4731970192218000e+03 },
    { 8.2900000000000001e-09, 4.4631963189399997e+00, 1.3828873468465999e+03 },
    { 8.2599999999999992e-09, 1.2226241023999999e-01, 2.8546403739102002e+03 },
    { 8.0600000000000007e-09, 4.4496427306199999e+00, 1.9125783119411999e+03 },
    { 8.0100000000000003e-09, 4.1244259101800003e+00, 2.4209286360333999e+03 },
    { 7.8000000000000004e-09, 5.4053020382400003e+00, 2.8808069400530002e+02 },
    { 7.8000000000000004e-09, 5.7675094355300001e+00, 1.3851749687070000e+02 },
    { 7.6100000000000000e-09, 5.1127418225100003e+00, 6.2885158605009997e+02 },
    { 7.6000000000000002e-09, 3.5324512853700001e+00, 1.3615467058442000e+03 },
    { 7.4899999999999996e-09, 3.6503216651599999e+00, 1.8094855377225999e+03 },
    { 7.2699999999999999e-09, 2.5027733088900002e+00, 2.4554242435239999e+02 },
    { 7.2699999999999999e-09, 2.3868528577000001e+00, 1.8105576652360000e+02 },
    { 7.1200000000000002e-09, 8.3128225609999995e-01, 3.1496916025644000e+03 },
    { 6.9900000000000001e-09, 4.1079818318000000e-01, 9.7795738083342003e+03 },
    { 6.9900000000000001e-09, 4.1950044225600003e+00, 1.0220399093211799e+04 },
};

static const VSOP87Term VSOP87A_SATURN_X4[87] = {
    { 7.9599209999999996e-05, 4.7052362336400000e+00, 2.0618554843720000e+02 },
    { 7.8366519999999997e-05, 1.3981693631000000e-01, 2.2041264243879999e+02 },
    { 1.5111959999999999e-05, 6.5615604620000004e-02, 2.2752618943959999e+02 },
    { 1.2230660000000000e-05, 5.4161848536099999e+00, 4.3371173787679999e+02 },
    { 8.6156900000000005e-06, 0.0000000000000000e+00, 0.0000000000000000e+00 },
    { 7.0277999999999999e-06, 4.8494192398599996e+00, 1.9907200143640000e+02 },
    { 5.5060600000000003e-06, 1.3737629607699999e+00, 7.1135470007999997e+00 },
    { 3.8663200000000003e-06, 5.0069828995899996e+00, 4.2659819087599999e+02 },
    { 3.8647600000000004e-06, 1.3381467686699999e+00, 1.4227094001599999e+01 },
    { 3.5395400000000000e-06, 4.1451070112500004e+00, 4.1948464387519999e+02 },
    { 2.0891699999999999e-06, 4.1416096867699999e+00, 6.3989728631399998e+02 },
    { 1.8561699999999999e-06, 5.4339446324500003e+00, 4.4082528487759998e+02 },
    { 1.7184200000000001e-06, 4.4745363058000001e+00, 6.4701083331480004e+02 },
    { 1.0762800000000001e-06, 3.1206678524200000e+00, 2.1329909543799999e+02 },
    { 8.5501000000000003e-07, 6.1991343294799996e+00, 2.3463973644039999e+02 },
    { 4.6688000000000001e-07, 3.2448829292800001e+00, 8.5319638175199998e+02 },
    { 4.4565000000000002e-07, 1.3791922361800000e+00, 1.1731986822019999e+02 },
    { 4.0674000000000000e-07, 3.8426460443199999e+00, 4.1237109687440000e+02 },
    { 3.1426000000000001e-07, 1.1051296603800000e+00, 2.1340641002400002e+01 },
    { 2.3454999999999999e-07, 1.8222105770999999e+00, 1.1020632121940000e+02 },
    { 2.2585000000000000e-07, 4.5457247222600001e+00, 6.5412438031559998e+02 },
    { 2.2100000000000001e-07, 4.9944297432600004e+00, 1.0309277421860000e+02 },
    { 2.1614000000000000e-07, 3.5766913345200000e+00, 8.6030992875280003e+02 },
    { 1.6780999999999999e-07, 5.9584577275599999e+00, 1.2443341522100000e+02 },
    { 1.3266000000000000e-07, 2.6254640628599999e+00, 8.8865680217000005e+01 },
    { 1.2767999999999999e-07, 4.2925928321200004e+00, 6.3278373931320004e+02 },
    { 1.2055000000000000e-07, 5.3155447991899996e+00, 4.4793883187839998e+02 },
    { 1.1680000000000000e-07, 7.0715558880000001e-02, 3.3061896365820002e+02 },
    { 1.1615000000000000e-07, 5.4866664522299997e+00, 1.9195845443560000e+02 },
    { 1.1159000000000000e-07, 5.1449343376000001e-01, 3.2350541665740002e+02 },
    { 9.4539999999999998e-08, 4.9794966687800004e+00, 3.0927832265580003e+02 },
    { 8.1829999999999999e-08, 6.2563706937800001e+00, 3.0216477565500003e+02 },
    { 8.1759999999999999e-08, 4.6957930675800004e+00, 4.2977958461370002e+02 },
    { 7.6479999999999994e-08, 2.4980954027300002e+00, 1.0664954771900000e+03 },
    { 7.2440000000000005e-08, 2.4515496659900000e+00, 3.1639186965660002e+02 },
    { 7.1219999999999998e-08, 3.4832048547699999e+00, 9.5979227217800002e+01 },
    { 6.8420000000000000e-08, 2.2105852149400000e+00, 2.0300415469949999e+02 },
    { 6.7210000000000004e-08, 5.8203492764100000e+00, 2.2359403617650000e+02 },
    { 6.0539999999999998e-08, 5.7667729741700002e+00, 6.2567019231239999e+02 },
    { 5.7949999999999999e-08, 7.7328842873000003e-01, 5.1546387109299997e+02 },
    { 5.1160000000000003e-08, 1.0248707118700000e+00, 5.2257741809380002e+02 },
    { 4.5660000000000000e-08, 2.3333663635900002e+00, 2.1011770170029999e+02 },
    { 4.1220000000000001e-08, 5.7505621744099997e+00, 2.1648048917570000e+02 },
    { 3.7060000000000002e-08, 9.1070583013999995e-01, 4.2341679713830001e+02 },
    { 3.4809999999999999e-08, 4.0766895754999999e-01, 4.0525754987360000e+02 },
    { 2.9140000000000002e-08, 4.7857894089200004e+00, 4.2266603761290003e+02 },
    { 2.7170000000000001e-08, 5.3592831999900001e+00, 1.3154696222179999e+02 },
    { 2.7090000000000000e-08, 4.7869910744999999e-01, 8.3185574074960005e+02 },
    { 2.5550000000000000e-08, 4.6550394624600004e+00, 3.3773251065900001e+02 },
    { 1.8320000000000001e-08, 1.0574832081000001e+00, 6.1855664531160005e+02 },
    { 1.7820000000000000e-08, 1.9496395296800000e+00, 1.8484490743480001e+02 },
    { 1.7820000000000000e-08, 6.0815792904999997e+00, 2.4175328344120001e+02 },
    { 1.6709999999999999e-08, 5.8234739921000003e+00, 7.2876296653099996e+02 },
    { 1.6600000000000000e-08, 5.2506542676199999e+00, 5.3680451209540001e+02 },
    { 1.6470000000000001e-08, 4.6949440020599997e+00, 5.2969096509459996e+02 },
    { 1.6460000000000000e-08, 9.7572157001000004e-01, 1.9513984817330001e+02 },
    { 1.6460000000000000e-08, 7.7231194299000006e-01, 2.3145834270270001e+02 },
    { 1.6070000000000001e-08, 4.5378185101100001e+00, 6.1144309831080000e+02 },
    { 1.5270000000000000e-08, 1.6925157947000000e-01, 7.4299006053259995e+02 },
    { 1.4240000000000000e-08, 1.2992197851700000e+00, 1.3615467058442000e+03 },
    { 1.3850000000000000e-08, 1.0334822503400001e+00, 9.3494851496820002e+02 },
    { 1.3780000000000000e-08, 4.6035784753100000e+00, 1.2584539316256000e+03 },
    { 1.3620000000000001e-08, 2.4349155164000000e-01, 2.9505122865419997e+02 },
    { 1.3599999999999999e-08, 4.3595823047999999e-01, 1.2513403846248000e+03 },
    { 1.3560000000000000e-08, 5.8904392870300004e+00, 1.4646394800628000e+03 },
    { 1.3540000000000001e-08, 5.6585154016999994e-01, 1.0522683831884001e+03 },
    { 1.3259999999999999e-08, 8.3065247009999996e-02, 7.3587651353180001e+02 },
    { 1.3210000000000000e-08, 4.3146339266300000e+00, 1.8912376709388000e+03 },
    { 1.3100000000000000e-08, 2.6671012691599998e+00, 2.1723124870109999e+02 },
    { 1.2760000000000000e-08, 5.2745147845100000e+00, 2.0936694217490000e+02 },
    { 1.2200000000000000e-08, 5.7769385926099996e+00, 1.6779385755008000e+03 },
    { 1.1609999999999999e-08, 6.0708034509499997e+00, 8.1752133216199994e+01 },
    { 1.1090000000000000e-08, 1.1344778680500001e+00, 9.4206206196899996e+02 },
    { 1.0789999999999999e-08, 6.0402130196800004e+00, 1.3686602528450001e+03 },
    { 1.0579999999999999e-08, 1.6007362787499999e+00, 7.5010360753340001e+02 },
    { 1.0400000000000000e-08, 6.1073954670299999e+00, 1.1482476104062000e+03 },
    { 9.6999999999999992e-09, 4.3578262748199998e+00, 1.5748458012822000e+03 },
    { 9.6299999999999992e-09, 3.1543671592000000e-01, 5.0835032409220003e+02 },
    { 8.7199999999999997e-09, 5.4058449057400004e+00, 1.9943304451573999e+03 },
    { 8.6900000000000004e-09, 4.0839453807600004e+00, 2.2076295405954002e+03 },
    { 8.6200000000000004e-09, 4.7510257301600003e+00, 4.3689313161450002e+02 },
    { 8.5899999999999995e-09, 1.1449140909400000e+00, 4.1630325013750002e+02 },
    { 8.1500000000000002e-09, 3.6867723111499999e+00, 1.8841241239379999e+03 },
    { 7.9099999999999994e-09, 4.8772914294000003e-01, 1.3757737998458001e+03 },
    { 7.1099999999999996e-09, 3.9534758907400001e+00, 8.3896928775039999e+02 },
    { 6.9900000000000001e-09, 6.1193893050400003e+00, 1.2726810256271999e+03 },
    { 6.8400000000000004e-09, 5.0970225648499996e+00, 6.6123792731640003e+02 },
};

static const VSOP87Term VSOP87A_SATURN_X5[31] = {
    { 5.8924999999999999e-06, 1.3910544483000001e-01, 2.0618554843720000e+02 },
    { 5.8587299999999999e-06, 4.6655922362400002e+00, 2.2041264243879999e+02 },
    { 2.2558500000000002e-06, 4.6065271030800004e+00, 2.2752618943959999e+02 },
    { 1.3092800000000000e-06, 3.6246402590200000e+00, 4.3371173787679999e+02 },
    { 9.6655000000000009e-07, 2.7211008149999999e-01, 1.9907200143640000e+02 },
    { 3.9233999999999998e-07, 5.8995804726200003e+00, 1.4227094001599999e+01 },
    { 3.4027999999999997e-07, 3.6560260282599999e+00, 4.4082528487759998e+02 },
    { 2.2888000000000000e-07, 2.6630839765199998e+00, 6.4701083331480004e+02 },
    { 1.3217000000000000e-07, 4.5062200344999997e-01, 4.1948464387519999e+02 },
    { 1.1098000000000001e-07, 2.2905344523800002e+00, 6.3989728631399998e+02 },
    { 1.0309000000000001e-07, 4.5168732199699999e+00, 2.3463973644039999e+02 },
    { 8.6179999999999995e-08, 3.5249681298300000e+00, 1.9195845443560000e+02 },
    { 7.2320000000000002e-08, 2.3567072366900002e+00, 4.2659819087599999e+02 },
    { 4.5699999999999999e-08, 4.4581262742499996e+00, 8.8865680217000005e+01 },
    { 4.3740000000000000e-08, 2.7651155148100002e+00, 6.5412438031559998e+02 },
    { 3.9519999999999999e-08, 2.2669651931699999e+00, 4.0525754987360000e+02 },
    { 3.4230000000000003e-08, 5.5242036550999996e-01, 2.1329909543799999e+02 },
    { 3.3449999999999998e-08, 5.7732318208000000e-01, 3.0216477565500003e+02 },
    { 3.3300000000000000e-08, 1.4152660348699999e+00, 8.5319638175199998e+02 },
    { 3.2590000000000003e-08, 3.8996634721799999e+00, 1.2443341522100000e+02 },
    { 3.2570000000000000e-08, 2.0150471206699998e+00, 7.1135470007999997e+00 },
    { 2.6779999999999999e-08, 1.7496877597600000e+00, 8.6030992875280003e+02 },
    { 2.6000000000000001e-08, 5.8886152683499997e+00, 1.1020632121940000e+02 },
    { 2.2930000000000001e-08, 3.3316379954599999e+00, 5.1546387109299997e+02 },
    { 2.1380000000000000e-08, 3.5867888807299999e+00, 4.4793883187839998e+02 },
    { 1.4340000000000000e-08, 1.5296246459000000e+00, 3.0927832265580003e+02 },
    { 1.1790000000000000e-08, 5.6183558745199997e+00, 1.1731986822019999e+02 },
    { 9.8400000000000008e-09, 3.9557922409300001e+00, 2.0300415469949999e+02 },
    { 9.8400000000000008e-09, 4.0754265792500002e+00, 2.2359403617650000e+02 },
    { 7.8299999999999996e-09, 3.1952056232400001e+00, 5.2257741809380002e+02 },
    { 7.7900000000000006e-09, 2.1292970706899998e+00, 3.3061896365820002e+02 },
};

static const VSOP87Term VSOP87A_SATURN_Y0[1658] = {
    { 9.5298688269900005e+00, 5.5860055666499999e+00, 2.1329909543799999e+02 },
    { 7.9387988805999998e-01, 3.1415926535900001e+00, 0.0000000000000000e+00 },
    { 2.6441781302000000e-01, 4.8352806184899997e+00, 4.2659819087599999e+02 },
    { 6.9166539149999998e-02, 2.5527940870600001e+00, 2.0618554843720000e+02 },
    { 6.6335707029999996e-02, 5.4625884828800002e+00, 2.2041264243879999e+02 },
    { 2.3456097419999999e-02, 4.4652132518999998e-01, 7.1135470007999997e+00 },
    { 1.2457904339999999e-02, 6.0367177974999997e-01, 1.1020632121940000e+02 },
    { 1.1838746520000000e-02, 1.3463829837100001e+00, 4.1948464387519999e+02 },
    { 1.0987511310000000e-02, 4.0860878281300002e+00, 6.3989728631399998e+02 },
    { 7.0084933599999996e-03, 1.1361129802500001e+00, 3.1639186965660002e+02 },
    { 4.3446617600000001e-03, 5.4247469626199996e+00, 5.2969096509459996e+02 },
    { 3.7332734200000001e-03, 4.7130872695799999e+00, 4.3371173787679999e+02 },
    { 3.3516236299999998e-03, 6.6422253983000001e-01, 1.0309277421860000e+02 },
    { 9.7837745000000005e-04, 5.7284429017300003e+00, 3.2350541665740002e+02 },
    { 8.3899691000000000e-04, 5.3320407026699996e+00, 2.2752618943959999e+02 },
    { 8.0571807999999998e-04, 4.0529544991000002e+00, 1.1045700263900001e+01 },
    { 7.0957224999999998e-04, 8.8888207566999999e-01, 6.3278373931320004e+02 },
    { 7.0158491000000005e-04, 5.5977796362900003e+00, 2.0936694217490000e+02 },
    { 6.8985859000000004e-04, 2.4446031261700001e+00, 2.1723124870109999e+02 },
    { 6.5937657000000004e-04, 1.2596960820800001e+00, 2.0225339517410001e+02 },
    { 5.8382264000000005e-04, 5.8978766922000003e-01, 2.2434479570190001e+02 },
    { 5.4049836000000001e-04, 3.3375790487899999e+00, 8.5319638175199998e+02 },
    { 4.5790929999999999e-04, 3.0331527631999999e-01, 1.4227094001599999e+01 },
    { 4.4697174999999998e-04, 9.0661238255999999e-01, 6.3735898303399999e+01 },
    { 4.1976402000000002e-04, 2.6259135594799998e+00, 1.9907200143640000e+02 },
    { 2.9666833000000002e-04, 6.0991063834499997e+00, 7.3587651353180001e+02 },
    { 2.5545907000000003e-04, 6.6262292520000002e-02, 1.1731986822019999e+02 },
    { 2.5199575000000001e-04, 5.0896350600600000e+00, 2.1648048917570000e+02 },
    { 2.4835150999999998e-04, 4.0263019057099996e+00, 4.1555249061209997e+02 },
    { 2.4827950000000002e-04, 3.9268142890000002e+00, 7.4781598567299994e+01 },
    { 2.4640836000000001e-04, 2.9544524728199999e+00, 2.1011770170029999e+02 },
    { 2.3396741999999999e-04, 3.9633739363500000e+00, 6.4701083331480004e+02 },
    { 2.3380691000000000e-04, 4.9005107227600000e+00, 1.4956319713459999e+02 },
    { 2.0272215000000001e-04, 2.3431954819800001e+00, 3.0927832265580003e+02 },
    { 2.0099551999999999e-04, 9.8365186365000001e-01, 5.2257741809380002e+02 },
    { 1.5383926999999999e-04, 3.1022782262700002e+00, 2.7703499374140000e+02 },
    { 1.2422966000000000e-04, 2.6255786574300002e+00, 4.9033408917940000e+02 },
    { 1.2048048000000000e-04, 4.0926598011599999e+00, 3.5181659230870002e+02 },
    { 1.1629210000000000e-04, 5.7410828377199996e+00, 1.0593819301891999e+03 },
    { 9.5517960000000000e-05, 3.4878804209399998e+00, 9.5979227217800002e+01 },
    { 9.0348769999999996e-05, 5.8681614419799999e+00, 1.0522683831884001e+03 },
    { 7.6703790000000002e-05, 1.1659427616400000e+00, 7.4299006053259995e+02 },
    { 6.9199460000000004e-05, 1.1709006388300001e+00, 4.1237109687440000e+02 },
    { 6.5367509999999994e-05, 8.4246459391999995e-01, 8.3896928775039999e+02 },
    { 6.0820969999999997e-05, 4.5875828072899996e+00, 4.4082528487759998e+02 },
    { 5.7688970000000001e-05, 3.7377669040199999e+00, 3.8133035637799999e+01 },
    { 5.2018490000000003e-05, 2.9748280242999998e+00, 2.1085141488319999e+02 },
    { 5.1565779999999999e-05, 5.0579699856399998e+00, 2.1574677599280000e+02 },
    { 5.0272109999999997e-05, 9.3213690546000005e-01, 8.4608283475120004e+02 },
    { 4.8381460000000001e-05, 4.9856381247500003e+00, 5.3680451209540001e+02 },
    { 4.0424630000000001e-05, 1.2447121101600001e+00, 1.3703302416240001e+02 },
    { 3.8811040000000001e-05, 3.3802664696300000e+00, 3.9321532631000000e+00 },
    { 3.8156910000000001e-05, 2.4416685115500001e+00, 6.2567019231239999e+02 },
    { 3.7923480000000002e-05, 4.4180604698100003e+00, 4.2266603761290003e+02 },
    { 3.7687510000000002e-05, 2.1495424735999999e+00, 2.1233588759150001e+02 },
    { 3.7520100000000000e-05, 5.8812543401799999e+00, 2.1426230328450001e+02 },
    { 3.3672700000000000e-05, 6.1329884705700000e+00, 4.3764389113990001e+02 },
    { 3.1716800000000001e-05, 4.1955307539500000e+00, 7.6266071275599998e+01 },
    { 3.1386280000000000e-05, 5.6305845592399999e+00, 1.3686602528450001e+03 },
    { 3.1189380000000002e-05, 5.5579939715900002e+00, 2.1325091132820000e+02 },
    { 3.1180560000000002e-05, 2.4725978010200000e+00, 2.1334727954780001e+02 },
    { 2.9821739999999999e-05, 2.5853510721299999e+00, 1.0664954771900000e+03 },
    { 2.8913520000000002e-05, 5.8531349710600002e+00, 8.5827298831199997e+01 },
    { 2.8299440000000002e-05, 5.4443422599800000e+00, 3.3061896365820002e+02 },
    { 2.7745950000000002e-05, 5.6159435130200004e+00, 2.0373786788240000e+02 },
    { 2.6707500000000001e-05, 1.8165311734000000e-01, 3.1813937377000001e+00 },
    { 2.6387150000000000e-05, 3.8367815681200002e+00, 2.8808069400530002e+02 },
    { 2.6206419999999998e-05, 6.9751279148000001e-01, 1.2747179660680000e+02 },
    { 2.5336320000000001e-05, 3.8978859092599998e+00, 1.3851749687070000e+02 },
    { 2.5323739999999999e-05, 3.5262937234099998e+00, 6.2885158605009997e+02 },
    { 2.5017749999999999e-05, 6.1992927439600001e+00, 9.5612275556000004e+00 },
    { 2.4484670000000001e-05, 5.4179432209000000e-01, 1.1553611574070001e+03 },
    { 2.4314960000000001e-05, 2.4012245139499999e+00, 2.2286032299359999e+02 },
    { 2.3397640000000001e-05, 4.2961905385200003e+00, 3.8846515523820000e+02 },
    { 2.2692260000000000e-05, 2.1240190510499999e+00, 4.3053034413910001e+02 },
    { 2.0780490000000001e-05, 1.9568234896400001e+00, 4.2341679713830001e+02 },
    { 1.9892529999999998e-05, 4.2496478369000001e-01, 1.4844727083000000e+00 },
    { 1.6789520000000000e-05, 2.0721171921399999e+00, 9.4917560896980001e+02 },
    { 1.6391809999999998e-05, 6.2823227848999996e-01, 2.1277783057619999e+02 },
    { 1.6365799999999998e-05, 2.9744013972699999e+00, 1.2530172972200001e+01 },
    { 1.6342350000000001e-05, 1.1204307321800000e+00, 2.1382036029980000e+02 },
    { 1.5402420000000002e-05, 3.2144977048299999e+00, 8.6030992875280003e+02 },
    { 1.4996650000000000e-05, 5.1686599057900002e+00, 5.2690198039499997e+01 },
    { 1.4629080000000001e-05, 4.3528569099299999e+00, 4.2977958461370002e+02 },
    { 1.4370630000000000e-05, 3.8429354329300001e+00, 3.5033211960040001e+02 },
    { 1.2763769999999999e-05, 2.9872898776999999e+00, 3.4077089204480001e+02 },
    { 1.1866470000000000e-05, 5.3832362055400003e+00, 1.6850521225016000e+03 },
    { 1.1628069999999999e-05, 2.3504084031699999e+00, 2.0076892246579999e+02 },
    { 1.1521639999999999e-05, 1.7873684830200001e+00, 9.4206206196899996e+02 },
    { 1.1208190000000000e-05, 1.8947869668299999e+00, 7.0363318461740005e+02 },
    { 1.1126170000000000e-05, 5.1911418314500004e+00, 2.3463973644039999e+02 },
    { 1.0910249999999999e-05, 4.2563837020499999e+00, 3.6286229257259998e+02 },
    { 1.0906780000000001e-05, 5.8508622621799997e+00, 1.7394221952280000e+02 },
    { 1.0110230000000001e-05, 2.7242160432000001e-01, 1.4717530270636000e+03 },
    { 1.0088980000000000e-05, 5.8011030244999997e+00, 2.2582926841020000e+02 },
    { 9.7944099999999996e-06, 4.3803036249999998e+00, 1.1624747044077999e+03 },
    { 9.3582999999999998e-06, 2.7832575386300000e+00, 3.0216477565500003e+02 },
    { 9.1974200000000008e-06, 3.2909839700900001e+00, 5.6511568774670002e+02 },
    { 9.1873499999999994e-06, 3.0060451770099998e+00, 5.6363121503840000e+02 },
    { 8.8094299999999994e-06, 6.1783004966500004e+00, 2.0910630974399999e+02 },
    { 8.7036599999999999e-06, 1.6637357334700000e+00, 2.1037833413120001e+02 },
    { 8.6881500000000008e-06, 8.0286526969999994e-02, 2.1621985674480001e+02 },
    { 8.6762199999999997e-06, 1.8435375081100001e+00, 2.1749188113200000e+02 },
    { 8.6189600000000006e-06, 2.1321481064299999e+00, 2.5265597135319999e+02 },
    { 8.5234799999999992e-06, 3.7749202699000000e+00, 8.8865680217000005e+01 },
    { 8.3547600000000006e-06, 1.3915262288300001e+00, 1.6060889739850001e+02 },
    { 8.3198900000000007e-06, 5.2929222226199997e+00, 2.1254833591260001e+02 },
    { 8.2584899999999999e-06, 2.7362972892599999e+00, 2.1404985496340001e+02 },
    { 7.9890900000000004e-06, 6.0669589372499999e+00, 2.2091400527800001e+01 },
    { 7.9590699999999993e-06, 4.4249466780000002e-01, 2.0788246946660001e+02 },
    { 7.8974699999999998e-06, 4.9520079389999996e+00, 2.2359403617650000e+02 },
    { 7.8776000000000006e-06, 1.0963692076900000e+00, 5.1546387109299997e+02 },
    { 7.7082899999999997e-06, 1.3212894955800001e+00, 2.1871572140940000e+02 },
    { 7.6461100000000002e-06, 1.4564177629000000e-01, 2.1340641002400002e+01 },
    { 7.3983400000000002e-06, 6.2774753196399997e+00, 7.2876296653099996e+02 },
    { 7.1442799999999998e-06, 5.6133677553599997e+00, 2.0767002114549999e+02 },
    { 7.1156000000000001e-06, 1.8390991301299999e+00, 4.1703696332039999e+02 },
    { 7.0779700000000003e-06, 1.0009225698799999e+00, 1.2655674786264001e+03 },
    { 7.0208900000000001e-06, 1.9750893461400001e+00, 6.2251425595100002e+01 },
    { 6.9863100000000002e-06, 5.2825913316199999e+00, 1.2443341522100000e+02 },
    { 6.9371400000000003e-06, 2.4054943755600000e+00, 2.1892816973050000e+02 },
    { 6.8604000000000004e-06, 3.0753112959000001e-01, 9.5628915597059995e+02 },
    { 6.6791799999999998e-06, 1.2522506790500001e+00, 4.2511371816769997e+02 },
    { 6.6461800000000003e-06, 1.8239901113000001e-01, 4.7928838891549998e+02 },
    { 6.2975600000000002e-06, 2.9538670066299999e+00, 6.5220371011699996e+01 },
    { 6.2904900000000003e-06, 1.2285880931399999e+00, 1.8159247264699999e+01 },
    { 6.1445999999999996e-06, 2.8521217312700000e+00, 2.1796496188399999e+02 },
    { 5.9401299999999997e-06, 1.3159871316500000e+00, 2.6598929347749998e+02 },
    { 5.8137599999999999e-06, 5.0782281722000000e-01, 2.3145834270270001e+02 },
    { 5.8126099999999999e-06, 2.3887279738300000e+00, 5.5406998748280000e+02 },
    { 5.5325599999999999e-06, 5.1763130651699996e+00, 2.0863322899200000e+02 },
    { 5.3353200000000001e-06, 9.6174816661999996e-01, 2.1478356814630001e+02 },
    { 5.2882599999999996e-06, 6.5918491216999997e-01, 2.1181462272970001e+02 },
    { 5.2034400000000003e-06, 5.0962251883900000e+00, 4.1406801790380001e+02 },
    { 5.1983500000000000e-06, 1.7091197246000001e+00, 9.9160620955499994e+01 },
    { 5.0317999999999998e-06, 2.8000849403900001e+00, 2.5143213107579999e+02 },
    { 5.0078800000000002e-06, 1.4783668563900001e+00, 1.2584539316256000e+03 },
    { 4.8325599999999999e-06, 9.1737302114999997e-01, 4.0843894361129998e+02 },
    { 4.7340300000000003e-06, 5.1405082226500003e+00, 2.0014439921582000e+03 },
    { 4.6552599999999999e-06, 3.8379366992800001e+00, 6.5412438031559998e+02 },
    { 4.6538600000000002e-06, 3.0578636262600001e+00, 2.0300415469949999e+02 },
    { 4.6506099999999998e-06, 1.4530286283600000e+00, 1.4244965013379999e+02 },
    { 4.5491100000000001e-06, 4.1770471657300003e+00, 1.4788665740644001e+03 },
    { 4.4367399999999996e-06, 7.3649664399999997e-03, 1.7881448967202000e+03 },
    { 4.3331600000000000e-06, 3.2055840499800001e+00, 3.1219908396260001e+02 },
    { 4.2621399999999997e-06, 6.0108962514900002e+00, 1.5890728952838001e+03 },
    { 4.2180699999999999e-06, 5.6326824272499998e+00, 7.0849445304200003e+01 },
    { 4.1451399999999998e-06, 2.0643078608800001e+00, 1.4807872442630000e+02 },
    { 4.0002999999999996e-06, 2.7120712594500000e+00, 1.7787437278589999e+02 },
    { 3.9838199999999996e-06, 7.9097747519999997e-02, 2.1341097002260000e+02 },
    { 3.9838199999999996e-06, 1.6692552490599999e+00, 2.1318722085339999e+02 },
    { 3.8477999999999998e-06, 4.1488026045400002e+00, 5.6622351302600002e+01 },
    { 3.7656200000000001e-06, 5.4829635116999997e-01, 1.5819593482830001e+03 },
    { 3.7017300000000000e-06, 3.8286185613199999e+00, 1.5104766984290001e+02 },
    { 3.6760500000000002e-06, 4.3644504650700000e+00, 7.5010360753340001e+02 },
    { 3.5891900000000000e-06, 4.9387802367300004e+00, 2.4476805547999998e+00 },
    { 3.5714300000000002e-06, 5.3321140834799996e+00, 1.7516605980020000e+02 },
    { 3.5043600000000000e-06, 1.1451350167200001e+00, 9.8899988524600005e+01 },
    { 3.4876500000000000e-06, 5.3311217212999997e+00, 2.4872381809010000e+02 },
    { 3.4158500000000001e-06, 3.4526440904500002e+00, 1.9832124191099999e+02 },
    { 3.3641199999999998e-06, 3.2974123533199999e+00, 1.1413847448250000e+02 },
    { 3.3475900000000001e-06, 2.2888972913600001e+00, 2.1308664711690000e+02 },
    { 3.3475900000000001e-06, 5.7426410123900000e+00, 2.1351154375909999e+02 },
    { 3.2773299999999999e-06, 2.6930943256400002e+00, 1.9195845443560000e+02 },
    { 3.1902399999999998e-06, 4.5759319996499999e+00, 2.2827694896500000e+02 },
    { 3.1259000000000000e-06, 9.6110658139000005e-01, 8.3185574074960005e+02 },
    { 3.0722699999999998e-06, 5.8329589928700001e+00, 4.8322054217860000e+02 },
    { 3.0658599999999999e-06, 4.9399647997900002e+00, 1.9513984817330001e+02 },
    { 3.0504500000000002e-06, 1.8263234555999999e-01, 6.2830758499914000e+03 },
    { 2.9684499999999999e-06, 6.2235748595300002e+00, 1.0294940738499999e+01 },
    { 2.9277400000000002e-06, 2.2916015539000001e+00, 4.2415051032119999e+02 },
    { 2.7399300000000002e-06, 5.9883002668399996e+00, 2.7851946644970002e+02 },
    { 2.7384899999999998e-06, 4.8976257090999997e-01, 1.0627416795630000e+02 },
    { 2.7020199999999998e-06, 3.5709821181999999e+00, 6.3596513305090002e+02 },
    { 2.6994299999999999e-06, 5.6914501963999997e-01, 3.2743756992049998e+02 },
    { 2.6927000000000001e-06, 1.9239293784500000e+00, 1.2125202148330000e+02 },
    { 2.6909599999999999e-06, 6.0840911065699999e+00, 1.0451548361876000e+03 },
    { 2.6506999999999999e-06, 4.0690045086300000e+00, 2.0522234059069999e+02 },
    { 2.5305199999999999e-06, 2.6722432827599998e+00, 4.0450679034820001e+02 },
    { 2.5169699999999998e-06, 1.6434343499199999e+00, 2.1303846300710001e+02 },
    { 2.5169699999999998e-06, 1.0491824010000000e-01, 2.1355972786890001e+02 },
    { 2.5135400000000000e-06, 5.4902748851099998e+00, 1.9120769491019999e+02 },
    { 2.4735100000000000e-06, 4.1496725935900001e+00, 2.7555052103309998e+02 },
    { 2.4161600000000000e-06, 4.1775708673700001e+00, 2.2137585028530000e+02 },
    { 2.4054400000000001e-06, 8.9336142219999998e-02, 2.6992144674060000e+02 },
    { 2.3739700000000001e-06, 3.3892599915199999e+00, 2.3539049596580000e+02 },
    { 2.3682000000000000e-06, 2.0031360394000002e+00, 2.1944943459230001e+02 },
    { 2.3679799999999999e-06, 1.1864480878699999e+00, 1.5748458012822000e+03 },
    { 2.3287799999999999e-06, 1.3293872166400000e+00, 3.9356875915200000e+01 },
    { 2.3090599999999998e-06, 2.4651126678400002e+00, 4.9744763618019999e+02 },
    { 2.1887200000000000e-06, 8.6295345781999999e-01, 8.0767548473000002e+00 },
    { 2.1648099999999999e-06, 4.5819056948199997e+00, 4.2904587143079999e+02 },
    { 2.1470700000000002e-06, 1.4055503319900000e+00, 4.2563498302950001e+02 },
    { 2.1243400000000000e-06, 5.3863758102799997e+00, 6.5094298657790000e+02 },
    { 2.1102900000000001e-06, 4.7339644655999997e+00, 5.4391805909619995e+02 },
    { 2.0682599999999999e-06, 2.8968795303400001e+00, 8.4215068148809996e+02 },
    { 2.0394599999999999e-06, 1.3852823832600001e+00, 2.8414854074220000e+02 },
    { 2.0266199999999999e-06, 6.0316988630099999e+00, 2.1045367663768002e+03 },
    { 1.9833999999999999e-06, 3.6954136158400002e+00, 4.8884961647109998e+02 },
    { 1.9713399999999999e-06, 2.4363457263999999e-01, 1.8983512179396000e+03 },
    { 1.9631200000000000e-06, 4.9043295394699999e+00, 2.3178358618148000e+03 },
    { 1.9444700000000002e-06, 2.6240484012200000e+00, 1.8892007304980001e+02 },
    { 1.9065900000000000e-06, 2.7228088989899999e+00, 3.5574874557179999e+02 },
    { 1.9052800000000001e-06, 6.0242734783399996e+00, 2.0714875628370001e+02 },
    { 1.8857100000000001e-06, 2.3354458554100002e+00, 6.1855664531160005e+02 },
    { 1.8612099999999999e-06, 5.2245500538599998e+00, 4.2756139872249997e+02 },
    { 1.8507500000000000e-06, 3.5536553913900000e+00, 6.0176425067620005e+02 },
    { 1.8254900000000001e-06, 5.9592392350300001e+00, 2.1059078245230000e+02 },
    { 1.8117000000000000e-06, 3.8522024255299998e+00, 1.7952584437210000e+03 },
    { 1.8082999999999999e-06, 1.5090420057800000e+00, 8.4342826122899993e+01 },
    { 1.8042100000000001e-06, 4.7621387511200002e+00, 4.9181856188770001e+02 },
    { 1.8039000000000000e-06, 1.0546957767400000e+00, 3.4470304530790003e+02 },
    { 1.7952800000000001e-06, 2.0719717928399999e+00, 2.1600740842370001e+02 },
    { 1.7755899999999999e-06, 1.6062867127500000e+00, 1.0213285546211000e+04 },
    { 1.7404800000000000e-06, 5.3880213533000001e+00, 2.3767811782620001e+02 },
    { 1.7375099999999999e-06, 1.7205111339800001e+00, 4.2664637498579998e+02 },
    { 1.7374600000000000e-06, 4.8059016185000001e+00, 4.2655000676620000e+02 },
    { 1.6975000000000001e-06, 3.2098762542900001e+00, 5.0137978944330001e+02 },
    { 1.6805099999999999e-06, 5.4652686265400003e+00, 3.1245971639350000e+02 },
    { 1.6754500000000000e-06, 1.8342000574899999e+00, 1.2797945726280000e+03 },
    { 1.6323199999999999e-06, 1.7995475541299999e+00, 7.3297125859000005e+01 },
    { 1.5944900000000001e-06, 3.9667536414300000e+00, 3.5893013930950002e+02 },
    { 1.5886800000000000e-06, 1.8481337815200001e+00, 1.7935884549420001e+02 },
    { 1.5601699999999999e-06, 1.9038381230200001e+00, 1.3757737998458001e+03 },
    { 1.5599099999999999e-06, 3.4211128826200001e+00, 2.8956516671359998e+02 },
    { 1.5167600000000000e-06, 2.5667531005500002e+00, 2.1345915413239999e+02 },
    { 1.5167600000000000e-06, 5.4644657196299997e+00, 2.1313903674360000e+02 },
    { 1.5142900000000000e-06, 2.3343401119600000e+00, 2.2046082654860001e+02 },
    { 1.4734400000000000e-06, 1.5244853916100001e+00, 6.1780588578619995e+02 },
    { 1.4661499999999999e-06, 5.4416879941899996e+00, 2.2036445832900000e+02 },
    { 1.4547199999999999e-06, 1.2185359365799999e+00, 6.3671589257630001e+02 },
    { 1.3896199999999999e-06, 1.6282899059600000e+00, 4.3615941843159999e+02 },
    { 1.3697600000000000e-06, 7.0428757869000003e-01, 1.1482476104062000e+03 },
    { 1.3480000000000001e-06, 1.2057628028500000e+00, 3.5424722652100002e+01 },
    { 1.3265299999999999e-06, 4.0817165877299999e+00, 4.6658664459999999e+00 },
    { 1.3179800000000000e-06, 9.3003850214999995e-01, 2.1202707105080000e+02 },
    { 1.3070999999999999e-06, 8.1771148870999999e-01, 2.1457111982519999e+02 },
    { 1.3001500000000000e-06, 5.0763003439599999e+00, 2.4554242435239999e+02 },
    { 1.2864900000000000e-06, 6.1692751034700004e+00, 2.4723934538180001e+02 },
    { 1.2625100000000000e-06, 2.7996154450100001e+00, 1.8105576652360000e+02 },
    { 1.2528399999999999e-06, 1.7624327658600001e+00, 6.4382943957709995e+02 },
    { 1.2342300000000000e-06, 1.7360907837000000e-01, 6.2173803904930003e+02 },
    { 1.2058400000000000e-06, 1.7406709675000001e-01, 3.4225536475310003e+02 },
    { 1.1759700000000001e-06, 9.0279986158000003e-01, 1.8912376709388000e+03 },
    { 1.1676300000000000e-06, 3.8343170840000003e-02, 1.1338771495710000e+02 },
    { 1.1511200000000000e-06, 3.6533309508400000e+00, 5.6290742925000004e+00 },
    { 1.1439400000000000e-06, 5.4245488042999996e+00, 3.2243328914400003e+01 },
    { 1.1253400000000001e-06, 5.6863581833900003e+00, 2.0613736432740001e+02 },
    { 1.1107499999999999e-06, 4.4185011961500003e+00, 1.0702492748170000e+02 },
    { 1.0902299999999999e-06, 3.9010601937499998e+00, 1.1439910691340000e+02 },
    { 1.0763599999999999e-06, 2.5706960693399998e+00, 2.0623373254699999e+02 },
    { 1.0622100000000000e-06, 2.4090572806999999e-01, 7.6736908292080000e+02 },
    { 1.0582800000000000e-06, 6.1688824784499996e+00, 2.1989137757699999e+02 },
    { 1.0466300000000000e-06, 5.8841999568399999e+00, 1.3615467058442000e+03 },
    { 1.0433700000000001e-06, 2.4623596009700002e+00, 1.0736090241908000e+03 },
    { 1.0251400000000000e-06, 5.9631304120199999e+00, 9.6320784650000002e-01 },
    { 1.0136500000000000e-06, 6.2398511188199999e+00, 2.2147430875962000e+03 },
    { 1.0048300000000001e-06, 1.9851594844500000e+00, 1.4000196957899999e+02 },
    { 9.9527999999999990e-07, 6.1353681213200000e+00, 4.2607692601420001e+02 },
    { 9.7745000000000006e-07, 6.2300588879900003e+00, 4.1927856940000003e+00 },
    { 9.5620000000000005e-07, 4.4535794068100003e+00, 4.4793883187839998e+02 },
    { 9.5114999999999999e-07, 3.5992246158099999e+00, 6.4307868005169996e+02 },
    { 9.4747000000000005e-07, 5.7756411029199999e+00, 2.4209286360333999e+03 },
    { 9.1385000000000005e-07, 2.2480633262800001e+00, 7.7693031047640000e+02 },
    { 9.1320999999999996e-07, 3.6728160716000002e-01, 4.2711945573780002e+02 },
    { 9.1142999999999995e-07, 2.0189468027199999e+00, 2.0670681329900000e+02 },
    { 9.0513000000000001e-07, 5.6690834129800001e+00, 9.2047073954699997e+01 },
    { 9.0474000000000004e-07, 2.8675765548099998e+00, 5.2549817940059995e+02 },
    { 8.7891999999999998e-07, 4.7674930808499996e+00, 8.7311771539500000e+01 },
    { 8.6568000000000001e-07, 9.1442175672000003e-01, 7.8713751830400000e+01 },
    { 8.6492999999999995e-07, 1.1548829236300000e+00, 9.1693228005540004e+02 },
    { 8.5046999999999997e-07, 3.1877498985899999e+00, 5.4174670747800000e+01 },
    { 8.3997999999999998e-07, 1.3045394153800001e+00, 6.9152524274800001e+01 },
    { 8.3341000000000000e-07, 4.6703802234599996e+00, 2.6342277314714001e+03 },
    { 8.1905999999999998e-07, 2.8233250500000001e-02, 3.5330106501699998e+02 },
    { 8.1847000000000000e-07, 1.6779902100299999e+00, 2.0606927819500001e+01 },
    { 8.0704999999999996e-07, 1.9389111279400000e+00, 3.2032402291969998e+02 },
    { 8.0179999999999999e-07, 1.7647369977700000e+00, 1.2598732389849999e+02 },
    { 7.9749000000000003e-07, 3.2635189849900001e+00, 1.5912442469019999e+02 },
    { 7.8833999999999998e-07, 4.6155167184700003e+00, 2.9912639426919998e+02 },
    { 7.8591000000000003e-07, 5.9064418458999999e+00, 4.6470422915999997e+01 },
    { 7.7990000000000004e-07, 1.6680932235099999e+00, 1.4014645680499999e+01 },
    { 7.7268999999999997e-07, 2.8001974743100000e+00, 6.7668051566499997e+01 },
    { 7.6856999999999995e-07, 4.8809505263300004e+00, 6.9258748435350003e+02 },
    { 7.6520000000000000e-07, 5.7042294455500002e+00, 1.4414657116320001e+02 },
    { 7.6329000000000000e-07, 5.9019559445500001e+00, 5.4302428721889999e+02 },
    { 7.6209999999999998e-07, 1.9196072124500001e+00, 9.3494851496820002e+02 },
    { 7.6008000000000001e-07, 4.5717410164599999e+00, 1.3458534360760001e+02 },
    { 7.5578999999999997e-07, 2.7206472581100001e+00, 1.5349535039770001e+02 },
    { 7.4957000000000001e-07, 4.6076834183899997e+00, 4.2096911658350001e+02 },
    { 7.3605999999999999e-07, 3.1879952981999998e-01, 1.2726810256271999e+03 },
    { 7.3570000000000000e-07, 6.1445189049800000e+00, 2.8659622129700000e+02 },
    { 7.1834000000000001e-07, 3.4731052493400001e+00, 2.1116503133776000e+03 },
    { 7.1099999999999995e-07, 8.9129238652999998e-01, 6.3033605875839999e+02 },
    { 7.0976999999999998e-07, 4.8623495471000000e+00, 4.8736514376280002e+02 },
    { 7.0849999999999997e-07, 4.6152158723100003e+00, 2.1160217440860001e+02 },
    { 7.0325000000000000e-07, 4.7399900994499999e+00, 3.3773251065900001e+02 },
    { 6.9325999999999995e-07, 4.4067081798799999e+00, 2.1207525516059999e+02 },
    { 6.9325999999999995e-07, 3.6245106402900000e+00, 2.1452293571540000e+02 },
    { 6.8948000000000000e-07, 6.0225054242000002e+00, 3.0534616939270001e+02 },
    { 6.7029000000000000e-07, 2.1050939193199998e+00, 3.3940249943799998e+01 },
    { 6.6790999999999995e-07, 4.9592172477699998e+00, 1.1642609634290000e+02 },
    { 6.6705000000000003e-07, 9.9016626570999999e-01, 2.2093390730060000e+02 },
    { 6.6683999999999995e-07, 7.6966880509000002e-01, 4.2367742956920000e+02 },
    { 6.6652000000000002e-07, 5.4259708963400000e+00, 2.9689454166000000e+00 },
    { 6.6194999999999997e-07, 4.8898073722299999e+00, 6.9651963761659999e+02 },
    { 6.6066000000000004e-07, 7.9541001399999998e-03, 1.1695882514085999e+03 },
    { 6.6000999999999999e-07, 2.5097533523500002e+00, 7.7841478318470001e+02 },
    { 6.5832999999999999e-07, 1.7401060214499999e+00, 4.1630325013750002e+02 },
    { 6.5595000000000005e-07, 3.2365149578199999e+00, 6.1503391543000001e+00 },
    { 6.4746999999999996e-07, 4.5792885624200004e+00, 1.2895626931510000e+02 },
    { 6.4685000000000000e-07, 4.1181703730499999e+00, 4.2193232442999999e+02 },
    { 6.4137000000000003e-07, 4.4216461549000002e-01, 1.4646394800628000e+03 },
    { 6.4125999999999996e-07, 1.3010538921299999e+00, 2.9207613067999998e+00 },
    { 6.3493999999999999e-07, 8.0826123303999997e-01, 3.9117346822389999e+02 },
    { 6.3366000000000002e-07, 4.9906591188900000e+00, 1.8498791978669999e+02 },
    { 6.3310999999999998e-07, 3.5081233772100000e+00, 5.7616138801060004e+02 },
    { 6.1865999999999996e-07, 9.6601636837000004e-01, 1.4977853527000001e+01 },
    { 6.1422000000000000e-07, 4.9333686470300000e+00, 2.6747376618580000e+02 },
    { 5.9971000000000001e-07, 5.4507640773799997e+00, 1.8227960680100000e+02 },
    { 5.9935000000000003e-07, 6.2832430984999998e-01, 2.2076295405954002e+03 },
    { 5.9393000000000003e-07, 5.0477721653999996e+00, 4.3912836384820002e+02 },
    { 5.9139000000000000e-07, 6.2593899388400001e+00, 5.1205725331200000e+01 },
    { 5.9022000000000000e-07, 3.4076903753400001e+00, 2.1499601646740001e+02 },
    { 5.6863999999999999e-07, 6.0203004829599998e+00, 4.4475743814070000e+02 },
    { 5.6846999999999995e-07, 9.7741250258000001e-01, 4.1852143602870001e+02 },
    { 5.5947000000000002e-07, 5.1535719965000004e-01, 6.3841281360569997e+02 },
    { 5.5591999999999996e-07, 3.4316529619199998e+00, 2.2116340196420001e+02 },
    { 5.5245000000000000e-07, 4.2100943077400004e+00, 4.3689313161450002e+02 },
    { 5.4608999999999999e-07, 7.9372732000000008e-03, 3.8724131496080003e+02 },
    { 5.4244000000000003e-07, 1.7776119647999999e+00, 3.2972519178089999e+02 },
    { 5.4036000000000000e-07, 4.6170832597600002e+00, 6.2736711334179995e+02 },
    { 5.3229000000000001e-07, 4.0932838378599996e+00, 2.9201284726839998e+02 },
    { 5.2982000000000002e-07, 2.9452166528700001e+00, 2.4161027108930000e+02 },
    { 5.2915000000000005e-07, 3.6630957053500000e+00, 1.9928444975750000e+02 },
    { 5.2883000000000000e-07, 2.8393166808300001e+00, 1.6682867252200001e+02 },
    { 5.2537999999999996e-07, 4.4709989513400004e+00, 2.0543478891180001e+02 },
    { 5.2407999999999997e-07, 2.4987108227300001e+00, 2.8245161971279998e+02 },
    { 5.1493000000000002e-07, 5.9742587352400003e+00, 2.5311349572528002e+03 },
    { 5.1412000000000000e-07, 5.7666129859200002e+00, 5.1864526483069994e+02 },
    { 5.0427000000000000e-07, 3.5821848772800000e+00, 2.1186280683950000e+02 },
    { 5.0427000000000000e-07, 4.4490339428899999e+00, 2.1473538403649999e+02 },
    { 5.0139000000000002e-07, 3.3981792988999998e-01, 7.0653628910000004e+00 },
    { 5.0113000000000004e-07, 1.8101616354600001e+00, 6.5922821389999999e+00 },
    { 4.9622000000000003e-07, 1.0739790251400001e+00, 4.3079097657000000e+02 },
    { 4.9146999999999999e-07, 4.6208557488400004e+00, 3.3406124266997999e+03 },
    { 4.8709999999999996e-07, 5.6064083658500001e+00, 4.2951895218279998e+02 },
    { 4.8678000000000002e-07, 1.5939843182700000e+00, 9.6872999095099999e+01 },
    { 4.8381999999999995e-07, 3.5164144533599999e+00, 7.1617311105999999e+00 },
    { 4.7714999999999995e-07, 5.1053493558599996e+00, 2.1966188291340001e+02 },
    { 4.7005000000000000e-07, 2.6164955843800000e+00, 3.1019488636999998e+01 },
    { 4.6801999999999998e-07, 1.6227114680000002e-02, 5.5800214074589996e+02 },
    { 4.6354999999999999e-07, 1.5377473101900001e+00, 4.3222726516850003e+02 },
    { 4.6208999999999998e-07, 4.5408364848099998e+00, 4.2584743135060000e+02 },
    { 4.6113000000000000e-07, 5.5509688946300004e+00, 2.1240532356070000e+02 },
    { 4.6113000000000000e-07, 2.4802499255399999e+00, 2.1419286731529999e+02 },
    { 4.5964000000000001e-07, 7.5454904800000000e-01, 2.0566428357539999e+02 },
    { 4.5853000000000002e-07, 3.1320535766500002e+00, 1.9269216761850001e+02 },
    { 4.5618000000000000e-07, 5.6670423655400004e+00, 2.7310284047829998e+02 },
    { 4.5106999999999998e-07, 4.4938008893800001e+00, 2.2731374111849999e+02 },
    { 4.5028999999999999e-07, 4.6131983164000001e-01, 4.2808266358430001e+02 },
    { 4.4467000000000002e-07, 5.5303731989999996e+00, 2.7373205056900001e+03 },
    { 4.4136999999999999e-07, 5.4832121559000004e+00, 1.3554855145409999e+02 },
    { 4.2519000000000001e-07, 2.7008450259900001e+00, 5.4166259714000002e+00 },
    { 4.2505000000000001e-07, 5.3145914559399996e+00, 4.2118156490460001e+02 },
    { 4.2003999999999999e-07, 9.1254981054999995e-01, 2.8096714700450002e+02 },
    { 4.1955999999999998e-07, 5.6954061002499996e+00, 1.6779385755008000e+03 },
    { 4.1851000000000001e-07, 2.3609963192699999e+00, 2.4431858407499999e+02 },
    { 4.1377999999999999e-07, 3.3870141572799999e+00, 5.9803745040300001e+01 },
    { 4.1254000000000001e-07, 4.0143199973100003e+00, 1.2035824960600000e+02 },
    { 4.0433000000000002e-07, 5.1621911573699997e+00, 2.3390602325750001e+02 },
    { 3.9779000000000002e-07, 4.7498520855599997e+00, 7.3879727483859995e+02 },
    { 3.9775000000000002e-07, 4.0454974341899996e+00, 3.3928641933649999e+02 },
    { 3.9341999999999999e-07, 6.2472997033000000e-01, 3.7390799283650000e+02 },
    { 3.9322999999999998e-07, 3.2013895412800002e+00, 7.5632338265689998e+02 },
    { 3.9214999999999997e-07, 4.1216248214600002e+00, 2.8311175651300001e+01 },
    { 3.8453000000000002e-07, 4.4245026634600002e+00, 2.0915449385380001e+02 },
    { 3.8170999999999999e-07, 4.7693941751400004e+00, 1.4563104387150000e+02 },
    { 3.7635000000000001e-07, 1.4465968262300000e+00, 3.4788443904560000e+02 },
    { 3.7471000000000000e-07, 3.6068122176399999e+00, 2.1744369702220001e+02 },
    { 3.7384000000000002e-07, 4.4569910831500001e+00, 3.1957326339430000e+02 },
    { 3.7350000000000000e-07, 1.6373584115399999e+00, 1.2513403846248000e+03 },
    { 3.7172000000000000e-07, 2.9364541722299999e+00, 2.0693630796260001e+02 },
    { 3.7160000000000002e-07, 3.1428932075999998e+00, 4.1644497775600001e+01 },
    { 3.6698000000000002e-07, 2.1152481218300001e+00, 4.3126405732199999e+02 },
    { 3.6628000000000002e-07, 2.0970754280500001e+00, 4.2734895040139997e+02 },
    { 3.6077000000000002e-07, 3.0847634678300002e+00, 8.6742347575359997e+02 },
    { 3.5891999999999999e-07, 4.4454028194799999e+00, 2.9506196011279999e+03 },
    { 3.5072000000000001e-07, 6.2391203927900003e+00, 1.8180652604900001e+02 },
    { 3.4964000000000000e-07, 4.9668686330999998e+00, 2.3575873236100001e+01 },
    { 3.4738999999999999e-07, 3.2638897124999999e-01, 7.7750543983900002e+01 },
    { 3.4731000000000000e-07, 5.8239347559099999e+00, 3.9617508346100003e+01 },
    { 3.4699000000000001e-07, 1.2759524240800000e+00, 3.8135160823740000e+02 },
    { 3.4214999999999998e-07, 5.1082259618099997e+00, 2.5976951835400001e+02 },
    { 3.4157000000000001e-07, 3.9413505242500002e+00, 8.1752133216199994e+01 },
    { 3.4058000000000000e-07, 1.7575156514700001e+00, 4.0525754987360000e+02 },
    { 3.3607000000000002e-07, 4.2072517101200004e+00, 3.9557870223899999e+02 },
    { 3.3556000000000002e-07, 3.8739633588500002e+00, 2.7087335373900000e+01 },
    { 3.3249999999999999e-07, 3.0954521222999998e-01, 1.6209337010679999e+02 },
    { 3.2207000000000002e-07, 1.9368116958699999e+00, 3.1321047591889999e+02 },
    { 3.1782000000000002e-07, 4.1337026231399996e+00, 2.4379022388199999e+01 },
    { 3.1736999999999999e-07, 2.9924236290600001e+00, 7.5075952540000002e-01 },
    { 3.1637000000000002e-07, 2.0711604394599998e+00, 1.6921656695024001e+03 },
    { 3.1497000000000002e-07, 4.1521979218000000e+00, 4.6824268865160002e+02 },
    { 3.1230000000000001e-07, 2.3280228607000000e-01, 4.0599126305649997e+02 },
    { 3.1118000000000001e-07, 1.6911634149000001e+00, 4.6595506679120001e+02 },
    { 3.1105000000000002e-07, 1.8786474496000000e-01, 1.7810313497193999e+03 },
    { 3.0193000000000000e-07, 3.5423807736000001e-01, 2.5240214102519999e+03 },
    { 2.9984999999999998e-07, 1.7514077277700000e+00, 7.1074673161819999e+02 },
    { 2.9765000000000002e-07, 4.5262613257700002e+00, 1.2991947716160001e+02 },
    { 2.9649999999999999e-07, 3.3720465027700000e+00, 2.7406604832480002e+02 },
    { 2.9611000000000002e-07, 3.0356326957399999e+00, 2.9505122865419997e+02 },
    { 2.9441000000000000e-07, 3.0354409228999999e+00, 2.4280421830342002e+03 },
    { 2.9304999999999999e-07, 1.9834042859400001e+00, 4.6473122651379998e+02 },
    { 2.9292000000000001e-07, 4.6110543339500003e+00, 1.3154696222179999e+02 },
    { 2.8756000000000002e-07, 4.8707650905100000e+00, 1.9285222631290000e+02 },
    { 2.8750000000000000e-07, 6.2312638205899997e+00, 3.3365734504400001e+02 },
    { 2.8673999999999999e-07, 5.9472116447100003e+00, 4.5490936652729999e+02 },
    { 2.7836000000000001e-07, 5.8892058379299996e+00, 2.1626804085460000e+02 },
    { 2.7273999999999999e-07, 2.4503005098299999e+00, 6.0766952886799999e+01 },
    { 2.7254999999999999e-07, 1.6944797524499999e+00, 2.4479166482700001e+02 },
    { 2.7189999999999999e-07, 3.4693321600900000e+00, 2.1494362684070001e+02 },
    { 2.7189999999999999e-07, 4.5618866600899999e+00, 2.1165456403530001e+02 },
    { 2.7057000000000002e-07, 2.0865845630800002e+00, 2.1033015002139999e+02 },
    { 2.7051000000000000e-07, 1.3548453806400000e+00, 3.6137781986430002e+02 },
    { 2.7048000000000002e-07, 4.8407693845999997e+00, 3.6648562929500002e+01 },
    { 2.6679000000000002e-07, 5.1729970181100002e+00, 4.9426624244250002e+02 },
    { 2.6627000000000001e-07, 1.2248505949899999e+00, 6.8154178408960001e+02 },
    { 2.6324999999999998e-07, 2.1695193397499999e+00, 5.2126486179999998e-01 },
    { 2.6221999999999998e-07, 2.8014312834899999e+00, 3.1492569389000000e+01 },
    { 2.6199999999999999e-07, 2.2076214161599998e+00, 4.5576651038700000e+01 },
    { 2.5741999999999998e-07, 1.2588885731599999e+00, 4.7780391620720002e+02 },
    { 2.5643999999999999e-07, 2.2921371268300001e+00, 2.2997386999439999e+02 },
    { 2.5611999999999999e-07, 1.7238892092500000e+00, 4.1162033734900001e+02 },
    { 2.5496000000000000e-07, 5.6907425855499998e+00, 2.8475268269093999e+03 },
    { 2.5384000000000000e-07, 3.2279175976899999e+00, 1.6674774556399999e+01 },
    { 2.5305000000000001e-07, 3.1604542432999998e+00, 2.3374596456309999e+02 },
    { 2.5195999999999999e-07, 1.2107682753000000e-01, 4.1258354519549999e+02 },
    { 2.5034000000000000e-07, 4.8855256945900001e+00, 4.2044785172169998e+02 },
    { 2.4839000000000002e-07, 2.6543898165000002e+00, 4.6209790485100001e+01 },
    { 2.4172000000000002e-07, 4.0173034839500001e+00, 1.8472728735580000e+02 },
    { 2.4105000000000000e-07, 9.7906190272000004e-01, 3.3455111692129998e+02 },
    { 2.3970000000000001e-07, 2.3488757145300001e+00, 2.6450482076920002e+02 },
    { 2.3913000000000000e-07, 3.8575143902600000e+00, 1.6289651925890001e+02 },
    { 2.3844000000000001e-07, 4.5588574802200004e+00, 5.9342686339800002e+02 },
    { 2.3832999999999999e-07, 6.1442468693899999e+00, 4.2240540518199998e+02 },
    { 2.3832999999999999e-07, 2.5892652090900001e+00, 1.4122580985639999e+02 },
    { 2.3815000000000000e-07, 6.0263285954899999e+00, 1.1265400177420000e+02 },
    { 2.3767000000000001e-07, 4.5029621200000003e+00, 4.6202291352809999e+02 },
    { 2.3510999999999999e-07, 5.1181734951599998e+00, 2.1249594628590000e+02 },
    { 2.3510999999999999e-07, 2.9130453250100001e+00, 2.1410224459010001e+02 },
    { 2.3225999999999999e-07, 6.0356308099799998e+00, 2.0707932031449999e+02 },
    { 2.2972000000000000e-07, 4.8520708754699999e+00, 4.5097721326419997e+02 },
    { 2.2961000000000000e-07, 2.5810989110699998e+00, 9.2940845831999994e+01 },
    { 2.2863999999999999e-07, 1.1268233469000000e-01, 2.1187638603783998e+03 },
    { 2.2695000000000000e-07, 6.2421473643100001e+00, 2.2333340374560001e+02 },
    { 2.2471999999999999e-07, 2.1563391858399998e+00, 7.6348118626000003e+00 },
    { 2.2398000000000000e-07, 6.2109177925300001e+00, 5.5651766803759995e+02 },
    { 2.2359000000000001e-07, 5.9467560340099999e+00, 4.8640193591629998e+02 },
    { 2.2333999999999999e-07, 2.4916471210000002e+00, 4.2362924545940001e+02 },
    { 2.2282999999999999e-07, 1.9977842460399999e+00, 2.1951887056149999e+02 },
    { 2.2198000000000001e-07, 9.1804268519999999e-01, 4.2648631629139999e+02 },
    { 2.2198000000000001e-07, 5.6110704908400004e+00, 4.2671006546059999e+02 },
    { 2.2079000000000001e-07, 1.9683581093100000e+00, 1.8621176006409999e+02 },
    { 2.1720000000000001e-07, 1.0252419139100000e+00, 4.3201481684740003e+02 },
    { 2.1568999999999999e-07, 1.6359215489900001e+00, 2.2460542813280000e+02 },
    { 2.1495000000000000e-07, 1.8000995496300001e+00, 2.0639799675830000e+02 },
    { 2.1409999999999999e-07, 8.6161640270999995e-01, 3.4321857259960001e+02 },
    { 2.1045000000000001e-07, 4.0072421378499996e+00, 2.4187090352019999e+02 },
    { 2.1012999999999999e-07, 2.1393953971499999e+00, 4.0221916848780000e+02 },
    { 2.0842999999999999e-07, 3.4317435243799999e+00, 4.7217484191469998e+02 },
    { 2.0776999999999999e-07, 1.6679985664700001e+00, 1.7542669223109999e+02 },
    { 2.0601999999999999e-07, 5.4790052586400000e+00, 2.8537238101960003e+02 },
    { 2.0564000000000001e-07, 1.7367006636100000e+00, 1.5667674413540001e+02 },
    { 2.0519000000000000e-07, 5.2871577894000001e+00, 3.0537123753465999e+03 },
    { 2.0403999999999999e-07, 1.4483545443800001e+00, 4.5884151979040001e+02 },
    { 2.0032000000000001e-07, 1.3566257245000000e+00, 1.5677322542813999e+03 },
    { 2.0029999999999999e-07, 2.8167540606500001e+00, 8.4926422848890002e+02 },
    { 1.9929999999999999e-07, 1.4722884397000000e-01, 5.3362311835770004e+02 },
    { 1.9922999999999999e-07, 4.1893912761600003e+00, 1.0775864066459999e+02 },
    { 1.9844000000000000e-07, 4.3567116631000002e-01, 1.4859801210651999e+03 },
    { 1.9726999999999999e-07, 6.1533785026399999e+00, 1.0064509366380000e+02 },
    { 1.9726000000000001e-07, 3.6485544402399999e+00, 9.6340270297140000e+02 },
    { 1.9518999999999999e-07, 1.3145350829300000e+00, 3.5744566660120000e+02 },
    { 1.9430000000000001e-07, 4.9889379659999999e-01, 3.9265794093220001e+02 },
    { 1.9371999999999999e-07, 2.4140775392000000e-01, 8.3110498122419995e+02 },
    { 1.9007999999999999e-07, 1.3953646693399999e+00, 7.2073285581600004e+01 },
    { 1.8946999999999999e-07, 1.5019438972000001e+00, 6.3744960575920004e+02 },
    { 1.8921000000000001e-07, 3.4584291141899999e+00, 1.7165459766239999e+02 },
    { 1.8879000000000001e-07, 3.1378071563900001e+00, 5.7222923474749996e+02 },
    { 1.8843000000000000e-07, 6.1721132988200003e+00, 5.5555446019110002e+02 },
    { 1.8789000000000000e-07, 1.5219362956100000e+00, 6.6704843720000000e+01 },
    { 1.8781000000000001e-07, 6.0552375639099996e+00, 2.4038643081190000e+02 },
    { 1.8652999999999999e-07, 4.9914284485399998e+00, 4.2681063919709999e+02 },
    { 1.8652999999999999e-07, 1.5376847275100001e+00, 4.2638574255489999e+02 },
    { 1.8568000000000001e-07, 2.7094042775200000e+00, 1.5253214255120000e+02 },
    { 1.8535000000000000e-07, 5.5713671422099997e+00, 5.4695644048199995e+02 },
    { 1.8381999999999999e-07, 1.8848462965000001e-01, 2.2020019411769999e+02 },
    { 1.8337000000000001e-07, 4.6770941761599998e+00, 1.7001006625970001e+02 },
    { 1.8321000000000001e-07, 7.1302333929999995e-01, 5.0402576179100002e+01 },
    { 1.8290999999999999e-07, 8.4252688004999998e-01, 1.0038446123289999e+02 },
    { 1.8248000000000000e-07, 1.1694031549399999e+00, 5.0835032409220003e+02 },
    { 1.8220999999999999e-07, 1.6636095390200001e+00, 5.6118353448360006e+02 },
    { 1.8185000000000000e-07, 3.0468976205199998e+00, 7.0214871190910003e+02 },
    { 1.8098000000000000e-07, 4.6799786693999996e+00, 2.3070758317729999e+02 },
    { 1.7952000000000001e-07, 2.7934804167000000e+00, 8.2858353414600003e+01 },
    { 1.7812000000000001e-07, 4.6787686021799999e+00, 4.6053844081979997e+02 },
    { 1.7672999999999999e-07, 4.7190181279400001e+00, 5.2575881183150000e+02 },
    { 1.7463000000000000e-07, 4.8340069824300000e+00, 5.6904784100979998e+02 },
    { 1.7455000000000001e-07, 1.7125048822400000e+00, 9.0588657979150003e+02 },
    { 1.7370000000000000e-07, 3.4828679532200000e+00, 5.5258551477449998e+02 },
    { 1.7326000000000000e-07, 1.3459708728299999e+00, 1.9980571461930001e+02 },
    { 1.7268999999999999e-07, 4.5253101235699997e+00, 1.9996577331370000e+02 },
    { 1.7263999999999999e-07, 3.7060120569400001e+00, 2.9519424100610001e+02 },
    { 1.7239000000000000e-07, 2.1963836463700002e+00, 1.0554497769261000e+03 },
    { 1.7235000000000000e-07, 5.6964246040799997e+00, 2.5494359321360000e+02 },
    { 1.7163999999999999e-07, 2.6663376440200000e+00, 2.8571808082200000e+01 },
    { 1.7127000000000000e-07, 1.4259332665100000e+00, 4.1953282798499998e+02 },
    { 1.7078000000000000e-07, 3.0365151122000000e+00, 1.4659425171800001e+02 },
    { 1.6675999999999999e-07, 4.0072067393199999e+00, 7.0511765732569995e+02 },
    { 1.6640999999999999e-07, 4.5268135213500003e+00, 4.1943645976540000e+02 },
    { 1.6595000000000000e-07, 3.1993527598900000e+00, 3.6434676528090000e+02 },
    { 1.6502999999999999e-07, 8.6644103914000004e-01, 1.1116952906589999e+02 },
    { 1.6498000000000001e-07, 5.5299795122999997e+00, 1.9943304451573999e+03 },
    { 1.6444000000000001e-07, 4.2898329517000002e+00, 2.6370167161709998e+02 },
    { 1.6411999999999999e-07, 1.1829129938100000e+00, 2.2189711514710001e+02 },
    { 1.6383000000000001e-07, 1.1030926780100001e+00, 2.0389792657679999e+02 },
    { 1.6374000000000001e-07, 4.0531803594999998e+00, 7.5721715453419995e+02 },
    { 1.6371000000000000e-07, 1.1183330033900001e+00, 8.1895145568100006e+01 },
    { 1.6290000000000001e-07, 4.4003701782700002e+00, 4.1053796944600002e+01 },
    { 1.6252999999999999e-07, 3.4326929033100000e+00, 1.0924311337290000e+02 },
    { 1.6067000000000000e-07, 3.4979473438499999e+00, 2.2663241756229999e+02 },
    { 1.5961000000000001e-07, 8.6204855849999998e-02, 2.8404132799086001e+03 },
    { 1.5958000000000000e-07, 6.1847428295500002e+00, 2.0974232193759999e+03 },
    { 1.5930000000000000e-07, 6.1706914429000004e-01, 1.7245774681450001e+02 },
    { 1.5926000000000001e-07, 4.0206286297999999e-01, 2.2679247625670001e+02 },
    { 1.5892000000000000e-07, 2.3824760428600000e+00, 2.3056457082540001e+02 },
    { 1.5877000000000001e-07, 4.2069144936100002e+00, 1.7265475387399999e+01 },
    { 1.5830999999999999e-07, 8.4929199819999995e-02, 2.5117149864490000e+02 },
    { 1.5790000000000000e-07, 1.3700145446600001e+00, 3.4884764689209999e+02 },
    { 1.5648000000000001e-07, 1.5668863601600000e+00, 2.1129335786790000e+02 },
    { 1.5648000000000001e-07, 1.8114715283999999e-01, 2.1530483300809999e+02 },
    { 1.5647000000000000e-07, 4.2292405336199996e+00, 3.2670114707846001e+03 },
    { 1.5578000000000001e-07, 2.7928788130600002e+00, 5.7074476203920005e+02 },
    { 1.5566999999999999e-07, 1.5431832518799999e+00, 2.2030076785419999e+02 },
    { 1.5482999999999999e-07, 5.0013865526399996e+00, 4.0992341631959999e+02 },
    { 1.5461999999999999e-07, 6.4414979340000000e-01, 2.2270026429920000e+02 },
    { 1.5393000000000000e-07, 1.5806990290000000e-01, 4.2000590873700003e+02 },
    { 1.5302000000000000e-07, 5.6500652382099998e+00, 9.8066817835879999e+02 },
    { 1.5064999999999999e-07, 4.4800831440000000e+00, 2.5887574647669999e+02 },
    { 1.5062000000000001e-07, 4.1743259741800003e+00, 4.4157604440300003e+02 },
    { 1.5012000000000000e-07, 6.2342754359299999e+00, 2.2052451702339999e+02 },
    { 1.4822000000000001e-07, 1.2437161534000001e-01, 1.9577298761969999e+02 },
    { 1.4819999999999999e-07, 5.6944121850700000e+00, 8.3503713448730002e+02 },
    { 1.4709999999999999e-07, 5.6450048893000004e-01, 1.1015813710960001e+02 },
    { 1.4705000000000001e-07, 3.7620580202100000e+00, 1.1025450532920000e+02 },
    { 1.4697000000000000e-07, 1.2652387723400000e+00, 4.3274853003030000e+02 },
    { 1.4433000000000000e-07, 2.2059095550399999e+00, 3.8698068252989998e+02 },
    { 1.4376000000000001e-07, 2.5947261198699998e+00, 4.4868959140380002e+02 },
    { 1.4282000000000000e-07, 4.2726394129300003e+00, 4.2538269652899999e+01 },
    { 1.4166000000000001e-07, 5.1135104221000001e+00, 7.7962992305000000e+01 },
    { 1.4161999999999999e-07, 4.8783024873400000e+00, 8.9484087952760001e+02 },
    { 1.4100999999999999e-07, 1.4546668740199999e+00, 9.9911380480899993e+01 },
    { 1.4016000000000001e-07, 5.6368912122400001e+00, 4.2685882330689998e+02 },
    { 1.4016000000000001e-07, 8.9222201488999997e-01, 4.2633755844510000e+02 },
    { 1.3957000000000000e-07, 4.6048213971200003e+00, 8.6424208201590000e+02 },
    { 1.3848999999999999e-07, 5.6971120252999996e-01, 8.5001498801430000e+02 },
    { 1.3829999999999999e-07, 3.4313566070800001e+00, 4.3467494572330003e+02 },
    { 1.3741000000000001e-07, 2.8116024584099999e+00, 8.1506334611420004e+02 },
    { 1.3631999999999999e-07, 2.3133127677399998e+00, 1.3333322124300000e+01 },
    { 1.3407000000000001e-07, 1.0990667569999999e-02, 5.4073666535849998e+02 },
    { 1.3350000000000000e-07, 6.5575474438000003e-01, 6.3893407846749994e+02 },
    { 1.3281000000000001e-07, 1.6208171761100001e+00, 2.3082520325630000e+02 },
    { 1.3236999999999999e-07, 4.8210221096500003e+00, 4.8077286162380000e+02 },
    { 1.3217000000000000e-07, 2.6659236626100000e+00, 1.8816931352440000e+02 },
    { 1.3199000000000001e-07, 4.3838858499000004e+00, 4.5761767951299998e+02 },
    { 1.3192000000000001e-07, 1.9730438033900000e+00, 2.0085575391590000e+03 },
    { 1.2993000000000000e-07, 5.7145876036300001e+00, 1.9365537546499999e+02 },
    { 1.2984999999999999e-07, 3.3332560315199999e+00, 2.5658812461629998e+02 },
    { 1.2977999999999999e-07, 5.0096771941900000e+00, 2.4175328344120001e+02 },
    { 1.2956999999999999e-07, 5.4088483303399997e+00, 3.1639186965660001e+03 },
    { 1.2848000000000000e-07, 5.6267232324299998e+00, 1.9603362005060001e+02 },
    { 1.2721000000000001e-07, 4.2631814945300004e+00, 1.7076082578510000e+02 },
    { 1.2620000000000001e-07, 9.2344635693999999e-01, 1.3140394986990000e+02 },
    { 1.2576999999999999e-07, 3.1187279533000001e-01, 2.3857188970350001e+02 },
    { 1.2359000000000001e-07, 5.1989412041899996e+00, 3.2668681039510000e+02 },
    { 1.2270999999999999e-07, 4.5950418130999998e-01, 6.6253120356299996e+02 },
    { 1.2127000000000000e-07, 2.5741410293300002e+00, 2.7444340526907999e+03 },
    { 1.2095000000000001e-07, 1.1454730664899999e+00, 2.5414044406150001e+02 },
    { 1.2048999999999999e-07, 1.7961818844699999e+00, 2.0326478713040001e+02 },
    { 1.1989000000000000e-07, 4.0219996362500003e+00, 6.4234496686880004e+02 },
    { 1.1924000000000000e-07, 2.7241103690399999e+00, 3.8012776796000003e+02 },
    { 1.1557000000000000e-07, 5.0381300412799996e+00, 2.8000393915799998e+02 },
    { 1.1401000000000000e-07, 4.6018712838500004e+00, 1.7224529849340001e+02 },
    { 1.1289000000000000e-07, 3.2473206267000001e+00, 3.8994962794650002e+02 },
    { 1.1265000000000001e-07, 3.7431351394900001e+00, 4.0302231763990000e+02 },
    { 1.1258000000000001e-07, 6.0066407105999997e-01, 5.2650957135689998e+02 },
    { 1.1233000000000000e-07, 4.5187214098600004e+00, 6.4086049416050002e+02 },
    { 1.1220000000000000e-07, 5.6078139689200004e+00, 4.5368552624990002e+02 },
    { 1.1206000000000000e-07, 2.5505916719299999e+00, 7.1467888488129995e+02 },
    { 1.1199000000000000e-07, 5.0663512405200004e+00, 3.9510562148700001e+02 },
    { 1.1094000000000000e-07, 3.7749436983800000e+00, 1.1976754877499999e+02 },
    { 1.1067000000000000e-07, 3.4379915485199999e+00, 2.4012579838100001e+02 },
    { 1.1058000000000000e-07, 3.5405181917000000e-01, 1.7526107818300002e+01 },
    { 1.1002000000000000e-07, 3.4533540557999998e-01, 3.5452490529440001e+02 },
    { 1.0935000000000000e-07, 2.9947488397900002e+00, 1.9122455111200001e+01 },
    { 1.0907999999999999e-07, 4.5039280809999998e+00, 1.8647239249500001e+02 },
    { 1.0891000000000000e-07, 1.0742508175100001e+00, 1.8841241239379999e+03 },
    { 1.0879000000000000e-07, 2.8497692680200002e+00, 8.2005928096030004e+02 },
    { 1.0842000000000000e-07, 9.7259949348999997e-01, 6.3994547042379997e+02 },
    { 1.0842000000000000e-07, 4.0579964946100002e+00, 6.3984910220419999e+02 },
    { 1.0715000000000000e-07, 4.3702763275900001e+00, 4.9575071515079998e+02 },
    { 1.0570000000000000e-07, 5.5555760559899996e+00, 6.0676018552230005e+02 },
    { 1.0518000000000001e-07, 2.6434945016900002e+00, 9.7676148247200004e+01 },
    { 1.0505000000000000e-07, 2.0513029140000000e-01, 2.0629742302180000e+02 },
    { 1.0478000000000000e-07, 3.7560099716000001e-01, 1.8972322220190000e+02 },
    { 1.0384999999999999e-07, 4.7526654224899998e+00, 1.9683676920270000e+02 },
    { 1.0379000000000000e-07, 5.2561803342899998e+00, 2.3842887735159999e+02 },
    { 1.0314999999999999e-07, 5.0474815237200001e+00, 3.3701042450032000e+03 },
    { 1.0267000000000001e-07, 4.2375685550400002e+00, 7.4692221379570003e+02 },
    { 1.0254000000000000e-07, 1.8412125850000000e-01, 2.0199276274319999e+02 },
    { 1.0249000000000000e-07, 1.2549450414500001e+00, 1.0728555991259999e+02 },
    { 1.0066000000000000e-07, 5.2081908270500001e+00, 4.2388987789030000e+02 },
    { 1.0066000000000000e-07, 1.3209224000799999e+00, 4.2930650386169998e+02 },
    { 9.9439999999999996e-08, 1.1090975687900000e+00, 4.7694263193399998e+01 },
    { 9.9200000000000002e-08, 2.8888805593000000e-01, 2.0470107572890001e+02 },
    { 9.8889999999999994e-08, 2.7959183672800001e+00, 2.9204947528600002e+01 },
    { 9.8630000000000004e-08, 1.0049850141200001e+00, 3.7619561469690001e+02 },
    { 9.8620000000000006e-08, 1.7872293404799999e+00, 3.3137100791700000e+01 },
    { 9.7349999999999995e-08, 4.7297282444800004e+00, 4.3366355376700000e+02 },
    { 9.6579999999999995e-08, 5.3752363956400000e+00, 3.8038840039090002e+02 },
    { 9.5939999999999998e-08, 1.6346615581899999e+00, 4.3375992198659998e+02 },
    { 9.5900000000000005e-08, 5.1909773983400003e+00, 3.4373983746139999e+02 },
    { 9.5459999999999996e-08, 7.5076951952000004e-01, 2.1914061805160000e+02 },
    { 9.5329999999999994e-08, 2.2910782719500000e+00, 2.3687496867409999e+02 },
    { 9.5259999999999994e-08, 3.3128897508700001e+00, 3.8405992122309999e+02 },
    { 9.4920000000000006e-08, 3.1148699345000002e+00, 1.0601353552540000e+02 },
    { 9.4360000000000006e-08, 1.1256700622200000e+00, 1.4930936680660000e+03 },
    { 9.4160000000000004e-08, 4.4885422243199997e+00, 4.1800017116689997e+02 },
    { 9.3740000000000004e-08, 1.7976205677699999e+00, 2.0607367385260000e+02 },
    { 9.3699999999999999e-08, 3.2785537367100002e+00, 2.2976142167329999e+02 },
    { 9.3550000000000001e-08, 1.5334738387600000e+00, 1.9417664032680000e+02 },
    { 9.2449999999999997e-08, 2.2155008098900000e+00, 4.0695447090300001e+02 },
    { 9.2280000000000003e-08, 2.2253166768999999e-01, 5.9465070367540000e+02 },
    { 9.1500000000000005e-08, 9.1468054201000004e-01, 5.6214674233009998e+02 },
    { 9.1199999999999996e-08, 4.5710225513099996e+00, 1.1291463420510000e+02 },
    { 9.1119999999999998e-08, 4.3497245521499996e+00, 1.6772244439930000e+02 },
    { 8.9589999999999997e-08, 5.6594132416600003e+00, 1.8743560034149999e+02 },
    { 8.9239999999999997e-08, 6.0388366536399998e+00, 1.9643719973000000e+01 },
    { 8.9029999999999997e-08, 8.6621106610999998e-01, 6.4945851386959998e+02 },
    { 8.8769999999999994e-08, 3.0514122646600002e+00, 1.9054647649404001e+03 },
    { 8.8759999999999995e-08, 1.4512444291700000e+00, 1.9040454575810000e+02 },
    { 8.8409999999999996e-08, 5.0392103517999998e-01, 1.8023719907218001e+03 },
    { 8.7330000000000002e-08, 4.6703792487499998e+00, 1.5763995198190000e+02 },
    { 8.7270000000000000e-08, 2.6584404497500000e+00, 1.9662432088160000e+02 },
    { 8.6319999999999995e-08, 5.9471629722400001e+00, 7.3980866679489998e+02 },
    { 8.5829999999999995e-08, 1.3818329997100001e+00, 2.9823262239190001e+02 },
    { 8.4909999999999997e-08, 6.0966643145299999e+00, 3.1568051495651998e+03 },
    { 8.4330000000000001e-08, 1.8157003040299999e+00, 4.2675824957039998e+02 },
    { 8.4330000000000001e-08, 4.7134129230999999e+00, 4.2643813218160000e+02 },
    { 8.4209999999999998e-08, 2.6546457460599999e+00, 5.0286426215159997e+02 },
    { 8.3890000000000005e-08, 3.0076958466999998e-01, 2.3242155054919999e+02 },
    { 8.3799999999999996e-08, 1.3066425659800001e+00, 8.5712853501510006e+02 },
    { 8.3659999999999996e-08, 2.8942065447399998e+00, 8.9759452094300002e+01 },
    { 8.3169999999999996e-08, 3.7083043397700002e+00, 6.6123792731640003e+02 },
    { 8.2459999999999999e-08, 7.1370891616999999e-01, 1.9438908864790000e+02 },
    { 8.0960000000000005e-08, 1.1310116459999999e+00, 8.2474219374879999e+02 },
    { 8.0859999999999997e-08, 3.3418434543800002e+00, 3.8554439393140001e+02 },
    { 8.0659999999999996e-08, 2.3398042751700001e+00, 8.4933526953900000e+01 },
    { 8.0260000000000005e-08, 2.6256400161800002e+00, 3.1368355667089997e+02 },
    { 7.9980000000000005e-08, 4.1981278448200001e+00, 2.5583736509089999e+02 },
    { 7.9780000000000004e-08, 1.6824569369100000e+00, 3.9435486196160002e+02 },
    { 7.9759999999999994e-08, 6.2761408611800000e+00, 2.8454188003199999e+01 },
    { 7.8209999999999997e-08, 5.1874169585900001e+00, 1.7665053250849999e+02 },
    { 7.7779999999999999e-08, 1.4362040257599999e+00, 2.5272794265500000e+01 },
    { 7.7669999999999993e-08, 3.4413205399400000e+00, 1.9589060769869999e+02 },
    { 7.7560000000000001e-08, 1.9340526653100001e+00, 5.5659143456099997e+01 },
    { 7.7120000000000005e-08, 5.9806089059399996e+00, 2.4138150890326001e+03 },
    { 7.6949999999999998e-08, 1.0099567587400000e+00, 2.0745757282439999e+02 },
    { 7.6630000000000005e-08, 4.5957567731899998e+00, 7.2483081326790000e+02 },
    { 7.6490000000000005e-08, 5.2410936804999997e+00, 1.3828873468465999e+03 },
    { 7.5699999999999996e-08, 1.4579740721600001e+00, 9.9022940591439999e+02 },
    { 7.5639999999999994e-08, 2.7244464937999999e+00, 3.2595309721220002e+02 },
    { 7.5440000000000006e-08, 2.3966013284300001e+00, 2.3916259053450000e+02 },
    { 7.5390000000000002e-08, 5.2942753585200002e+00, 2.5602862665600000e+01 },
    { 7.5120000000000000e-08, 5.5740369507000000e+00, 2.5129781913599999e+01 },
    { 7.4499999999999999e-08, 6.0075865484800000e+00, 3.6679444583570000e+02 },
    { 7.4449999999999995e-08, 2.9603600768999999e-01, 2.3619364511789999e+02 },
    { 7.4180000000000006e-08, 4.5399267392600002e+00, 7.0066423920080001e+02 },
    { 7.3580000000000001e-08, 4.8917643147999996e+00, 4.4923210812500002e+02 },
    { 7.3269999999999994e-08, 1.0343250321499999e+00, 2.3220910222809999e+02 },
    { 7.2979999999999996e-08, 1.7912713995999999e-01, 4.2532616648880003e+02 },
    { 7.2979999999999996e-08, 6.6800779990000003e-02, 4.2787021526320001e+02 },
    { 7.2680000000000000e-08, 5.5017036505000005e-01, 1.2836556848410001e+02 },
    { 7.1690000000000002e-08, 7.8094936112000002e-01, 2.9126208774300000e+02 },
    { 7.0949999999999997e-08, 4.0759092049200003e+00, 3.5834033404411998e+03 },
    { 7.0440000000000001e-08, 7.4967263787000005e-01, 1.4148644228730001e+02 },
    { 7.0379999999999999e-08, 2.3642889750800000e+00, 1.1072758608120000e+02 },
    { 7.0370000000000001e-08, 5.8991876375399999e+00, 4.1896337901340002e+02 },
    { 6.9650000000000005e-08, 4.2059301280500003e+00, 9.0981873305459999e+02 },
    { 6.9349999999999996e-08, 1.6886662274899999e+00, 1.2869081196288000e+03 },
    { 6.9279999999999996e-08, 3.9261887908999998e-01, 1.4393412284210001e+02 },
    { 6.9219999999999994e-08, 5.8614855598500002e+00, 5.1153171782990000e+02 },
    { 6.9009999999999994e-08, 2.9395395665800002e+00, 2.4994765836750000e+02 },
    { 6.8970000000000002e-08, 1.1175007033400000e+00, 3.1071461125430000e+02 },
    { 6.8410000000000002e-08, 6.0546192730600001e+00, 3.9739324334740002e+02 },
    { 6.7200000000000006e-08, 1.6601582149000000e+00, 1.3533610313299999e+02 },
    { 6.6889999999999999e-08, 5.6758779058900002e+00, 1.4096517742550000e+02 },
    { 6.6880000000000001e-08, 2.7075829679600001e+00, 3.1394418910180002e+02 },
    { 6.6609999999999999e-08, 3.4525813844700002e+00, 9.5388526386799995e+01 },
    { 6.6489999999999995e-08, 5.4498691118800000e+00, 2.3107223148140001e+03 },
    { 6.6279999999999995e-08, 2.3104314100100001e+00, 6.1144309831080000e+02 },
    { 6.6240000000000003e-08, 3.3847840471000001e+00, 2.6895823889410002e+02 },
    { 6.5719999999999996e-08, 4.6207708292999999e-01, 1.1302313754934000e+03 },
    { 6.5330000000000003e-08, 1.4180633629799999e+00, 7.2333918012500007e+01 },
    { 6.5079999999999998e-08, 5.5152963915899997e+00, 5.6660016045500004e+02 },
    { 6.4620000000000006e-08, 4.1139531150800002e+00, 3.6997583957339998e+02 },
    { 6.4270000000000006e-08, 5.7851678564500002e+00, 3.2892204262880000e+02 },
    { 6.4160000000000000e-08, 4.2307449455000001e-01, 6.9364972595899999e+01 },
    { 6.4140000000000004e-08, 1.1822184892000001e+00, 6.2491943278700001e+02 },
    { 6.3899999999999996e-08, 2.8282010515899998e+00, 2.6087903141574199e+04 },
    { 6.3759999999999997e-08, 3.0496904596699999e+00, 1.1791056905120000e+02 },
    { 6.3339999999999997e-08, 1.8495194994499999e+00, 1.0968505635760000e+02 },
    { 6.3049999999999999e-08, 5.0764438040800002e+00, 3.4803105662225998e+03 },
    { 6.2349999999999999e-08, 2.6489029998300002e+00, 6.1632141307790005e+02 },
    { 6.1690000000000005e-08, 2.8811624550000001e-02, 1.0457724692690000e+02 },
    { 6.1469999999999994e-08, 5.3881799045300003e+00, 6.3937602145220001e+02 },
    { 6.1010000000000002e-08, 5.2713056948199997e+00, 5.5137878594299998e+01 },
    { 6.0850000000000006e-08, 5.3017977139700001e+00, 4.3319047301500001e+02 },
    { 6.0819999999999998e-08, 5.9052490381900000e+00, 6.4041855117579996e+02 },
    { 6.0570000000000006e-08, 2.8582653073299999e+00, 8.5637777548969996e+02 },
    { 5.9590000000000000e-08, 8.1674403093000003e-01, 2.2005159935945999e+03 },
    { 5.9319999999999998e-08, 4.4835034837299999e+00, 1.2099138905240000e+02 },
    { 5.9050000000000003e-08, 2.1120305084700002e+00, 9.6962247809489998e+02 },
    { 5.8740000000000002e-08, 1.4496939666199999e+00, 3.9346109008430000e+02 },
    { 5.8379999999999997e-08, 4.6994514489699997e+00, 5.0531194270639998e+02 },
    { 5.8320000000000002e-08, 3.2357488024199998e+00, 6.3523141986799999e+02 },
    { 5.8190000000000000e-08, 4.8038871377600003e+00, 2.1595922431389999e+02 },
    { 5.8190000000000000e-08, 3.2273316824200000e+00, 2.1063896656209999e+02 },
    { 5.8180000000000002e-08, 3.3280800634299998e+00, 4.3289029178299998e+01 },
    { 5.8080000000000001e-08, 3.2485416032600001e+00, 2.5435289238260000e+02 },
    { 5.7730000000000001e-08, 5.3490987071199996e+00, 9.7415515816300001e+01 },
    { 5.7709999999999999e-08, 4.7657414506700002e+00, 1.6454105066160000e+02 },
    { 5.7660000000000001e-08, 4.8460184938799999e+00, 8.8114920691600005e+01 },
    { 5.7509999999999997e-08, 1.4238078677999999e-01, 3.0508553696180002e+02 },
    { 5.7480000000000003e-08, 4.2984212350000002e-02, 8.4363515419639998e+02 },
    { 5.7019999999999997e-08, 4.8503886441199997e+00, 6.1069233878540001e+02 },
    { 5.6769999999999998e-08, 2.2462654591000000e-01, 2.1840690486870000e+02 },
    { 5.6769999999999998e-08, 1.5234069670899999e+00, 2.0819128600729999e+02 },
    { 5.6370000000000001e-08, 5.4236451798700003e+00, 4.9989531673499999e+02 },
    { 5.5910000000000002e-08, 9.7716211165000000e-01, 2.8511174858869998e+02 },
    { 5.5579999999999999e-08, 3.3718806428499999e+00, 4.9330303459599997e+02 },
    { 5.5500000000000001e-08, 2.1680369235899999e+00, 3.0608259223474001e+03 },
    { 5.5159999999999999e-08, 1.4615818799999999e-03, 6.3182053146670000e+02 },
    { 5.5159999999999999e-08, 4.2992082819300004e+00, 4.5516999895819998e+02 },
    { 5.4790000000000003e-08, 4.1040066045600003e+00, 5.5103160609700001e+02 },
    { 5.4569999999999998e-08, 6.2936317015999999e-01, 4.8173606947029998e+02 },
    { 5.4189999999999998e-08, 6.0345604452800004e+00, 3.2058465535059997e+02 },
    { 5.3850000000000002e-08, 3.0815887732000002e-01, 1.0333583763982999e+03 },
    { 5.3279999999999998e-08, 6.2350075986500002e+00, 5.8106824010899999e+01 },
    { 5.3209999999999998e-08, 1.2369593259300000e+00, 5.9735901666109999e+02 },
    { 5.2889999999999999e-08, 1.7271596636600000e+00, 4.8470501488690002e+02 },
    { 5.2600000000000001e-08, 2.1562284075000000e+00, 1.1562294719080001e+02 },
    { 5.2329999999999999e-08, 4.0052954089200004e+00, 9.5209637027660006e+02 },
    { 5.1539999999999997e-08, 4.7238526865499999e+00, 8.0672595883600002e+02 },
    { 5.1289999999999998e-08, 3.8748590111499999e+00, 2.0762183703570000e+02 },
    { 5.1289999999999998e-08, 4.1563598090299996e+00, 2.1897635384029999e+02 },
    { 5.1049999999999997e-08, 3.4379119967600000e+00, 8.8796244247800004e+01 },
    { 5.0939999999999998e-08, 3.0474548229299998e+00, 2.2218566345969998e+03 },
    { 5.0750000000000001e-08, 3.4142558372999998e+00, 3.1787634236489998e+02 },
    { 5.0619999999999999e-08, 2.7156281585099999e+00, 2.0055647414469999e+02 },
    { 5.0549999999999999e-08, 1.4881200768000000e+00, 5.3287235883230005e+02 },
    { 5.0540000000000001e-08, 1.3008149475399999e+00, 4.3128970483899998e+01 },
    { 5.0349999999999998e-08, 5.2751555113400004e+00, 6.5805653357870005e+02 },
    { 5.0320000000000003e-08, 4.7839435664699996e+00, 3.6864961146597998e+03 },
    { 5.0290000000000002e-08, 4.8115733292499998e+00, 1.9725746032800001e+02 },
    { 5.0209999999999998e-08, 4.7766325341400000e+00, 1.0872184851110001e+02 },
    { 5.0099999999999999e-08, 1.8791923276700000e+00, 2.3249494088155998e+03 },
    { 4.9780000000000000e-08, 3.2101789006099999e+00, 1.7808682110699999e+02 },
    { 4.9570000000000000e-08, 2.4957521995400000e+00, 5.2698265210889997e+02 },
    { 4.9310000000000003e-08, 5.4126655839800000e+00, 7.7130123618389996e+02 },
    { 4.9170000000000003e-08, 5.3349675573099997e+00, 7.6981676347560006e+02 },
    { 4.9010000000000000e-08, 1.9419248795199999e+00, 3.1931263096340001e+02 },
    { 4.8860000000000002e-08, 3.1679823076400000e+00, 3.0609692891809999e+02 },
    { 4.8739999999999998e-08, 5.9494711255199997e+00, 6.0447256366190004e+02 },
    { 4.8730000000000000e-08, 3.8671434121299999e+00, 7.6025553592000006e+02 },
    { 4.8650000000000002e-08, 3.1098684161599999e+00, 1.2450285119020000e+02 },
    { 4.8450000000000000e-08, 2.2769778216300001e+00, 2.2604171673130000e+02 },
    { 4.8119999999999997e-08, 3.0486160057600000e+00, 3.2769820235140003e+02 },
    { 4.7829999999999999e-08, 2.3167756143000000e+00, 5.0456118318099999e+02 },
    { 4.7699999999999997e-08, 1.6232275229599999e+00, 9.9171387862270001e+02 },
    { 4.7290000000000002e-08, 2.7525880017000000e+00, 1.8409414790939999e+02 },
    { 4.7050000000000001e-08, 3.7224856879299999e+00, 4.2490126984659997e+02 },
    { 4.6929999999999997e-08, 5.8370119736000003e-01, 1.1143016149680000e+02 },
    { 4.6639999999999999e-08, 4.2053608853700002e+00, 3.1883955021140002e+02 },
    { 4.6329999999999999e-08, 4.4190283437900000e+00, 2.6826702943000001e+01 },
    { 4.6280000000000001e-08, 1.0376305610100001e+00, 7.4527768239299996e+02 },
    { 4.6129999999999997e-08, 1.0623373342000000e-01, 7.7448262992160005e+02 },
    { 4.6059999999999997e-08, 3.9010137294999998e+00, 6.3426821202149995e+02 },
    { 4.5610000000000003e-08, 6.0972538967299998e+00, 4.9155792945680003e+02 },
    { 4.5499999999999997e-08, 4.6405961741099997e+00, 9.5857677783099996e+02 },
    { 4.5349999999999999e-08, 3.2221946759900000e+00, 2.2934073054800001e+02 },
    { 4.5109999999999998e-08, 3.8744538075100001e+00, 7.0756533788050001e+02 },
    { 4.5090000000000002e-08, 2.9424181057099998e+00, 2.7140591944890002e+02 },
    { 4.4920000000000001e-08, 3.6984177539000002e-01, 7.3269511979410004e+02 },
    { 4.4850000000000001e-08, 3.8024738845899999e+00, 6.2322251175760005e+02 },
    { 4.4640000000000001e-08, 9.6353376399999999e-03, 6.3697652500720005e+02 },
    { 4.4540000000000000e-08, 6.2035547449899999e+00, 1.9848130060540001e+02 },
    { 4.4510000000000000e-08, 1.2701359473800000e+00, 1.1312708252620000e+02 },
    { 4.4500000000000001e-08, 2.8288566659100001e+00, 6.2960234557549995e+02 },
    { 4.3779999999999999e-08, 5.8269212551500003e+00, 3.4731970192218000e+03 },
    { 4.3719999999999997e-08, 2.7222653532000001e+00, 2.2507850888479999e+02 },
    { 4.3499999999999999e-08, 4.0771687427499996e+00, 8.4066620877979994e+02 },
    { 4.3390000000000000e-08, 3.4836109176300001e+00, 6.0887779767699999e+02 },
    { 4.3270000000000003e-08, 5.7019584378700001e+00, 2.7302069586891998e+03 },
    { 4.3210000000000001e-08, 2.2863802448600001e+00, 2.0151968199120000e+02 },
    { 4.3130000000000003e-08, 3.8843631643799998e+00, 4.3993151300030001e+02 },
    { 4.2920000000000003e-08, 4.2310834999000004e+00, 1.8016199464630000e+02 },
    { 4.2890000000000002e-08, 5.6065340002999997e-01, 1.8769623277240001e+02 },
    { 4.1929999999999999e-08, 2.6989832197300001e+00, 7.7523338944700004e+02 },
    { 4.1729999999999997e-08, 2.8763595583000001e-01, 2.1101147357759999e+02 },
    { 4.1729999999999997e-08, 1.4603975571700001e+00, 2.1558671729840000e+02 },
    { 4.1649999999999999e-08, 3.7588049347800001e+00, 6.2759623029905997e+03 },
    { 4.1320000000000002e-08, 1.5196778100000000e-01, 4.3423300273860002e+02 },
    { 4.1159999999999999e-08, 5.2488100528399997e+00, 1.2050126195790000e+02 },
    { 4.1110000000000002e-08, 3.9936357638500000e+00, 5.6193429400900004e+02 },
    { 4.1059999999999998e-08, 2.0343070819000002e+00, 6.8547393735269998e+02 },
    { 4.0770000000000000e-08, 3.4671568883299999e+00, 6.5019222705250002e+02 },
    { 4.0620000000000002e-08, 2.1817718478499999e+00, 9.2307706385600000e+01 },
    { 4.0600000000000000e-08, 1.8276643368200001e+00, 2.2811689027060001e+02 },
    { 4.0490000000000001e-08, 2.5484253209499999e+00, 3.7242352012819998e+02 },
    { 4.0410000000000003e-08, 5.3698247052800001e+00, 3.3120966448920001e+02 },
    { 4.0399999999999998e-08, 1.3382773490700000e+00, 4.9721252622900003e+01 },
    { 4.0340000000000003e-08, 3.4574809812699998e+00, 2.6205714021440002e+02 },
    { 4.0189999999999998e-08, 8.7322430940999995e-01, 1.1411340634054000e+03 },
    { 4.0079999999999999e-08, 3.4416873822400000e+00, 1.0444040766621999e+03 },
    { 3.9920000000000003e-08, 6.0635929849599997e+00, 6.6820846196529999e+02 },
    { 3.9920000000000003e-08, 4.8210414614700001e+00, 2.4851136976900000e+02 },
    { 3.9729999999999999e-08, 5.1794118872099997e+00, 1.0206171999210201e+04 },
    { 3.9690000000000000e-08, 3.9996413726100002e+00, 3.9828701522469999e+02 },
    { 3.9529999999999997e-08, 6.1602279734699996e+00, 1.0186893394120000e+02 },
    { 3.9480000000000000e-08, 3.1743797635300002e+00, 7.1812653150700001e+01 },
    { 3.9029999999999999e-08, 6.4285376703999997e-01, 3.7890392768260000e+02 },
    { 3.8940000000000003e-08, 1.2453221944199999e+00, 2.6843697403229999e+02 },
    { 3.8799999999999997e-08, 7.5632369741000005e-01, 5.4153981451059997e+02 },
    { 3.8799999999999997e-08, 4.8275422266700003e+00, 3.1735507750310001e+02 },
    { 3.8659999999999997e-08, 5.4436408108599998e+00, 3.7872403206900003e+01 },
    { 3.8600000000000002e-08, 3.2859647337900002e+00, 5.1242548970719997e+02 },
    { 3.8549999999999998e-08, 3.6556553833600001e+00, 4.2537435059860002e+02 },
    { 3.8549999999999998e-08, 2.8734578437699998e+00, 4.2782203115340002e+02 },
    { 3.8509999999999999e-08, 3.4483701402200002e+00, 3.0560680182359999e+02 },
    { 3.8280000000000003e-08, 6.1046859893600001e+00, 1.1494162363460001e+02 },
    { 3.8040000000000002e-08, 5.9412609161500001e+00, 8.0198224538700003e+01 },
    { 3.7879999999999999e-08, 2.5655551914400001e+00, 2.5854567807659998e+02 },
    { 3.7879999999999999e-08, 4.1613967901999999e-01, 6.7214061522839995e+02 },
    { 3.7870000000000001e-08, 4.2860012034199997e+00, 6.5242745928620002e+02 },
    { 3.7870000000000001e-08, 5.7268078376000000e+00, 1.0562005364515001e+03 },
    { 3.7830000000000002e-08, 3.1601328807000000e+00, 7.7229279122099996e+01 },
    { 3.7700000000000000e-08, 3.2287593124599998e+00, 2.1467169356170001e+02 },
    { 3.7700000000000000e-08, 4.8024595077200001e+00, 2.1192649731430001e+02 },
    { 3.7520000000000001e-08, 1.3127521300900000e+00, 1.8727554164710000e+02 },
    { 3.7499999999999998e-08, 3.9166189765500001e+00, 1.4541859555040000e+02 },
    { 3.7370000000000003e-08, 2.6727034607000000e-01, 2.5863495096499999e+01 },
    { 3.7319999999999999e-08, 1.8740638441299999e+00, 4.8758044776399998e+01 },
    { 3.7280000000000000e-08, 5.0614521726500001e+00, 4.0747573576479999e+02 },
    { 3.6909999999999998e-08, 5.1995701168100004e+00, 1.9428851491140000e+02 },
    { 3.6489999999999998e-08, 1.9674639618700001e+00, 4.4412429869430002e+02 },
    { 3.6419999999999998e-08, 3.5969305989000002e+00, 3.4863519857099999e+02 },
    { 3.6389999999999997e-08, 4.6550139833499999e+00, 6.3448066034260000e+02 },
    { 3.6169999999999999e-08, 1.0965520379300000e+00, 3.1542866181009998e+02 },
    { 3.6130000000000000e-08, 1.4690318882500000e+00, 5.1939602435610004e+02 },
    { 3.5719999999999998e-08, 6.0835076013800000e+00, 8.5171190904369996e+02 },
    { 3.5660000000000003e-08, 1.2244612880600001e+00, 2.3890195810360001e+02 },
    { 3.5409999999999997e-08, 5.3353274088000002e-01, 2.3932264922889999e+02 },
    { 3.5390000000000001e-08, 5.7112707037800003e+00, 2.3294281541100000e+02 },
    { 3.5380000000000003e-08, 1.3155294494400001e+00, 3.1644005376640001e+02 },
    { 3.5269999999999998e-08, 3.5907562410299998e+00, 4.9596316347189997e+02 },
    { 3.5170000000000003e-08, 4.0385279506399998e+00, 3.8997952100978000e+03 },
    { 3.5059999999999998e-08, 4.4045461435200002e+00, 3.1634368554679997e+02 },
    { 3.5000000000000002e-08, 2.8134907207600000e+00, 7.8946048344860003e+02 },
    { 3.4709999999999998e-08, 9.5985137600000003e-01, 1.1191856752295000e+03 },
    { 3.4669999999999999e-08, 2.7527214782699998e+00, 4.2829511190540001e+02 },
    { 3.4620000000000002e-08, 4.8766538475800001e+00, 1.8254023923189999e+02 },
    { 3.4079999999999998e-08, 4.5776763876700004e+00, 4.9893210888850001e+02 },
    { 3.4049999999999997e-08, 3.0824367777299999e+00, 1.6708930495289999e+02 },
    { 3.3769999999999997e-08, 1.4104942976999999e+00, 7.6588461021249998e+02 },
    { 3.3729999999999998e-08, 6.2251098799599998e+00, 1.0212956637210000e+02 },
    { 3.3580000000000000e-08, 5.0356104951600003e+00, 1.6560483224460000e+02 },
    { 3.3470000000000001e-08, 1.4036401631000001e-01, 1.0945556169400000e+02 },
    { 3.3209999999999998e-08, 6.2461208452000005e-01, 1.2273649419160000e+02 },
    { 3.3169999999999999e-08, 1.0550853542500001e+00, 7.2164941953020002e+02 },
    { 3.3120000000000001e-08, 6.0497070096599996e+00, 6.9110301164520001e+02 },
    { 3.3099999999999999e-08, 5.3771489354000002e-01, 2.5169078632512001e+03 },
    { 3.3010000000000002e-08, 2.4629879672700001e+00, 2.8266406803389998e+02 },
    { 3.2819999999999999e-08, 6.0551273936300003e+00, 2.1176223310300000e+02 },
    { 3.2819999999999999e-08, 1.9760914265500000e+00, 2.1483595777299999e+02 },
    { 3.2770000000000002e-08, 2.8316491719200001e+00, 2.3230967596459999e+02 },
    { 3.2700000000000002e-08, 1.1350105903000001e+00, 9.2404582705619998e+02 },
    { 3.2609999999999999e-08, 6.1573250931199999e+00, 4.5342489381899998e+02 },
    { 3.2590000000000003e-08, 4.7399380346299997e+00, 3.7967024358792000e+03 },
    { 3.2350000000000002e-08, 2.5804921903300002e+00, 3.3848327018440000e+02 },
    { 3.2310000000000003e-08, 2.6344372350700000e+00, 1.5961864422845999e+03 },
    { 3.2189999999999999e-08, 4.4066925886300004e+00, 6.5716276170139997e+02 },
    { 3.2119999999999999e-08, 4.0223202974200003e+00, 3.0683064210100002e+02 },
    { 3.1940000000000000e-08, 2.6003120904999999e-01, 9.1396333463880001e+02 },
    { 3.1930000000000002e-08, 2.6678800345500000e+00, 2.5382485042536000e+03 },
    { 3.1779999999999997e-08, 4.0465336810399997e+00, 1.1095708074480000e+02 },
    { 3.1650000000000002e-08, 4.2796343489000002e+00, 1.0633140834523001e+03 },
    { 3.1419999999999999e-08, 3.7464733722100001e+00, 1.3044074202339999e+02 },
    { 3.1249999999999999e-08, 3.7984446695099998e+00, 5.3197858695499997e+02 },
    { 3.1009999999999998e-08, 4.0792170155100003e+00, 1.0405598206510000e+02 },
    { 3.0860000000000000e-08, 5.8439609304099998e+00, 4.4386366626339998e+02 },
    { 3.0559999999999997e-08, 3.8425839722199999e+00, 1.7957129381530001e+02 },
    { 3.0540000000000001e-08, 4.3154116704699996e+00, 6.9970103135429997e+02 },
    { 3.0330000000000001e-08, 1.4545863260300000e+00, 1.8802630117250001e+02 },
    { 3.0320000000000003e-08, 3.5222572678500002e+00, 9.3531546663000000e+01 },
    { 3.0199999999999999e-08, 5.0389037638399996e+00, 1.5126011816400000e+02 },
    { 3.0080000000000002e-08, 2.5311755453000000e+00, 4.3446249740219997e+02 },
    { 2.9980000000000001e-08, 2.9591968693299999e+00, 3.2824071907259997e+02 },
    { 2.9940000000000002e-08, 4.3874501249800000e+00, 9.1786441523799994e+01 },
    { 2.9859999999999997e-08, 1.7278687010000002e-02, 4.2325821331800000e+01 },
    { 2.9700000000000001e-08, 4.6278023778000001e+00, 4.0028879843164000e+03 },
    { 2.9519999999999999e-08, 7.4298822983000001e-01, 6.4552636060650002e+02 },
    { 2.9220000000000000e-08, 3.4333931533199999e+00, 3.3514181775230003e+02 },
    { 2.9169999999999999e-08, 1.8633195093399999e+00, 3.5723321828010000e+02 },
    { 2.9060000000000000e-08, 5.3661463177600002e+00, 5.5013783421970004e+02 },
    { 2.9049999999999999e-08, 5.8902203587299997e+00, 3.5164090221199999e+01 },
    { 2.9000000000000002e-08, 1.1951252426000001e+00, 8.4290144101349995e+02 },
    { 2.8950000000000001e-08, 5.5016420855300003e+00, 9.4524345570670005e+02 },
    { 2.8900000000000001e-08, 4.2347728109100000e+00, 9.2797833480099996e+01 },
    { 2.8819999999999999e-08, 3.0788122684200001e+00, 1.2353964334370001e+02 },
    { 2.8460000000000001e-08, 5.9348034929000004e-01, 1.0314095832840000e+02 },
    { 2.8400000000000000e-08, 5.6378766129600004e+00, 3.3683873878169999e+02 },
    { 2.8330000000000000e-08, 4.4907622267400003e+00, 7.8234693644779998e+02 },
    { 2.8110000000000002e-08, 6.1021905162400003e+00, 6.4138175902230000e+02 },
    { 2.8100000000000000e-08, 4.4491999855200000e+00, 1.0412226829245001e+03 },
    { 2.8080000000000001e-08, 2.3033031948899998e+00, 8.3769726336320002e+02 },
    { 2.8040000000000002e-08, 2.8311320807599998e+00, 4.2516190227750002e+02 },
    { 2.8040000000000002e-08, 3.6979811463700001e+00, 4.2803447947450002e+02 },
    { 2.7739999999999999e-08, 2.0259766584599999e+00, 1.6805251279940001e+02 },
    { 2.7579999999999999e-08, 3.6782578885000001e+00, 1.0304459010879999e+02 },
    { 2.7510000000000000e-08, 2.0034349205000002e+00, 3.2202094394910000e+02 },
    { 2.7280000000000000e-08, 1.1360581258400000e+00, 3.5426427286350003e+02 },
    { 2.7260000000000001e-08, 1.9495873268499999e+00, 9.5310776223289997e+02 },
    { 2.7220000000000002e-08, 3.3833300116999998e-01, 6.4409007200799999e+02 },
    { 2.7150000000000002e-08, 1.4074737307500000e+00, 6.2025356634100001e+02 },
    { 2.7120000000000001e-08, 3.2676686151899998e+00, 6.3129926660490003e+02 },
    { 2.7070000000000000e-08, 1.1590621753999999e-01, 3.1097524368519998e+02 },
    { 2.7010000000000002e-08, 3.7394716454000001e+00, 3.0061086697750000e+02 },
    { 2.7000000000000000e-08, 2.1228249444100000e+00, 9.2783496796739996e+02 },
    { 2.6989999999999999e-08, 4.0977036122300001e+00, 1.4169889060840001e+02 },
    { 2.6930000000000000e-08, 4.3580664415200001e+00, 4.3296097835140000e+02 },
    { 2.6890000000000001e-08, 1.0721977806999999e-01, 1.2566151699982800e+04 },
    { 2.6860000000000000e-08, 4.8716434489099996e+00, 6.4281804762080003e+02 },
    { 2.6670000000000000e-08, 2.3489918221699999e+00, 1.0807225711916001e+03 },
    { 2.6600000000000000e-08, 1.5824586002500001e+00, 4.4327296543239999e+02 },
    { 2.6550000000000000e-08, 2.7142442432400000e+00, 1.2518417474640000e+02 },
    { 2.6550000000000000e-08, 3.7358282521600001e+00, 4.4061283655649999e+02 },
    { 2.6490000000000001e-08, 3.4131232912399998e+00, 2.4405795164410000e+02 },
    { 2.6410000000000000e-08, 3.3519682114800000e+00, 7.3905790726949999e+02 },
    { 2.6340000000000000e-08, 4.9836678917299997e+00, 1.0335340664950000e+02 },
    { 2.6099999999999999e-08, 4.7725798535399999e+00, 1.1939672737968001e+03 },
    { 2.6060000000000000e-08, 1.4368418549000000e-01, 1.0490869894507000e+03 },
    { 2.6029999999999999e-08, 3.1489275126399998e+00, 5.0849333644410001e+02 },
    { 2.5909999999999998e-08, 5.3503321369899997e+00, 1.7017012495410000e+02 },
    { 2.5889999999999999e-08, 1.1614919117300000e+00, 7.1600204829600003e+01 },
    { 2.5810000000000001e-08, 1.0593909995099999e+00, 1.5896436599579999e+02 },
    { 2.5640000000000000e-08, 1.7291971290199999e+00, 4.2749196275330002e+02 },
    { 2.5640000000000000e-08, 4.7999160981099998e+00, 4.2570441899870002e+02 },
    { 2.5590000000000000e-08, 3.7936808829299999e+00, 6.3914652678860000e+02 },
    { 2.5559999999999999e-08, 1.6606741054800001e+00, 3.3772177920039999e+03 },
    { 2.5550000000000000e-08, 5.9114170276599998e+00, 6.2588264063350005e+02 },
    { 2.5480000000000001e-08, 1.6625667138700000e+00, 9.5228467692400002e+01 },
    { 2.5419999999999999e-08, 4.9929130318499997e+00, 3.5478553772530000e+02 },
    { 2.5360000000000000e-08, 3.9331699360500001e+00, 6.4959738580800007e+01 },
    { 2.5329999999999999e-08, 4.2337625046999999e-01, 9.0562601246400007e+01 },
    { 2.4960000000000000e-08, 4.0808534064000002e-01, 2.9764192156090002e+02 },
    { 2.4949999999999998e-08, 5.4896673854799998e+00, 2.6271141844705999e+03 },
    { 2.4890000000000000e-08, 1.2638706947200000e+00, 2.4874160449477999e+03 },
    { 2.4669999999999999e-08, 5.8737976951200004e+00, 4.2956713629260003e+02 },
    { 2.4660000000000000e-08, 4.1947454251799998e+00, 2.4702689706070001e+02 },
    { 2.4590000000000000e-08, 2.8320780597700002e+00, 4.8588067105450000e+02 },
    { 2.4340000000000002e-08, 5.2656666788599997e+00, 1.8324281464750001e+02 },
    { 2.4290000000000001e-08, 2.8352181202700000e+00, 2.3593301268700000e+02 },
    { 2.4030000000000001e-08, 6.9911286193000000e-01, 4.5220105354160000e+02 },
    { 2.3960000000000001e-08, 3.7877174078100002e+00, 1.7368158709190001e+02 },
    { 2.3870000000000001e-08, 4.2305506193099998e+00, 6.1929035849449997e+02 },
    { 2.3860000000000000e-08, 2.8017823474600001e+00, 4.3074279246020001e+02 },
    { 2.3860000000000000e-08, 5.7305491876800003e+00, 2.5234706759822191e+04 },
    { 2.3790000000000000e-08, 4.9192182148000002e-01, 1.7638990007760000e+02 },
    { 2.3730000000000001e-08, 5.4771035745600001e+00, 3.0465988283458000e+03 },
    { 2.3730000000000001e-08, 5.4685550764300004e+00, 3.3780194662820003e+02 },
    { 2.3709999999999999e-08, 7.4039668491999999e-01, 7.9961241183519996e+02 },
    { 2.3689999999999999e-08, 5.5570519726900001e+00, 2.0597310011610000e+02 },
    { 2.3689999999999999e-08, 5.6157595010800003e+00, 2.2062509075989999e+02 },
    { 2.3600000000000000e-08, 3.9023157090299998e+00, 1.1081399749656000e+03 },
    { 2.3460000000000000e-08, 4.3988155180900002e+00, 4.4720511869550000e+02 },
    { 2.3390000000000000e-08, 3.1958101401799999e+00, 4.7535623565240002e+02 },
    { 2.3370000000000000e-08, 6.0252486880699996e+00, 6.3570450061999998e+02 },
    { 2.3339999999999999e-08, 3.5627509360800000e+00, 6.0554504565700000e+01 },
    { 2.3289999999999999e-08, 4.1179908373899998e+00, 3.9814400287280000e+02 },
    { 2.3250000000000000e-08, 5.5852244183300002e+00, 3.7895888888784002e+03 },
    { 2.3230000000000000e-08, 3.0503465408100001e+00, 5.3591074021810005e+02 },
    { 2.3129999999999999e-08, 4.7443261248799997e+00, 4.6988722005429997e+02 },
    { 2.3090000000000000e-08, 1.5288985765600001e+00, 1.5497982310600000e+02 },
    { 2.3050000000000001e-08, 1.2790633182200000e+00, 7.6885355562910001e+02 },
    { 2.3040000000000000e-08, 4.3942115684000003e+00, 4.9508804301799998e+01 },
    { 2.2939999999999999e-08, 4.6159733821300000e+00, 9.1247886193049999e+02 },
    { 2.2930000000000001e-08, 3.7483310222399999e+00, 1.8084331820250000e+02 },
    { 2.2889999999999999e-08, 3.5352764724800001e+00, 4.8225733433210002e+02 },
    { 2.2710000000000000e-08, 3.5246250838300002e+00, 1.9280404220310001e+02 },
    { 2.2710000000000000e-08, 4.5065937363500002e+00, 2.3379414867290001e+02 },
    { 2.2650000000000001e-08, 5.4111634611700001e+00, 8.2792358748649997e+02 },
    { 2.2530000000000001e-08, 6.5614704540000002e-01, 3.1017209453309999e+02 },
    { 2.2500000000000000e-08, 3.8071398952300002e+00, 3.6604368631030002e+02 },
    { 2.2469999999999999e-08, 1.3038829647200001e+00, 3.1516802937919999e+02 },
    { 2.2150000000000000e-08, 1.9800387989199999e+00, 2.6413412784722000e+03 },
    { 2.2029999999999999e-08, 4.7411172578800000e+00, 2.8117959532560002e+02 },
    { 2.2010000000000000e-08, 2.1031092553700002e+00, 7.8404385747720005e+02 },
    { 2.1999999999999998e-08, 4.9315116144099997e+00, 6.6812248533995999e+03 },
    { 2.1850000000000000e-08, 2.2964127331499999e+00, 1.0994568878849999e+02 },
    { 2.1839999999999999e-08, 6.4755103697000005e-01, 1.4575259330619999e+03 },
    { 2.1769999999999999e-08, 4.1777961151299996e+00, 6.7880499887599996e+01 },
    { 2.1590000000000000e-08, 2.7857977041400002e+00, 3.4741135829360002e+02 },
    { 2.1590000000000000e-08, 5.2289789337499997e+00, 6.6672398925699997e+02 },
    { 2.1150000000000001e-08, 2.4914434086699999e+00, 4.7631944349890000e+02 },
    { 2.1150000000000001e-08, 3.3792079669300001e+00, 6.7091677495099998e+02 },
    { 2.1130000000000001e-08, 3.7173554235999999e+00, 9.0823233677299996e+01 },
    { 2.1109999999999998e-08, 3.6734032721999998e+00, 4.2245358929179997e+02 },
    { 2.1039999999999998e-08, 5.5396515934700004e+00, 1.3725924061081000e+03 },
    { 2.1010000000000001e-08, 3.1710014334099998e+00, 4.1873388434980001e+02 },
    { 2.0980000000000000e-08, 4.8000789939099997e+00, 2.5291660378410000e+02 },
    { 2.0980000000000000e-08, 4.3222554055000000e+00, 7.5483890994859996e+02 },
    { 2.0940000000000001e-08, 2.6977808376799999e+00, 8.8379517926369999e+02 },
    { 2.0890000000000000e-08, 2.7467783024000001e+00, 1.0554045477340000e+02 },
    { 2.0840000000000000e-08, 3.8161264497300000e+00, 5.2165803728013998e+03 },
    { 2.0840000000000000e-08, 1.1965409334899999e+00, 4.1481877742920000e+02 },
    { 2.0800000000000001e-08, 6.1695106041000001e+00, 5.8720708827450005e+02 },
    { 2.0700000000000000e-08, 4.8705338682999999e-01, 4.5246582638600003e+01 },
    { 2.0689999999999999e-08, 3.4263828680400001e+00, 2.5642806592189999e+02 },
    { 2.0630000000000000e-08, 4.4229054453999996e+00, 1.7712361326050001e+02 },
    { 2.0420000000000000e-08, 4.9056999981600002e+00, 1.0257150935679999e+02 },
    { 2.0409999999999999e-08, 5.1157885404299996e+00, 4.0918970313670002e+02 },
    { 2.0350000000000000e-08, 4.4182616152399996e+00, 1.3576145525811000e+03 },
    { 2.0269999999999999e-08, 5.5262474705200004e+00, 1.6992792165031999e+03 },
    { 2.0179999999999999e-08, 1.2314392190300001e+00, 2.5020829079839999e+02 },
    { 2.0160000000000000e-08, 2.7655532486199998e+00, 2.4335537622850001e+02 },
    { 2.0070000000000000e-08, 6.2062595370000001e-02, 6.1462449204849997e+02 },
    { 2.0059999999999999e-08, 1.8943750286400001e+00, 5.2894020556919997e+02 },
    { 1.9950000000000000e-08, 4.0714276476600002e+00, 4.0933271548860000e+02 },
    { 1.9910000000000001e-08, 7.5708701449999999e-02, 1.0970942747017000e+03 },
    { 1.9820000000000001e-08, 5.3560812448700004e+00, 1.0361403908040000e+02 },
    { 1.9690000000000000e-08, 5.5912523198899997e+00, 5.3044172461999995e+02 },
    { 1.9650000000000001e-08, 4.2962096751600001e+00, 4.0160025067299998e+01 },
    { 1.9589999999999999e-08, 5.9636324053700003e+00, 1.2466574718362999e+03 },
    { 1.9550000000000000e-08, 1.1403431213199999e+00, 9.1375088631769995e+02 },
    { 1.9530000000000000e-08, 2.7768477227199999e+00, 1.1583539551190000e+02 },
    { 1.9490000000000001e-08, 2.2901482804200000e+00, 1.1718758732690001e+03 },
    { 1.9429999999999999e-08, 5.2469081084899996e+00, 9.1467900690210008e+03 },
    { 1.9420000000000001e-08, 5.3374925334299999e+00, 2.4643619622969999e+02 },
    { 1.9329999999999998e-08, 2.3814004490600000e+00, 1.0625633239269000e+03 },
    { 1.9230000000000001e-08, 4.7654346656700000e+00, 5.3388375078859997e+02 },
    { 1.9130000000000000e-08, 1.1131849219300001e+00, 8.3578789401270001e+02 },
    { 1.9119999999999998e-08, 5.8315431791999996e+00, 1.3310087089929999e+02 },
    { 1.9079999999999999e-08, 5.2223826537000004e+00, 4.6675821594330000e+02 },
    { 1.8909999999999999e-08, 3.4372872391099998e+00, 1.1653954657146000e+03 },
    { 1.8900000000000000e-08, 2.0778152690899998e+00, 7.6440013750419996e+02 },
    { 1.8870000000000000e-08, 6.1570625515300001e+00, 1.0837609525774001e+03 },
    { 1.8780000000000000e-08, 5.6034818539999998e-01, 1.3406407874580000e+02 },
    { 1.8730000000000000e-08, 3.8262826955899998e+00, 3.4484605765980001e+02 },
    { 1.8660000000000000e-08, 5.1689415228100000e+00, 5.8319272331999997e+01 },
    { 1.8590000000000000e-08, 2.4345513431599999e+00, 4.6972716135989998e+02 },
    { 1.8550000000000001e-08, 5.9920515148400000e+00, 3.0141401612959999e+02 },
    { 1.8510000000000002e-08, 1.3225791672400000e+00, 5.8631331639719997e+02 },
    { 1.8480000000000001e-08, 4.2015926215599997e+00, 6.3374694715969997e+02 },
    { 1.8480000000000001e-08, 9.8286127486999997e-01, 4.1969709219629999e+02 },
    { 1.8439999999999998e-08, 7.8924215599999997e-03, 2.4351557300350000e+03 },
    { 1.8390000000000001e-08, 6.0187235739100000e+00, 5.8949471013489995e+02 },
    { 1.8390000000000001e-08, 6.3974984192999995e-01, 5.2401370669230005e+02 },
    { 1.8380000000000000e-08, 1.9492753765999999e+00, 2.8546403739102002e+03 },
    { 1.8360000000000000e-08, 1.3561461448800001e+00, 6.4456315275999998e+02 },
    { 1.8349999999999999e-08, 4.2833552375200004e+00, 2.4575487267349999e+02 },
    { 1.8329999999999999e-08, 6.8864623996999996e-01, 2.6763382488020000e+02 },
    { 1.8299999999999998e-08, 5.3471573963100001e+00, 2.5950888592310002e+02 },
    { 1.8220000000000000e-08, 1.3681681797100000e+00, 1.1807062774560001e+02 },
    { 1.8130000000000001e-08, 4.7493499523000002e+00, 2.8489930026759998e+02 },
    { 1.8040000000000001e-08, 5.2474225956599998e+00, 3.3629906980024002e+03 },
    { 1.8019999999999999e-08, 3.8839664126700000e+00, 2.7078286873866000e+03 },
    { 1.7999999999999999e-08, 1.1375429584400001e+00, 1.3891071219701000e+03 },
    { 1.7840000000000000e-08, 3.4113027703899998e+00, 3.4618751801619999e+02 },
    { 1.7820000000000000e-08, 3.2528987676000001e+00, 9.7724602613333991e+03 },
    { 1.7809999999999999e-08, 4.9115892345300001e+00, 2.0592491600630001e+02 },
    { 1.7809999999999999e-08, 6.2612222392400003e+00, 2.2067327486970001e+02 },
    { 1.7750000000000000e-08, 8.1870406809999996e-01, 1.9810879358989999e+02 },
    { 1.7750000000000000e-08, 4.0709220984899996e+00, 2.2848939728610000e+02 },
    { 1.7739999999999999e-08, 3.3374714043300000e+00, 2.0644618086809999e+02 },
    { 1.7739999999999999e-08, 1.5521547622600000e+00, 2.2015201000790000e+02 },
    { 1.7739999999999999e-08, 2.0557745178300002e+00, 9.8853248488500003e+02 },
    { 1.7649999999999999e-08, 5.0342973611100001e+00, 7.5780785536520000e+02 },
    { 1.7630000000000000e-08, 3.7990707114000002e+00, 3.8330916169770001e+02 },
    { 1.7620000000000002e-08, 4.8306578111800000e+00, 5.5110104206619997e+02 },
    { 1.7610000000000000e-08, 4.5481534383500000e+00, 5.9282480178500002e+01 },
    { 1.7590000000000001e-08, 3.0150296475500000e+00, 2.0003520928290001e+02 },
    { 1.7590000000000001e-08, 1.8745965190400000e+00, 2.2656298159310001e+02 },
    { 1.7579999999999999e-08, 1.9605568983199999e+00, 4.5246168597249999e+02 },
    { 1.7490000000000000e-08, 2.1742115184499999e+00, 2.5455119402261000e+04 },
    { 1.7420000000000000e-08, 5.0929948144599999e+00, 3.4936891175390002e+02 },
    { 1.7199999999999999e-08, 6.0415911136300000e+00, 6.2901893969922003e+03 },
    { 1.7150000000000001e-08, 3.4812452115000001e+00, 7.0904981058880003e+02 },
    { 1.7150000000000001e-08, 4.5121839706799998e+00, 7.9186832582400001e+01 },
    { 1.7070000000000000e-08, 5.4350776110899996e+00, 1.1443154571431000e+03 },
    { 1.7070000000000000e-08, 4.8133465395000002e-01, 9.8357471803400003e+01 },
    { 1.7059999999999999e-08, 5.5353279572000003e-01, 7.4920983565610004e+02 },
    { 1.7030000000000001e-08, 2.2929918731800001e+00, 1.4638180339690001e+02 },
    { 1.7000000000000000e-08, 2.5269058926399999e+00, 1.0916252591036000e+03 },
    { 1.6989999999999999e-08, 1.8358915128900000e+00, 5.8493641121146002e+03 },
    { 1.6960000000000001e-08, 3.7496250825300002e+00, 1.3411226285559999e+02 },
    { 1.6940000000000001e-08, 6.2401720911099998e+00, 5.1838463239980001e+02 },
    { 1.6890000000000001e-08, 3.0605255770899999e+00, 2.7533807271199998e+02 },
    { 1.6870000000000002e-08, 9.6512635264000002e-01, 1.0034308307600000e+01 },
    { 1.6870000000000002e-08, 3.2471038210600001e+00, 1.1514290041439001e+03 },
    { 1.6840000000000001e-08, 1.2022940181999999e+00, 1.0220399093211799e+04 },
    { 1.6829999999999999e-08, 2.4439854691299998e+00, 6.5034884924301996e+03 },
    { 1.6810000000000000e-08, 3.2723374178700002e+00, 9.7795738083342003e+03 },
    { 1.6680000000000001e-08, 1.5464960467000000e-01, 3.1867949151699997e+02 },
    { 1.6660000000000002e-08, 2.2154934055200002e+00, 7.0045179087969996e+02 },
    { 1.6650000000000000e-08, 5.3422615771800004e+00, 1.6357784281510001e+02 },
    { 1.6630000000000001e-08, 2.4097601232999999e-01, 1.0298089963400000e+02 },
    { 1.6610000000000001e-08, 3.6710516357000000e-01, 1.0129115072732000e+03 },
    { 1.6600000000000000e-08, 2.1468624951100002e+00, 6.6109491496450005e+02 },
    { 1.6580000000000000e-08, 4.5133729472000000e-01, 6.2439816792520003e+02 },
    { 1.6549999999999999e-08, 1.0024943880199999e+00, 1.0478969524800000e+02 },
    { 1.6519999999999998e-08, 2.2941700619700001e+00, 1.3647280995819001e+03 },
    { 1.6510000000000000e-08, 1.3840493911800000e+00, 1.0380412891868000e+03 },
    { 1.6490000000000001e-08, 3.7636192105699999e+00, 4.2161870797543997e+03 },
    { 1.6470000000000001e-08, 2.0710880859800000e+00, 1.0931254934210000e+02 },
    { 1.6440000000000000e-08, 3.6083149843900002e+00, 2.4947457761550001e+02 },
    { 1.6370000000000000e-08, 1.3771147165199999e+00, 4.5172797278960002e+02 },
    { 1.6230000000000000e-08, 2.8270252232300002e+00, 2.9140142358237999e+03 },
    { 1.6169999999999999e-08, 8.2146396585000003e-01, 8.3379618276399995e+01 },
    { 1.6149999999999999e-08, 3.2870658817599998e+00, 7.6343692965770003e+02 },
    { 1.5990000000000000e-08, 3.8676916640800001e+00, 1.0433698188649800e+04 },
    { 1.5970000000000000e-08, 1.0822958181300000e+00, 3.0386169668439999e+02 },
    { 1.5900000000000000e-08, 5.8826619728100003e+00, 2.6484548254729998e+03 },
    { 1.5880000000000001e-08, 3.9282400266400002e+00, 1.4833935685719999e+02 },
    { 1.5840000000000002e-08, 3.7141320251800001e+00, 3.7672106175757999e+03 },
    { 1.5799999999999999e-08, 3.3044498213300000e+00, 1.4775945496771999e+03 },
    { 1.5790000000000001e-08, 2.1805676969999999e+00, 6.8005731138130000e+02 },
    { 1.5729999999999999e-08, 6.0072712597200004e+00, 2.7515475996916002e+03 },
    { 1.5690000000000000e-08, 2.9659094904400001e+00, 1.2247586176070000e+02 },
    { 1.5690000000000000e-08, 4.9663745508000003e+00, 1.1789894202697999e+03 },
    { 1.5650000000000001e-08, 5.5086607595199997e+00, 6.7049608382570000e+02 },
    { 1.5640000000000000e-08, 6.1637325384899997e+00, 3.3914448860056000e+03 },
    { 1.5630000000000002e-08, 3.1606391853500000e+00, 7.3683972137830006e+02 },
    { 1.5600000000000001e-08, 1.3555573994000001e+00, 4.1385556958270001e+02 },
    { 1.5570000000000000e-08, 4.7286874219600001e+00, 5.7533848848968000e+03 },
    { 1.5550000000000000e-08, 1.7521420683100000e+00, 6.7925416222920001e+02 },
    { 1.5489999999999998e-08, 3.0007952515100000e+00, 7.3582832942200002e+02 },
    { 1.5370000000000001e-08, 2.3081556193999999e+00, 3.2105773610260002e+02 },
    { 1.5290000000000000e-08, 3.4923015058100000e+00, 1.3797059531089001e+03 },
    { 1.5220000000000000e-08, 3.9554329597500000e+00, 4.0615132175090002e+02 },
    { 1.5130000000000000e-08, 1.5806353839400000e+00, 6.9407195706180005e+02 },
    { 1.5119999999999999e-08, 5.6285509166200001e+00, 1.7480372612796000e+03 },
    { 1.5119999999999999e-08, 1.4647922236699999e+00, 5.7462713378959997e+03 },
    { 1.5119999999999999e-08, 2.7182793635700002e+00, 4.2824272227870000e+02 },
    { 1.5049999999999999e-08, 2.4327863246199999e+00, 4.4704506000110001e+02 },
    { 1.5010000000000000e-08, 2.8745054636699998e+00, 9.6764810341156008e+03 },
    { 1.4999999999999999e-08, 1.5324241124200000e+00, 2.3139037085517002e+03 },
    { 1.4979999999999999e-08, 4.1523796773999999e-01, 1.7469297904819999e+02 },
    { 1.4960000000000000e-08, 4.7985703308299996e+00, 7.7623812138408583e+04 },
    { 1.4960000000000000e-08, 6.1386447861300004e+00, 5.4369930152402003e+03 },
    { 1.4950000000000002e-08, 2.9095019343000000e-01, 1.5578297225809999e+02 },
    { 1.4920000000000001e-08, 3.8704126206699998e+00, 8.7832616366560001e+02 },
    { 1.4769999999999999e-08, 2.9773862405800000e+00, 1.0538039607899999e+02 },
    { 1.4750000000000000e-08, 5.6136660061899999e+00, 5.6431785636774002e+03 },
    { 1.4750000000000000e-08, 4.0702476724299999e+00, 2.8109214616052000e+03 },
    { 1.4730000000000000e-08, 5.4155495683300003e+00, 1.4528430202735001e+03 },
    { 1.4710000000000001e-08, 6.1030032816000002e-01, 2.6936015933849999e+03 },
    { 1.4640000000000001e-08, 3.8537420457699998e+00, 2.6302034806090001e+02 },
    { 1.4640000000000001e-08, 2.8423006713499999e+00, 3.5989334715600000e+02 },
    { 1.4640000000000001e-08, 2.1951774645800000e+00, 1.2687488723641000e+03 },
    { 1.4600000000000000e-08, 1.4223512530100000e+00, 1.0200250542740000e+03 },
    { 1.4600000000000000e-08, 1.8106236816500001e+00, 1.2442268376239999e+03 },
    { 1.4600000000000000e-08, 4.0201450815499999e+00, 1.1285344544640000e+03 },
    { 1.4590000000000000e-08, 4.6204734122000000e+00, 9.3529756174581998e+03 },
    { 1.4580000000000001e-08, 7.7581736647999999e-01, 6.3283192342300003e+02 },
    { 1.4550000000000000e-08, 7.4361321398000002e-01, 2.5661304950698199e+04 },
    { 1.4510000000000001e-08, 5.6662870385100002e+00, 3.1665250208750001e+02 },
    { 1.4500000000000001e-08, 6.0389400302000000e+00, 7.3592469764160001e+02 },
    { 1.4419999999999999e-08, 4.2387754926500003e+00, 4.1130943055358002e+03 },
    { 1.4349999999999999e-08, 5.2420756307099996e+00, 1.1474968508807999e+03 },
    { 1.4349999999999999e-08, 4.7369900504700002e+00, 7.4591082183940000e+02 },
    { 1.4320000000000000e-08, 4.3785992960200000e+00, 4.3192798539730002e+03 },
    { 1.4320000000000000e-08, 3.4980510915000002e-01, 1.1664068576709001e+03 },
    { 1.4320000000000000e-08, 1.9615689989300000e+00, 1.4670701142751000e+03 },
    { 1.4289999999999999e-08, 2.5787001661500000e+00, 3.8872578766909999e+02 },
    { 1.4289999999999999e-08, 1.3559097260599999e+00, 9.3672027114598004e+03 },
    { 1.4260000000000000e-08, 3.5466841104500002e+00, 8.2399143422340001e+02 },
    { 1.4160000000000001e-08, 1.2755790892200001e+00, 8.4045376045870000e+02 },
    { 1.4140000000000000e-08, 3.8704360621600000e+00, 6.3273555520340005e+02 },
    { 1.4130000000000000e-08, 5.1928246196299996e+00, 3.2261164478009999e+02 },
    { 1.4079999999999999e-08, 1.4891642217500001e+00, 1.5606187072805999e+03 },
    { 1.4079999999999999e-08, 3.7622940212499998e+00, 1.1120721282287000e+03 },
    { 1.4060000000000000e-08, 8.4231326932999995e-01, 5.6071045373159996e+02 },
    { 1.4060000000000000e-08, 3.9436369388100001e+00, 9.5920991727740000e+02 },
    { 1.4009999999999999e-08, 4.9723926267499996e+00, 1.9502906564537000e+03 },
    { 1.3970000000000000e-08, 3.8839140036600002e+00, 1.4614410399825999e+03 },
    { 1.3939999999999999e-08, 3.5900843466799999e+00, 6.3203297978779995e+02 },
    { 1.3939999999999999e-08, 3.7066643553800001e+00, 1.0089793540101000e+03 },
    { 1.3930000000000000e-08, 2.2406031055700000e+00, 2.5558212176479599e+04 },
    { 1.3910000000000000e-08, 1.5944577714199999e+00, 5.8604098123784997e+03 },
    { 1.3830000000000000e-08, 2.9111533848000000e+00, 8.8002308469499997e+02 },
    { 1.3809999999999999e-08, 5.2417965677499998e+00, 7.1861103814440003e+02 },
    { 1.3790000000000000e-08, 5.7948132627200000e+00, 3.1587060479479999e+02 },
    { 1.3770000000000000e-08, 1.4934322695600000e+00, 1.3544331588434000e+03 },
    { 1.3770000000000000e-08, 2.4545446635900001e+00, 1.8355163118819999e+02 },
    { 1.3770000000000000e-08, 5.5766741565900002e+00, 2.4304655968780000e+02 },
    { 1.3760000000000000e-08, 2.3895578039999998e+00, 9.7355463135800005e+02 },
    { 1.3750000000000001e-08, 1.6043092730799999e+00, 1.6708250284999999e+03 },
    { 1.3739999999999999e-08, 5.3241005595599997e+00, 5.3532003938710000e+02 },
    { 1.3730000000000000e-08, 5.1421386804300004e+00, 1.1887377689770000e+02 },
    { 1.3730000000000000e-08, 4.6944603869900003e+00, 5.8525455058523003e+03 },
    { 1.3720000000000000e-08, 5.0908565714099998e+00, 1.1829215735329001e+03 },
    { 1.3640000000000000e-08, 3.6567046836400001e+00, 6.7383753625780002e+02 },
    { 1.3610000000000001e-08, 9.4127367769999998e-02, 3.3136972318360000e+02 },
    { 1.3540000000000001e-08, 2.1900809841800002e+00, 5.2980283967920002e+02 },
    { 1.3510000000000000e-08, 5.9069524131100000e+00, 7.9568025857210000e+02 },
    { 1.3500000000000000e-08, 5.0974551320800003e+00, 2.5440892308259390e+04 },
    { 1.3440000000000000e-08, 5.3567451867400004e+00, 1.0850940018999999e+02 },
    { 1.3420000000000000e-08, 1.8466374522900000e+00, 3.8427236954419999e+02 },
    { 1.3389999999999999e-08, 5.7976012941999999e-01, 1.1592933106701000e+03 },
    { 1.3370000000000000e-08, 6.0300610560700001e+00, 6.3627874754000002e+00 },
    { 1.3350000000000000e-08, 1.0817972203999999e+00, 7.3045988756040003e+02 },
    { 1.3319999999999999e-08, 6.8009283969000001e-01, 8.9632535223590003e+02 },
    { 1.3300000000000000e-08, 3.4671858878299999e+00, 2.7405018994277002e+03 },
    { 1.3210000000000000e-08, 3.6554718986000001e-01, 2.7161836777000002e+02 },
    { 1.3210000000000000e-08, 2.8659084813700000e+00, 3.2324478422649997e+02 },
    { 1.3200000000000001e-08, 5.9204894501999998e+00, 1.1272624300768000e+03 },
    { 1.3140000000000001e-08, 2.0760783081200000e+00, 7.0193626358799997e+02 },
    { 1.3119999999999999e-08, 4.5645992033100002e+00, 3.1172600321060003e+02 },
    { 1.3110000000000000e-08, 2.4777750742400002e+00, 3.3217287233569999e+02 },
    { 1.3070000000000001e-08, 5.0909643309100003e+00, 9.6250893109410003e+02 },
    { 1.3030000000000000e-08, 2.1241860613500001e+00, 1.0962429414398000e+03 },
    { 1.3030000000000000e-08, 1.9920032998499999e+00, 9.4599421523210003e+02 },
    { 1.3010000000000000e-08, 3.0179118550299999e+00, 9.7906195085980999e+03 },
    { 1.3010000000000000e-08, 6.2268340726700000e+00, 8.7583029900099996e+02 },
    { 1.2989999999999999e-08, 7.1540574041000005e-01, 2.5097943162503998e+03 },
    { 1.2970000000000000e-08, 3.4602920466799998e+00, 4.5325789494110004e+03 },
    { 1.2970000000000000e-08, 2.8010150282300001e+00, 1.7842127434571000e+03 },
    { 1.2950000000000000e-08, 2.8030875021499999e+00, 5.1795897827258004e+03 },
    { 1.2940000000000000e-08, 5.5006700626500002e+00, 1.0820158474381999e+03 },
    { 1.2919999999999999e-08, 4.9489217954900004e+00, 1.0160830151030000e+02 },
    { 1.2900000000000000e-08, 4.3166466238699996e+00, 1.4033342159716999e+03 },
    { 1.2880000000000000e-08, 5.1360746189700004e+00, 1.3405615255148000e+03 },
    { 1.2870000000000000e-08, 2.7168560659600001e+00, 1.1306332694800000e+01 },
    { 1.2830000000000000e-08, 6.1179141340900003e+00, 9.7827552020718995e+03 },
    { 1.2820000000000000e-08, 2.4539082102499998e+00, 1.0483362299252999e+03 },
    { 1.2820000000000000e-08, 5.5100433530800004e+00, 3.1490739694830000e+02 },
    { 1.2790000000000001e-08, 5.3402896197200000e+00, 2.0889386142289999e+02 },
    { 1.2790000000000001e-08, 5.8325218540500003e+00, 2.1770432945310000e+02 },
    { 1.2790000000000001e-08, 1.9452534270799999e+00, 2.2312095542450001e+02 },
    { 1.2790000000000001e-08, 2.9443727395099999e+00, 2.0347723545150001e+02 },
    { 1.2779999999999999e-08, 3.3906549097799998e+00, 8.5149946072260002e+02 },
    { 1.2760000000000000e-08, 4.9281961699799997e+00, 6.1387373252309999e+02 },
    { 1.2740000000000000e-08, 2.1325961648299998e+00, 1.6534419981369999e+02 },
    { 1.2720000000000001e-08, 4.5550263382400002e+00, 5.2677020378780003e+02 },
    { 1.2700000000000000e-08, 2.9546944513200000e+00, 3.7301422095919997e+02 },
    { 1.2690000000000000e-08, 8.3510696952999997e-01, 1.2395439248355001e+03 },
    { 1.2670000000000000e-08, 3.7548776755800000e+00, 1.1487218766540001e+02 },
    { 1.2660000000000001e-08, 5.9242666115700002e+00, 5.8675233593793000e+03 },
    { 1.2610000000000000e-08, 3.0134478694800002e+00, 5.2012973753899996e+02 },
    { 1.2600000000000000e-08, 4.2179286242599998e+00, 9.5016019371300004e+01 },
    { 1.2580000000000001e-08, 6.9213323851999997e-01, 9.5733882598969994e+03 },
    { 1.2569999999999999e-08, 4.1197726402000001e-01, 8.7757540414020002e+02 },
    { 1.2540000000000000e-08, 4.1052834082800000e+00, 7.5877106321170004e+02 },
    { 1.2520000000000001e-08, 3.4989378235000002e-01, 1.1160042814917999e+03 },
    { 1.2480000000000000e-08, 5.8188058531499998e+00, 1.3922885157077999e+03 },
    { 1.2450000000000001e-08, 3.0843227251899998e+00, 2.9349731997670000e+02 },
    { 1.2439999999999999e-08, 5.0029623332900002e+00, 7.6174000862829996e+02 },
    { 1.2410000000000000e-08, 1.1637463504000001e+00, 9.3812990870589999e+02 },
    { 1.2410000000000000e-08, 2.0171719130000001e-02, 7.4840668650400005e+02 },
    { 1.2380000000000001e-08, 9.2078746342999995e-01, 1.3836904959987000e+03 },
    { 1.2380000000000001e-08, 2.8808201148800001e+00, 5.5969906177530004e+02 },
    { 1.2360000000000000e-08, 5.8782888180299997e+00, 5.3909213395580002e+02 },
    { 1.2299999999999999e-08, 1.3956830728000000e-01, 2.5867490499135391e+04 },
    { 1.2280000000000000e-08, 4.5781246485800002e+00, 9.1456373123700004e+01 },
    { 1.2270000000000000e-08, 2.3750983049199998e+00, 1.9731328165069001e+03 },
    { 1.2260000000000000e-08, 4.3969293618599998e+00, 2.5344913081041599e+04 },
    { 1.2220000000000000e-08, 1.6399626005200001e+00, 7.7596710262989995e+02 },
    { 1.2210000000000000e-08, 3.0618508394499999e+00, 1.4584720945659999e+03 },
    { 1.2210000000000000e-08, 4.5070753586699999e+00, 4.7306861379200001e+02 },
    { 1.2200000000000000e-08, 5.7661528642500004e+00, 6.5994677196479997e+03 },
    { 1.2140000000000000e-08, 4.4212007726799998e+00, 2.9466874478649002e+03 },
    { 1.2110000000000001e-08, 7.2754477376000004e-01, 2.7081521861789997e+02 },
    { 1.2100000000000001e-08, 3.9977363288299999e+00, 5.3213864564940002e+02 },
    { 1.2100000000000001e-08, 6.1876561574600002e+00, 2.5976223661672002e+03 },
    { 1.2080000000000000e-08, 6.0394954681400002e+00, 1.1588357962169999e+02 },
    { 1.2029999999999999e-08, 1.0799692880200000e+00, 5.3339002410215999e+03 },
    { 1.2010000000000000e-08, 1.5274119728900000e+00, 5.2846712481719999e+02 },
    { 1.1990000000000000e-08, 2.5452578550900000e+00, 3.0831511480929998e+02 },
    { 1.1970000000000001e-08, 6.6740409249999993e-02, 4.7061123747000003e+01 },
    { 1.1920000000000000e-08, 2.4147418416500002e+00, 5.2724328453980002e+02 },
    { 1.1889999999999999e-08, 1.4469766843100000e+00, 9.3419775544280003e+02 },
    { 1.1880000000000000e-08, 5.7506060179100000e+00, 1.3947361962626001e+03 },
    { 1.1870000000000000e-08, 4.7758362885199999e+00, 1.2616353253632999e+03 },
    { 1.1870000000000000e-08, 4.3003079109400000e+00, 5.8635912061161998e+03 },
    { 1.1860000000000000e-08, 7.3895067297000006e-01, 7.7299815721330003e+02 },
    { 1.1840000000000000e-08, 1.0645525274400001e+00, 9.7977330555988992e+03 },
    { 1.1830000000000001e-08, 4.1723614042400001e+00, 4.1529185818119998e+02 },
    { 1.1819999999999999e-08, 6.0471014226699999e+00, 3.1496916025644000e+03 },
    { 1.1819999999999999e-08, 5.4150343075700000e+00, 5.5400857894587998e+03 },
    { 1.1800000000000000e-08, 2.3718062095199999e+00, 1.2262106027112000e+03 },
    { 1.1790000000000000e-08, 9.6347481459999995e-01, 2.7258157561650000e+02 },
    { 1.1790000000000000e-08, 3.9990668192199998e+00, 2.6099335863139999e+02 },
    { 1.1760000000000001e-08, 1.3327973704300000e+00, 2.5190521182780000e+02 },
    { 1.1760000000000001e-08, 2.5747287404000002e-01, 3.0305854753230000e+02 },
    { 1.1749999999999999e-08, 4.5124710680000002e-01, 1.7739178027185999e+03 },
    { 1.1720000000000000e-08, 2.0951776841000000e-01, 9.7918370565049997e+02 },
    { 1.1700000000000001e-08, 5.4627504485000000e-01, 7.0485702489480002e+02 },
    { 1.1700000000000001e-08, 3.0461931097799999e+00, 1.1767017984094000e+03 },
    { 1.1679999999999999e-08, 1.9640397826300000e+00, 8.3258945393249996e+02 },
    { 1.1670000000000000e-08, 3.0271719444900000e+00, 2.5771511271917600e+04 },
    { 1.1660000000000000e-08, 3.4888028532100002e+00, 2.4673990589494001e+03 },
    { 1.1609999999999999e-08, 8.0096292279999995e-02, 2.8332997329077998e+03 },
    { 1.1609999999999999e-08, 5.3333722233099996e+00, 6.3278373931320002e+03 },
    { 1.1600000000000000e-08, 4.6667569271399998e+00, 1.5274459087229999e+02 },
    { 1.1560000000000001e-08, 2.0995802140299999e+00, 2.9577331481288002e+03 },
    { 1.1560000000000001e-08, 4.9132081850100002e+00, 8.8543971066639995e+02 },
    { 1.1549999999999999e-08, 2.2997009336999999e-01, 3.0852756313039998e+02 },
    { 1.1520000000000000e-08, 3.5051240732000000e-01, 7.7544583776809998e+02 },
    { 1.1520000000000000e-08, 1.0409390254299999e+00, 3.6936096616606001e+03 },
    { 1.1520000000000000e-08, 5.3398477548300001e+00, 9.8778172535960005e+02 },
    { 1.1500000000000000e-08, 5.0537495634300003e+00, 4.1059807585349999e+03 },
    { 1.1500000000000000e-08, 2.7516439772900001e+00, 6.2248879857470001e+02 },
    { 1.1479999999999999e-08, 3.6615251089699998e+00, 2.0479144150742000e+03 },
    { 1.1469999999999999e-08, 5.4745912726299997e+00, 6.3330500417500002e+02 },
    { 1.1440000000000000e-08, 1.0161269620500000e+00, 9.8215265106710001e+02 },
    { 1.1420000000000001e-08, 3.1372536553399999e+00, 1.3780614217062000e+03 },
    { 1.1390000000000000e-08, 2.6843580300499998e+00, 1.9902381732660001e+02 },
    { 1.1390000000000000e-08, 2.2052681365400000e+00, 2.2757437354940001e+02 },
    { 1.1390000000000000e-08, 4.1496903696999998e-01, 2.5345912050530001e+02 },
    { 1.1390000000000000e-08, 2.8846267486999999e-01, 5.8454319588515000e+03 },
    { 1.1320000000000000e-08, 3.3032386479300002e+00, 1.4678208738005001e+03 },
    { 1.1310000000000000e-08, 3.2144655170000001e-02, 6.8961853893689999e+02 },
    { 1.1290000000000001e-08, 5.6146098189200000e+00, 1.0547160637432000e+03 },
    { 1.1280000000000001e-08, 1.5025003123100000e+00, 1.2222784494481000e+03 },
    { 1.1269999999999999e-08, 7.5894620303000004e-01, 1.2481419445445999e+03 },
    { 1.1269999999999999e-08, 5.0275544921500002e+00, 7.8477008520160583e+04 },
    { 1.1240000000000000e-08, 1.9505027301600000e+00, 4.0841348623499997e+01 },
    { 1.1220000000000001e-08, 9.2184815863000003e-01, 4.3281796599950002e+02 },
    { 1.1220000000000001e-08, 2.6740802950200000e+00, 5.2502509864859996e+02 },
    { 1.1220000000000001e-08, 6.9750945683999999e-01, 8.7710232338820003e+02 },
    { 1.1210000000000001e-08, 6.0613213778799997e+00, 1.0798287993143001e+03 },
    { 1.1199999999999999e-08, 5.7251806245200001e+00, 5.0759956456679998e+02 },
    { 1.1190000000000000e-08, 1.8058684742100000e+00, 4.9662906872878002e+03 },
    { 1.1160000000000000e-08, 1.6896008048900000e+00, 5.1716079212240004e+02 },
    { 1.1140000000000001e-08, 4.8066713860099997e+00, 6.7442823708879996e+02 },
    { 1.1129999999999999e-08, 2.7889574934099999e+00, 1.8484490743480001e+02 },
    { 1.1100000000000000e-08, 9.4798767960000002e-02, 1.3977051416792001e+03 },
    { 1.1090000000000000e-08, 4.5251069064099996e+00, 1.2591876448085000e+03 },
    { 1.1080000000000001e-08, 4.5250154391299997e+00, 5.2354062594029995e+02 },
    { 1.1050000000000000e-08, 3.1520749893300000e+00, 9.7110695080319999e+02 },
    { 1.1050000000000000e-08, 5.8615863941999997e+00, 1.9912018554619999e+02 },
    { 1.1050000000000000e-08, 5.3112250795699998e+00, 2.2747800532980000e+02 },
    { 1.1040000000000000e-08, 4.6996392943099998e+00, 1.2766131788903001e+03 },
    { 1.1030000000000000e-08, 3.0597573589799998e+00, 5.3169670266469996e+02 },
    { 1.1010000000000001e-08, 5.0726658786099996e+00, 4.9447869076360001e+02 },
    { 1.0999999999999999e-08, 4.4256070270199999e+00, 5.2768522752449996e+02 },
    { 1.0999999999999999e-08, 3.2299798339199999e+00, 6.0824465823059995e+02 },
    { 1.0989999999999999e-08, 2.1479393978800001e+00, 5.8238116313410001e+02 },
    { 1.0940000000000001e-08, 4.0064664506499996e+00, 9.8968936765544004e+03 },
    { 1.0920000000000000e-08, 2.1014969393500000e+00, 1.6811199692385001e+03 },
    { 1.0890000000000000e-08, 4.8926994408300004e+00, 4.1656388256840000e+02 },
    { 1.0880000000000000e-08, 1.8753617552999999e+00, 9.7673602509570003e+02 },
    { 1.0859999999999999e-08, 1.0244633959700000e+00, 5.9220302312060005e+02 },
    { 1.0850000000000000e-08, 3.7811767801000001e-01, 1.5465346256309001e+03 },
    { 1.0820000000000000e-08, 6.0851188696099996e+00, 4.4263982598600001e+02 },
    { 1.0820000000000000e-08, 3.6806320705300002e+00, 2.1510071892927999e+03 },
    { 1.0820000000000000e-08, 5.6504229674799999e+00, 2.6404295011230799e+04 },
    { 1.0810000000000001e-08, 5.4332720512400003e+00, 1.5126467653138000e+03 },
    { 1.0810000000000001e-08, 5.0889794960200003e+00, 3.0412232911529998e+02 },
    { 1.0789999999999999e-08, 3.7232261437799998e+00, 9.0291763437489999e+02 },
    { 1.0750000000000000e-08, 5.7719130648699997e+00, 1.7408487739300000e+01 },
    { 1.0719999999999999e-08, 2.4400313310300001e+00, 2.2057270113320001e+02 },
    { 1.0719999999999999e-08, 2.4495948355600001e+00, 2.0602548974280000e+02 },
    { 1.0719999999999999e-08, 5.8350675236700003e+00, 2.0634560713159999e+02 },
    { 1.0719999999999999e-08, 5.3377439501000001e+00, 2.2025258374440000e+02 },
    { 1.0719999999999999e-08, 3.4120813713999998e+00, 5.7297999427289994e+02 },
    { 1.0719999999999999e-08, 5.8230284341200003e+00, 7.8159617692239999e+02 },
    { 1.0710000000000000e-08, 2.3713494049500001e+00, 9.4753107756710006e+02 },
    { 1.0660000000000001e-08, 3.5413270141999997e-01, 3.1347110834979998e+02 },
    { 1.0660000000000001e-08, 4.7942092975000000e-01, 2.3067901615508999e+03 },
    { 1.0649999999999999e-08, 1.7119150645600001e+00, 9.7756416550711001e+03 },
    { 1.0570000000000000e-08, 6.0303045816500003e+00, 3.7078367556622002e+03 },
    { 1.0560000000000000e-08, 2.3536879128399999e+00, 6.8318631549229997e+02 },
    { 1.0560000000000000e-08, 8.4339591319999996e-01, 6.2038977273999997e+01 },
    { 1.0550000000000000e-08, 5.0731905875800001e+00, 1.4607073267997000e+03 },
    { 1.0530000000000001e-08, 4.9039004753100004e+00, 1.9015326116773001e+03 },
    { 1.0530000000000001e-08, 1.9192041513100000e+00, 8.5941615687549995e+02 },
    { 1.0519999999999999e-08, 3.1131636217900001e+00, 6.2617402095243997e+03 },
    { 1.0490000000000000e-08, 5.3328599229400000e+00, 1.0183281332446001e+03 },
    { 1.0480000000000000e-08, 1.9489816041800001e+00, 1.4710022675381999e+03 },
    { 1.0470000000000000e-08, 6.1397488626800003e+00, 2.0903096723752001e+03 },
    { 1.0449999999999999e-08, 1.5408168232299999e+00, 1.9872168981566001e+03 },
    { 1.0440000000000000e-08, 2.1653776350500000e+00, 8.3652160719560004e+02 },
    { 1.0420000000000000e-08, 1.8402802718500000e+00, 1.3049243545416000e+03 },
    { 1.0410000000000000e-08, 1.7114011434700001e+00, 1.1868537267960000e+03 },
    { 1.0379999999999999e-08, 5.1297577196999999e-01, 1.1521797636693000e+03 },
    { 1.0370000000000000e-08, 2.3768919609600001e+00, 6.4531391228539997e+02 },
    { 1.0370000000000000e-08, 5.0288726688400001e+00, 5.8052599750381996e+03 },
    { 1.0350000000000000e-08, 1.5807945489599999e+00, 1.2184272231430000e+02 },
    { 1.0330000000000001e-08, 1.5245018920700000e+00, 4.0100015313171998e+03 },
    { 1.0320000000000001e-08, 1.1751734306599999e+00, 3.6570042963564001e+03 },
    { 1.0320000000000001e-08, 4.6795245642300003e+00, 9.0195442652839995e+02 },
    { 1.0309999999999999e-08, 3.4224973086600001e+00, 7.1453587252939997e+02 },
    { 1.0309999999999999e-08, 6.0335376470000002e-01, 6.8865533109039995e+02 },
    { 1.0300000000000000e-08, 1.1914034740499999e+00, 5.3857753311630004e+03 },
    { 1.0300000000000000e-08, 4.8264992678200000e+00, 1.4395096981491999e+03 },
    { 1.0290000000000000e-08, 6.1863257742500002e+00, 1.3654788591073000e+03 },
    { 1.0260000000000001e-08, 7.6317034721999999e-01, 5.2161421024729998e+02 },
    { 1.0260000000000001e-08, 3.0932479108300002e+00, 2.4422692770357999e+03 },
    { 1.0220000000000000e-08, 3.8558153258000001e+00, 9.6813800538659996e+02 },
    { 1.0210000000000000e-08, 1.4013949400200001e+00, 1.6969210293999999e+00 },
    { 1.0200000000000000e-08, 5.9046500078799999e+00, 1.4756851803267000e+03 },
    { 1.0190000000000001e-08, 2.0483353539999999e-01, 6.2475131155228000e+03 },
    { 1.0190000000000001e-08, 2.5303734535100002e+00, 5.9964663852149999e+02 },
    { 1.0190000000000001e-08, 5.4857952412500000e+00, 1.0320464880320000e+02 },
    { 1.0179999999999999e-08, 1.3451658876599999e+00, 6.4064804583939997e+02 },
    { 1.0160000000000000e-08, 4.7092188411200002e+00, 9.8141893788419998e+02 },
    { 1.0150000000000000e-08, 4.2879637919900002e+00, 1.0223126761344000e+03 },
    { 1.0150000000000000e-08, 1.0530028205000000e-01, 1.0017201291180000e+02 },
    { 1.0109999999999999e-08, 4.9627990216200004e+00, 6.0054041039879996e+02 },
    { 1.0099999999999999e-08, 1.9822011758900000e+00, 4.8025159676200002e+02 },
    { 1.0090000000000000e-08, 4.0854433922000000e-01, 1.9392449561898000e+03 },
    { 1.0080000000000000e-08, 5.4254440599800002e+00, 5.5703893289940004e+02 },
    { 1.0070000000000000e-08, 5.8781227521000003e+00, 1.0283214178770000e+02 },
    { 1.0050000000000001e-08, 2.8351221663999998e-01, 9.7938009023358009e+03 },
    { 1.0029999999999999e-08, 5.7243912875699996e+00, 3.0679394693482000e+03 },
    { 1.0020000000000000e-08, 2.6128292303500000e+00, 5.2182919830149200e+04 },
    { 9.9900000000000005e-09, 9.0076451227999998e-01, 3.5691762464396002e+03 },
    { 9.9900000000000005e-09, 1.1331464215600000e+00, 9.2086443331850001e+02 },
    { 9.9800000000000007e-09, 1.0463166052399999e+00, 6.0697767545533998e+03 },
    { 9.9599999999999995e-09, 2.0876442702000000e-01, 1.3324847706675000e+03 },
    { 9.9400000000000000e-09, 1.6095067258000000e+00, 8.2962050851590004e+02 },
    { 9.9300000000000002e-09, 3.8133950222799999e+00, 1.7598337210689001e+03 },
    { 9.9300000000000002e-09, 7.0716715579000000e-01, 1.1262992222303001e+03 },
    { 9.9100000000000007e-09, 3.1479754936300002e+00, 7.2089866000480004e+02 },
    { 9.8899999999999996e-09, 6.2188818076699999e+00, 3.9494556279260001e+02 },
    { 9.8600000000000003e-09, 5.1043043394799996e+00, 3.6793825676590000e+03 },
    { 9.8400000000000008e-09, 3.9615540142999999e+00, 3.2812385647862002e+03 },
    { 9.8400000000000008e-09, 2.5300411689600000e+00, 7.2322514589612001e+03 },
    { 9.8299999999999993e-09, 6.1459713455699996e+00, 1.4323961511483999e+03 },
    { 9.7399999999999999e-09, 3.9168803324599999e+00, 5.1890589726159999e+02 },
    { 9.7399999999999999e-09, 2.4257091423000001e+00, 7.3832419408659996e+02 },
    { 9.7300000000000001e-09, 2.3619399302100001e+00, 1.7313907037070001e+02 },
    { 9.7300000000000001e-09, 4.2572964156999999e+00, 5.2252923398400003e+02 },
    { 9.7200000000000003e-09, 1.5861857420800001e+00, 8.9255325766720000e+02 },
    { 9.6999999999999992e-09, 2.4966410556100000e+00, 5.1322609901396390e+04 },
    { 9.6799999999999997e-09, 1.1565571052000000e-01, 2.7385360000370002e+02 },
    { 9.6699999999999999e-09, 9.8988062069000005e-01, 1.0529677415867600e+04 },
    { 9.6500000000000004e-09, 1.8371287236300000e+00, 5.5990744266009997e+03 },
    { 9.6299999999999992e-09, 1.1398671683000000e-01, 3.1691313451840000e+02 },
    { 9.6299999999999992e-09, 2.0943184390000000e-01, 7.7033802833740003e+02 },
    { 9.6099999999999997e-09, 1.7800795484000000e+00, 1.3332355301928999e+03 },
    { 9.5999999999999999e-09, 6.2093387350100002e+00, 2.0156710861598001e+03 },
    { 9.5999999999999999e-09, 1.8166086465600000e+00, 5.8422505651137999e+03 },
    { 9.5900000000000002e-09, 3.2306531541900001e+00, 1.0413831347085879e+05 },
    { 9.5900000000000002e-09, 1.3762135638999999e+00, 2.4968702593660001e+02 },
    { 9.5900000000000002e-09, 3.7181994909999999e-01, 1.7691116493940001e+02 },
    { 9.5900000000000002e-09, 1.0478926973300000e+00, 6.0768903015542001e+03 },
    { 9.5599999999999992e-09, 2.9202726691300001e+00, 1.5276246188407999e+03 },
    { 9.5300000000000000e-09, 1.6463302800699999e+00, 9.5146323083020002e+02 },
    { 9.5300000000000000e-09, 5.1927892471000003e-01, 1.2301427559742999e+03 },
    { 9.5200000000000002e-09, 9.0233317635999999e-01, 1.1978994270599001e+03 },
    { 9.5100000000000005e-09, 5.4670432236000002e+00, 1.7097327410619999e+02 },
    { 9.5000000000000007e-09, 1.3432883350600000e+00, 5.6829708148440000e+02 },
    { 9.5000000000000007e-09, 5.9456934148999996e-01, 9.4498282327580000e+02 },
    { 9.4799999999999995e-09, 3.4399814401200000e+00, 9.7463699926100006e+01 },
    { 9.4699999999999998e-09, 1.7040616871600001e+00, 1.7913262904579001e+03 },
    { 9.4699999999999998e-09, 3.0874634321599999e+00, 4.4779581952650000e+02 },
    { 9.4500000000000002e-09, 2.3474859211600001e+00, 7.3168372783780001e+02 },
    { 9.4300000000000007e-09, 7.1258336966000002e-01, 6.0185590704761998e+03 },
    { 9.4300000000000007e-09, 4.1740333814100001e+00, 1.1096244476739000e+03 },
    { 9.3999999999999998e-09, 1.3720426637400001e+00, 4.6176228109720000e+02 },
    { 9.3900000000000000e-09, 4.7669821039300002e+00, 6.3226247445139995e+02 },
    { 9.3900000000000000e-09, 3.1507362170600000e+00, 1.2573265246983599e+04 },
    { 9.3700000000000005e-09, 3.3474972560000000e+00, 5.4450875992720000e+02 },
    { 9.3700000000000005e-09, 4.2725418233000001e-01, 6.8189316428300003e+01 },
    { 9.3600000000000008e-09, 7.2034877665999997e-01, 8.4189004905720003e+02 },
    { 9.3499999999999994e-09, 1.8350234649899999e+00, 6.1288217748599997e+01 },
    { 9.3299999999999998e-09, 5.1299958961000003e-01, 8.4853051530599998e+02 },
    { 9.3299999999999998e-09, 1.0105086252999999e-01, 4.0153784493159998e+02 },
    { 9.3299999999999998e-09, 4.1536443771800000e+00, 1.6740064222377000e+03 },
    { 9.3100000000000003e-09, 2.1521830264000000e-01, 3.5499798604640000e+02 },
    { 9.3100000000000003e-09, 3.0243358241800000e+00, 5.9666839803348003e+03 },
    { 9.3100000000000003e-09, 2.9970677094100000e+00, 5.2957909051000001e+02 },
    { 9.3000000000000006e-09, 4.2101638406799999e+00, 8.1857480825200003e+02 },
    { 9.2900000000000008e-09, 2.6270977368500001e+00, 1.1201488830759999e+03 },
    { 9.2799999999999994e-09, 4.7778888711700001e+00, 5.8564776591153995e+03 },
    { 9.2699999999999996e-09, 4.1984943595999997e+00, 2.0491352405000001e+02 },
    { 9.2699999999999996e-09, 6.9113180698999999e-01, 2.2168466682600001e+02 },
    { 9.2699999999999996e-09, 2.6642994083599998e+00, 1.9620871162430001e+03 },
    { 9.2699999999999996e-09, 3.3029059972899999e+00, 1.0254416802454000e+03 },
    { 9.2300000000000006e-09, 1.5129825021600001e+00, 3.1710322435667999e+03 },
    { 9.1899999999999999e-09, 3.4732521119099999e+00, 1.0262448293975001e+03 },
    { 9.1899999999999999e-09, 4.8346745244299996e+00, 1.0551891444952000e+03 },
    { 9.1800000000000001e-09, 6.0382932158699996e+00, 5.2168692736147590e+04 },
    { 9.1800000000000001e-09, 3.4299044782800001e+00, 8.0354456509830004e+02 },
    { 9.1700000000000004e-09, 5.1544073899500003e+00, 6.6502706822760001e+02 },
    { 9.1600000000000006e-09, 3.7788199378399998e+00, 6.5541765656220002e+02 },
    { 9.1499999999999992e-09, 1.5291133436000001e-01, 1.3293033769297999e+03 },
    { 9.1399999999999995e-09, 6.1911370419900003e+00, 8.3821852822500000e+02 },
    { 9.1399999999999995e-09, 1.0655271622400000e+00, 3.8575684225250001e+02 },
    { 9.0799999999999993e-09, 2.6950699609000001e-01, 7.3342883297699996e+02 },
    { 9.0699999999999995e-09, 2.1402813154400002e+00, 6.6380322795020004e+02 },
    { 9.0599999999999997e-09, 5.1176519017500004e+00, 2.9616653013918999e+03 },
    { 9.0500000000000000e-09, 2.7196041246100000e+00, 1.2577031721001999e+03 },
    { 9.0200000000000007e-09, 4.4376013107799999e+00, 1.4072663692348001e+03 },
    { 8.9899999999999998e-09, 5.6653913725100002e+00, 6.0595703637019994e+02 },
    { 8.9600000000000005e-09, 1.2483572464299999e+00, 6.3861686242100004e+03 },
    { 8.9600000000000005e-09, 2.7836778819000002e-01, 5.8009354127370000e+02 },
    { 8.9500000000000007e-09, 4.3423282898500002e+00, 7.7400954916959995e+02 },
    { 8.9299999999999996e-09, 9.1586743264000003e-01, 6.3847772888000002e+01 },
    { 8.8699999999999994e-09, 1.7033455697900000e+00, 2.6514501332450200e+04 },
    { 8.8200000000000006e-09, 6.1484264379000004e+00, 9.6835945811164001e+03 },
    { 8.8200000000000006e-09, 1.9514325519500000e+00, 8.7046185713939997e+02 },
    { 8.8100000000000008e-09, 4.5802111414900004e+00, 2.8435946736463002e+03 },
    { 8.8100000000000008e-09, 4.5352093773100002e+00, 5.2262560220360001e+02 },
    { 8.8100000000000008e-09, 4.1400449705300000e+00, 1.5742750366080000e+02 },
    { 8.8100000000000008e-09, 3.8911738496499999e+00, 2.6917068721520002e+02 },
    { 8.7999999999999994e-09, 1.3023886546999999e-01, 5.4785021235930003e+02 },
    { 8.7799999999999999e-09, 6.2476789800799999e+00, 1.6623797169100001e+02 },
    { 8.7700000000000001e-09, 5.4124600185700000e+00, 4.6427852706304002e+03 },
    { 8.7700000000000001e-09, 4.5226859243100002e+00, 6.5323060843830001e+02 },
    { 8.7500000000000006e-09, 4.7626060282999999e-01, 7.1149749114359997e+02 },
    { 8.7299999999999994e-09, 4.0865661944899996e+00, 6.3624023718799997e+01 },
    { 8.7199999999999997e-09, 3.4984427654900001e+00, 1.0293271552420001e+02 },
    { 8.7099999999999999e-09, 4.8742243907999999e-01, 1.0325283291300001e+02 },
    { 8.7099999999999999e-09, 5.2354702088000005e-01, 9.4702954856783999e+03 },
    { 8.6800000000000006e-09, 4.4886276523000002e-01, 4.4949274055590001e+02 },
    { 8.6699999999999992e-09, 5.4709289480300001e+00, 1.5519227142710000e+02 },
    { 8.6599999999999995e-09, 4.2820654405300003e+00, 1.5354889253670001e+03 },
    { 8.6599999999999995e-09, 5.1243219681700003e+00, 1.2938786542777000e+03 },
    { 8.5899999999999995e-09, 1.0553060905400000e+00, 1.8865547581502999e+03 },
    { 8.5799999999999997e-09, 5.1225152968499996e+00, 1.0350029078010000e+03 },
    { 8.5700000000000000e-09, 3.1763375990400000e+00, 1.1585425511446999e+03 },
    { 8.5500000000000005e-09, 3.9885387894000002e+00, 1.3103933701397000e+03 },
    { 8.5400000000000007e-09, 2.3340791395999999e-01, 6.1799830757727996e+03 },
    { 8.5400000000000007e-09, 4.6873622405999998e-01, 1.7849635029824999e+03 },
    { 8.5400000000000007e-09, 9.8057544228000004e-01, 2.6036021918500001e+02 },
    { 8.5299999999999993e-09, 4.5937367787500003e+00, 4.7601051388506003e+03 },
    { 8.5299999999999993e-09, 1.5997180148800001e+00, 1.2356117715723999e+03 },
    { 8.5199999999999995e-09, 5.1301141845999998e-01, 6.0043319764746002e+03 },
    { 8.5000000000000000e-09, 6.0051998782200000e+00, 8.9335640681929999e+02 },
    { 8.4900000000000003e-09, 2.6905396323200002e+00, 1.4266209845489999e+02 },
    { 8.4700000000000007e-09, 2.9761030043000001e+00, 1.0523165672982000e+03 },
    { 8.4599999999999993e-09, 6.0229513030400001e+00, 9.3101636170510005e+02 },
    { 8.4599999999999993e-09, 3.2817929651000000e-01, 6.6964475056380002e+02 },
    { 8.4499999999999996e-09, 2.8528073608200000e+00, 7.2631528597620002e+02 },
    { 8.4300000000000000e-09, 3.1259990623600000e+00, 2.4846318565920001e+02 },
    { 8.4300000000000000e-09, 4.9052197578200003e+00, 1.7813500521680001e+02 },
    { 8.4100000000000005e-09, 2.4574554186399999e+00, 1.0513051753418999e+03 },
    { 8.4000000000000008e-09, 2.8106757743399999e+00, 9.8085381846614000e+03 },
    { 8.4000000000000008e-09, 5.6005062336700000e+00, 1.3364169239306000e+03 },
    { 8.4000000000000008e-09, 1.4823353128300001e+00, 1.3851749687070001e+03 },
    { 8.3899999999999994e-09, 4.7165462544599999e+00, 1.2316272286825999e+03 },
    { 8.3899999999999994e-09, 5.6475821892900004e+00, 1.3362449706799200e+04 },
    { 8.3799999999999996e-09, 2.9688727807099999e+00, 1.8094855377225999e+03 },
    { 8.3600000000000001e-09, 3.8530973007500000e+00, 5.7053231371810000e+02 },
    { 8.3099999999999996e-09, 5.4907671784000001e-01, 2.7483662059539001e+03 },
    { 8.3099999999999996e-09, 5.6460075539899996e+00, 8.9877303279069997e+02 },
    { 8.3099999999999996e-09, 5.4851921128800001e+00, 4.3663249918359998e+02 },
    { 8.3099999999999996e-09, 3.4729909833599999e+00, 5.9571448525840003e+02 },
    { 8.2999999999999999e-09, 1.7740765484700001e+00, 1.0168436605363000e+03 },
    { 8.2900000000000001e-09, 4.1373531425500003e+00, 1.3032798231388999e+03 },
    { 8.2499999999999994e-09, 6.2630510792000005e-01, 2.0007086448546801e+04 },
    { 8.2499999999999994e-09, 5.0649811527499997e+00, 1.1634379122543000e+03 },
    { 8.2499999999999994e-09, 4.4056572951100001e+00, 1.0128633231634000e+03 },
    { 8.1899999999999992e-09, 2.6737806704899998e+00, 1.0316378320429600e+04 },
    { 8.1799999999999995e-09, 3.1030222579200002e+00, 3.9197661737599998e+02 },
    { 8.1699999999999997e-09, 2.8553061077700002e+00, 4.8631979130691998e+03 },
    { 8.1500000000000002e-09, 6.1645957460199998e+00, 5.2175806283148399e+04 },
    { 8.1400000000000004e-09, 3.7021427088999997e-01, 5.4547196777370004e+02 },
    { 8.1099999999999995e-09, 4.2702580747600001e+00, 1.7991905969841000e+03 },
    { 8.0999999999999997e-09, 3.8136828400799998e+00, 6.1483694036960003e+02 },
    { 8.0600000000000007e-09, 1.6685922332600001e+00, 1.0110192771992401e+04 },
    { 8.0499999999999993e-09, 1.9626723777999999e+00, 9.1300012679229997e+02 },
    { 8.0299999999999998e-09, 4.0010516634300002e+00, 1.9992859354545191e+04 },
    { 8.0299999999999998e-09, 5.3952030138499998e+00, 1.6889842757647000e+03 },
    { 8.0000000000000005e-09, 4.1317704970899998e+00, 4.6356717236295999e+03 },
    { 8.0000000000000005e-09, 5.5998643548300002e+00, 5.2973914920439995e+02 },
    { 8.0000000000000005e-09, 4.9960220120300001e+00, 1.4711551657979999e+02 },
    { 8.0000000000000005e-09, 3.0351968081499998e+00, 2.7948267429620000e+02 },
    { 7.9900000000000007e-09, 5.1468826889999999e+00, 7.3491330568529997e+02 },
    { 7.9799999999999993e-09, 8.9814256580999996e-01, 4.3790452357080000e+02 },
    { 7.9799999999999993e-09, 2.9956126386599999e+00, 1.8823874949360001e+02 },
    { 7.9500000000000001e-09, 2.4039429677999999e+00, 5.2964278098479997e+02 },
    { 7.9500000000000001e-09, 2.5469172056499998e+00, 8.6652970387630000e+02 },
    { 7.9500000000000001e-09, 3.1851522807400001e+00, 1.5780271950199001e+03 },
    { 7.9300000000000005e-09, 4.2607307276900004e+00, 7.4617145427030005e+02 },
    { 7.9300000000000005e-09, 3.1940500524200002e+00, 5.4227659212385997e+03 },
    { 7.8899999999999998e-09, 3.2094579002099999e+00, 1.0309277421859999e+03 },
    { 7.8899999999999998e-09, 1.2332567631300000e+00, 6.7803032195180003e+02 },
    { 7.8700000000000003e-09, 5.1310452400899997e+00, 4.8715269544170002e+02 },
    { 7.8600000000000006e-09, 1.0931070227799999e+00, 1.8770105769372001e+03 },
    { 7.8500000000000008e-09, 4.6068916559100002e+00, 3.8926816630970002e+03 },
    { 7.8399999999999994e-09, 5.9984182425799997e+00, 5.1582491417233996e+03 },
    { 7.8299999999999996e-09, 4.5069582657299998e+00, 6.6427630870220003e+02 },
    { 7.8299999999999996e-09, 6.0564215365700003e+00, 2.2700492457780001e+02 },
    { 7.8299999999999996e-09, 5.1163899372000001e+00, 1.9959326629820001e+02 },
    { 7.8199999999999999e-09, 5.6561734425500001e+00, 3.0923013854599998e+02 },
    { 7.8100000000000001e-09, 2.1919449583000001e+00, 1.2552725378878999e+03 },
    { 7.7900000000000006e-09, 2.5407875857900000e+00, 6.8940609061580005e+02 },
    { 7.7799999999999992e-09, 4.0481084334900004e+00, 2.3217680150779001e+03 },
    { 7.7699999999999994e-09, 2.7859528602800001e+00, 2.0140265547571000e+03 },
    { 7.7699999999999994e-09, 4.4872832349299996e+00, 1.4599565672742999e+03 },
    { 7.7599999999999997e-09, 1.1552710493000000e-01, 1.7032113697663001e+03 },
    { 7.7099999999999992e-09, 2.2702233329400001e+00, 8.8772733252679996e+02 },
    { 7.7099999999999992e-09, 3.1090923033800002e+00, 6.0587310542895002e+03 },
    { 7.6899999999999997e-09, 5.8709275819000002e+00, 1.0522201990786000e+03 },
    { 7.6799999999999999e-09, 1.6275658032600000e+00, 8.4533207522580005e+02 },
    { 7.6799999999999999e-09, 4.5591121701399997e+00, 2.0934910661129002e+03 },
    { 7.6700000000000002e-09, 4.6630794514899998e+00, 4.9819839570559998e+02 },
    { 7.6700000000000002e-09, 4.4676289062399999e+00, 2.4390878832981002e+03 },
    { 7.6600000000000004e-09, 6.2479960584400001e+00, 1.1010264279648000e+03 },
    { 7.5800000000000007e-09, 3.4567197825000001e-01, 2.5200892569889002e+03 },
    { 7.5699999999999993e-09, 2.6598749702600002e+00, 5.4959816523824002e+03 },
    { 7.5200000000000005e-09, 5.6556241429999998e-01, 4.1310481005730003e+02 },
    { 7.4899999999999996e-09, 2.2964936693300002e+00, 7.8552833018549995e+02 },
    { 7.4400000000000000e-09, 8.3167351532000000e-01, 1.2623860848887000e+03 },
    { 7.4300000000000002e-09, 2.2170768460399999e+00, 1.0160929010109001e+03 },
    { 7.4199999999999996e-09, 5.5272611798300000e+00, 2.3320629558164001e+03 },
    { 7.4199999999999996e-09, 5.7093516071400003e+00, 7.3194436026870005e+02 },
    { 7.4099999999999998e-09, 5.2308522475199997e+00, 2.6302955782083000e+03 },
    { 7.3700000000000000e-09, 2.6503786305600001e+00, 2.8563301345050002e+02 },
    { 7.3600000000000002e-09, 5.9131680274300003e+00, 4.0370364119610002e+02 },
    { 7.3099999999999998e-09, 2.6013947494499998e+00, 9.9564603188579997e+02 },
    { 7.2900000000000003e-09, 4.6594656153100003e+00, 1.6496273998495001e+03 },
    { 7.2900000000000003e-09, 1.6195016605699999e+00, 9.3888066823129998e+02 },
    { 7.2600000000000002e-09, 1.7410508691700000e+00, 1.1939504175950000e+02 },
    { 7.2000000000000000e-09, 1.5650065552000001e+00, 1.0371171527998279e+05 },
    { 7.2000000000000000e-09, 2.3831524300000001e-01, 2.5562491676980000e+02 },
    { 7.2000000000000000e-09, 4.5325436078400001e+00, 9.9889407505091003e+03 },
    { 7.1900000000000002e-09, 5.7933919071500002e+00, 4.6743953949949997e+02 },
    { 7.1099999999999996e-09, 5.1488865944999995e-01, 2.5739127376840003e+02 },
    { 7.1099999999999996e-09, 7.8076574240999996e-01, 4.6112914165080002e+02 },
    { 7.0900000000000001e-09, 3.1502712383599998e+00, 2.4312235767718998e+03 },
    { 7.0699999999999998e-09, 3.1281754154899999e+00, 5.6542242639412998e+03 },
    { 7.0699999999999998e-09, 3.0072030778199998e+00, 1.1169079392770000e+02 },
    { 7.0600000000000000e-09, 3.1178317096999999e+00, 7.4028174754689996e+02 },
    { 7.0500000000000003e-09, 5.8226093816800004e+00, 3.4660834722210002e+03 },
    { 7.0299999999999999e-09, 3.4276578699999999e-01, 8.0901358069640003e+02 },
    { 7.0200000000000002e-09, 1.2454469547900000e+00, 5.9884348936940000e+02 },
    { 6.8500000000000001e-09, 4.2901016487400003e+00, 4.5505237887919998e+02 },
    { 6.8400000000000004e-09, 5.8507603441100002e+00, 1.5851407420206999e+03 },
    { 6.8100000000000003e-09, 8.3326775082000004e-01, 4.5809076026499997e+02 },
    { 6.8100000000000003e-09, 5.7128158592100000e+00, 2.1438936422920001e+03 },
    { 6.8100000000000003e-09, 1.8401512669900000e+00, 2.9545517543911001e+03 },
};

static const VSOP87Term VSOP87A_SATURN_Y1[917] = {
    { 5.3738891349999998e-02, 0.0000000000000000e+00, 0.0000000000000000e+00 },
    { 3.0905751520000000e-02, 2.7034689090600001e+00, 4.2659819087599999e+02 },
    { 2.7415943120000001e-02, 4.2666763601500000e+00, 2.0618554843720000e+02 },
    { 2.6474896769999999e-02, 3.7613229888899999e+00, 2.2041264243879999e+02 },
    { 6.3152052699999997e-03, 5.0324550527999996e+00, 7.1135470007999997e+00 },
    { 3.1227193000000000e-03, 3.2585020502300002e+00, 4.1948464387519999e+02 },
    { 2.5679970100000002e-03, 1.9535181975800000e+00, 6.3989728631399998e+02 },
    { 1.8943331900000001e-03, 2.9150184081899999e+00, 4.3371173787679999e+02 },
    { 1.6413355299999999e-03, 5.2923929006600003e+00, 2.1329909543799999e+02 },
    { 1.1679122699999999e-03, 5.8914667575999999e+00, 1.1020632121940000e+02 },
    { 6.7210919000000005e-04, 2.1704263634399998e+00, 3.1639186965660002e+02 },
    { 6.7003291999999999e-04, 3.6310107551400002e+00, 2.2752618943959999e+02 },
    { 3.3002405999999997e-04, 4.3552740580100000e+00, 1.9907200143640000e+02 },
    { 3.0628998000000000e-04, 4.8886176077199996e+00, 1.4227094001599999e+01 },
    { 2.2234714000000000e-04, 4.6221277923099997e+00, 1.0309277421860000e+02 },
    { 1.8945004000000000e-04, 1.2041249384499999e+00, 8.5319638175199998e+02 },
    { 1.8079958999999999e-04, 3.5156615325099998e+00, 2.0936694217490000e+02 },
    { 1.7791543000000000e-04, 4.5321414064900001e+00, 2.1723124870109999e+02 },
    { 1.6717121999999999e-04, 3.0027079275199999e+00, 6.3278373931320004e+02 },
    { 1.6320701000000000e-04, 3.2978403097000002e+00, 2.1648048917570000e+02 },
    { 1.6149947000000000e-04, 4.0418643251699997e+00, 3.2350541665740002e+02 },
    { 1.5944258000000000e-04, 4.7450326516899999e+00, 2.1011770170029999e+02 },
    { 1.4481431000000001e-04, 2.1029829864999998e+00, 6.4701083331480004e+02 },
    { 1.1084040000000000e-04, 4.7407387175400002e+00, 1.1731986822019999e+02 },
    { 9.9054910000000000e-05, 3.6025859937500000e+00, 2.0225339517410001e+02 },
    { 8.7260509999999996e-05, 4.4634134287700000e+00, 2.2434479570190001e+02 },
    { 6.5855969999999999e-05, 4.0732632048699999e+00, 3.0927832265580003e+02 },
    { 5.5059780000000003e-05, 2.8320739023999999e+00, 4.4082528487759998e+02 },
    { 5.4240409999999997e-05, 1.0319768440999999e+00, 1.1045700263900001e+01 },
    { 4.1782660000000000e-05, 3.0103851207600001e+00, 4.1237109687440000e+02 },
    { 4.0499050000000003e-05, 5.1748876764500000e+00, 9.5979227217800002e+01 },
    { 2.7352560000000000e-05, 2.5397585040899999e+00, 1.4956319713459999e+02 },
    { 2.3690239999999998e-05, 2.4049792791700000e+00, 5.2257741809380002e+02 },
    { 1.7452580000000002e-05, 5.5057601545599999e+00, 2.7703499374140000e+02 },
    { 1.6927900000000001e-05, 2.3992650252900001e+00, 4.2266603761290003e+02 },
    { 1.5460059999999998e-05, 3.7315692559899998e+00, 3.3061896365820002e+02 },
    { 1.5197320000000001e-05, 6.1288066463700002e+00, 3.9321532631000000e+00 },
    { 1.3936220000000000e-05, 4.6221427717500001e+00, 3.1813937377000001e+00 },
    { 1.3893539999999999e-05, 4.6207025894999998e-01, 1.0664954771900000e+03 },
    { 1.3448910000000000e-05, 1.4179359368500000e+00, 2.1085141488319999e+02 },
    { 1.3325530000000000e-05, 3.4919981229600001e+00, 2.3463973644039999e+02 },
    { 1.3317860000000001e-05, 3.3834520814000002e-01, 2.1574677599280000e+02 },
    { 1.3141250000000001e-05, 4.3889165659999998e+00, 6.2567019231239999e+02 },
    { 1.3009340000000000e-05, 8.3727681905999995e-01, 4.1555249061209997e+02 },
    { 1.1743190000000001e-05, 2.7260998433500001e+00, 8.4608283475120004e+02 },
    { 1.1480100000000000e-05, 3.0437473888200000e+00, 5.3680451209540001e+02 },
    { 1.1279520000000001e-05, 3.8730969230699999e+00, 4.2341679713830001e+02 },
    { 1.1276460000000000e-05, 1.3108890621300000e+00, 8.6030992875280003e+02 },
    { 1.1023610000000000e-05, 2.5053530601400000e+00, 4.2977958461370002e+02 },
    { 9.6135799999999992e-06, 4.0227802588700001e+00, 5.2969096509459996e+02 },
    { 8.8305800000000001e-06, 3.8461836072000000e+00, 4.3764389113990001e+02 },
    { 8.4526600000000001e-06, 2.6103774900099999e+00, 8.3896928775039999e+02 },
    { 8.2112000000000003e-06, 3.1747699226199999e+00, 2.2359403617650000e+02 },
    { 8.1619299999999993e-06, 4.7330173583499997e+00, 2.1340641002400002e+01 },
    { 7.1152099999999999e-06, 4.5495568954300003e+00, 4.3053034413910001e+02 },
    { 6.9886999999999997e-06, 4.5669190868199996e+00, 3.0216477565500003e+02 },
    { 6.9002800000000001e-06, 5.4865680869000002e+00, 8.8865680217000005e+01 },
    { 6.6371800000000000e-06, 5.4456026111300000e+00, 7.4299006053259995e+02 },
    { 5.9758300000000000e-06, 3.5268718753299999e+00, 1.2443341522100000e+02 },
    { 5.3660599999999999e-06, 4.9975389583099998e+00, 1.0593819301891999e+03 },
    { 5.1667000000000000e-06, 5.8813204065400004e+00, 7.3587651353180001e+02 },
    { 4.7985300000000003e-06, 4.8589786751800004e+00, 2.0300415469949999e+02 },
    { 4.7183899999999997e-06, 2.0382697744099998e+00, 6.5412438031559998e+02 },
    { 4.0362399999999998e-06, 2.7113905278399999e+00, 5.1546387109299997e+02 },
    { 4.0249799999999999e-06, 2.8546669052899998e+00, 8.5827298831199997e+01 },
    { 3.8014600000000001e-06, 4.4284964588799998e+00, 1.9195845443560000e+02 },
    { 3.7698599999999999e-06, 2.1407639628999999e+00, 7.6266071275599998e+01 },
    { 3.3977699999999999e-06, 1.1673233717300000e+00, 2.1796496188399999e+02 },
    { 3.3675099999999998e-06, 1.5100695523800001e+00, 2.0373786788240000e+02 },
    { 3.2938499999999998e-06, 2.2509041220500001e+00, 1.1553611574070001e+03 },
    { 3.1327900000000000e-06, 7.9690491776000005e-01, 6.2885158605009997e+02 },
    { 3.1250799999999998e-06, 3.5286093609600000e+00, 9.4206206196899996e+02 },
    { 3.0493199999999999e-06, 4.8966455563100002e+00, 2.3145834270270001e+02 },
    { 2.9996699999999999e-06, 5.8486918118999998e-01, 2.0863322899200000e+02 },
    { 2.9247999999999999e-06, 2.4003627114000001e-01, 2.2286032299359999e+02 },
    { 2.8899800000000002e-06, 1.6242129656199999e+00, 7.2876296653099996e+02 },
    { 2.7155300000000001e-06, 1.1338071529200000e+00, 2.8808069400530002e+02 },
    { 2.6379199999999999e-06, 1.9850073782900000e+00, 3.6286229257259998e+02 },
    { 2.6166500000000002e-06, 5.5804380018200002e+00, 1.8159247264699999e+01 },
    { 2.5974100000000002e-06, 4.4204995933099998e+00, 1.0294940738499999e+01 },
    { 2.3087600000000000e-06, 5.8064500748999999e+00, 9.4917560896980001e+02 },
    { 2.1389299999999999e-06, 4.4602586751100004e+00, 2.0788246946660001e+02 },
    { 2.1222600000000002e-06, 5.1516577793999996e-01, 1.3851749687070000e+02 },
    { 2.0835500000000002e-06, 2.7939257402500002e+00, 4.0843894361129998e+02 },
    { 2.0810600000000000e-06, 3.5933300777300001e+00, 2.1871572140940000e+02 },
    { 1.9575299999999999e-06, 1.9228651196300000e+00, 5.2690198039499997e+01 },
    { 1.9462199999999998e-06, 5.1559231982600000e+00, 3.4077089204480001e+02 },
    { 1.9400900000000000e-06, 4.6400220620899999e+00, 2.0076892246579999e+02 },
    { 1.7811800000000001e-06, 4.7000850902500000e+00, 1.6060889739850001e+02 },
    { 1.7782200000000001e-06, 4.6497015984800001e+00, 2.1621985674480001e+02 },
    { 1.7751100000000000e-06, 3.3700615066799999e+00, 2.1037833413120001e+02 },
    { 1.7027000000000000e-06, 1.0616988642500000e+00, 2.0767002114549999e+02 },
    { 1.6951399999999999e-06, 5.9001086524000002e+00, 3.5033211960040001e+02 },
    { 1.6790299999999999e-06, 3.4316587067400000e+00, 2.2582926841020000e+02 },
    { 1.6594200000000001e-06, 2.6591592808400000e+00, 8.3185574074960005e+02 },
    { 1.6487400000000000e-06, 3.0458452880100002e+00, 2.2091400527800001e+01 },
    { 1.6397400000000000e-06, 6.6806989122000004e-01, 2.1892816973050000e+02 },
    { 1.6377199999999999e-06, 4.3381090445400003e+00, 9.5628915597059995e+02 },
    { 1.6035300000000000e-06, 3.8234469262999998e-01, 1.9513984817330001e+02 },
    { 1.5327700000000000e-06, 5.8539434181500001e+00, 5.6622351302600002e+01 },
    { 1.5155600000000001e-06, 3.6739662394699999e+00, 1.2747179660680000e+02 },
    { 1.5113900000000000e-06, 1.5689792575199999e+00, 6.3596513305090002e+02 },
    { 1.4503399999999999e-06, 5.0152187712799998e+00, 9.5612275556000004e+00 },
    { 1.3710699999999999e-06, 5.5803433170499996e+00, 4.9033408917940000e+02 },
    { 1.3703200000000000e-06, 2.9467138477099999e+00, 5.4391805909619995e+02 },
    { 1.3683800000000000e-06, 1.9835248691900000e+00, 1.4717530270636000e+03 },
    { 1.3590900000000000e-06, 2.8249723288499999e+00, 1.4244965013379999e+02 },
    { 1.3575200000000000e-06, 3.2018158405900001e+00, 1.2584539316256000e+03 },
    { 1.3465400000000000e-06, 5.9692733582299997e+00, 7.0363318461740005e+02 },
    { 1.3105700000000000e-06, 2.4591848289999998e+00, 7.5010360753340001e+02 },
    { 1.2885300000000000e-06, 4.0908143942399997e+00, 6.1855664531160005e+02 },
    { 1.2442800000000000e-06, 2.7156076148500001e+00, 4.4793883187839998e+02 },
    { 1.2363200000000000e-06, 1.4236071000000000e-01, 1.2530172972200001e+01 },
    { 1.2293100000000001e-06, 1.2863850557600001e+00, 4.8322054217860000e+02 },
    { 1.1714500000000000e-06, 1.9340824149800000e+00, 2.6992144674060000e+02 },
    { 1.0963100000000001e-06, 3.5803490283300001e+00, 2.6598929347749998e+02 },
    { 1.0822199999999999e-06, 1.1761412803700000e+00, 5.6511568774670002e+02 },
    { 1.0753100000000000e-06, 4.7701573313000001e-01, 4.2415051032119999e+02 },
    { 1.0535800000000000e-06, 2.0108608770099998e+00, 2.0910630974399999e+02 },
    { 1.0507700000000000e-06, 6.0139497839700002e+00, 2.1749188113200000e+02 },
    { 1.0503100000000000e-06, 1.4616217214600000e+00, 1.0451548361876000e+03 },
    { 9.9879000000000001e-07, 6.1426324114600002e+00, 1.2655674786264001e+03 },
    { 9.8664000000000005e-07, 5.9831519267799997e+00, 1.2797945726280000e+03 },
    { 9.2818000000000004e-07, 4.7222918448100000e+00, 7.0849445304200003e+01 },
    { 9.1314000000000004e-07, 7.3720537016999999e-01, 4.9744763618019999e+02 },
    { 8.8418999999999998e-07, 5.2765914900999999e-01, 1.0736090241908000e+03 },
    { 8.4107999999999996e-07, 2.0089720995999999e-01, 2.1944943459230001e+02 },
    { 8.4025000000000002e-07, 4.6015099021800001e+00, 1.1338771495710000e+02 },
    { 8.2305000000000000e-07, 1.7146935133500001e+00, 6.4307868005169996e+02 },
    { 8.0531000000000000e-07, 3.1392282501100000e+00, 6.5094298657790000e+02 },
    { 7.9808000000000001e-07, 4.9828411835499997e+00, 1.0627416795630000e+02 },
    { 7.6736000000000001e-07, 6.1812871360399999e+00, 2.8414854074220000e+02 },
    { 7.3483999999999998e-07, 2.7571015645400001e+00, 3.4470304530790003e+02 },
    { 7.2994000000000004e-07, 2.4091607913200002e+00, 1.1482476104062000e+03 },
    { 7.0938999999999997e-07, 8.3631473279000001e-01, 2.5143213107579999e+02 },
    { 6.8746000000000004e-07, 1.7244811779400000e+00, 1.9120769491019999e+02 },
    { 6.7693999999999997e-07, 3.2916164353299999e+00, 6.3671589257630001e+02 },
    { 6.7011000000000001e-07, 6.0753473546999999e+00, 1.0702492748170000e+02 },
    { 6.6519000000000004e-07, 2.9608741255600002e+00, 3.3773251065900001e+02 },
    { 6.5944000000000003e-07, 1.5872702055800001e+00, 2.0714875628370001e+02 },
    { 6.3898000000000002e-07, 4.1807886669000000e-01, 2.3539049596580000e+02 },
    { 6.3641999999999998e-07, 2.9062317329399998e+00, 1.5748458012822000e+03 },
    { 6.3066999999999997e-07, 2.2578302187100001e+00, 3.5893013930950002e+02 },
    { 6.3038999999999997e-07, 2.4083443840999998e+00, 4.3689313161450002e+02 },
    { 6.2162999999999999e-07, 1.2620447187999999e+00, 7.4781598567299994e+01 },
    { 6.1778999999999998e-07, 4.7658912542999998e-01, 1.3757737998458001e+03 },
    { 6.0208999999999996e-07, 3.5929274758999998e+00, 4.7928838891549998e+02 },
    { 5.9885999999999995e-07, 1.7246210112500000e+00, 1.7881448967202000e+03 },
    { 5.9381999999999995e-07, 5.8999971226900003e+00, 1.0522683831884001e+03 },
    { 5.8620999999999995e-07, 2.6943174364100000e+00, 4.1703696332039999e+02 },
    { 5.6655999999999996e-07, 2.8534686756299998e+00, 9.8899988524600005e+01 },
    { 5.4845000000000002e-07, 3.6330591335800002e+00, 4.1630325013750002e+02 },
    { 5.2992000000000002e-07, 2.3735797368600000e+00, 4.6658664459999999e+00 },
    { 5.2203000000000002e-07, 6.2684541286700002e+00, 1.2125202148330000e+02 },
    { 5.2198999999999998e-07, 5.2231362459100001e+00, 4.0450679034820001e+02 },
    { 5.2017000000000003e-07, 4.4065942412999998e-01, 2.2046082654860001e+02 },
    { 5.1982999999999996e-07, 9.8560774735000001e-01, 1.7516605980020000e+02 },
    { 5.1724000000000004e-07, 5.9717397378600001e+00, 1.5819593482830001e+03 },
    { 5.1547999999999995e-07, 1.7653026657999999e+00, 1.1624747044077999e+03 },
    { 5.1282999999999997e-07, 5.3281815875599996e+00, 5.5406998748280000e+02 },
    { 5.1114999999999997e-07, 3.6315467759200000e+00, 9.3494851496820002e+02 },
    { 5.0399999999999996e-07, 6.2333404621500001e+00, 4.2904587143079999e+02 },
    { 4.9994999999999997e-07, 1.0898057383299999e+00, 1.8105576652360000e+02 },
    { 4.8966000000000001e-07, 5.3744540006200001e+00, 1.1413847448250000e+02 },
    { 4.8930000000000002e-07, 6.1944877125999998e-01, 2.4554242435239999e+02 },
    { 4.8559000000000000e-07, 1.5144841375300000e+00, 1.5104766984290001e+02 },
    { 4.7109000000000002e-07, 3.9163135029000001e-01, 3.5181659230870002e+02 },
    { 4.5475000000000002e-07, 4.0542517807000000e-01, 8.4215068148809996e+02 },
    { 4.4083000000000001e-07, 6.8386602503000005e-01, 2.0522234059069999e+02 },
    { 4.1900999999999999e-07, 5.6360397538000004e+00, 8.1752133216199994e+01 },
    { 4.1367000000000002e-07, 1.2803657127400001e+00, 1.3615467058442000e+03 },
    { 4.0978000000000001e-07, 2.1772706738699998e+00, 6.2173803904930003e+02 },
    { 4.0865000000000000e-07, 4.2862360556099999e+00, 6.4382943957709995e+02 },
    { 4.0579999999999999e-07, 1.2522343584300000e+00, 8.6742347575359997e+02 },
    { 3.8938000000000001e-07, 2.8874663384499999e+00, 1.3154696222179999e+02 },
    { 3.8753999999999999e-07, 9.5034448967999996e-01, 1.9832124191099999e+02 },
    { 3.7557000000000002e-07, 5.1334090025399997e+00, 1.5890728952838001e+03 },
    { 3.7519000000000001e-07, 8.1273995928999998e-01, 2.2827694896500000e+02 },
    { 3.6820999999999999e-07, 1.4074640471400000e+00, 2.0613736432740001e+02 },
    { 3.6570000000000000e-07, 3.1823772130000000e-01, 1.4807872442630000e+02 },
    { 3.6296999999999997e-07, 4.0937854150900002e+00, 4.4475743814070000e+02 },
    { 3.6203999999999998e-07, 1.1792236480700000e+00, 2.1404985496340001e+02 },
    { 3.5352000000000001e-07, 8.0858694073000004e-01, 5.0137978944330001e+02 },
    { 3.5277000000000000e-07, 4.7954858194999996e+00, 2.9505122865419997e+02 },
    { 3.4507000000000001e-07, 2.1510552220100001e+00, 1.4646394800628000e+03 },
    { 3.4261000000000002e-07, 1.3083237539300001e+00, 2.8956516671359998e+02 },
    { 3.3504000000000001e-07, 5.7575527640099997e+00, 4.3615941843159999e+02 },
    { 3.3253999999999999e-07, 3.4604465898800001e+00, 4.0525754987360000e+02 },
    { 3.3004000000000001e-07, 3.6631725462600002e+00, 1.7787437278589999e+02 },
    { 3.2822000000000001e-07, 6.2395825569999996e-01, 2.1254833591260001e+02 },
    { 3.2207000000000002e-07, 1.6511035319100000e+00, 4.1406801790380001e+02 },
    { 3.2160999999999998e-07, 2.3596694801000001e-01, 2.7555052103309998e+02 },
    { 3.2108000000000001e-07, 1.4188824770599999e+00, 1.4788665740644001e+03 },
    { 3.1458000000000000e-07, 9.7131994074000005e-01, 3.2243328914400003e+01 },
    { 3.1052000000000000e-07, 2.6252234173400000e+00, 1.8912376709388000e+03 },
    { 2.9408000000000000e-07, 2.7613052986900000e+00, 7.8713751830400000e+01 },
    { 2.9374999999999999e-07, 4.1420148317100001e+00, 2.1478356814630001e+02 },
    { 2.9162000000000001e-07, 4.3594149706599996e+00, 2.4872381809010000e+02 },
    { 2.7476000000000001e-07, 2.2891631804900001e+00, 1.1439910691340000e+02 },
    { 2.7350000000000001e-07, 4.2896012531099998e+00, 3.5574874557179999e+02 },
    { 2.7145000000000001e-07, 3.1547938457000002e-01, 1.6921656695024001e+03 },
    { 2.6801999999999999e-07, 1.4891302818400001e+00, 2.1045367663768002e+03 },
    { 2.5470999999999998e-07, 2.4587068953000002e+00, 6.9152524274800001e+01 },
    { 2.5319000000000001e-07, 5.9372282252000002e+00, 4.2193232442999999e+02 },
    { 2.4965000000000002e-07, 3.3594203695500000e+00, 1.2513403846248000e+03 },
    { 2.4810000000000000e-07, 5.5456448736399997e+00, 4.2563498302950001e+02 },
    { 2.4789999999999999e-07, 4.1775395345400002e+00, 9.9160620955499994e+01 },
    { 2.4663999999999999e-07, 2.9908891620400002e+00, 4.2756139872249997e+02 },
    { 2.4545000000000002e-07, 5.6907563866200004e+00, 1.8983512179396000e+03 },
    { 2.4097000000000002e-07, 1.2939290377399999e+00, 3.9356875915200000e+01 },
    { 2.3967999999999999e-07, 3.6677339014999999e-01, 4.3126405732199999e+02 },
    { 2.3643999999999999e-07, 1.4419819853200000e+00, 2.7851946644970002e+02 },
    { 2.3120000000000001e-07, 2.9033784338499999e+00, 2.3070758317729999e+02 },
    { 2.2380999999999999e-07, 1.2831191556499999e+00, 5.7616138801060004e+02 },
    { 2.2252999999999999e-07, 1.4040018752800001e+00, 6.0176425067620005e+02 },
    { 2.2179999999999999e-07, 3.6951497184300002e+00, 3.1321047591889999e+02 },
    { 2.1878999999999999e-07, 5.8246964818600002e+00, 5.4166259714000002e+00 },
    { 2.1820999999999999e-07, 2.1694414463900000e+00, 2.9912639426919998e+02 },
    { 2.1610999999999999e-07, 3.3437691811199999e+00, 3.1245971639350000e+02 },
    { 2.1598000000000001e-07, 3.8849202348800000e+00, 3.2032402291969998e+02 },
    { 2.1367000000000001e-07, 3.9956958157800000e+00, 3.2743756992049998e+02 },
    { 2.1267000000000001e-07, 5.2661491350400000e+00, 9.1693228005540004e+02 },
    { 2.1170000000000000e-07, 1.2053964813300000e+00, 2.2137585028530000e+02 },
    { 2.1066999999999999e-07, 3.3230326883500001e+00, 4.2118156490460001e+02 },
    { 2.0940000000000001e-07, 2.4705097476600000e+00, 3.1957326339430000e+02 },
    { 2.0886000000000001e-07, 4.2975138859299999e+00, 1.1695882514085999e+03 },
    { 2.0788999999999999e-07, 3.2925868316600000e+00, 2.4175328344120001e+02 },
    { 2.0377999999999999e-07, 2.6726086563900000e+00, 4.2655000676620000e+02 },
    { 2.0375000000000001e-07, 5.8711420489700004e+00, 4.2664637498579998e+02 },
    { 1.9616999999999999e-07, 4.6012011082999998e-01, 6.9651963761659999e+02 },
    { 1.9320000000000001e-07, 6.1941620059100000e+00, 1.8892007304980001e+02 },
    { 1.9224999999999999e-07, 6.2039027209400004e+00, 6.3735898303399999e+01 },
    { 1.8692999999999999e-07, 3.7991561250300001e+00, 2.1181462272970001e+02 },
    { 1.8458999999999999e-07, 1.1236951070900001e+00, 1.7952584437210000e+03 },
    { 1.8307000000000001e-07, 4.6170694193599999e+00, 1.6682867252200001e+02 },
    { 1.8272000000000001e-07, 5.1880680372899999e+00, 6.2759623029905997e+03 },
    { 1.8160000000000001e-07, 3.8025664823700001e+00, 2.0543478891180001e+02 },
    { 1.7965000000000000e-07, 3.0897157373600002e+00, 1.4977853527000001e+01 },
    { 1.7830000000000000e-07, 5.8946997232199996e+00, 1.2035824960600000e+02 },
    { 1.7811000000000000e-07, 4.6050733947899998e+00, 6.2901893969922003e+03 },
    { 1.7163999999999999e-07, 3.2859484938000000e-01, 1.0206171999210201e+04 },
    { 1.7109999999999999e-07, 6.0038535250600003e+00, 4.3222726516850003e+02 },
    { 1.6894999999999999e-07, 1.8781412204200001e+00, 1.7810313497193999e+03 },
    { 1.6719999999999999e-07, 6.0294321608299999e+00, 1.0220399093211799e+04 },
    { 1.6691000000000000e-07, 1.0933209332000000e+00, 1.6779385755008000e+03 },
    { 1.6565999999999999e-07, 2.7786108840799999e+00, 5.0835032409220003e+02 },
    { 1.6563000000000001e-07, 4.0867991410200002e+00, 1.3458534360760001e+02 },
    { 1.6458000000000001e-07, 2.7372598163399999e+00, 4.3912836384820002e+02 },
    { 1.6429000000000000e-07, 2.3567503260099998e+00, 2.2076295405954002e+03 },
    { 1.6372000000000001e-07, 3.6437338170000002e+00, 1.2726810256271999e+03 },
    { 1.6077999999999999e-07, 3.5327615385000000e-01, 7.7841478318470001e+02 },
    { 1.5871000000000000e-07, 3.6823272432000000e+00, 1.8227960680100000e+02 },
    { 1.5744000000000001e-07, 1.8800344234399999e+00, 2.3767811782620001e+02 },
    { 1.5557999999999999e-07, 1.8019095777200000e+00, 3.3365734504400001e+02 },
    { 1.5239000000000000e-07, 3.7525862593900001e+00, 4.2951895218279998e+02 },
    { 1.5127000000000000e-07, 6.2273484191800001e+00, 7.1074673161819999e+02 },
    { 1.4844000000000000e-07, 1.5783751840800000e+00, 9.2047073954699997e+01 },
    { 1.4553000000000001e-07, 2.9802367235000000e+00, 3.8135160823740000e+02 },
    { 1.4181000000000000e-07, 5.7689257512300003e+00, 3.1219908396260001e+02 },
    { 1.4070000000000001e-07, 2.4926761993300000e+00, 3.9557870223899999e+02 },
    { 1.3948999999999999e-07, 1.7514853742700001e+00, 1.1642609634290000e+02 },
    { 1.3925000000000001e-07, 1.7631391814799999e+00, 5.5800214074589996e+02 },
    { 1.3895999999999999e-07, 4.6891474958600003e+00, 5.2549817940059995e+02 },
    { 1.3577000000000000e-07, 4.8864569464600001e+00, 9.6872999095099999e+01 },
    { 1.3500000000000000e-07, 4.3606233683199997e+00, 1.9662432088160000e+02 },
    { 1.3442000000000001e-07, 3.0690278027300000e+00, 1.5677322542813999e+03 },
    { 1.3372000000000001e-07, 2.1744387273700001e+00, 5.9803745040300001e+01 },
    { 1.3100000000000000e-07, 8.5481187045999996e-01, 8.4926422848890002e+02 },
    { 1.3084000000000000e-07, 6.2189733050399996e+00, 1.2991947716160001e+02 },
    { 1.3054000000000000e-07, 3.3380601341500000e+00, 1.9603362005060001e+02 },
    { 1.3038999999999999e-07, 1.8649928580999999e+00, 8.7311771539500000e+01 },
    { 1.2848000000000000e-07, 1.8644820787800001e+00, 3.0534616939270001e+02 },
    { 1.2749000000000001e-07, 1.2604348696000001e+00, 2.4209286360333999e+03 },
    { 1.2464000000000001e-07, 2.2228457450099999e+00, 7.5721715453419995e+02 },
    { 1.2234000000000000e-07, 6.1880920526300001e+00, 5.6363121503840000e+02 },
    { 1.2205000000000001e-07, 1.9367330990300000e+00, 6.6123792731640003e+02 },
    { 1.2158000000000001e-07, 4.2670010339199997e+00, 3.7390799283650000e+02 },
    { 1.1819000000000000e-07, 5.2527600317400003e+00, 4.3079097657000000e+02 },
    { 1.1815000000000000e-07, 9.2992400010000004e-02, 2.0085575391590000e+03 },
    { 1.1808999999999999e-07, 5.4365274935899999e+00, 2.2147430875962000e+03 },
    { 1.1807000000000000e-07, 3.7653543711399999e+00, 6.2251425595100002e+01 },
    { 1.1766000000000000e-07, 4.6510251736199999e+00, 2.8454188003199999e+01 },
    { 1.1698000000000001e-07, 4.1793855725000002e+00, 4.5576651038700000e+01 },
    { 1.1593000000000001e-07, 3.4215088199900001e+00, 5.8493641121146002e+03 },
    { 1.1528000000000000e-07, 3.2485111087499998e+00, 4.3201481684740003e+02 },
    { 1.1504000000000000e-07, 4.5634493496899999e+00, 1.4414657116320001e+02 },
    { 1.1381000000000000e-07, 6.1551282113800001e+00, 1.5667674413540001e+02 },
    { 1.1229000000000000e-07, 4.6441338800000000e+00, 2.3056457082540001e+02 },
    { 1.1215000000000000e-07, 4.7800879075699996e+00, 1.5349535039770001e+02 },
    { 1.1166000000000000e-07, 7.9599093090000003e-02, 7.7693031047640000e+02 },
    { 1.0914999999999999e-07, 1.2236542472600000e+00, 1.9996577331370000e+02 },
    { 1.0848000000000000e-07, 4.8479754630000000e+00, 9.7795738083342003e+03 },
    { 1.0726000000000000e-07, 5.8851960934400003e+00, 1.9269216761850001e+02 },
    { 1.0716000000000000e-07, 8.3453298030000000e-01, 2.1116503133776000e+03 },
    { 1.0712000000000000e-07, 4.5170601962300001e+00, 4.2711945573780002e+02 },
    { 1.0705999999999999e-07, 5.3268331320100000e+00, 8.4342826122899993e+01 },
    { 1.0673000000000000e-07, 4.2495920358500001e+00, 2.2116340196420001e+02 },
    { 1.0670999999999999e-07, 4.0254950396600000e+00, 4.2607692601420001e+02 },
    { 1.0521000000000000e-07, 5.9635443681000000e-01, 9.2940845831999994e+01 },
    { 1.0497000000000001e-07, 6.1717798048599999e+00, 4.1644497775600001e+01 },
    { 1.0476000000000001e-07, 1.2699169807199999e+00, 6.5220371011699996e+01 },
    { 1.0417000000000000e-07, 2.5460502834800001e+00, 3.4321857259960001e+02 },
    { 1.0377000000000000e-07, 8.6654919643999995e-01, 2.1240532356070000e+02 },
    { 1.0377000000000000e-07, 8.8148431656000004e-01, 2.1419286731529999e+02 },
    { 1.0352000000000000e-07, 2.8529317787899999e+00, 1.4563104387150000e+02 },
    { 1.0345000000000000e-07, 5.0847865931699996e+00, 2.0606927819500001e+01 },
    { 1.0237000000000000e-07, 4.2815276741900004e+00, 2.4431858407499999e+02 },
    { 1.0230000000000000e-07, 5.8816292138100001e+00, 1.9928444975750000e+02 },
    { 1.0188000000000000e-07, 5.6939090877299998e+00, 3.4788443904560000e+02 },
    { 1.0107000000000000e-07, 2.8782819845100001e+00, 5.8635912061161998e+03 },
    { 1.0085000000000001e-07, 3.3066826977899999e+00, 2.5976951835400001e+02 },
    { 1.0068000000000000e-07, 6.0759510221099999e+00, 2.0670681329900000e+02 },
    { 1.0043000000000001e-07, 5.2437948082999997e-01, 2.2663241756229999e+02 },
    { 9.9190000000000003e-08, 2.3299804230799999e+00, 2.8096714700450002e+02 },
    { 9.8599999999999996e-08, 1.3102722948400001e+00, 5.7222923474749996e+02 },
    { 9.7790000000000004e-08, 8.8460460263999996e-01, 1.8816931352440000e+02 },
    { 9.7460000000000001e-08, 3.3857921289299999e+00, 1.8498791978669999e+02 },
    { 9.6719999999999995e-08, 4.7095083963700004e+00, 2.3857188970350001e+02 },
    { 9.6670000000000005e-08, 3.3832780204700001e+00, 3.2668681039510000e+02 },
    { 9.6550000000000001e-08, 5.3967121907399997e+00, 7.6736908292080000e+02 },
    { 9.6089999999999996e-08, 1.7731166054300000e+00, 1.8180652604900001e+02 },
    { 9.5859999999999999e-08, 4.5566717751000002e-01, 1.5912442469019999e+02 },
    { 9.4609999999999998e-08, 4.0383604975900003e+00, 4.6470422915999997e+01 },
    { 9.4469999999999998e-08, 4.3043067963399997e+00, 9.7938009023358009e+03 },
    { 9.3699999999999999e-08, 2.1496396418999999e+00, 2.2731374111849999e+02 },
    { 9.2029999999999998e-08, 2.7307341598999999e+00, 2.1457111982519999e+02 },
    { 9.2029999999999998e-08, 5.3004846602800004e+00, 2.1202707105080000e+02 },
    { 9.1920000000000005e-08, 1.6795984874500001e+00, 5.5651766803759995e+02 },
    { 9.1559999999999994e-08, 5.0239673628899997e+00, 6.1780588578619995e+02 },
    { 9.0610000000000002e-08, 1.1590681479500000e+00, 2.0623373254699999e+02 },
    { 9.0079999999999997e-08, 5.8979002993499998e+00, 6.3744960575920004e+02 },
    { 8.9249999999999995e-08, 2.2292955847900000e+00, 2.3390602325750001e+02 },
    { 8.8819999999999997e-08, 4.2515468402599996e+00, 4.8640193591629998e+02 },
    { 8.8510000000000003e-08, 2.8983254637600000e+00, 5.1205725331200000e+01 },
    { 8.7670000000000004e-08, 5.4054203021699996e+00, 4.7217484191469998e+02 },
    { 8.7069999999999998e-08, 3.9416894968600000e+00, 6.7668051566499997e+01 },
    { 8.6850000000000000e-08, 3.7303289034799998e+00, 2.2036445832900000e+02 },
    { 8.6630000000000002e-08, 1.3041640272199999e+00, 2.6747376618580000e+02 },
    { 8.5959999999999997e-08, 1.2321089718600000e+00, 5.4695644048199995e+02 },
    { 8.5539999999999997e-08, 9.7000952693999998e-01, 1.3703302416240001e+02 },
    { 8.4130000000000000e-08, 2.0583648946099999e+00, 2.5240214102519999e+03 },
    { 8.3729999999999996e-08, 4.6108382819699996e+00, 2.4161027108930000e+02 },
    { 8.2539999999999997e-08, 1.6161848939400001e+00, 9.6340270297140000e+02 },
    { 8.2399999999999997e-08, 1.5754540389600000e+00, 2.0974232193759999e+03 },
    { 8.0760000000000003e-08, 5.3515938995800001e+00, 1.9589060769869999e+02 },
    { 8.0639999999999999e-08, 2.1160042977600000e+00, 5.2650957135689998e+02 },
    { 8.0599999999999994e-08, 2.8144865026499999e+00, 6.0766952886799999e+01 },
    { 8.0379999999999996e-08, 1.2694546060000000e-01, 2.5887574647669999e+02 },
    { 8.0290000000000000e-08, 1.5188617899999999e-03, 2.4479166482700001e+02 },
    { 8.0260000000000005e-08, 7.6761905754000004e-01, 2.3842887735159999e+02 },
    { 8.0109999999999994e-08, 1.9177203347600000e+00, 2.3575873236100001e+01 },
    { 8.0010000000000000e-08, 2.8222159526600001e+00, 8.2474219374879999e+02 },
    { 7.8240000000000004e-08, 3.3668688701599998e+00, 9.9911380480899993e+01 },
    { 7.6660000000000000e-08, 5.5961823450299999e+00, 8.1895145568100006e+01 },
    { 7.6459999999999998e-08, 1.0123250231300001e+00, 1.7394221952280000e+02 },
    { 7.6370000000000002e-08, 1.8145037593000000e+00, 2.9519424100610001e+02 },
    { 7.5969999999999998e-08, 2.9921148472099999e+00, 1.8647239249500001e+02 },
    { 7.3070000000000005e-08, 4.0107477086300003e+00, 6.1144309831080000e+02 },
    { 7.2989999999999994e-08, 2.3788343945000001e+00, 4.5884151979040001e+02 },
    { 7.2149999999999994e-08, 2.7521642398299999e+00, 1.8841241239379999e+03 },
    { 7.1869999999999994e-08, 2.3375682080700000e+00, 8.6424208201590000e+02 },
    { 7.1169999999999995e-08, 4.2029983867800000e+00, 2.8245161971279998e+02 },
    { 7.1079999999999999e-08, 6.2757218772799996e+00, 4.2734895040139997e+02 },
    { 7.0910000000000004e-08, 2.8913841193700001e+00, 6.9258748435350003e+02 },
    { 7.0760000000000006e-08, 6.2240143516600002e+00, 4.6473122651379998e+02 },
    { 7.0700000000000004e-08, 1.1151617181100000e+00, 5.7074476203920005e+02 },
    { 7.0290000000000003e-08, 4.2590595476199997e+00, 3.2972519178089999e+02 },
    { 7.0140000000000005e-08, 1.6594364685200000e+00, 6.2736711334179995e+02 },
    { 6.9530000000000001e-08, 6.1587872013300000e+00, 3.3928641933649999e+02 },
    { 6.9209999999999996e-08, 6.1009355281600000e+00, 3.5744566660120000e+02 },
    { 6.8919999999999998e-08, 3.5237959264600001e+00, 7.7962992305000000e+01 },
    { 6.8190000000000004e-08, 2.7441847198999998e+00, 1.3140394986990000e+02 },
    { 6.7620000000000006e-08, 7.0345352932000005e-01, 4.2096911658350001e+02 },
    { 6.7230000000000000e-08, 4.9077197315800003e+00, 2.4012579838100001e+02 },
    { 6.6609999999999999e-08, 6.1519152302700002e+00, 5.6118353448360006e+02 },
    { 6.6479999999999997e-08, 6.0271596963700000e+00, 1.2869081196288000e+03 },
    { 6.5400000000000003e-08, 4.8894597514499996e+00, 1.4859801210651999e+03 },
    { 6.5180000000000005e-08, 4.6472410425800001e+00, 6.3841281360569997e+02 },
    { 6.4760000000000005e-08, 5.9173858077999997e+00, 5.3287235883230005e+02 },
    { 6.4700000000000004e-08, 3.6840476461999998e+00, 1.6209337010679999e+02 },
    { 6.4259999999999994e-08, 1.2754661419400000e+00, 1.9365537546499999e+02 },
    { 6.4130000000000006e-08, 5.2851663211100002e+00, 1.4930936680660000e+03 },
    { 6.3710000000000006e-08, 1.5240383762600000e+00, 5.1864526483069994e+02 },
    { 6.2999999999999995e-08, 5.9808952593300004e+00, 4.1162033734900001e+02 },
    { 6.2909999999999999e-08, 3.2263636094399999e+00, 4.2367742956920000e+02 },
    { 6.2890000000000003e-08, 9.4099317700999996e-01, 1.9943304451573999e+03 },
    { 6.2859999999999995e-08, 5.6526751002799998e+00, 4.3274853003030000e+02 },
    { 6.2159999999999995e-08, 4.5259148972000002e-01, 1.7165459766239999e+02 },
    { 6.2110000000000005e-08, 2.7615270864500001e+00, 1.0775864066459999e+02 },
    { 6.1719999999999999e-08, 4.6208511931600000e+00, 1.2598732389849999e+02 },
    { 6.1700000000000003e-08, 6.0245638714199998e+00, 1.2895626931510000e+02 },
    { 6.1290000000000002e-08, 3.3954933152100000e+00, 2.1101147357759999e+02 },
    { 6.1290000000000002e-08, 4.6357255049699999e+00, 2.1558671729840000e+02 },
    { 6.1239999999999998e-08, 9.8855335751999995e-01, 2.7373205056900001e+03 },
    { 6.0609999999999998e-08, 5.0690771703000004e-01, 2.5265597135319999e+02 },
    { 6.0119999999999998e-08, 5.3998869705999997e-01, 2.4280421830342002e+03 },
    { 5.9950000000000004e-08, 5.1746982116700000e+00, 3.3455111692129998e+02 },
    { 5.9179999999999998e-08, 2.2875045054899998e+00, 4.2240540518199998e+02 },
    { 5.9050000000000003e-08, 9.3769875190999996e-01, 8.5637777548969996e+02 },
    { 5.9039999999999998e-08, 5.1490726814699999e+00, 2.5311349572528002e+03 },
    { 5.8570000000000001e-08, 2.9950781275599998e+00, 5.4302428721889999e+02 },
    { 5.8260000000000000e-08, 2.3167082164299999e+00, 5.5103160609700001e+02 },
    { 5.8210000000000003e-08, 8.3210890700000009e-03, 1.0554497769261000e+03 },
    { 5.7579999999999997e-08, 1.7498573649200000e+00, 8.3503713448730002e+02 },
    { 5.7030000000000002e-08, 4.8994114508599997e+00, 2.7406604832480002e+02 },
    { 5.6319999999999997e-08, 5.4983367871400004e+00, 5.4174670747800000e+01 },
    { 5.5959999999999999e-08, 4.2035096734300001e+00, 3.1019488636999998e+01 },
    { 5.4609999999999997e-08, 2.6617678900600001e+00, 3.4225536475310003e+02 },
    { 5.4399999999999997e-08, 4.9338802540999999e-01, 2.2997386999439999e+02 },
    { 5.2240000000000003e-08, 1.1212214312000000e+00, 4.6824268865160002e+02 },
    { 5.1480000000000002e-08, 5.9267267670999999e+00, 4.8884961647109998e+02 },
    { 5.0519999999999998e-08, 4.3140093036999999e-01, 2.4379022388199999e+01 },
    { 5.0039999999999997e-08, 1.6396778927400000e+00, 6.5019222705250002e+02 },
    { 4.9980000000000002e-08, 6.1456701151499997e+00, 4.4868959140380002e+02 },
    { 4.8590000000000000e-08, 6.4868543867999995e-01, 2.9204947528600002e+01 },
    { 4.8580000000000002e-08, 1.8166389092799999e+00, 3.3940249943799998e+01 },
    { 4.8130000000000002e-08, 1.7168274937800001e+00, 1.9054647649404001e+03 },
    { 4.7799999999999998e-08, 6.0503504889600004e+00, 2.3249494088155998e+03 },
    { 4.7750000000000001e-08, 6.1687186328000001e+00, 3.9435486196160002e+02 },
    { 4.7239999999999998e-08, 5.5969411593900000e+00, 2.0566428357539999e+02 },
    { 4.6460000000000000e-08, 5.5476910221900004e+00, 2.9126208774300000e+02 },
    { 4.6399999999999999e-08, 4.6565314298700002e+00, 2.6450482076920002e+02 },
    { 4.6240000000000002e-08, 1.3028375506700001e+00, 5.3362311835770004e+02 },
    { 4.5590000000000000e-08, 1.5524505071000000e-01, 4.1852143602870001e+02 },
    { 4.5289999999999997e-08, 4.0937583902899997e+00, 2.7310284047829998e+02 },
    { 4.5289999999999997e-08, 5.0050985259500003e+00, 2.1187638603783998e+03 },
    { 4.5270000000000001e-08, 5.5756535628600004e+00, 2.2093390730060000e+02 },
    { 4.5020000000000002e-08, 2.2409105415799999e+00, 1.6772244439930000e+02 },
    { 4.4869999999999997e-08, 5.6012010972000004e-01, 4.0992341631959999e+02 },
    { 4.4719999999999999e-08, 5.5799251625100004e+00, 3.6137781986430002e+02 },
    { 4.4689999999999998e-08, 4.0780236478200003e+00, 4.2808266358430001e+02 },
    { 4.4269999999999999e-08, 2.0180520161400000e+00, 2.5494359321360000e+02 },
    { 4.4150000000000002e-08, 3.5074452985200000e+00, 2.9823262239190001e+02 },
    { 4.3919999999999999e-08, 2.7052041573499999e+00, 4.9181856188770001e+02 },
    { 4.3320000000000000e-08, 1.7855328626800000e+00, 2.8404132799086001e+03 },
    { 4.3079999999999999e-08, 4.7850809730200004e+00, 6.2960234557549995e+02 },
    { 4.2949999999999997e-08, 4.8438218526999997e+00, 1.8023719907218001e+03 },
    { 4.2809999999999998e-08, 2.9654692110099998e+00, 4.0599126305649997e+02 },
    { 4.2690000000000000e-08, 3.8574651996300000e+00, 2.0151968199120000e+02 },
    { 4.2690000000000000e-08, 1.0321609669600000e+00, 2.2507850888479999e+02 },
    { 4.2459999999999998e-08, 2.5477976946199998e+00, 1.1411340634054000e+03 },
    { 4.2429999999999997e-08, 4.9030583479000001e+00, 3.0609692891809999e+02 },
    { 4.2410000000000001e-08, 2.2412264305699998e+00, 3.6997583957339998e+02 },
    { 4.2030000000000000e-08, 8.4999684255999997e-01, 3.6434676528090000e+02 },
    { 4.1929999999999999e-08, 6.0163848977300001e+00, 1.7076082578510000e+02 },
    { 4.1400000000000000e-08, 1.7282022317600001e+00, 3.9739324334740002e+02 },
    { 4.1210000000000003e-08, 3.4476352465900000e+00, 1.3828873468465999e+03 },
    { 4.1150000000000001e-08, 1.3822647370200001e+00, 2.4138150890326001e+03 },
    { 4.0520000000000002e-08, 3.8817687254400002e+00, 4.2511371816769997e+02 },
    { 3.9990000000000003e-08, 9.6003854854000004e-01, 4.9426624244250002e+02 },
    { 3.9680000000000002e-08, 3.6110768682500001e+00, 1.2050126195790000e+02 },
    { 3.9629999999999998e-08, 2.4968295317300000e+00, 2.2005159935945999e+03 },
    { 3.9410000000000000e-08, 5.9120903057599996e+00, 4.3375992198659998e+02 },
    { 3.9349999999999998e-08, 2.1394757352100000e+00, 4.2584743135060000e+02 },
    { 3.9150000000000003e-08, 6.3612829951000005e-01, 5.0456118318099999e+02 },
    { 3.9139999999999998e-08, 5.2229319176800004e+00, 2.8311175651300001e+01 },
    { 3.8659999999999997e-08, 1.1823135039099999e+00, 3.8846515523820000e+02 },
    { 3.7849999999999998e-08, 3.4692432158600002e+00, 2.1494362684070001e+02 },
    { 3.7849999999999998e-08, 4.5619756043199997e+00, 2.1165456403530001e+02 },
    { 3.7399999999999997e-08, 3.3163676746599999e+00, 6.5805653357870005e+02 },
    { 3.7329999999999997e-08, 5.6064994319399997e+00, 4.6595506679120001e+02 },
    { 3.7130000000000002e-08, 3.5483179852000002e-01, 2.0326478713040001e+02 },
    { 3.7130000000000002e-08, 4.5347943680700000e+00, 2.2333340374560001e+02 },
    { 3.7000000000000000e-08, 3.3249569390099998e+00, 1.7001006625970001e+02 },
    { 3.6780000000000002e-08, 6.1697020720099998e+00, 4.1800017116689997e+02 },
    { 3.6160000000000001e-08, 5.3513462883500003e+00, 2.9201284726839998e+02 },
    { 3.6150000000000003e-08, 3.0744652302100000e+00, 1.3533610313299999e+02 },
    { 3.5840000000000002e-08, 4.2209220280700004e+00, 7.7750543983900002e+01 },
    { 3.5660000000000003e-08, 2.2715156215999999e-01, 2.1160217440860001e+02 },
    { 3.5660000000000003e-08, 4.6624746044299998e+00, 2.1499601646740001e+02 },
    { 3.5189999999999999e-08, 4.9764644914999998e+00, 3.3137100791700000e+01 },
    { 3.5089999999999999e-08, 2.9807887753800002e+00, 1.8972322220190000e+02 },
    { 3.4599999999999999e-08, 2.5816671675700000e+00, 8.5001498801430000e+02 },
    { 3.4520000000000001e-08, 2.0824642597400000e+00, 7.3269511979410004e+02 },
    { 3.3860000000000000e-08, 2.9410676093700001e+00, 7.3297125859000005e+01 },
    { 3.3790000000000000e-08, 5.4813205856099998e+00, 2.3687496867409999e+02 },
    { 3.3370000000000000e-08, 4.8417271078899997e+00, 1.2836556848410001e+02 },
    { 3.3190000000000001e-08, 1.0744515519600000e+00, 7.3194436026870005e+02 },
    { 3.2700000000000002e-08, 2.4934461993000001e-01, 7.1467888488129995e+02 },
    { 3.2320000000000001e-08, 6.9234918409000001e-01, 5.5137878594299998e+01 },
    { 3.1870000000000000e-08, 3.1993428379900002e+00, 5.1939602435610004e+02 },
    { 3.1690000000000001e-08, 6.0670691554399996e+00, 9.2797833480099996e+01 },
    { 3.1690000000000001e-08, 4.9932840784300003e+00, 2.8475268269093999e+03 },
    { 3.1669999999999998e-08, 4.9649689322499997e+00, 2.0199276274319999e+02 },
    { 3.1669999999999998e-08, 6.2078425415199998e+00, 2.2460542813280000e+02 },
    { 3.1650000000000002e-08, 4.3198884532299999e+00, 2.0055647414469999e+02 },
    { 3.1650000000000002e-08, 5.6973771336000001e-01, 2.2604171673130000e+02 },
    { 3.1020000000000002e-08, 3.8218442244499999e+00, 9.2783496796739996e+02 },
    { 3.1000000000000000e-08, 4.1370254603900003e+00, 4.0695447090300001e+02 },
    { 3.0909999999999997e-08, 1.6050431076400000e+00, 6.0626632075526004e+03 },
    { 3.0670000000000003e-08, 8.9802750598000003e-01, 4.4157604440300003e+02 },
    { 3.0239999999999998e-08, 6.0870614979899997e+00, 4.0302231763990000e+02 },
    { 3.0180000000000003e-08, 1.9040552693899999e+00, 4.5097721326419997e+02 },
    { 2.9919999999999999e-08, 2.8102213064599999e+00, 4.6202291352809999e+02 },
    { 2.9870000000000002e-08, 1.9558496529700000e+00, 1.6850521225016000e+03 },
    { 2.9740000000000000e-08, 4.7538727403600003e+00, 1.8409414790939999e+02 },
    { 2.9740000000000000e-08, 5.1583249559400004e+00, 6.3033605875839999e+02 },
    { 2.9460000000000000e-08, 1.3712840308800001e+00, 7.3980866679489998e+02 },
    { 2.9429999999999999e-08, 2.7319884516499999e+00, 2.5583736509089999e+02 },
    { 2.8979999999999999e-08, 5.0716531857000002e-01, 1.0807225711916001e+03 },
    { 2.8979999999999999e-08, 3.0253624045700001e+00, 9.9928729037722005e+03 },
    { 2.8740000000000001e-08, 1.6579183756400000e+00, 5.8106824010899999e+01 },
    { 2.8620000000000001e-08, 1.6617227354399999e+00, 2.2218566345969998e+03 },
    { 2.8500000000000000e-08, 3.8197380638400000e+00, 2.2052451702339999e+02 },
    { 2.8290000000000001e-08, 2.1658900675799999e+00, 5.2575881183150000e+02 },
    { 2.8190000000000000e-08, 4.7062642328699997e+00, 2.5658812461629998e+02 },
    { 2.8010000000000001e-08, 4.2649927804200001e+00, 8.4933526953900000e+01 },
    { 2.7820000000000000e-08, 4.8158599096400003e+00, 6.3893407846749994e+02 },
    { 2.7649999999999999e-08, 4.2440366790199997e+00, 5.6360650166766000e+03 },
    { 2.7470000000000000e-08, 5.9959266925999999e-01, 4.0918970313670002e+02 },
    { 2.7470000000000000e-08, 5.9415209174000005e-01, 3.0537123753465999e+03 },
    { 2.7470000000000000e-08, 5.2434420540200000e+00, 6.0768903015542001e+03 },
    { 2.7210000000000000e-08, 4.0718988080700003e+00, 8.5712853501510006e+02 },
    { 2.7120000000000001e-08, 5.9355527706000002e+00, 5.0402576179100002e+01 },
    { 2.7030000000000001e-08, 2.1002443426499999e+00, 3.4484605765980001e+02 },
    { 2.6689999999999999e-08, 6.6381570061999995e-01, 8.1506334611420004e+02 },
    { 2.6499999999999999e-08, 5.9948806565200004e+00, 3.9814400287280000e+02 },
    { 2.6390000000000000e-08, 4.5248063549499999e+00, 4.8736514376280002e+02 },
    { 2.6359999999999999e-08, 5.0126701076400000e+00, 6.4945851386959998e+02 },
    { 2.6340000000000000e-08, 2.6482595015999999e-01, 2.7444340526907999e+03 },
    { 2.6149999999999999e-08, 8.9208215786000000e-01, 3.8133035637799999e+01 },
    { 2.6110000000000000e-08, 1.0538431716400001e+00, 7.3905790726949999e+02 },
    { 2.5970000000000000e-08, 5.0666225958600002e+00, 4.2648631629139999e+02 },
    { 2.5970000000000000e-08, 3.4764650943299999e+00, 4.2671006546059999e+02 },
    { 2.5930000000000001e-08, 2.4109780915000001e-01, 1.7224529849340001e+02 },
    { 2.5629999999999999e-08, 3.8164149000999997e-01, 1.0007099997773799e+04 },
    { 2.5600000000000001e-08, 2.6077767585100000e+00, 1.0601353552540000e+02 },
    { 2.5450000000000000e-08, 2.5280362653499999e+00, 4.5505237887919998e+02 },
    { 2.5259999999999999e-08, 3.8249020124499999e+00, 5.4073666535849998e+02 },
    { 2.5230000000000002e-08, 1.4658225109400000e+00, 5.1153171782990000e+02 },
    { 2.5189999999999999e-08, 3.6199265608900002e+00, 9.8066817835879999e+02 },
    { 2.5100000000000000e-08, 2.3230724533200000e+00, 6.4086049416050002e+02 },
    { 2.4949999999999998e-08, 4.0795173628899999e+00, 6.4892613984285999e+03 },
    { 2.4739999999999999e-08, 6.2132217324100001e+00, 4.1953282798499998e+02 },
    { 2.4599999999999999e-08, 3.6518642275400000e+00, 5.6502921106781996e+03 },
    { 2.4470000000000000e-08, 1.5464128175600000e+00, 2.0693630796260001e+02 },
    { 2.4470000000000000e-08, 3.3432133490300000e+00, 2.1966188291340001e+02 },
    { 2.4439999999999999e-08, 1.6866047394300001e+00, 4.2044785172169998e+02 },
    { 2.4380000000000001e-08, 1.2802364775599999e+00, 2.7302069586891998e+03 },
    { 2.4290000000000001e-08, 4.6032805429200003e+00, 4.7780391620720002e+02 },
    { 2.4039999999999999e-08, 5.0853587236899997e+00, 9.5805018068978006e+03 },
    { 2.3790000000000000e-08, 2.0045589151100001e+00, 5.9465070367540000e+02 },
    { 2.3420000000000001e-08, 5.7488193568900003e+00, 4.0221916848780000e+02 },
    { 2.3250000000000000e-08, 5.5049728515499998e+00, 1.0419471094648199e+04 },
    { 2.2950000000000000e-08, 5.7260746210000002e-01, 5.0286426215159997e+02 },
    { 2.2790000000000001e-08, 6.6039697096000005e-01, 4.5490936652729999e+02 },
    { 2.2609999999999999e-08, 2.5404962753400002e+00, 4.2538269652899999e+01 },
    { 2.2399999999999999e-08, 2.3584611430200000e+00, 1.4575259330619999e+03 },
    { 2.2280000000000002e-08, 5.5026880881400002e+00, 5.9342686339800002e+02 },
    { 2.2189999999999999e-08, 5.6874366068300004e+00, 2.5272794265500000e+01 },
    { 2.2010000000000000e-08, 1.1261021327100000e+00, 4.3467494572330003e+02 },
    { 2.1979999999999999e-08, 1.8348775842200000e+00, 2.0014439921582000e+03 },
    { 2.1820000000000000e-08, 2.8568230520200002e+00, 4.2681063919709999e+02 },
    { 2.1820000000000000e-08, 5.6862646381699999e+00, 4.2638574255489999e+02 },
    { 2.1780000000000000e-08, 1.4779241937600001e+00, 2.5435289238260000e+02 },
    { 2.1520000000000000e-08, 3.8778465107000000e+00, 2.3294281541100000e+02 },
    { 2.1360000000000001e-08, 5.8081526963899996e+00, 1.8621176006409999e+02 },
    { 2.1170000000000000e-08, 1.4289451533499999e+00, 3.1568051495651998e+03 },
    { 2.1150000000000001e-08, 1.9066694649800000e+00, 9.0588657979150003e+02 },
    { 2.0619999999999999e-08, 2.5449925704199998e+00, 6.9970103135429997e+02 },
    { 2.0599999999999999e-08, 8.3760116572999999e-01, 2.3107223148140001e+03 },
    { 2.0549999999999999e-08, 5.7738804592099999e+00, 2.6413412784722000e+03 },
    { 2.0400000000000001e-08, 2.3821420349300002e+00, 4.8173606947029998e+02 },
    { 2.0170000000000001e-08, 1.1645581629799999e+00, 1.1265400177420000e+02 },
    { 2.0129999999999999e-08, 3.8394857572099998e+00, 6.8547393735269998e+02 },
    { 1.9919999999999999e-08, 1.2120028521999999e+00, 1.6674774556399999e+01 },
    { 1.9799999999999999e-08, 4.5139617900299998e+00, 2.0629742302180000e+02 },
    { 1.9740000000000000e-08, 5.1210235016499999e+00, 6.3994547042379997e+02 },
    { 1.9690000000000000e-08, 2.1124308051999998e+00, 5.1653626887241999e+03 },
    { 1.9610000000000002e-08, 1.7770016010300000e+00, 1.4393412284210001e+02 },
    { 1.9580000000000001e-08, 5.1006487072999995e-01, 6.1069233878540001e+02 },
    { 1.9470000000000002e-08, 3.0397994170399998e+00, 2.6843697403229999e+02 },
    { 1.9399999999999998e-08, 4.4942255304599996e+00, 6.2248879857470001e+02 },
    { 1.9370000000000001e-08, 3.5092026257800000e+00, 1.4122580985639999e+02 },
    { 1.9370000000000001e-08, 3.1244783533699998e+00, 8.4290144101349995e+02 },
    { 1.9180000000000000e-08, 6.1493329924799998e+00, 4.9508804301799998e+01 },
    { 1.9099999999999999e-08, 3.0447779293599999e+00, 1.7542669223109999e+02 },
    { 1.9040000000000000e-08, 5.3808554251400000e+00, 3.8012776796000003e+02 },
    { 1.9029999999999999e-08, 1.8670736120700000e+00, 1.0490869894507000e+03 },
    { 1.9000000000000001e-08, 4.6450717286599996e+00, 2.7140591944890002e+02 },
    { 1.8880000000000001e-08, 5.6923543297599997e+00, 6.4234496686880004e+02 },
    { 1.8850000000000000e-08, 1.1575573798500001e+00, 5.2755690099435997e+03 },
    { 1.8839999999999999e-08, 1.4471659067500000e+00, 2.3178358618148000e+03 },
    { 1.8810000000000001e-08, 6.3571507860999998e-01, 1.5961864422845999e+03 },
    { 1.8749999999999999e-08, 7.7039124871999998e-01, 7.6981676347560006e+02 },
    { 1.8710000000000000e-08, 5.1195036498600004e+00, 2.0607367385260000e+02 },
    { 1.8369999999999998e-08, 1.6712025894999999e-01, 1.6289651925890001e+02 },
    { 1.8299999999999998e-08, 3.4171303407000000e+00, 4.8470501488690002e+02 },
    { 1.8240000000000000e-08, 5.1439763258699998e+00, 4.2000590873700003e+02 },
    { 1.8200000000000001e-08, 1.0574231798000000e+00, 4.8077286162380000e+02 },
    { 1.8050000000000000e-08, 5.6621327216699999e+00, 9.5662747128962001e+03 },
    { 1.8040000000000001e-08, 4.8312211537899996e+00, 3.5330106501699998e+02 },
    { 1.7999999999999999e-08, 2.9943567364399999e+00, 7.1600204829600003e+01 },
    { 1.7960000000000000e-08, 2.2230675714600001e+00, 2.4038643081190000e+02 },
    { 1.7949999999999999e-08, 1.7756060771900000e+00, 3.0508553696180002e+02 },
    { 1.7929999999999999e-08, 1.8596538837600001e+00, 4.1943645976540000e+02 },
    { 1.7910000000000000e-08, 5.8005957654499998e+00, 6.2491943278700001e+02 },
    { 1.7859999999999999e-08, 2.7026588556500002e+00, 1.1939672737968001e+03 },
    { 1.7789999999999999e-08, 7.3316360703000005e-01, 7.0214871190910003e+02 },
    { 1.7789999999999999e-08, 3.3209015529300001e+00, 3.3334988796990001e+03 },
    { 1.7730000000000001e-08, 5.6273970815099998e+00, 9.9022940591439999e+02 },
    { 1.7590000000000001e-08, 2.5645517288700002e+00, 7.2164941953020002e+02 },
    { 1.7360000000000001e-08, 1.5229852583600001e+00, 2.6080789594573391e+04 },
    { 1.7339999999999998e-08, 1.8261195970299999e+00, 7.0511765732569995e+02 },
    { 1.7339999999999998e-08, 9.6745963615999997e-01, 7.4692221379570003e+02 },
    { 1.7330000000000000e-08, 4.4354117395100001e+00, 1.3324847706675000e+03 },
    { 1.7319999999999999e-08, 7.9114238966999995e-01, 7.2483081326790000e+02 },
    { 1.7319999999999999e-08, 5.3596700681999998e+00, 1.1262992222303001e+03 },
    { 1.7299999999999999e-08, 1.1474604214399999e+00, 6.0697767545533998e+03 },
    { 1.7280000000000000e-08, 1.0396760956500000e+00, 3.0465988283458000e+03 },
    { 1.7260000000000000e-08, 2.9294772579799999e+00, 8.3578789401270001e+02 },
    { 1.7240000000000001e-08, 4.8024773821800002e+00, 1.8484490743480001e+02 },
    { 1.7129999999999999e-08, 6.0142121241800002e+00, 4.5761767951299998e+02 },
    { 1.7030000000000001e-08, 5.7950995317000000e+00, 1.0064509366380000e+02 },
    { 1.7010000000000001e-08, 4.2752531542099996e+00, 2.4351557300350000e+03 },
    { 1.6910000000000001e-08, 5.0775020288999995e-01, 7.8946048344860003e+02 },
    { 1.6880000000000000e-08, 2.7946526017400002e+00, 3.3477259737006002e+03 },
    { 1.6750000000000001e-08, 1.7765469492499999e+00, 2.4476805547999998e+00 },
    { 1.6720000000000000e-08, 3.5340557970700002e+00, 1.2442268376239999e+03 },
    { 1.6700000000000001e-08, 5.2461494281799999e+00, 3.4863519857099999e+02 },
    { 1.6700000000000001e-08, 3.2006821802599998e+00, 1.5606187072805999e+03 },
    { 1.6660000000000002e-08, 3.8059362684300000e+00, 1.1272624300768000e+03 },
    { 1.6619999999999999e-08, 9.9863125096000005e-01, 2.6095016688575000e+04 },
    { 1.6519999999999998e-08, 1.3806525696300000e+00, 1.5253214255120000e+02 },
    { 1.6460000000000000e-08, 5.8798737474499996e+00, 1.1120721282287000e+03 },
    { 1.6400000000000001e-08, 4.7948796214299998e+00, 7.7448262992160005e+02 },
    { 1.6390000000000000e-08, 3.5022866934699999e+00, 4.2685882330689998e+02 },
    { 1.6370000000000000e-08, 3.7607532277000000e-01, 9.0981873305459999e+02 },
    { 1.6330000000000001e-08, 4.0890276277100002e+00, 1.6805251279940001e+02 },
    { 1.6330000000000001e-08, 4.8372724080999996e+00, 3.8405992122309999e+02 },
    { 1.6199999999999999e-08, 3.0763317476000002e-01, 7.8404385747720005e+02 },
    { 1.6050000000000001e-08, 2.0545424450300001e+00, 4.8715269544170002e+02 },
    { 1.6020000000000001e-08, 2.5520762861200001e+00, 9.9999864507729999e+03 },
    { 1.6009999999999999e-08, 1.1487473298399999e+00, 3.1492569389000000e+01 },
    { 1.5910000000000002e-08, 1.7835063764400001e+00, 3.7301422095919997e+02 },
    { 1.5869999999999999e-08, 1.5390055393299999e+00, 2.5382485042536000e+03 },
    { 1.5780000000000000e-08, 4.4146956379200004e+00, 1.9040454575810000e+02 },
    { 1.5780000000000000e-08, 3.6165231822599999e+00, 2.3619364511789999e+02 },
    { 1.5720000000000001e-08, 6.0954698733399999e+00, 9.1786441523799994e+01 },
    { 1.5659999999999999e-08, 5.3165970760599999e+00, 3.9265794093220001e+02 },
    { 1.5659999999999999e-08, 4.8537128056899999e+00, 3.1639186965660001e+03 },
    { 1.5539999999999999e-08, 5.5840540882700003e+00, 7.6025553592000006e+02 },
    { 1.5510000000000001e-08, 9.0565423938000000e-01, 6.1503391543000001e+00 },
    { 1.5480000000000000e-08, 1.1508956250800000e+00, 1.0483362299252999e+03 },
    { 1.5469999999999999e-08, 3.4348210204299998e+00, 2.1325091132820000e+02 },
    { 1.5460000000000001e-08, 8.5174846192999998e-01, 2.5854567807659998e+02 },
    { 1.5410000000000000e-08, 3.0828029911799999e+00, 1.5126011816400000e+02 },
    { 1.5370000000000001e-08, 4.0025478700099999e+00, 3.9346109008430000e+02 },
    { 1.5329999999999999e-08, 5.3589768240500000e+00, 9.0684978763799995e+02 },
    { 1.5309999999999999e-08, 5.9626199139600002e+00, 5.2165803728013998e+03 },
    { 1.5279999999999999e-08, 6.2403445655500001e+00, 3.0608259223474001e+03 },
    { 1.5239999999999999e-08, 4.8646667926799996e+00, 9.2107688163959995e+02 },
    { 1.5170000000000000e-08, 4.5650586602100001e+00, 3.4803105662225998e+03 },
    { 1.5060000000000001e-08, 1.6749581608299999e+00, 1.1010264279648000e+03 },
    { 1.5060000000000001e-08, 4.9864427149699999e+00, 2.5117149864490000e+02 },
    { 1.5010000000000000e-08, 1.7682022033200000e+00, 3.9140223040994001e+03 },
    { 1.4890000000000000e-08, 1.4725656511900000e+00, 2.6370167161709998e+02 },
    { 1.4629999999999999e-08, 1.3381728103000001e-01, 6.9364972595899999e+01 },
    { 1.4520000000000000e-08, 4.7002334837199999e+00, 1.1302313754934000e+03 },
    { 1.4460000000000000e-08, 5.3569225316300004e+00, 2.2189711514710001e+02 },
    { 1.4460000000000000e-08, 5.8158889421400000e+00, 2.0470107572890001e+02 },
    { 1.4429999999999999e-08, 3.0715889083899999e+00, 6.9407195706180005e+02 },
    { 1.4310000000000000e-08, 1.6608244840799999e+00, 2.8332997329077998e+03 },
    { 1.4270000000000000e-08, 5.5616656252599999e+00, 9.7676148247200004e+01 },
    { 1.4210000000000000e-08, 4.9081346336799996e+00, 2.8793768165339998e+02 },
    { 1.4090000000000001e-08, 8.0723610506999999e-01, 1.3725924061081000e+03 },
    { 1.4079999999999999e-08, 3.8283955280100002e+00, 3.4884764689209999e+02 },
    { 1.3970000000000000e-08, 2.3013709531300002e+00, 2.5169078632512001e+03 },
    { 1.3960000000000001e-08, 1.1028776112500001e+00, 9.1467900690210008e+03 },
    { 1.3960000000000001e-08, 1.2955848185100001e+00, 1.0633140834523001e+03 },
    { 1.3869999999999999e-08, 5.8643967906899999e+00, 3.3683873878169999e+02 },
    { 1.3860000000000000e-08, 4.1677856243700004e+00, 5.2724328453980002e+02 },
    { 1.3850000000000000e-08, 2.6904735026100002e+00, 4.1053796944600002e+01 },
    { 1.3790000000000000e-08, 1.7513481904199999e+00, 5.5555446019110002e+02 },
    { 1.3690000000000000e-08, 4.6233206716000002e+00, 6.8154178408960001e+02 },
    { 1.3680000000000001e-08, 2.2576802749999998e+00, 6.0676018552230005e+02 },
    { 1.3610000000000001e-08, 6.0826072973000000e-01, 3.1394418910180002e+02 },
    { 1.3529999999999999e-08, 1.2956244608200000e+00, 1.0562005364515001e+03 },
    { 1.3510000000000000e-08, 1.4688911791799999e+00, 1.7935884549420001e+02 },
    { 1.3480000000000001e-08, 3.9755621633099998e+00, 1.7408487739300000e+01 },
    { 1.3469999999999999e-08, 9.5659867183000002e-01, 1.1767017984094000e+03 },
    { 1.3450000000000000e-08, 4.0544096187000001e-01, 4.4923210812500002e+02 },
    { 1.3430000000000000e-08, 1.0248792715199999e+00, 7.2333918012500007e+01 },
    { 1.3329999999999999e-08, 1.4445225131399999e+00, 5.1838463239980001e+02 },
    { 1.3329999999999999e-08, 8.7141945338000004e-01, 2.8266406803389998e+02 },
    { 1.3300000000000000e-08, 3.2956329995000000e+00, 4.6053844081979997e+02 },
    { 1.3300000000000000e-08, 4.7614788679300002e+00, 2.0003520928290001e+02 },
    { 1.3300000000000000e-08, 1.2814729865999999e-01, 2.2656298159310001e+02 },
    { 1.3220000000000000e-08, 3.8188875566700000e+00, 5.0123677709139997e+02 },
    { 1.3210000000000000e-08, 5.4053393298200003e+00, 4.0132539661049998e+02 },
    { 1.3130000000000001e-08, 1.0052934890000000e+00, 3.1931263096340001e+02 },
    { 1.3110000000000000e-08, 1.6982924665700001e+00, 1.0404719233991000e+03 },
    { 1.3040000000000000e-08, 2.1079618478700000e+00, 1.3686602528450001e+03 },
    { 1.2979999999999999e-08, 1.2634337068099999e+00, 2.6342277314714001e+03 },
    { 1.2940000000000000e-08, 4.3494678615999999e-01, 3.2598979237837998e+03 },
    { 1.2940000000000000e-08, 1.7807131838400001e+00, 6.9110301164520001e+02 },
    { 1.2900000000000000e-08, 2.7816604045800002e+00, 1.4169889060840001e+02 },
    { 1.2880000000000000e-08, 2.8913908894000001e-01, 2.4723934538180001e+02 },
    { 1.2870000000000000e-08, 4.9774360616899997e+00, 2.8659622129700000e+02 },
    { 1.2870000000000000e-08, 3.0537827584900001e+00, 1.4000196957899999e+02 },
    { 1.2840000000000000e-08, 1.2538950583299999e+00, 7.1617311105999999e+00 },
    { 1.2810000000000000e-08, 1.1735422108800000e+00, 3.2261164478009999e+02 },
    { 1.2779999999999999e-08, 2.9163592956800000e+00, 1.0380412891868000e+03 },
    { 1.2690000000000000e-08, 6.0500062366999996e-01, 1.8016199464630000e+02 },
    { 1.2590000000000001e-08, 4.7794736201300001e+00, 8.2399143422340001e+02 },
    { 1.2550000000000000e-08, 2.5834846913799998e+00, 9.4524345570670005e+02 },
    { 1.2470000000000000e-08, 3.0758453502299998e+00, 1.8743560034149999e+02 },
    { 1.2470000000000000e-08, 4.9553734699499996e+00, 2.3916259053450000e+02 },
    { 1.2460000000000000e-08, 3.3663890365400002e+00, 2.4874160449477999e+03 },
    { 1.2460000000000000e-08, 4.8885453525300004e+00, 7.3879727483859995e+02 },
    { 1.2439999999999999e-08, 4.4760999619999998e-02, 6.7442823708879996e+02 },
    { 1.2369999999999999e-08, 4.9698209585499997e+00, 1.1926254413668799e+04 },
    { 1.2320000000000000e-08, 3.6724596043100002e+00, 1.8802630117250001e+02 },
    { 1.2310000000000001e-08, 4.0173321131400002e+00, 5.1535908996834391e+04 },
    { 1.2299999999999999e-08, 3.9589322370600000e+00, 8.3110498122419995e+02 },
    { 1.2270000000000000e-08, 3.2275941589100001e+00, 1.3544331588434000e+03 },
    { 1.2270000000000000e-08, 3.6729374328700000e+00, 4.1273213995373999e+03 },
    { 1.2089999999999999e-08, 4.4068563547000000e-01, 5.5258551477449998e+02 },
    { 1.2080000000000000e-08, 5.5094685421000000e-01, 3.3701042450032000e+03 },
    { 1.1980000000000000e-08, 4.8262910294000000e-01, 1.0625633239269000e+03 },
    { 1.1959999999999999e-08, 1.6006433516900000e+00, 1.4599565672742999e+03 },
    { 1.1920000000000000e-08, 5.2566865291699996e+00, 9.5952390892233998e+03 },
    { 1.1860000000000000e-08, 4.2962086979800000e+00, 9.3888066823129998e+02 },
    { 1.1800000000000000e-08, 1.0745911746300001e+00, 2.9211277828246002e+03 },
    { 1.1780000000000000e-08, 1.7068538925000001e-01, 7.6348118626000003e+00 },
    { 1.1780000000000000e-08, 4.8756623437200002e+00, 1.3285526174044001e+03 },
    { 1.1690000000000001e-08, 2.5543154213400001e+00, 1.4685716333258999e+03 },
    { 1.1679999999999999e-08, 2.6533089291400000e+00, 7.4638586215399997e+01 },
    { 1.1670000000000000e-08, 2.7509889613200000e+00, 6.3448066034260000e+02 },
    { 1.1640000000000000e-08, 4.4654028791500000e+00, 3.3914448860056000e+03 },
    { 1.1549999999999999e-08, 1.8127838782500001e+00, 2.0903096723752001e+03 },
    { 1.1539999999999999e-08, 1.6772926731500000e+00, 7.8234693644779998e+02 },
    { 1.1500000000000000e-08, 2.8114008219400000e+00, 1.1521797636693000e+03 },
    { 1.1479999999999999e-08, 3.7503398495800000e+00, 1.3576145525811000e+03 },
    { 1.1460000000000000e-08, 5.0501307092600003e+00, 2.2030076785419999e+02 },
    { 1.1420000000000001e-08, 3.6209439606100000e+00, 4.8025159676200002e+02 },
    { 1.1409999999999999e-08, 1.2364083242199999e+00, 1.1443154571431000e+03 },
    { 1.1370000000000000e-08, 5.4222333040000001e-02, 4.3335069479745998e+03 },
    { 1.1370000000000000e-08, 5.0785963489999997e-01, 6.4679257579615996e+03 },
    { 1.1350000000000001e-08, 5.5555474328000001e+00, 2.5668418497699000e+04 },
    { 1.1330000000000000e-08, 3.2860881519200000e+00, 1.0444040766621999e+03 },
    { 1.1310000000000000e-08, 2.0866602351300001e+00, 1.7739178027185999e+03 },
    { 1.1310000000000000e-08, 1.8026035486300001e+00, 5.4817545583807996e+03 },
    { 1.1310000000000000e-08, 1.9212770622999999e+00, 6.3984910220419999e+02 },
    { 1.1300000000000000e-08, 4.3330277812200002e+00, 1.9022833712027000e+03 },
    { 1.1280000000000001e-08, 2.3531864142900001e+00, 3.9828701522469999e+02 },
    { 1.1210000000000001e-08, 2.1331434784800001e+00, 3.6604368631030002e+02 },
    { 1.1190000000000000e-08, 3.8425599314399999e+00, 1.5347381658416000e+03 },
    { 1.1140000000000001e-08, 1.5778080363900000e+00, 2.5234706759822191e+04 },
    { 1.1140000000000001e-08, 5.5140817598399998e+00, 3.1883955021140002e+02 },
    { 1.0970000000000000e-08, 2.3260591866100002e+00, 9.8853248488500003e+02 },
    { 1.0950000000000000e-08, 4.6976742311000003e-01, 5.4153981451059997e+02 },
    { 1.0920000000000000e-08, 5.2593835180599999e+00, 1.3143255234027999e+03 },
    { 1.0910000000000000e-08, 1.8371087630200000e+00, 1.2353964334370001e+02 },
    { 1.0859999999999999e-08, 6.5554097743999995e-01, 1.4096517742550000e+02 },
    { 1.0859999999999999e-08, 4.7229877833399998e+00, 1.9982625984204999e+03 },
    { 1.0820000000000000e-08, 3.4826723375399999e+00, 7.3168372783780001e+02 },
    { 1.0800000000000001e-08, 1.5676903250000001e+00, 2.9069006888230001e+03 },
    { 1.0780000000000000e-08, 2.9701172794500001e+00, 1.8770105769372001e+03 },
    { 1.0700000000000000e-08, 1.3051411132099999e+00, 8.2792358748649997e+02 },
    { 1.0680000000000000e-08, 5.7944248367000002e+00, 2.9577331481288002e+03 },
    { 1.0649999999999999e-08, 2.4754993686700000e+00, 6.1462449204849997e+02 },
    { 1.0610000000000000e-08, 2.5326733221700000e+00, 7.8643065262000000e+00 },
    { 1.0600000000000001e-08, 6.0487094184099996e+00, 2.5654191403697390e+04 },
    { 1.0579999999999999e-08, 3.1050635133900002e+00, 4.9447869076360001e+02 },
    { 1.0560000000000000e-08, 4.7899593383900001e+00, 2.1277783057619999e+02 },
    { 1.0560000000000000e-08, 2.9641806208400001e+00, 2.1426230328450001e+02 },
    { 1.0500000000000000e-08, 3.3377791983699998e+00, 1.7913262904579001e+03 },
    { 1.0480000000000000e-08, 2.9636046284200002e+00, 7.4591082183940000e+02 },
    { 1.0410000000000000e-08, 3.5211836573899999e+00, 1.6708250284999999e+03 },
    { 1.0390000000000001e-08, 3.0565447323099999e+00, 2.3320629558164001e+03 },
    { 1.0370000000000000e-08, 1.6188390920500000e+00, 7.5632338265689998e+02 },
    { 1.0350000000000000e-08, 2.0803921650400001e+00, 2.1077181601144998e+03 },
    { 1.0350000000000000e-08, 8.4548736210999997e-01, 6.7214061522839995e+02 },
    { 1.0340000000000000e-08, 4.5924898474300004e+00, 2.3593301268700000e+02 },
    { 1.0320000000000001e-08, 9.9207081398999997e-01, 2.4187090352019999e+02 },
    { 1.0320000000000001e-08, 7.5596269901000002e-01, 1.8472728735580000e+02 },
    { 1.0320000000000001e-08, 2.5012467220499999e+00, 7.7523338944700004e+02 },
    { 1.0239999999999999e-08, 2.2088505430200001e+00, 1.0262448293975001e+03 },
    { 1.0230000000000000e-08, 6.1735985263000002e+00, 3.0683064210100002e+02 },
    { 1.0230000000000000e-08, 2.6266711772600000e+00, 9.3531546663000000e+01 },
    { 1.0210000000000000e-08, 9.7936116353000002e-01, 5.8631331639719997e+02 },
    { 1.0179999999999999e-08, 1.9827209218000000e-01, 9.6962247809489998e+02 },
    { 1.0169999999999999e-08, 8.0348484501999995e-01, 1.2648167191010000e+03 },
    { 1.0140000000000000e-08, 2.2741993883200000e+00, 2.4643619622969999e+02 },
    { 1.0120000000000001e-08, 4.4789996995999998e-01, 1.3797059531089001e+03 },
    { 1.0109999999999999e-08, 2.4100142230700001e+00, 1.3866050922260001e+02 },
    { 1.0080000000000000e-08, 4.3406246107099999e+00, 7.0653628910000004e+00 },
    { 1.0010000000000000e-08, 1.6376917921200000e+00, 4.2304141737560003e+03 },
    { 1.0000000000000000e-08, 3.7044716925799999e+00, 1.1664068576709001e+03 },
    { 9.9699999999999993e-09, 1.4978148855800000e+00, 4.8631979130691998e+03 },
    { 9.9300000000000002e-09, 5.4181402641799998e+00, 6.3523141986799999e+02 },
    { 9.8600000000000003e-09, 3.5264614170400002e+00, 5.8422505651137999e+03 },
    { 9.8500000000000005e-09, 3.1169201834900000e+00, 2.0046253858959001e+03 },
    { 9.8199999999999996e-09, 6.1733564256599998e+00, 1.4710022675381999e+03 },
    { 9.8099999999999998e-09, 6.1206352826999999e-01, 7.0066423920080001e+02 },
    { 9.7700000000000008e-09, 1.8719970529300001e+00, 1.9872168981566001e+03 },
    { 9.7399999999999999e-09, 3.5149736614600000e+00, 9.8778172535960005e+02 },
    { 9.7300000000000001e-09, 4.5868924143700003e+00, 1.6960978227655000e+03 },
    { 9.7200000000000003e-09, 5.5466768668999999e-01, 3.4731970192218000e+03 },
    { 9.5999999999999999e-09, 2.1605806062699999e+00, 2.8261861859070000e+03 },
    { 9.5800000000000004e-09, 1.0151262579500000e+00, 5.3786617841622001e+03 },
    { 9.5700000000000007e-09, 3.8715546735099999e+00, 2.7515475996916002e+03 },
    { 9.5700000000000007e-09, 2.7704268486100001e+00, 4.3366355376700000e+02 },
    { 9.5100000000000005e-09, 2.5509580072200002e+00, 4.1873388434980001e+02 },
    { 9.4899999999999993e-09, 2.8735177801399998e+00, 1.1592933106701000e+03 },
    { 9.4699999999999998e-09, 2.8042577126400001e+00, 3.5691762464396002e+03 },
    { 9.4500000000000002e-09, 9.7464731674000005e-01, 2.0597310011610000e+02 },
    { 9.4500000000000002e-09, 3.9149788498500002e+00, 2.2062509075989999e+02 },
    { 9.4300000000000007e-09, 3.7705412340700000e+00, 4.4386366626339998e+02 },
    { 9.4199999999999993e-09, 1.9304243856300001e+00, 1.1487218766540001e+02 },
    { 9.4099999999999996e-09, 2.1016452781199999e+00, 1.1976754877499999e+02 },
    { 9.3999999999999998e-09, 8.1063715212999998e-01, 9.7415515816300001e+01 },
    { 9.3900000000000000e-09, 3.5034347316400001e+00, 2.0213271996984000e+04 },
    { 9.3600000000000008e-09, 3.4142554909600000e+00, 5.5013783421970004e+02 },
    { 9.3200000000000001e-09, 4.7619301180700004e+00, 8.3821852822500000e+02 },
    { 9.3000000000000006e-09, 8.5222232622000005e-01, 1.3103933701397000e+03 },
    { 9.2199999999999992e-09, 4.9499521512899998e+00, 9.7724602613333991e+03 },
    { 9.1399999999999995e-09, 7.2203323860000002e-02, 1.6811199692385001e+03 },
    { 9.1299999999999997e-09, 4.4973289349999996e+00, 2.8537238101960003e+02 },
    { 9.1100000000000002e-09, 1.3475027951100000e+00, 1.2505896250994001e+03 },
    { 9.1000000000000004e-09, 5.5994188395700002e+00, 2.1382036029980000e+02 },
    { 9.0500000000000000e-09, 4.4194841216000003e-01, 1.2779450795420800e+04 },
    { 9.0200000000000007e-09, 1.4695768539300000e+00, 4.7700076705510003e+02 },
    { 9.0099999999999993e-09, 2.1883519804999998e+00, 4.4400667861530002e+02 },
    { 8.9800000000000000e-09, 4.2035123570700001e+00, 5.4369930152402003e+03 },
    { 8.9600000000000005e-09, 4.0906181392300001e+00, 2.5661304950698199e+04 },
    { 8.9399999999999993e-09, 5.2051581337100004e+00, 3.8038159828799999e+03 },
    { 8.9299999999999996e-09, 1.4084770367899999e+00, 3.1496916025644000e+03 },
    { 8.9199999999999998e-09, 3.3891302538799999e+00, 4.4437132691939996e+03 },
    { 8.8699999999999994e-09, 8.1421674913999997e-01, 3.3629906980024002e+03 },
    { 8.8499999999999998e-09, 1.6438870907000001e+00, 6.0887779767699999e+02 },
    { 8.8400000000000001e-09, 2.9190021301100000e+00, 1.1494162363460001e+02 },
    { 8.7600000000000004e-09, 3.2389876225399998e+00, 1.6992792165031999e+03 },
    { 8.7500000000000006e-09, 4.5006844849499998e+00, 3.4945376602242000e+03 },
    { 8.7500000000000006e-09, 3.4142949239600000e+00, 9.4599421523210003e+02 },
    { 8.7199999999999997e-09, 3.5835124426999998e-01, 1.0970942747017000e+03 },
    { 8.7099999999999999e-09, 5.9741352066199997e+00, 1.0213285546211000e+04 },
    { 8.6800000000000006e-09, 2.8175400917600002e+00, 1.2146667056107601e+04 },
    { 8.6699999999999992e-09, 1.4060045406599999e+00, 2.4422692770357999e+03 },
    { 8.6699999999999992e-09, 1.8750001515200001e+00, 9.5209637027660006e+02 },
    { 8.6599999999999995e-09, 9.3216349587000003e-01, 5.1962507187710398e+04 },
    { 8.6499999999999997e-09, 5.1942079567999999e-01, 2.1258774073792001e+03 },
    { 8.6100000000000007e-09, 5.3501548505600001e+00, 1.4749344208012999e+03 },
    { 8.5899999999999995e-09, 4.6415509955800003e+00, 1.3473196118425999e+03 },
    { 8.5799999999999997e-09, 4.1245532367799997e+00, 7.8159617692239999e+02 },
    { 8.5400000000000007e-09, 5.9283415543300002e+00, 5.3998590583309999e+02 },
    { 8.5199999999999995e-09, 6.9770700154999998e-01, 2.9506196011279999e+03 },
    { 8.5199999999999995e-09, 4.0130688056899997e+00, 9.7355463135800005e+02 },
    { 8.4900000000000003e-09, 4.5838712447100001e+00, 8.3258945393249996e+02 },
    { 8.4800000000000005e-09, 2.7983322592400000e+00, 5.1582491417233996e+03 },
    { 8.4599999999999993e-09, 1.6046449083900001e+00, 2.1148317071153001e+03 },
    { 8.4599999999999993e-09, 4.4699164095199997e+00, 1.9902381732660001e+02 },
    { 8.4599999999999993e-09, 4.1970975707000002e-01, 2.2757437354940001e+02 },
    { 8.4499999999999996e-09, 3.4272409540200002e+00, 1.2395439248355001e+03 },
    { 8.4499999999999996e-09, 2.8540436706399999e+00, 1.5709136480191000e+03 },
    { 8.4100000000000005e-09, 4.4558446881300000e+00, 5.3591074021810005e+02 },
    { 8.4100000000000005e-09, 2.3176558914299998e+00, 4.7306861379200001e+02 },
    { 8.3899999999999994e-09, 4.4907719753000003e-01, 2.2020019411769999e+02 },
    { 8.3899999999999994e-09, 4.4405489690600000e+00, 2.0639799675830000e+02 },
    { 8.3699999999999998e-09, 1.1878762285800000e+00, 2.8546403739102002e+03 },
    { 8.3300000000000008e-09, 5.7984102398499999e+00, 8.8379517926369999e+02 },
    { 8.3199999999999994e-09, 6.2391186482799998e+00, 6.1632141307790005e+02 },
    { 8.3099999999999996e-09, 5.6279975699699998e+00, 9.3672027114598004e+03 },
    { 8.2900000000000001e-09, 3.7604270790799998e+00, 3.9069087570985998e+03 },
    { 8.2700000000000006e-09, 2.1039973576200000e+00, 2.1989137757699999e+02 },
    { 8.2599999999999992e-09, 5.9843078238500000e+00, 2.8489930026759998e+02 },
    { 8.2499999999999994e-09, 2.8868081042500000e+00, 4.9893210888850001e+02 },
    { 8.2499999999999994e-09, 3.1336123785500001e+00, 8.1762864674800005e+02 },
    { 8.2100000000000004e-09, 2.1321593679299999e+00, 4.2233006267552000e+03 },
    { 8.1500000000000002e-09, 4.5528392341400004e+00, 3.3558771510015999e+03 },
    { 8.0800000000000002e-09, 2.0132966365199998e+00, 5.8564776591153995e+03 },
    { 8.0100000000000003e-09, 4.0319522555000002e+00, 1.2552725378878999e+03 },
    { 7.9900000000000007e-09, 1.9049155591700000e+00, 4.9596316347189997e+02 },
    { 7.9599999999999998e-09, 8.5220984274000000e-01, 1.9786673806107989e+04 },
    { 7.9200000000000008e-09, 5.0764093520199998e+00, 5.6193429400900004e+02 },
    { 7.8800000000000001e-09, 2.0229400720599999e+00, 7.4920983565610004e+02 },
    { 7.8700000000000003e-09, 4.1428515446500001e+00, 9.5310776223289997e+02 },
    { 7.8600000000000006e-09, 3.0302576590600001e+00, 1.3851749687070001e+03 },
    { 7.8500000000000008e-09, 3.2646993692600002e+00, 1.5851407420206999e+03 },
    { 7.8299999999999996e-09, 2.3925372988000002e+00, 9.8426907772600003e+01 },
    { 7.8100000000000001e-09, 1.0908846780999999e+00, 4.7529915918498000e+03 },
    { 7.8000000000000004e-09, 4.3481230697899997e+00, 2.3217680150779001e+03 },
    { 7.7799999999999992e-09, 4.4616779686300001e+00, 4.1896337901340002e+02 },
    { 7.7499999999999999e-09, 4.1741104286199997e+00, 6.2025356634100001e+02 },
    { 7.6299999999999995e-09, 2.6850230038000000e+00, 1.9810879358989999e+02 },
    { 7.6299999999999995e-09, 2.2046031627899998e+00, 2.2848939728610000e+02 },
    { 7.6299999999999995e-09, 5.6712596765500001e+00, 1.9125783119411999e+03 },
    { 7.5800000000000007e-09, 4.1795441788599996e+00, 1.0931254934210000e+02 },
    { 7.5200000000000005e-09, 1.9385664303800001e+00, 3.1347110834979998e+02 },
    { 7.5200000000000005e-09, 6.0926523897999996e+00, 1.1312708252620000e+02 },
    { 7.4499999999999997e-09, 1.3232829741600001e+00, 4.0171150783180001e+03 },
    { 7.4499999999999997e-09, 5.5420631544800001e+00, 1.5780271950199001e+03 },
    { 7.4400000000000000e-09, 5.0129989579799998e+00, 5.4227659212385997e+03 },
    { 7.4300000000000002e-09, 4.9446569174399997e+00, 1.7842127434571000e+03 },
    { 7.4000000000000001e-09, 3.3707021115000002e-01, 1.3647280995819001e+03 },
    { 7.3600000000000002e-09, 4.1175297790899998e+00, 1.4607073267997000e+03 },
    { 7.3000000000000000e-09, 1.5402948421200000e+00, 1.3654788591073000e+03 },
    { 7.2900000000000003e-09, 2.8163536399600000e+00, 5.1742094545271590e+04 },
    { 7.2300000000000001e-09, 6.1840550973799999e+00, 6.6253120356299996e+02 },
    { 7.2000000000000000e-09, 2.1471901312199999e+00, 8.7832616366560001e+02 },
    { 7.1699999999999998e-09, 5.5637447164599996e+00, 1.0392501437542078e+05 },
    { 7.1399999999999997e-09, 2.8902349293200000e+00, 2.1155824666407002e+03 },
    { 7.1300000000000000e-09, 4.3661215256999997e+00, 1.6637114814992001e+03 },
    { 7.1099999999999996e-09, 3.2918457858000000e-01, 2.0592491600630001e+02 },
    { 7.1099999999999996e-09, 4.5604415880099998e+00, 2.2067327486970001e+02 },
    { 7.0600000000000000e-09, 1.8864197629999999e-02, 6.6427630870220003e+02 },
    { 7.0600000000000000e-09, 5.8356746408799998e+00, 6.4456315275999998e+02 },
    { 7.0500000000000003e-09, 4.4993795367700002e+00, 2.7533807271199998e+02 },
    { 6.9500000000000002e-09, 1.7570296417600000e+00, 8.1294573395949999e+02 },
    { 6.9399999999999996e-09, 3.9210820027200000e+00, 1.0728555991259999e+02 },
    { 6.9299999999999999e-09, 8.6742075682999997e-01, 2.5448005855260191e+04 },
    { 6.9200000000000001e-09, 1.2476055657400000e+00, 1.1653954657146000e+03 },
    { 6.9200000000000001e-09, 4.7052737815999999e+00, 3.5905168874420001e+03 },
    { 6.8699999999999996e-09, 1.2627909107899999e+00, 1.2616353253632999e+03 },
    { 6.8100000000000003e-09, 1.8362616925799999e+00, 1.0038446123289999e+02 },
    { 6.8100000000000003e-09, 2.4673062664400001e+00, 3.8855681160961999e+03 },
};

static const VSOP87Term VSOP87A_SATURN_Y2[465] = {
    { 5.6370653700000004e-03, 5.9711587824199999e+00, 2.0618554843720000e+02 },
    { 5.4701211599999999e-03, 2.0515497342600000e+00, 2.2041264243879999e+02 },
    { 4.5851861300000001e-03, 0.0000000000000000e+00, 0.0000000000000000e+00 },
    { 3.6229424899999999e-03, 8.9540100508999998e-01, 2.1329909543799999e+02 },
    { 2.2552164200000000e-03, 9.1699821444999996e-01, 4.2659819087599999e+02 },
    { 8.8390611000000005e-04, 3.3028944991700002e+00, 7.1135470007999997e+00 },
    { 5.0101314000000001e-04, 1.1297616383500000e+00, 4.3371173787679999e+02 },
    { 4.5516403000000002e-04, 5.0766946653899998e+00, 4.1948464387519999e+02 },
    { 3.2896744999999999e-04, 2.0890579380000000e-02, 6.3989728631399998e+02 },
    { 2.7199743000000002e-04, 1.9263841764000000e+00, 2.2752618943959999e+02 },
    { 1.3251505000000000e-04, 6.0769309940399996e+00, 1.9907200143640000e+02 },
    { 1.0425984000000000e-04, 3.1824686902799999e+00, 1.4227094001599999e+01 },
    { 6.6735560000000002e-05, 4.2474763388700003e+00, 1.1020632121940000e+02 },
    { 4.9340940000000003e-05, 1.5130117951600000e+00, 2.1648048917570000e+02 },
    { 4.7895539999999998e-05, 2.4337901915999999e-01, 2.1011770170029999e+02 },
    { 4.6585909999999998e-05, 2.6557833757999999e-01, 6.4701083331480004e+02 },
    { 4.1672679999999997e-05, 3.7320367139099999e+00, 3.1639186965660002e+02 },
    { 3.5095370000000001e-05, 5.4928144056799999e+00, 8.5319638175199998e+02 },
    { 2.7434700000000000e-05, 6.2193908388599999e+00, 1.0309277421860000e+02 },
    { 2.6611719999999999e-05, 1.5879541273600000e+00, 2.0936694217490000e+02 },
    { 2.5680180000000001e-05, 1.6811216098000001e-01, 2.1723124870109999e+02 },
    { 2.5411910000000000e-05, 1.0796465357400000e+00, 4.4082528487759998e+02 },
    { 2.5077380000000000e-05, 3.1188274629000001e+00, 1.1731986822019999e+02 },
    { 2.1590890000000000e-05, 4.9991256702399998e+00, 6.3278373931320004e+02 },
    { 1.8284120000000001e-05, 2.4336865058999999e+00, 3.2350541665740002e+02 },
    { 1.3516290000000001e-05, 4.8167388936400002e+00, 4.1237109687440000e+02 },
    { 1.1773050000000001e-05, 5.8448441218899996e+00, 3.0927832265580003e+02 },
    { 8.9156100000000005e-06, 5.7880622764000000e-01, 9.5979227217800002e+01 },
    { 8.0655400000000004e-06, 5.8016056051099998e+00, 2.0225339517410001e+02 },
    { 8.0491200000000003e-06, 1.7866305029800000e+00, 2.3463973644039999e+02 },
    { 7.3751799999999996e-06, 2.2900963330200002e+00, 2.2434479570190001e+02 },
    { 4.3932800000000002e-06, 2.0277682750600001e+00, 3.3061896365820002e+02 },
    { 4.3873800000000000e-06, 3.0400313376299999e+00, 2.1340641002400002e+01 },
    { 4.2744199999999998e-06, 5.7175757609900000e+00, 8.6030992875280003e+02 },
    { 4.0459499999999998e-06, 1.3825351741600000e+00, 2.2359403617650000e+02 },
    { 4.0004900000000002e-06, 6.4827543917999997e-01, 4.2977958461370002e+02 },
    { 3.7711600000000002e-06, 5.3963843240499996e+00, 1.1045700263900001e+01 },
    { 3.5601099999999999e-06, 4.9515346589999998e-01, 4.2266603761290003e+02 },
    { 3.3552900000000000e-06, 4.7222379297200003e+00, 1.0664954771900000e+03 },
    { 3.2297000000000002e-06, 3.7469251809299999e+00, 5.2969096509459996e+02 },
    { 3.1355700000000001e-06, 2.8742054766899998e+00, 3.1813937377000001e+00 },
    { 2.8380100000000000e-06, 9.2471217472000000e-01, 8.8865680217000005e+01 },
    { 2.7666400000000002e-06, 6.7598868270000001e-02, 6.2567019231239999e+02 },
    { 2.7426700000000000e-06, 7.9902442990000003e-02, 3.0216477565500003e+02 },
    { 2.6549799999999999e-06, 5.8058642355299996e+00, 4.2341679713830001e+02 },
    { 2.6323600000000002e-06, 1.7583807824200000e+00, 1.2443341522100000e+02 },
    { 2.4398400000000002e-06, 2.4814637624999999e-01, 6.5412438031559998e+02 },
    { 2.3480400000000001e-06, 1.9844138645900000e+00, 3.9321532631000000e+00 },
    { 2.2904199999999998e-06, 4.0578151728000000e-01, 2.0300415469949999e+02 },
    { 2.2288400000000000e-06, 6.1711598034600001e+00, 1.9195845443560000e+02 },
    { 1.9406600000000001e-06, 7.7928033729000001e-01, 1.4956319713459999e+02 },
    { 1.7505800000000001e-06, 3.2479150339000000e+00, 5.2257741809380002e+02 },
    { 1.5853500000000000e-06, 1.2484393583700000e+00, 5.3680451209540001e+02 },
    { 1.2378199999999999e-06, 1.7107877514000001e+00, 4.3764389113990001e+02 },
    { 1.1558800000000001e-06, 8.9164509194999997e-01, 2.7703499374140000e+02 },
    { 1.0912300000000000e-06, 2.5723411143099999e+00, 1.0294940738499999e+01 },
    { 1.0416699999999999e-06, 4.2924380295900004e+00, 5.1546387109299997e+02 },
    { 1.0207400000000001e-06, 5.3363642468399997e+00, 8.4608283475120004e+02 },
    { 9.4079999999999995e-07, 3.4838778915000002e-01, 4.3053034413910001e+02 },
    { 8.9665999999999998e-07, 2.3863497917099998e+00, 4.1555249061209997e+02 },
    { 8.2635000000000004e-07, 9.8367627117000001e-01, 4.4793883187839998e+02 },
    { 8.0157999999999996e-07, 3.0240381125200000e+00, 2.3145834270270001e+02 },
    { 5.9355000000000002e-07, 3.2563942738799998e+00, 7.2876296653099996e+02 },
    { 5.6164999999999995e-07, 3.5797654172600000e+00, 1.8159247264699999e+01 },
    { 5.5420999999999998e-07, 4.7153607677200000e+00, 8.3896928775039999e+02 },
    { 4.7324999999999998e-07, 4.4340304504399999e+00, 1.0593819301891999e+03 },
    { 4.6591999999999998e-07, 2.1639737230599998e+00, 1.9513984817330001e+02 },
    { 4.6404999999999998e-07, 1.1693691483399999e+00, 5.4391805909619995e+02 },
    { 4.5783999999999998e-07, 4.7212781395199999e+00, 4.0843894361129998e+02 },
    { 4.5494000000000002e-07, 4.3571773533300000e+00, 8.3185574074960005e+02 },
    { 4.5096000000000001e-07, 5.8798477695600004e+00, 6.1855664531160005e+02 },
    { 4.4256000000000001e-07, 5.3245575947499999e+00, 9.4206206196899996e+02 },
    { 4.3351000000000002e-07, 4.5003329155499996e+00, 7.4299006053259995e+02 },
    { 4.2263000000000002e-07, 5.9144159711800004e+00, 6.3596513305090002e+02 },
    { 3.9173999999999999e-07, 4.8986370388399996e+00, 1.0736090241908000e+03 },
    { 3.7270000000000000e-07, 5.7321060807000002e-01, 8.5827298831199997e+01 },
    { 3.6125999999999999e-07, 3.1114791003400000e+00, 2.0373786788240000e+02 },
    { 3.6033000000000000e-07, 4.8895861869299999e+00, 2.2286032299359999e+02 },
    { 3.4387999999999998e-07, 6.0988955485699998e+00, 6.4307868005169996e+02 },
    { 3.4252000000000003e-07, 6.2495429734499996e+00, 3.6286229257259998e+02 },
    { 3.3921999999999999e-07, 5.9559534499000000e-01, 4.3689313161450002e+02 },
    { 3.1901000000000000e-07, 1.1448175651800001e+00, 3.3773251065900001e+02 },
    { 3.0303999999999999e-07, 1.2867623414000000e+00, 5.6622351302600002e+01 },
    { 2.9536000000000002e-07, 3.7633131477799999e+00, 2.6992144674060000e+02 },
    { 2.8840000000000001e-07, 3.9416203778100001e+00, 1.2797945726280000e+03 },
    { 2.8666000000000000e-07, 2.8921088112400000e+00, 1.1338771495710000e+02 },
    { 2.6875000000000002e-07, 1.1451640226499999e+00, 1.3154696222179999e+02 },
    { 2.5348999999999998e-07, 3.0189454434099998e+00, 4.8322054217860000e+02 },
    { 2.5259000000000001e-07, 6.9371609826000002e-01, 7.5010360753340001e+02 },
    { 2.4872999999999998e-07, 4.4942144561000003e-01, 7.6266071275599998e+01 },
    { 2.4820000000000001e-07, 1.1029403429000000e+00, 8.1752133216199994e+01 },
    { 2.4694000000000001e-07, 4.1747484297200002e+00, 1.1553611574070001e+03 },
    { 2.3416000000000000e-07, 2.2510509693300000e+00, 9.5628915597059995e+02 },
    { 2.3209000000000001e-07, 5.7129379519699999e+00, 8.6742347575359997e+02 },
    { 2.1917000000000000e-07, 3.1503247693400001e+00, 1.0451548361876000e+03 },
    { 2.1337000000000001e-07, 2.8233505484999999e-01, 2.9505122865419997e+02 },
    { 2.1014000000000000e-07, 2.2228642589300001e+00, 1.6060889739850001e+02 },
    { 2.0842999999999999e-07, 5.5879824032699998e+00, 2.8808069400530002e+02 },
    { 2.0760000000000001e-07, 5.6217178132000001e+00, 4.1630325013750002e+02 },
    { 2.0311000000000000e-07, 5.3682587226000000e-01, 2.2091400527800001e+01 },
    { 2.0269000000000000e-07, 3.5975053180100001e+00, 7.0849445304200003e+01 },
    { 1.9674000000000000e-07, 5.1504291323999996e-01, 2.0076892246579999e+02 },
    { 1.9660000000000000e-07, 4.1234370419499999e+00, 1.1482476104062000e+03 },
    { 1.9564000000000000e-07, 2.2796975476200001e+00, 6.3735898303399999e+01 },
    { 1.9338000000000000e-07, 4.9977524941800002e+00, 1.2584539316256000e+03 },
    { 1.9266999999999999e-07, 5.2862399794200003e+00, 4.9744763618019999e+02 },
    { 1.8701000000000000e-07, 4.6001587154300001e+00, 1.4244965013379999e+02 },
    { 1.8325000000000000e-07, 4.2646327621099998e+00, 5.2690198039499997e+01 },
    { 1.8183999999999999e-07, 1.2333557601200000e+00, 2.2582926841020000e+02 },
    { 1.8136999999999999e-07, 5.3639557185799998e+00, 9.3494851496820002e+02 },
    { 1.7699000000000000e-07, 1.0918311345000000e+00, 3.4077089204480001e+02 },
    { 1.7317999999999999e-07, 1.5033942974500001e+00, 1.0702492748170000e+02 },
    { 1.6596000000000001e-07, 5.1961242360800002e+00, 6.2885158605009997e+02 },
    { 1.6443000000000000e-07, 5.1102434477000003e+00, 4.0525754987360000e+02 },
    { 1.6201999999999999e-07, 4.4779785723799996e+00, 3.4470304530790003e+02 },
    { 1.5757000000000000e-07, 1.0209047483400000e+00, 6.5094298657790000e+02 },
    { 1.5416000000000000e-07, 1.5954491462500000e+00, 2.4175328344120001e+02 },
    { 1.5349000000000001e-07, 4.0290044230500000e+00, 4.9033408917940000e+02 },
    { 1.5097999999999999e-07, 5.9383698261199998e+00, 2.6598929347749998e+02 },
    { 1.5048000000000001e-07, 2.4524830828300002e+00, 1.3851749687070000e+02 },
    { 1.3925000000000001e-07, 5.4675817756600003e+00, 6.3671589257630001e+02 },
    { 1.3873000000000000e-07, 4.1101559714800002e+00, 1.0522683831884001e+03 },
    { 1.3721000000000000e-07, 5.4585194233000001e-01, 3.5893013930950002e+02 },
    { 1.3637000000000000e-07, 4.6394588203099998e+00, 2.8414854074220000e+02 },
    { 1.3441000000000000e-07, 2.9322606434199998e+00, 6.2901893969922003e+03 },
    { 1.3344000000000001e-07, 3.3382239165100001e+00, 1.0627416795630000e+02 },
    { 1.3152000000000000e-07, 5.7129323430000001e-01, 6.2759623029905997e+03 },
    { 1.2996000000000001e-07, 9.4108602765000005e-01, 7.3587651353180001e+02 },
    { 1.2938000000000001e-07, 5.5722867319700002e+00, 1.8105576652360000e+02 },
    { 1.2863000000000001e-07, 5.7338127998999999e+00, 9.4917560896980001e+02 },
    { 1.2692000000000000e-07, 5.2230014731700001e+00, 1.3757737998458001e+03 },
    { 1.2627999999999999e-07, 4.3556339462900002e+00, 1.0220399093211799e+04 },
    { 1.2354999999999999e-07, 1.9946833335200000e+00, 1.0206171999210201e+04 },
    { 1.2303000000000001e-07, 2.1906609729099999e+00, 4.4475743814070000e+02 },
    { 1.1807000000000000e-07, 4.7748447611199998e+00, 4.2415051032119999e+02 },
    { 1.1522999999999999e-07, 4.2803815091599997e+00, 1.9120769491019999e+02 },
    { 1.1347000000000000e-07, 5.5221073531900000e+00, 3.5181659230870002e+02 },
    { 1.1234000000000000e-07, 1.2584048383399999e+00, 3.5033211960040001e+02 },
    { 1.0816000000000000e-07, 5.5765778361500002e+00, 1.2747179660680000e+02 },
    { 1.0468000000000000e-07, 2.4728291692300002e+00, 2.4554242435239999e+02 },
    { 1.0243000000000000e-07, 4.1103699793499997e+00, 7.0363318461740005e+02 },
    { 1.0114000000000000e-07, 3.9758591613699998e+00, 1.4717530270636000e+03 },
    { 1.0107000000000000e-07, 4.0146496757100003e+00, 2.3539049596580000e+02 },
    { 9.8239999999999998e-08, 4.1418322429499996e+00, 1.2530172972200001e+01 },
    { 9.4889999999999998e-08, 4.0743347272700001e+00, 1.3686602528450001e+03 },
    { 9.1979999999999994e-08, 3.9100363170999999e+00, 1.4646394800628000e+03 },
    { 9.1209999999999994e-08, 4.7227013887600000e+00, 1.5748458012822000e+03 },
    { 8.9799999999999997e-08, 5.0855039688600003e+00, 1.2513403846248000e+03 },
    { 8.4219999999999996e-08, 1.7788593244000001e-01, 6.6123792731640003e+02 },
    { 8.4019999999999994e-08, 1.7264511191400000e+00, 6.2830758499914000e+03 },
    { 8.2329999999999997e-08, 5.1104594832999997e+00, 5.8493641121146002e+03 },
    { 8.0500000000000000e-08, 4.4263357278299997e+00, 5.0835032409220003e+02 },
    { 7.9819999999999996e-08, 2.9446795101499998e+00, 1.3615467058442000e+03 },
    { 7.9790000000000002e-08, 3.1617062340600000e+00, 1.0213285546211000e+04 },
    { 7.9380000000000000e-08, 1.1891569133500000e+00, 5.8635912061161998e+03 },
    { 7.8969999999999998e-08, 1.4444386143999999e-01, 1.1624747044077999e+03 },
    { 7.8849999999999995e-08, 5.6193110431999997e+00, 5.6511568774670002e+02 },
    { 7.7639999999999999e-08, 4.2278131851199996e+00, 6.2173803904930003e+02 },
    { 7.6990000000000003e-08, 2.5085366400999998e-01, 9.7795738083342003e+03 },
    { 7.6440000000000002e-08, 1.2931681071500001e+00, 4.0450679034820001e+02 },
    { 7.4229999999999997e-08, 2.6127399789700001e+00, 9.7938009023358009e+03 },
    { 7.3500000000000003e-08, 5.4924305757100003e+00, 3.1321047591889999e+02 },
    { 6.9380000000000003e-08, 5.3945681814200004e+00, 2.0863322899200000e+02 },
    { 6.7780000000000002e-08, 5.7759538459800002e+00, 2.1796496188399999e+02 },
    { 6.5239999999999994e-08, 3.7725128474199998e+00, 4.2904587143079999e+02 },
    { 6.3619999999999997e-08, 5.2150509836000003e-01, 3.1957326339430000e+02 },
    { 6.2040000000000005e-08, 3.5627205836200000e+00, 1.9589060769869999e+02 },
    { 6.0530000000000000e-08, 1.3019557800899999e+00, 2.3070758317729999e+02 },
    { 5.9599999999999998e-08, 8.5552901966999995e-01, 1.1413847448250000e+02 },
    { 5.5320000000000002e-08, 4.7402028993799998e+00, 1.2655674786264001e+03 },
    { 5.3260000000000002e-08, 4.9844049312100003e+00, 1.6921656695024001e+03 },
    { 5.2850000000000000e-08, 4.7033345331199996e+00, 7.8713751830400000e+01 },
    { 5.0829999999999999e-08, 4.3049532282999996e+00, 8.4215068148809996e+02 },
    { 4.9089999999999998e-08, 3.7109358040500000e+00, 1.7881448967202000e+03 },
    { 4.9059999999999997e-08, 6.0344041525699996e+00, 4.7928838891549998e+02 },
    { 4.8860000000000002e-08, 4.7844921824999997e-01, 6.4382943957709995e+02 },
    { 4.5870000000000000e-08, 3.6757874813600000e+00, 1.6850521225016000e+03 },
    { 4.5739999999999998e-08, 3.6598249179400000e+00, 4.3615941843159999e+02 },
    { 4.5300000000000002e-08, 4.8006180438900001e+00, 1.5677322542813999e+03 },
    { 4.5130000000000001e-08, 1.8071659685399999e+00, 3.9814400287280000e+02 },
    { 4.4999999999999999e-08, 6.0395807719399999e+00, 9.9160620955499994e+01 },
    { 4.4960000000000000e-08, 4.5535353233999998e-01, 7.5721715453419995e+02 },
    { 4.4850000000000001e-08, 4.2714509127699998e+00, 1.2125202148330000e+02 },
    { 4.4279999999999997e-08, 6.2180889197899996e+00, 1.4788665740644001e+03 },
    { 4.3910000000000001e-08, 3.4868549142499998e+00, 7.4781598567299994e+01 },
    { 4.3840000000000001e-08, 3.6983811341299999e+00, 3.3365734504400001e+02 },
    { 4.3639999999999999e-08, 4.5087466606300000e+00, 8.2474219374879999e+02 },
    { 4.3480000000000003e-08, 5.7746492630000000e-02, 5.8564776591153995e+03 },
    { 4.2419999999999999e-08, 5.7340666911499998e+00, 2.8956516671359998e+02 },
    { 4.2400000000000002e-08, 2.8066849471499999e+00, 1.7516605980020000e+02 },
    { 4.2030000000000000e-08, 5.5140932678099999e+00, 5.7616138801060004e+02 },
    { 4.1980000000000003e-08, 4.3781505660300004e+00, 1.8912376709388000e+03 },
    { 4.0690000000000002e-08, 1.4828402394700000e+00, 9.7866873553349997e+03 },
    { 4.0270000000000003e-08, 4.4590287572899996e+00, 7.1074673161819999e+02 },
    { 4.0259999999999998e-08, 5.4031656080500001e+00, 2.5143213107579999e+02 },
    { 4.0259999999999998e-08, 1.4795397887099999e+00, 1.2035824960600000e+02 },
    { 3.9360000000000003e-08, 1.6750852833500001e+00, 3.2668681039510000e+02 },
    { 3.9319999999999997e-08, 2.5560332048099998e+00, 1.1695882514085999e+03 },
    { 3.8770000000000002e-08, 3.7099848289500001e+00, 1.7810313497193999e+03 },
    { 3.8759999999999998e-08, 2.8238124293000002e+00, 1.3458534360760001e+02 },
    { 3.7959999999999997e-08, 4.5459886117600004e+00, 9.8899988524600005e+01 },
    { 3.7679999999999997e-08, 4.9915288386699999e+00, 5.0137978944330001e+02 },
    { 3.7420000000000000e-08, 5.3355751225699999e+00, 8.4926422848890002e+02 },
    { 3.7380000000000001e-08, 3.7349355614999999e+00, 5.2650957135689998e+02 },
    { 3.6480000000000000e-08, 5.7534271347599999e+00, 6.1144309831080000e+02 },
    { 3.6410000000000000e-08, 4.0032748090000000e-02, 1.6682867252200001e+02 },
    { 3.5730000000000003e-08, 4.3775044717800000e+00, 1.5667674413540001e+02 },
    { 3.5329999999999999e-08, 5.7565855893999995e-01, 1.1439910691340000e+02 },
    { 3.3939999999999998e-08, 1.5284479023899999e+00, 1.9603362005060001e+02 },
    { 3.3780000000000002e-08, 3.6371546019999999e-02, 2.9912639426919998e+02 },
    { 3.3740000000000003e-08, 4.2496134927600000e+00, 6.9152524274800001e+01 },
    { 3.2469999999999999e-08, 2.6742361066100000e+00, 1.6779385755008000e+03 },
    { 3.0470000000000001e-08, 2.1958597766999999e-01, 2.3056457082540001e+02 },
    { 3.0010000000000002e-08, 8.7555285269000005e-01, 6.0697767545533998e+03 },
    { 2.9799999999999999e-08, 5.9425178239500003e+00, 1.5104766984290001e+02 },
    { 2.9270000000000000e-08, 7.5690409845999995e-01, 3.9557870223899999e+02 },
    { 2.9220000000000000e-08, 4.3827137534200000e+00, 1.5819593482830001e+03 },
    { 2.9090000000000001e-08, 2.5907881725299999e+00, 1.3703302416240001e+02 },
    { 2.8930000000000002e-08, 4.6819562998700004e+00, 3.8135160823740000e+02 },
    { 2.8800000000000000e-08, 5.6316732699000005e-01, 5.5103160609700001e+02 },
    { 2.8489999999999999e-08, 3.7778968209200001e+00, 9.2047073954699997e+01 },
    { 2.8190000000000000e-08, 4.6249688597900001e+00, 4.3126405732199999e+02 },
    { 2.8150000000000001e-08, 3.7083318615100000e+00, 4.1406801790380001e+02 },
    { 2.8130000000000001e-08, 4.0970094387699998e+00, 4.2193232442999999e+02 },
    { 2.8030000000000000e-08, 1.5479317553700001e+00, 2.5976951835400001e+02 },
    { 2.7990000000000001e-08, 1.6443592728100000e+00, 1.2991947716160001e+02 },
    { 2.7829999999999998e-08, 4.1215898476500001e+00, 1.2869081196288000e+03 },
    { 2.7710000000000001e-08, 5.5144192303799997e+00, 3.2032402291969998e+02 },
    { 2.7669999999999999e-08, 2.2344028588299998e+00, 9.9999864507729999e+03 },
    { 2.6689999999999999e-08, 2.3163089942000001e-01, 3.0609692891809999e+02 },
    { 2.6490000000000001e-08, 8.6332395470000003e-02, 8.4933526953900000e+01 },
    { 2.6040000000000000e-08, 5.5479650063200001e+00, 9.2783496796739996e+02 },
    { 2.5790000000000001e-08, 3.4161381949099998e+00, 1.4930936680660000e+03 },
    { 2.5570000000000000e-08, 4.1569961768000002e-01, 5.9803745040300001e+01 },
    { 2.5550000000000000e-08, 4.9678798459999998e+00, 2.2997386999439999e+02 },
    { 2.5550000000000000e-08, 6.2049316277699997e+00, 1.9662432088160000e+02 },
    { 2.5360000000000000e-08, 2.3408500977600002e+00, 6.9651963761659999e+02 },
    { 2.5119999999999999e-08, 1.2440226617800001e+00, 3.1219908396260001e+02 },
    { 2.4649999999999999e-08, 5.8153406852599998e+00, 5.7222923474749996e+02 },
    { 2.4610000000000000e-08, 2.3841513690700000e+00, 4.0918970313670002e+02 },
    { 2.4430000000000001e-08, 1.2924918844900000e+00, 1.2726810256271999e+03 },
    { 2.4420000000000000e-08, 2.9573583498899998e+00, 5.4695644048199995e+02 },
    { 2.4280000000000000e-08, 6.2367977799899998e+00, 4.5576651038700000e+01 },
    { 2.3980000000000000e-08, 4.2870700771300001e+00, 5.3287235883230005e+02 },
    { 2.3899999999999999e-08, 5.2906862086600004e+00, 8.5637777548969996e+02 },
    { 2.3530000000000000e-08, 6.1016829134500004e+00, 6.5019222705250002e+02 },
    { 2.3320000000000000e-08, 3.9976208733299998e+00, 1.4414657116320001e+02 },
    { 2.3180000000000000e-08, 4.4263791785700004e+00, 3.4321857259960001e+02 },
    { 2.2980000000000001e-08, 1.8553127362899999e+00, 1.8802630117250001e+02 },
    { 2.2819999999999999e-08, 4.7292670814999999e-01, 4.3912836384820002e+02 },
    { 2.2729999999999999e-08, 4.4999682649299997e+00, 1.3533610313299999e+02 },
    { 2.2679999999999999e-08, 3.6137027839800000e+00, 2.1045367663768002e+03 },
    { 2.2609999999999999e-08, 1.5398282964200001e+00, 4.2118156490460001e+02 },
    { 2.2560000000000002e-08, 1.1165088993700001e+00, 5.5406998748280000e+02 },
    { 2.2110000000000001e-08, 2.0917007819600002e+00, 2.7555052103309998e+02 },
    { 2.1929999999999999e-08, 2.5407408307499999e+00, 2.0522234059069999e+02 },
    { 2.1929999999999999e-08, 2.3488853358399999e+00, 2.2137585028530000e+02 },
    { 2.1929999999999999e-08, 6.2724795458799996e+00, 2.0714875628370001e+02 },
    { 2.1929999999999999e-08, 4.9003319278899999e+00, 2.1944943459230001e+02 },
    { 2.1750000000000000e-08, 4.0592269984499998e+00, 4.3222726516850003e+02 },
    { 2.1620000000000001e-08, 3.0295358476600001e+00, 2.3857188970350001e+02 },
    { 2.1480000000000001e-08, 3.4258678892900001e+00, 9.1693228005540004e+02 },
    { 2.1340000000000001e-08, 1.3838722461699999e+00, 1.7394221952280000e+02 },
    { 2.1220000000000001e-08, 4.7534140504100000e+00, 9.2940845831999994e+01 },
    { 2.1069999999999999e-08, 2.9612111629800002e+00, 1.4859801210651999e+03 },
    { 2.1030000000000000e-08, 3.5625737777099999e+00, 2.0014439921582000e+03 },
    { 2.0809999999999999e-08, 3.3607732482400001e+00, 1.8892007304980001e+02 },
    { 2.0440000000000000e-08, 5.9820106840199996e+00, 1.7952584437210000e+03 },
    { 2.0219999999999998e-08, 4.8206217292200000e+00, 2.0085575391590000e+03 },
    { 2.0070000000000000e-08, 2.6003774720000001e-02, 5.6431785636774002e+03 },
    { 1.9880000000000000e-08, 4.2030799124399998e+00, 1.1411340634054000e+03 },
    { 1.9659999999999999e-08, 4.8098907612000001e+00, 1.3140394986990000e+02 },
    { 1.9550000000000000e-08, 3.1798328110799998e+00, 2.4138150890326001e+03 },
    { 1.9540000000000002e-08, 3.4136511463999999e+00, 2.0974232193759999e+03 },
    { 1.9410000000000000e-08, 1.8052329131500000e+00, 3.7390799283650000e+02 },
    { 1.9309999999999999e-08, 2.2197985859200000e+00, 2.5887574647669999e+02 },
    { 1.9280000000000001e-08, 1.6484715495100000e+00, 3.1245971639350000e+02 },
    { 1.9130000000000000e-08, 7.5145133040000006e-02, 2.9519424100610001e+02 },
    { 1.8950000000000001e-08, 3.4764652655299999e+00, 2.9126208774300000e+02 },
    { 1.8940000000000000e-08, 1.6998005057200001e+00, 1.3828873468465999e+03 },
    { 1.8909999999999999e-08, 3.3757750809200000e+00, 5.5651766803759995e+02 },
    { 1.8250000000000001e-08, 2.9368090914000000e-01, 3.4484605765980001e+02 },
    { 1.8200000000000001e-08, 3.0912306843600001e+00, 3.4788443904560000e+02 },
    { 1.8189999999999999e-08, 1.4686151529000000e+00, 1.2050126195790000e+02 },
    { 1.8189999999999999e-08, 5.2235008104200000e+00, 2.2046082654860001e+02 },
    { 1.8189999999999999e-08, 5.9493106633500004e+00, 2.0613736432740001e+02 },
    { 1.8189999999999999e-08, 2.8639097132900000e+00, 2.0623373254699999e+02 },
    { 1.8189999999999999e-08, 2.0257164532999998e+00, 2.2036445832900000e+02 },
    { 1.8150000000000000e-08, 1.4154454958900000e+00, 9.5733882598969994e+03 },
    { 1.8119999999999999e-08, 5.7428797036599999e+00, 5.7074476203920005e+02 },
    { 1.8050000000000000e-08, 4.7458311902899997e+00, 4.2161870797543997e+03 },
    { 1.7949999999999999e-08, 4.2504631383699998e+00, 9.1300012679229997e+02 },
    { 1.7910000000000000e-08, 5.6660097125400002e+00, 9.9911380480899993e+01 },
    { 1.7859999999999999e-08, 4.6704462957299997e+00, 2.3767811782620001e+02 },
    { 1.7770000000000000e-08, 4.0675670544200004e+00, 1.5890728952838001e+03 },
    { 1.7750000000000000e-08, 4.2810069413700003e+00, 1.1642609634290000e+02 },
    { 1.7640000000000001e-08, 1.2279143228900000e+00, 1.8647239249500001e+02 },
    { 1.7490000000000000e-08, 4.5258550139200002e+00, 4.1703696332039999e+02 },
    { 1.7490000000000000e-08, 1.7892000727099999e+00, 8.9877303279069997e+02 },
    { 1.7420000000000000e-08, 3.8038243649700001e+00, 4.2563498302950001e+02 },
    { 1.7369999999999999e-08, 1.7489616820000001e-01, 1.6772244439930000e+02 },
    { 1.7260000000000000e-08, 2.5606355998399999e+00, 4.8640193591629998e+02 },
    { 1.7080000000000001e-08, 3.7157115674400001e+00, 8.1895145568100006e+01 },
    { 1.7030000000000001e-08, 1.2563524115700000e+00, 4.2756139872249997e+02 },
    { 1.6910000000000001e-08, 8.6915956974999997e-01, 4.5505237887919998e+02 },
    { 1.6899999999999999e-08, 4.0688827113699997e+00, 2.2076295405954002e+03 },
    { 1.6890000000000001e-08, 1.9408709330200000e+00, 3.2743756992049998e+02 },
    { 1.6870000000000002e-08, 4.5855191858800000e+00, 5.1939602435610004e+02 },
    { 1.6639999999999999e-08, 3.6159363492000002e-01, 2.5265597135319999e+02 },
    { 1.6630000000000001e-08, 7.0583379772999999e-01, 1.4563104387150000e+02 },
    { 1.6569999999999999e-08, 3.9981930277400002e+00, 1.4575259330619999e+03 },
    { 1.6460000000000000e-08, 4.5914077695600000e+00, 5.6363121503840000e+02 },
    { 1.6079999999999999e-08, 1.1666514504700001e+00, 3.2527843767830000e+03 },
    { 1.5970000000000000e-08, 4.1577046538199998e+00, 2.8096714700450002e+02 },
    { 1.5959999999999999e-08, 4.5957915236600000e+00, 1.8841241239379999e+03 },
    { 1.5900000000000000e-08, 4.8232889058100001e+00, 6.8547393735269998e+02 },
    { 1.5860000000000001e-08, 1.1779101405700001e+00, 3.8038159828799999e+03 },
    { 1.5810000000000001e-08, 5.2730422347600001e+00, 1.2442268376239999e+03 },
    { 1.5799999999999999e-08, 4.9964987358200004e+00, 5.6118353448360006e+02 },
    { 1.5790000000000001e-08, 5.0119049352599996e+00, 2.1085141488319999e+02 },
    { 1.5559999999999998e-08, 1.0892488499499999e+00, 1.9054647649404001e+03 },
    { 1.5360000000000000e-08, 1.0370420846799999e+00, 3.6936096616606001e+03 },
    { 1.4950000000000002e-08, 6.3703973970000005e-01, 1.0562005364515001e+03 },
    { 1.4909999999999999e-08, 5.2012009126000003e-01, 2.4012579838100001e+02 },
    { 1.4640000000000001e-08, 9.2036676895000002e-01, 4.7217484191469998e+02 },
    { 1.4570000000000001e-08, 8.2523140285999996e-01, 3.5976304344427999e+03 },
    { 1.4410000000000000e-08, 3.5171984643399998e+00, 7.3269511979410004e+02 },
    { 1.4390000000000000e-08, 1.2364835405000001e-01, 6.9970103135429997e+02 },
    { 1.4380000000000000e-08, 3.0748347400500000e+00, 1.8023719907218001e+03 },
    { 1.4230000000000001e-08, 5.5544219359899998e+00, 2.0151968199120000e+02 },
    { 1.4230000000000001e-08, 5.6183895377799997e+00, 2.2507850888479999e+02 },
    { 1.4140000000000000e-08, 1.5014870586700000e+00, 1.3718416465826999e+03 },
    { 1.3700000000000000e-08, 4.9750246295900000e+00, 8.3578789401270001e+02 },
    { 1.3510000000000000e-08, 8.2163520410000002e-01, 5.4298794682394000e+03 },
    { 1.3430000000000000e-08, 4.9551592782000000e+00, 1.0554497769261000e+03 },
    { 1.3389999999999999e-08, 4.4221289335199998e+00, 1.8983512179396000e+03 },
    { 1.3280000000000000e-08, 1.0187657372500001e+00, 2.1426230328450001e+02 },
    { 1.3280000000000000e-08, 3.8708604293400000e+00, 2.1233588759150001e+02 },
    { 1.3259999999999999e-08, 1.4166201514100001e+00, 9.8066817835879999e+02 },
    { 1.3210000000000000e-08, 5.6906762227599996e+00, 3.5574874557179999e+02 },
    { 1.3130000000000001e-08, 5.4663805264900001e+00, 3.8133035637799999e+01 },
    { 1.3110000000000000e-08, 4.6861825168099998e+00, 4.1130943055358002e+03 },
    { 1.3049999999999999e-08, 2.2526181909099998e+00, 9.6872999095099999e+01 },
    { 1.2870000000000000e-08, 2.7372331400599998e+00, 2.8454188003199999e+01 },
    { 1.2760000000000000e-08, 4.5778245543100002e+00, 1.1767017984094000e+03 },
    { 1.2600000000000000e-08, 5.6725795748000003e+00, 8.5001498801430000e+02 },
    { 1.2569999999999999e-08, 5.9358948653499999e+00, 6.2960234557549995e+02 },
    { 1.2450000000000001e-08, 2.2451812084500000e+00, 9.3600891644589992e+03 },
    { 1.2410000000000000e-08, 4.4122400421999997e+00, 2.1934024465938001e+03 },
    { 1.2229999999999999e-08, 3.4589530512900000e+00, 9.0588657979150003e+02 },
    { 1.2220000000000000e-08, 4.1409538618299999e+00, 2.1334727954780001e+02 },
    { 1.2220000000000000e-08, 7.4867230476000002e-01, 2.1325091132820000e+02 },
    { 1.2120000000000000e-08, 2.7410599509500000e+00, 2.6271141844705999e+03 },
    { 1.2050000000000000e-08, 3.2877293985199998e+00, 1.0490869894507000e+03 },
    { 1.1970000000000001e-08, 5.0166782062699999e+00, 4.2090735327536004e+03 },
    { 1.1910000000000000e-08, 5.4419432212999996e-01, 1.5349535039770001e+02 },
    { 1.1880000000000000e-08, 6.2567152281600000e+00, 5.2549817940059995e+02 },
    { 1.1840000000000000e-08, 2.7707810302200002e+00, 1.9943304451573999e+03 },
    { 1.1679999999999999e-08, 2.5468840622000000e-01, 3.9069087570985998e+03 },
    { 1.1670000000000000e-08, 3.0227873943199999e+00, 1.9365537546499999e+02 },
    { 1.1670000000000000e-08, 1.8668387722699999e+00, 2.3294281541100000e+02 },
    { 1.1640000000000000e-08, 4.9462197707000000e+00, 1.4807872442630000e+02 },
    { 1.1600000000000000e-08, 4.3268843013299998e+00, 6.7668051566499997e+01 },
    { 1.1580000000000000e-08, 5.4728893003000003e+00, 3.9957744373156002e+03 },
    { 1.1560000000000001e-08, 4.9657182728700002e+00, 1.5606187072805999e+03 },
    { 1.1560000000000001e-08, 1.1664875642899999e+00, 2.9648466951296000e+03 },
    { 1.1370000000000000e-08, 3.5902787935700000e+00, 2.6413412784722000e+03 },
    { 1.1339999999999999e-08, 1.9781526766699999e+00, 9.2797833480099996e+01 },
    { 1.1339999999999999e-08, 5.9416197367399999e+00, 3.7824753418775999e+03 },
    { 1.1250000000000000e-08, 6.1961503719000000e-01, 3.9140223040994001e+03 },
    { 1.1220000000000001e-08, 2.4585656712200001e+00, 1.2836556848410001e+02 },
    { 1.1160000000000000e-08, 4.6404981880799996e+00, 2.3249494088155998e+03 },
    { 1.1129999999999999e-08, 3.8689220377200000e+00, 2.5240214102519999e+03 },
    { 1.1069999999999999e-08, 2.5401599996700002e+00, 2.3107223148140001e+03 },
    { 1.1030000000000000e-08, 5.2499698231899998e+00, 7.3905790726949999e+02 },
    { 1.1030000000000000e-08, 5.2990683290399998e+00, 4.0028879843164000e+03 },
    { 1.1010000000000001e-08, 3.1366717265699999e+00, 2.3320629558164001e+03 },
    { 1.0940000000000001e-08, 3.0663052004700000e+00, 2.7302069586891998e+03 },
    { 1.0920000000000000e-08, 1.5329393473000000e-01, 5.2575881183150000e+02 },
    { 1.0540000000000000e-08, 3.2429800430300002e+00, 2.6342277314714001e+03 },
    { 1.0420000000000000e-08, 4.5107280664399996e+00, 4.5325789494110004e+03 },
    { 1.0410000000000000e-08, 3.2323357127899999e+00, 3.1781457905676002e+03 },
    { 1.0239999999999999e-08, 6.0364586990999998e+00, 5.6360650166766000e+03 },
    { 1.0179999999999999e-08, 3.5953664565899999e+00, 5.5800214074589996e+02 },
    { 1.0140000000000000e-08, 3.5696233238700001e+00, 2.5169078632512001e+03 },
    { 1.0120000000000001e-08, 2.0249023622800002e+00, 2.1160217440860001e+02 },
    { 1.0120000000000001e-08, 2.8647238043100001e+00, 2.1499601646740001e+02 },
    { 1.0080000000000000e-08, 4.0017662993199998e+00, 2.5494359321360000e+02 },
    { 1.0080000000000000e-08, 2.0592142418699999e+00, 2.0326478713040001e+02 },
    { 1.0080000000000000e-08, 2.8304119247199999e+00, 2.2333340374560001e+02 },
    { 1.0039999999999999e-08, 2.1415189260899998e+00, 9.4599421523210003e+02 },
    { 1.0020000000000000e-08, 4.4639484364299999e+00, 6.0766952886799999e+01 },
    { 1.0000000000000000e-08, 4.5956532081399999e+00, 7.7693031047640000e+02 },
    { 9.9900000000000005e-09, 3.1126558126199999e+00, 2.4067015420317998e+03 },
    { 9.9900000000000005e-09, 4.0667674472100002e+00, 1.7165459766239999e+02 },
    { 9.8999999999999993e-09, 1.3039500260000000e-02, 4.1202078525365996e+03 },
    { 9.8400000000000008e-09, 8.8646918919999995e-01, 2.2218566345969998e+03 },
    { 9.7300000000000001e-09, 6.2734636369399999e+00, 1.8180652604900001e+02 },
    { 9.7300000000000001e-09, 1.7577551832400000e+00, 2.4479166482700001e+02 },
    { 9.6400000000000006e-09, 5.7985297292200002e+00, 3.2972519178089999e+02 },
    { 9.6199999999999995e-09, 3.7639268504299999e+00, 2.1404985496340001e+02 },
    { 9.6199999999999995e-09, 1.1256993161600000e+00, 2.1254833591260001e+02 },
    { 9.5599999999999992e-09, 3.5926493461100000e+00, 2.3178358618148000e+03 },
    { 9.5399999999999997e-09, 1.5110888401600000e+00, 2.0670681329900000e+02 },
    { 9.5399999999999997e-09, 3.3785373264300000e+00, 2.1989137757699999e+02 },
    { 9.5399999999999997e-09, 1.0189462293000000e+00, 2.0566428357539999e+02 },
    { 9.5399999999999997e-09, 3.8706799372899998e+00, 2.2093390730060000e+02 },
    { 9.5000000000000007e-09, 3.8275761652700000e+00, 1.8094855377225999e+03 },
    { 9.4799999999999995e-09, 1.1270822957000000e-01, 2.1574677599280000e+02 },
    { 9.4500000000000002e-09, 5.9853957925900003e+00, 1.0515176236630000e+03 },
    { 9.4500000000000002e-09, 5.8345027460100001e+00, 3.8846515523820000e+02 },
    { 9.4300000000000007e-09, 5.2469754605999996e-01, 9.5310776223289997e+02 },
    { 9.4099999999999996e-09, 3.5309521085100002e+00, 7.3980866679489998e+02 },
    { 9.3900000000000000e-09, 5.1950272882100004e+00, 2.9363925071263998e+03 },
    { 9.2599999999999999e-09, 3.7725974556200002e+00, 6.0768903015542001e+03 },
    { 9.2500000000000001e-09, 5.2121565897000002e+00, 7.3168372783780001e+02 },
    { 9.1999999999999997e-09, 1.6974937347900001e+00, 4.5884151979040001e+02 },
    { 9.1800000000000001e-09, 7.1003260735999996e-01, 7.1453587252939997e+02 },
    { 9.1700000000000004e-09, 5.9681009316400004e+00, 6.0176425067620005e+02 },
    { 9.0900000000000007e-09, 1.3037115847500000e+00, 7.4591082183940000e+02 },
    { 9.0300000000000005e-09, 1.7043388679100000e+00, 8.3821852822500000e+02 },
    { 9.0300000000000005e-09, 1.8176721950999999e-01, 8.2399143422340001e+02 },
    { 8.9899999999999998e-09, 1.1627624936200001e+00, 9.5662747128962001e+03 },
    { 8.9700000000000003e-09, 2.7376157899100000e+00, 3.4945376602242000e+03 },
    { 8.9100000000000000e-09, 8.2111399765000004e-01, 1.2623860848887000e+03 },
    { 8.8699999999999994e-09, 3.4685732011099999e+00, 1.0372905296613999e+03 },
    { 8.8300000000000003e-09, 1.4085296374000000e-01, 6.2248879857470001e+02 },
    { 8.8200000000000006e-09, 3.4528137734400000e+00, 1.7739178027185999e+03 },
    { 8.7899999999999996e-09, 5.0606087953500003e+00, 3.4874241132234001e+03 },
    { 8.7500000000000006e-09, 5.1770378207799999e+00, 7.3194436026870005e+02 },
    { 8.7299999999999994e-09, 4.1444329692200004e+00, 2.5097943162503998e+03 },
    { 8.7099999999999999e-09, 4.7013342669200000e+00, 3.9739324334740002e+02 },
    { 8.6800000000000006e-09, 4.1889561494100001e+00, 6.2491943278700001e+02 },
    { 8.6399999999999999e-09, 5.1976279566599999e+00, 1.0007099997773799e+04 },
    { 8.5299999999999993e-09, 1.9577960856200001e+00, 1.9125783119411999e+03 },
    { 8.4700000000000007e-09, 4.9755100958599998e+00, 3.3772177920039999e+03 },
    { 8.4599999999999993e-09, 1.1479666168100000e+00, 3.0679394693482000e+03 },
    { 8.4000000000000008e-09, 1.9766601110799999e+00, 6.1069233878540001e+02 },
    { 8.3600000000000001e-09, 3.3036773151299998e+00, 2.8332997329077998e+03 },
    { 8.2599999999999992e-09, 3.9906360480599998e+00, 2.2147430875962000e+03 },
    { 8.1799999999999995e-09, 4.7783435309200000e+00, 3.7007232086613999e+03 },
    { 8.1799999999999995e-09, 3.4286719873100000e+00, 3.0534616939270001e+02 },
    { 8.1300000000000007e-09, 3.8362505901900001e+00, 2.4209286360333999e+03 },
    { 8.0999999999999997e-09, 4.4126148230900002e+00, 8.3503713448730002e+02 },
    { 8.0399999999999995e-09, 7.7955990249999996e-02, 4.1162033734900001e+02 },
    { 8.0299999999999998e-09, 1.2271627005000001e-01, 4.6658664459999999e+00 },
    { 8.0100000000000003e-09, 3.2882477813400000e+00, 3.6864961146597998e+03 },
    { 7.9099999999999994e-09, 4.4199384843000000e-01, 2.0199276274319999e+02 },
    { 7.9099999999999994e-09, 4.4476323181600002e+00, 2.2460542813280000e+02 },
    { 7.7599999999999997e-09, 3.1123801142800001e+00, 6.0626632075526004e+03 },
    { 7.6100000000000000e-09, 4.4472900208299997e+00, 4.4294861751924000e+03 },
    { 7.5900000000000005e-09, 4.7992861896200001e+00, 7.7841478318470001e+02 },
    { 7.5599999999999995e-09, 4.7549309740999997e+00, 1.0380412891868000e+03 },
    { 7.5300000000000003e-09, 3.4751224194999999e+00, 3.0508553696180002e+02 },
    { 7.4999999999999993e-09, 4.6348346967199996e+00, 4.4223726281915997e+03 },
    { 7.4600000000000003e-09, 4.9442007374200001e+00, 1.0807225711916001e+03 },
    { 7.4499999999999997e-09, 2.2621276375599999e+00, 8.4290144101349995e+02 },
    { 7.4400000000000000e-09, 3.0865309398599998e+00, 2.7851946644970002e+02 },
    { 7.4400000000000000e-09, 5.2855155486400003e+00, 5.0456118318099999e+02 },
    { 7.4300000000000002e-09, 3.1377566208899998e+00, 1.1191856752295000e+03 },
    { 7.3200000000000004e-09, 5.1775969447700003e+00, 1.7224529849340001e+02 },
    { 7.3200000000000004e-09, 2.8536218754100000e+00, 2.5435289238260000e+02 },
    { 7.2600000000000002e-09, 4.5431368848100000e+00, 9.9928729037722005e+03 },
    { 7.0900000000000001e-09, 3.2407302508000000e+00, 2.1187638603783998e+03 },
    { 6.9299999999999999e-09, 3.9423943603699998e+00, 6.3744960575920004e+02 },
};

static const VSOP87Term VSOP87A_SATURN_Y3[201] = {
    { 7.7376615000000000e-04, 1.4039104896100001e+00, 2.0618554843720000e+02 },
    { 7.5564350999999998e-04, 3.1962896379000000e-01, 2.2041264243879999e+02 },
    { 2.2843837000000001e-04, 3.1415926535900001e+00, 0.0000000000000000e+00 },
    { 1.0672263000000000e-04, 5.3649566381999998e+00, 4.2659819087599999e+02 },
    { 9.0101749999999995e-05, 5.6286514664500000e+00, 4.3371173787679999e+02 },
    { 8.2987229999999999e-05, 1.5226256351900000e+00, 7.1135470007999997e+00 },
    { 7.4180179999999997e-05, 2.1442310101000001e-01, 2.2752618943959999e+02 },
    { 4.5070610000000002e-05, 6.7248969479999998e-01, 4.1948464387519999e+02 },
    { 3.5816820000000000e-05, 1.5146678602999999e+00, 1.9907200143640000e+02 },
    { 2.9819690000000001e-05, 4.4286895162700004e+00, 6.3989728631399998e+02 },
    { 2.3762210000000002e-05, 1.4623277917999999e+00, 1.4227094001599999e+01 },
    { 1.0242630000000000e-05, 4.7233791719599996e+00, 6.4701083331480004e+02 },
    { 8.0791599999999994e-06, 5.9938557555200003e+00, 2.1648048917570000e+02 },
    { 7.8882599999999992e-06, 5.6107810412300001e+00, 4.4082528487759998e+02 },
    { 7.7083099999999994e-06, 2.0423480891699999e+00, 2.1011770170029999e+02 },
    { 4.5644300000000003e-06, 3.5652337109299999e+00, 8.5319638175199998e+02 },
    { 3.8421000000000000e-06, 1.4854244509000001e+00, 1.1731986822019999e+02 },
    { 3.5576900000000000e-06, 5.4070106655699997e+00, 2.1329909543799999e+02 },
    { 3.3801600000000000e-06, 2.4722672157300001e+00, 1.1020632121940000e+02 },
    { 3.2414800000000001e-06, 8.0746674759999998e-02, 2.3463973644039999e+02 },
    { 2.9660300000000000e-06, 3.2077836251000003e-01, 4.1237109687440000e+02 },
    { 2.4137800000000000e-06, 1.7977237542400000e+00, 1.0309277421860000e+02 },
    { 1.9820200000000000e-06, 5.5185797871600002e+00, 3.1639186965660002e+02 },
    { 1.9523400000000002e-06, 7.2153771654999999e-01, 6.3278373931320004e+02 },
    { 1.5402800000000000e-06, 1.3154362642499999e+00, 2.1340641002400002e+01 },
    { 1.5169799999999999e-06, 1.3828135136499999e+00, 3.0927832265580003e+02 },
    { 1.5077700000000000e-06, 7.7624311647999999e-01, 3.2350541665740002e+02 },
    { 1.3165300000000000e-06, 2.2672257459899998e+00, 9.5979227217800002e+01 },
    { 1.2517000000000000e-06, 6.1497233827800004e+00, 2.0936694217490000e+02 },
    { 1.1453700000000001e-06, 5.8772983775799998e+00, 2.2359403617650000e+02 },
    { 1.1322100000000000e-06, 1.8414105264500000e+00, 2.1723124870109999e+02 },
    { 1.1123900000000001e-06, 3.8466874991800002e+00, 8.6030992875280003e+02 },
    { 8.7336000000000003e-07, 5.0819554229600001e+00, 4.2977958461370002e+02 },
    { 8.7163000000000003e-07, 1.6017432881400000e+00, 1.9195845443560000e+02 },
    { 8.5241000000000005e-07, 3.3009287076999999e-01, 3.3061896365820002e+02 },
    { 8.5135999999999997e-07, 4.7461446521499999e+00, 6.5412438031559998e+02 },
    { 7.9372000000000004e-07, 2.6726096624100002e+00, 8.8865680217000005e+01 },
    { 7.8273999999999998e-07, 6.2741942132300004e+00, 1.2443341522100000e+02 },
    { 7.4114000000000003e-07, 1.8890347757999999e+00, 3.0216477565500003e+02 },
    { 5.7973999999999998e-07, 2.2635933486000002e+00, 2.0300415469949999e+02 },
    { 5.6135999999999999e-07, 2.7572892579000001e+00, 1.0664954771900000e+03 },
    { 5.5318000000000003e-07, 1.5695136165900001e+00, 2.0225339517410001e+02 },
    { 5.4081000000000004e-07, 1.6896886000000000e-01, 2.2434479570190001e+02 },
    { 4.5788999999999998e-07, 2.0067417998899999e+00, 6.2567019231239999e+02 },
    { 4.3284000000000000e-07, 4.9481644970899996e+00, 4.2266603761290003e+02 },
    { 3.6708999999999999e-07, 5.5164347759699996e+00, 4.4793883187839998e+02 },
    { 2.7097000000000000e-07, 1.9581976595299999e+00, 4.2341679713830001e+02 },
    { 2.5777000000000001e-07, 3.7841277356799998e+00, 3.9321532631000000e+00 },
    { 2.5216000000000000e-07, 4.2134626931700003e+00, 5.2257741809380002e+02 },
    { 1.8670000000000000e-07, 5.7748790307400002e+00, 5.1546387109299997e+02 },
    { 1.7669000000000000e-07, 3.8615276485200001e+00, 1.9513984817330001e+02 },
    { 1.5477000000000000e-07, 5.7927538987000000e+00, 5.3680451209540001e+02 },
    { 1.5244000000000001e-07, 3.6877639623200000e+00, 1.1045700263900001e+01 },
    { 1.2459000000000001e-07, 5.8516108564099998e+00, 4.3764389113990001e+02 },
    { 1.1913000000000000e-07, 5.0398107754700003e+00, 4.3689313161450002e+02 },
    { 1.1569000000000000e-07, 1.4583492141600001e+00, 6.1855664531160005e+02 },
    { 1.1351000000000000e-07, 3.0213594527600001e+00, 1.0736090241908000e+03 },
    { 1.0183000000000001e-07, 5.5701396781800003e+00, 3.3773251065900001e+02 },
    { 9.9509999999999996e-08, 5.6601756060700001e+00, 5.4391805909619995e+02 },
    { 9.8850000000000002e-08, 5.6820637002899996e+00, 1.3154696222179999e+02 },
    { 9.6629999999999999e-08, 2.8919238932900000e+00, 8.1752133216199994e+01 },
    { 8.8119999999999998e-08, 6.0812092660900001e+00, 8.3185574074960005e+02 },
    { 8.6319999999999995e-08, 4.2277982480500000e+00, 6.4307868005169996e+02 },
    { 8.3290000000000000e-08, 3.9026356415499999e+00, 8.6742347575359997e+02 },
    { 8.2910000000000006e-08, 4.9136384630299998e+00, 7.2876296653099996e+02 },
    { 8.2220000000000004e-08, 2.0913167394499999e+00, 2.9505122865419997e+02 },
    { 7.5610000000000000e-08, 5.3868939753299996e+00, 1.4956319713459999e+02 },
    { 7.3399999999999996e-08, 6.2446686550999997e-01, 8.4608283475120004e+02 },
    { 7.1089999999999997e-08, 4.0750815196300003e+00, 6.3596513305090002e+02 },
    { 7.0340000000000006e-08, 1.5202187462400001e+00, 1.8159247264699999e+01 },
    { 6.9320000000000001e-08, 1.0798167985100000e+00, 2.3145834270270001e+02 },
    { 6.5499999999999998e-08, 4.9524007864000003e-01, 4.0843894361129998e+02 },
    { 6.2649999999999995e-08, 3.2184191833400000e+00, 7.3587651353180001e+02 },
    { 6.0720000000000004e-08, 3.2796840981300002e+00, 4.3053034413910001e+02 },
    { 5.8420000000000003e-08, 5.5980618938599997e+00, 2.6992144674060000e+02 },
    { 5.7369999999999997e-08, 1.9266833291100001e+00, 1.2797945726280000e+03 },
    { 5.4259999999999998e-08, 6.2300450980799997e+00, 2.4175328344120001e+02 },
    { 5.3279999999999998e-08, 3.5649462022299998e+00, 4.1630325013750002e+02 },
    { 5.2749999999999999e-08, 3.8815756115600002e+00, 7.4299006053259995e+02 },
    { 5.0930000000000000e-08, 3.8521730597800001e+00, 4.1555249061209997e+02 },
    { 5.0050000000000001e-08, 2.3452224008200000e+00, 2.7703499374140000e+02 },
    { 4.6700000000000001e-08, 6.2193866939699998e+00, 5.2969096509459996e+02 },
    { 4.5870000000000000e-08, 3.0544158275600002e+00, 5.6622351302600002e+01 },
    { 4.5170000000000000e-08, 1.0167829913899999e+00, 9.4206206196899996e+02 },
    { 4.0819999999999998e-08, 3.4545096844700001e+00, 1.0593819301891999e+03 },
    { 4.0779999999999999e-08, 1.2238702593899999e+00, 8.3896928775039999e+02 },
    { 4.0490000000000001e-08, 4.7679805622500000e+00, 4.8322054217860000e+02 },
    { 3.8560000000000003e-08, 1.0053459991400000e+00, 9.3494851496820002e+02 },
    { 3.7580000000000003e-08, 4.6973879218199999e+00, 6.6123792731640003e+02 },
    { 3.7020000000000003e-08, 4.9976693131000003e+00, 1.8484490743480001e+02 },
    { 3.6830000000000000e-08, 1.9266769698999999e+00, 7.0849445304200003e+01 },
    { 3.5579999999999998e-08, 6.1401519846500001e+00, 5.0835032409220003e+02 },
    { 3.5439999999999998e-08, 5.1994385004200003e+00, 7.5010360753340001e+02 },
    { 3.5310000000000003e-08, 1.9316056256000000e-01, 9.5628915597059995e+02 },
    { 3.3309999999999999e-08, 5.7729544689700001e+00, 1.1482476104062000e+03 },
    { 3.1340000000000001e-08, 1.0698641533000000e-01, 9.9911380480899993e+01 },
    { 3.0160000000000000e-08, 8.5072422053999996e-01, 4.0525754987360000e+02 },
    { 2.9880000000000000e-08, 3.6589980996999998e+00, 3.9814400287280000e+02 },
    { 2.9130000000000000e-08, 4.8538922402200004e+00, 1.0451548361876000e+03 },
    { 2.6700000000000001e-08, 5.3431985312499997e+00, 1.9589060769869999e+02 },
    { 2.6700000000000001e-08, 5.8296129425199998e+00, 2.3070758317729999e+02 },
    { 2.6400000000000001e-08, 8.5341396103000000e-01, 1.1338771495710000e+02 },
    { 2.6340000000000000e-08, 3.6855828080999999e+00, 4.9744763618019999e+02 },
    { 2.6250000000000000e-08, 4.3567106754899996e+00, 3.6286229257259998e+02 },
    { 2.5970000000000000e-08, 4.7099450841000001e-01, 3.1813937377000001e+00 },
    { 2.4879999999999998e-08, 5.5999459770800000e+00, 1.4646394800628000e+03 },
    { 2.3560000000000001e-08, 6.9069106203999997e-01, 1.0294940738499999e+01 },
    { 2.3409999999999999e-08, 5.9778909734999996e-01, 2.1116503133776000e+03 },
    { 2.2849999999999999e-08, 1.1971745582900000e+00, 6.1144309831080000e+02 },
    { 2.1629999999999999e-08, 4.6527659677599997e+00, 8.5827298831199997e+01 },
    { 2.1390000000000002e-08, 8.3753177594999995e-01, 1.2584539316256000e+03 },
    { 2.1320000000000002e-08, 6.2419505054400002e+00, 8.2474219374879999e+02 },
    { 2.1260000000000000e-08, 5.7006889210400002e+00, 2.6413412784722000e+03 },
    { 2.0910000000000000e-08, 3.7568513415700000e+00, 1.0627416795630000e+02 },
    { 2.0860000000000000e-08, 6.7510294900000004e-02, 4.4475743814070000e+02 },
    { 2.0650000000000000e-08, 2.1633597233100001e+00, 6.3671589257630001e+02 },
    { 2.0580000000000000e-08, 9.7339401280000004e-01, 9.2783496796739996e+02 },
    { 1.9849999999999999e-08, 1.4834443079599999e+00, 2.5382485042536000e+03 },
    { 1.9770000000000001e-08, 3.5420931303000001e+00, 1.3757737998458001e+03 },
    { 1.9470000000000002e-08, 5.0053798977900001e+00, 6.5094298657790000e+02 },
    { 1.9390000000000000e-08, 2.0705779518699998e+00, 2.4209286360333999e+03 },
    { 1.9119999999999998e-08, 1.5560947246200001e+00, 2.6342277314714001e+03 },
    { 1.8810000000000001e-08, 6.2294212249500003e+00, 3.4470304530790003e+02 },
    { 1.8030000000000000e-08, 2.8058928923999998e-01, 1.9120769491019999e+02 },
    { 1.7070000000000000e-08, 1.7672896815800001e+00, 2.3178358618148000e+03 },
    { 1.6660000000000002e-08, 3.2720656179000001e+00, 1.8770105769372001e+03 },
    { 1.6510000000000000e-08, 6.7163171570000002e-01, 1.2513403846248000e+03 },
    { 1.6320000000000000e-08, 1.3405941759200000e+00, 2.9435060541272001e+03 },
    { 1.6219999999999999e-08, 4.3858843395599996e+00, 1.5961864422845999e+03 },
    { 1.6169999999999999e-08, 3.4686805547700001e+00, 1.3544331588434000e+03 },
    { 1.6090000000000000e-08, 1.5497424117600000e+00, 2.6271141844705999e+03 },
    { 1.6079999999999999e-08, 3.6802699659999999e-01, 1.5748458012822000e+03 },
    { 1.5959999999999999e-08, 3.0789610986299998e+00, 2.0156710861598001e+03 },
    { 1.5679999999999999e-08, 1.4688371202499999e+00, 2.3539049596580000e+02 },
    { 1.5489999999999998e-08, 1.8531407263199999e+00, 2.8404132799086001e+03 },
    { 1.5410000000000000e-08, 3.4826430677600002e+00, 7.2164941953020002e+02 },
    { 1.4950000000000002e-08, 2.9814802549000001e+00, 2.8414854074220000e+02 },
    { 1.4800000000000000e-08, 1.2042933606999999e+00, 6.2901893969922003e+03 },
    { 1.4800000000000000e-08, 5.5460477853000001e-01, 5.8493641121146002e+03 },
    { 1.4650000000000000e-08, 5.7512832497900002e+00, 5.8635912061161998e+03 },
    { 1.4650000000000000e-08, 2.2908001786600001e+00, 6.2759623029905997e+03 },
    { 1.4570000000000001e-08, 9.9248805006999996e-01, 2.5311349572528002e+03 },
    { 1.4500000000000001e-08, 5.5337290193999999e-01, 1.2726810256271999e+03 },
    { 1.4170000000000000e-08, 4.0366562493600000e+00, 1.0380412891868000e+03 },
    { 1.4050000000000000e-08, 6.8391140599000000e-01, 1.2442268376239999e+03 },
    { 1.4009999999999999e-08, 3.1555359079000000e-01, 1.4244965013379999e+02 },
    { 1.3990000000000000e-08, 3.2528948257199999e+00, 1.6708250284999999e+03 },
    { 1.3890000000000001e-08, 6.0226350211400002e+00, 2.5169078632512001e+03 },
    { 1.3830000000000000e-08, 3.5835644874299999e+00, 9.4917560896980001e+02 },
    { 1.3660000000000000e-08, 6.2195734733499997e+00, 1.6060889739850001e+02 },
    { 1.3110000000000000e-08, 3.3644374941000001e+00, 1.6921656695024001e+03 },
    { 1.3100000000000000e-08, 4.9608779552100000e+00, 1.3615467058442000e+03 },
    { 1.3010000000000000e-08, 5.0929049992800000e+00, 1.6779385755008000e+03 },
    { 1.3000000000000001e-08, 5.3961456874399998e+00, 1.7810313497193999e+03 },
    { 1.2930000000000001e-08, 4.4818099389899997e+00, 2.7515475996916002e+03 },
    { 1.2880000000000000e-08, 3.5594697353800000e+00, 1.0522683831884001e+03 },
    { 1.2880000000000000e-08, 1.2994233388000000e+00, 2.9506196011279999e+03 },
    { 1.2290000000000000e-08, 8.7236235179999996e-02, 1.5677322542813999e+03 },
    { 1.1930000000000000e-08, 7.8248411075000002e-01, 2.0863322899200000e+02 },
    { 1.1930000000000000e-08, 4.1071420558399998e+00, 2.1796496188399999e+02 },
    { 1.1840000000000000e-08, 8.0111846131999997e-01, 2.2147430875962000e+03 },
    { 1.1710000000000000e-08, 6.0413103411400000e+00, 6.7668051566499997e+01 },
    { 1.1409999999999999e-08, 5.7256866176000001e+00, 1.1411340634054000e+03 },
    { 1.1380000000000000e-08, 1.0766904598500000e+00, 3.2598979237837998e+03 },
    { 1.1310000000000000e-08, 2.7979459723500000e+00, 1.9872168981566001e+03 },
    { 1.1140000000000001e-08, 1.5199263502300000e+00, 1.8023719907218001e+03 },
    { 1.0719999999999999e-08, 4.6041335524999996e+00, 2.4351557300350000e+03 },
    { 1.0519999999999999e-08, 3.9800410793000002e-01, 1.5606187072805999e+03 },
    { 1.0430000000000000e-08, 2.3782893715300002e+00, 1.5667674413540001e+02 },
    { 1.0260000000000001e-08, 6.1823553841000001e-01, 3.1568051495651998e+03 },
    { 1.0230000000000000e-08, 8.9342926109999998e-01, 1.3828873468465999e+03 },
    { 1.0039999999999999e-08, 1.7939365989000000e+00, 1.1695882514085999e+03 },
    { 9.8600000000000003e-09, 5.9865579369899997e+00, 2.2005159935945999e+03 },
    { 9.8000000000000001e-09, 1.8116489090900001e+00, 2.6598929347749998e+02 },
    { 9.6400000000000006e-09, 4.0794527692200004e+00, 1.7952584437210000e+03 },
    { 9.4500000000000002e-09, 5.3328732196099997e+00, 1.7739178027185999e+03 },
    { 9.3999999999999998e-09, 3.3745097450000000e-01, 5.2690198039499997e+01 },
    { 8.9199999999999998e-09, 3.2999876697700001e+00, 3.4077089204480001e+02 },
    { 8.8900000000000005e-09, 5.1346266230399999e+00, 3.5893013930950002e+02 },
    { 8.8900000000000005e-09, 2.3310671381099999e+00, 1.0702492748170000e+02 },
    { 8.8300000000000003e-09, 5.9756879305300004e+00, 1.8912376709388000e+03 },
    { 8.7000000000000001e-09, 1.7225649920000000e-01, 6.2173803904930003e+02 },
    { 8.5600000000000002e-09, 3.3022785265699999e+00, 2.0085575391590000e+03 },
    { 8.5600000000000002e-09, 8.5986163418999995e-01, 2.8475268269093999e+03 },
    { 8.4499999999999996e-09, 4.4964995675399999e+00, 2.7302069586891998e+03 },
    { 8.4100000000000005e-09, 5.7673739017700001e+00, 2.8332997329077998e+03 },
    { 8.0999999999999997e-09, 6.0505419829599996e+00, 2.5240214102519999e+03 },
    { 8.0700000000000005e-09, 2.8788037174399999e+00, 1.9125783119411999e+03 },
    { 7.8700000000000003e-09, 4.1063399376999998e-01, 2.3320629558164001e+03 },
    { 7.8000000000000004e-09, 3.8345057114399999e+00, 2.8808069400530002e+02 },
    { 7.8000000000000004e-09, 4.1967131087400000e+00, 1.3851749687070000e+02 },
    { 7.7300000000000004e-09, 3.9992471639099998e+00, 1.1624747044077999e+03 },
    { 7.7200000000000006e-09, 5.3704469427400001e+00, 3.1957326339430000e+02 },
    { 7.6100000000000000e-09, 3.5419454957099998e+00, 6.2885158605009997e+02 },
    { 7.4899999999999996e-09, 2.0794975730699998e+00, 1.8094855377225999e+03 },
    { 7.4700000000000001e-09, 5.4984971945799996e+00, 1.9943304451573999e+03 },
    { 7.2699999999999999e-09, 9.3197698209000002e-01, 2.4554242435239999e+02 },
    { 7.2699999999999999e-09, 8.1605653090999997e-01, 1.8105576652360000e+02 },
    { 7.0900000000000001e-09, 5.4540362468600003e+00, 1.4575259330619999e+03 },
    { 7.0100000000000004e-09, 1.9828093324700000e+00, 9.7795738083342003e+03 },
    { 7.0100000000000004e-09, 2.6229932732700001e+00, 1.0220399093211799e+04 },
};

static const VSOP87Term VSOP87A_SATURN_Y4[88] = {
    { 7.9788859999999995e-05, 3.1322926801099999e+00, 2.0618554843720000e+02 },
    { 7.8683790000000006e-05, 4.8494026002100004e+00, 2.2041264243879999e+02 },
    { 1.5148350000000000e-05, 4.7767573386700004e+00, 2.2752618943959999e+02 },
    { 1.2255690000000000e-05, 3.8450013857399998e+00, 4.3371173787679999e+02 },
    { 7.2057299999999999e-06, 3.2377318159500001e+00, 1.9907200143640000e+02 },
    { 6.0997899999999998e-06, 5.9646176994799998e+00, 7.1135470007999997e+00 },
    { 4.6933999999999998e-06, 3.1415926535900001e+00, 0.0000000000000000e+00 },
    { 4.0366200000000002e-06, 3.4161417380199999e+00, 4.2659819087599999e+02 },
    { 3.8299399999999998e-06, 6.0039958155399997e+00, 1.4227094001599999e+01 },
    { 3.5385999999999999e-06, 2.5754831822400002e+00, 4.1948464387519999e+02 },
    { 2.0825899999999999e-06, 2.5779115763500000e+00, 6.3989728631399998e+02 },
    { 1.8546499999999999e-06, 3.8627421243100000e+00, 4.4082528487759998e+02 },
    { 1.7211800000000000e-06, 2.9047898767400002e+00, 6.4701083331480004e+02 },
    { 9.7307000000000007e-07, 3.4022342268900001e+00, 2.1329909543799999e+02 },
    { 8.4185000000000004e-07, 4.6236866333200002e+00, 2.3463973644039999e+02 },
    { 4.6432000000000002e-07, 1.6743328256800001e+00, 8.5319638175199998e+02 },
    { 4.3450999999999999e-07, 6.0881237988999999e+00, 1.1731986822019999e+02 },
    { 4.2226000000000002e-07, 2.1300093478800002e+00, 4.1237109687440000e+02 },
    { 2.9947000000000002e-07, 5.9054331658899999e+00, 2.1340641002400002e+01 },
    { 2.2585000000000000e-07, 2.9749280660799999e+00, 6.5412438031559998e+02 },
    { 2.1614000000000000e-07, 2.0058913256199999e+00, 8.6030992875280003e+02 },
    { 2.0732999999999999e-07, 3.4006784841000002e+00, 1.0309277421860000e+02 },
    { 1.8022999999999999e-07, 3.3286481089400000e+00, 3.0927832265580003e+02 },
    { 1.6785000000000001e-07, 4.3874301449799997e+00, 1.2443341522100000e+02 },
    { 1.6096000000000001e-07, 4.5485888673700003e+00, 8.8865680217000005e+01 },
    { 1.5292000000000000e-07, 3.7696277354599999e+00, 3.0216477565500003e+02 },
    { 1.4999999999999999e-07, 4.1001853764899998e+00, 9.5979227217800002e+01 },
    { 1.3392000000000000e-07, 2.7220207331099999e+00, 6.3278373931320004e+02 },
    { 1.2174000000000000e-07, 3.4289860245799999e+00, 1.9195845443560000e+02 },
    { 1.2053999999999999e-07, 3.7447527037400001e+00, 4.4793883187839998e+02 },
    { 1.1836999999999999e-07, 6.6874515167000004e-01, 1.1020632121940000e+02 },
    { 1.1675999999999999e-07, 5.2706357345399999e+00, 3.2350541665740002e+02 },
    { 1.1585000000000000e-07, 4.7660578077200002e+00, 3.3061896365820002e+02 },
    { 9.4180000000000000e-08, 1.3715445383400000e+00, 3.1639186965660002e+02 },
    { 8.1759999999999999e-08, 3.1249967407799999e+00, 4.2977958461370002e+02 },
    { 7.9249999999999998e-08, 9.9159987248000003e-01, 1.0664954771900000e+03 },
    { 7.7729999999999995e-08, 3.7714070064200000e+00, 6.2567019231239999e+02 },
    { 6.8420000000000000e-08, 6.3976654748999995e-01, 2.0300415469949999e+02 },
    { 6.7210000000000004e-08, 4.2495558204800004e+00, 2.2359403617650000e+02 },
    { 5.0209999999999998e-08, 4.0837981544500002e+00, 2.1648048917570000e+02 },
    { 4.5660000000000000e-08, 7.6257003680000002e-01, 2.1011770170029999e+02 },
    { 3.7490000000000000e-08, 5.1236280848300000e+00, 4.0525754987360000e+02 },
    { 3.7060000000000002e-08, 5.6230948105199996e+00, 4.2341679713830001e+02 },
    { 3.3150000000000002e-08, 5.8418273571799997e+00, 5.2257741809380002e+02 },
    { 3.0080000000000002e-08, 1.0288107303300000e+00, 5.1546387109299997e+02 },
    { 2.9140000000000002e-08, 3.2149930821299999e+00, 4.2266603761290003e+02 },
    { 2.7179999999999999e-08, 3.7884643714199999e+00, 1.3154696222179999e+02 },
    { 2.5559999999999999e-08, 3.0839146416099998e+00, 3.3773251065900001e+02 },
    { 2.4159999999999999e-08, 3.8895849255899999e+00, 2.9505122865419997e+02 },
    { 2.2950000000000000e-08, 3.0677325077000002e+00, 7.4299006053259995e+02 },
    { 1.9709999999999999e-08, 4.1028673839899996e+00, 1.3615467058442000e+03 },
    { 1.9280000000000001e-08, 2.9493979246299999e+00, 6.1855664531160005e+02 },
    { 1.7820000000000000e-08, 3.7884320289000001e-01, 1.8484490743480001e+02 },
    { 1.7820000000000000e-08, 4.5107829636999996e+00, 2.4175328344120001e+02 },
    { 1.7109999999999999e-08, 5.3403109635400003e+00, 1.0522683831884001e+03 },
    { 1.7039999999999999e-08, 1.1083475037999999e+00, 8.3185574074960005e+02 },
    { 1.6919999999999999e-08, 5.9694424259000001e-01, 7.2876296653099996e+02 },
    { 1.6460000000000000e-08, 5.6881105504000002e+00, 1.9513984817330001e+02 },
    { 1.6460000000000000e-08, 5.4847009233700001e+00, 2.3145834270270001e+02 },
    { 1.6070000000000001e-08, 6.1086514204200002e+00, 6.1144309831080000e+02 },
    { 1.6029999999999999e-08, 2.5546977463500000e+00, 1.2513403846248000e+03 },
    { 1.5049999999999999e-08, 3.9679478651900002e+00, 1.6779385755008000e+03 },
    { 1.4790000000000000e-08, 4.8785149789999999e-01, 5.2969096509459996e+02 },
    { 1.3920000000000000e-08, 1.4975629322699999e+00, 5.0835032409220003e+02 },
    { 1.3599999999999999e-08, 1.7676635719800000e+00, 9.5628915597059995e+02 },
    { 1.3100000000000000e-08, 1.0961762889000000e+00, 2.1723124870109999e+02 },
    { 1.2760000000000000e-08, 3.7036504425099999e+00, 2.0936694217490000e+02 },
    { 1.2740000000000000e-08, 4.6334747658900000e+00, 7.3587651353180001e+02 },
    { 1.2270000000000000e-08, 2.5416292999999999e+00, 9.3494851496820002e+02 },
    { 1.2140000000000000e-08, 1.1373243225100000e+00, 1.4646394800628000e+03 },
    { 1.1590000000000000e-08, 1.3591257028499999e+00, 8.1752133216199994e+01 },
    { 1.1120000000000000e-08, 3.7370285639400000e+00, 1.9943304451573999e+03 },
    { 1.0789999999999999e-08, 4.4693059908599997e+00, 1.3686602528450001e+03 },
    { 1.0379999999999999e-08, 2.3683012908699999e+00, 1.5677322542813999e+03 },
    { 1.0370000000000000e-08, 1.3141531458600000e+00, 1.1482476104062000e+03 },
    { 9.9499999999999998e-09, 3.1270003630200001e+00, 1.8912376709388000e+03 },
    { 9.9300000000000002e-09, 2.6725731434800002e+00, 9.4206206196899996e+02 },
    { 8.8800000000000008e-09, 4.9997381138400003e+00, 1.7881448967202000e+03 },
    { 8.6200000000000004e-09, 3.1802294033599998e+00, 4.3689313161450002e+02 },
    { 8.5899999999999995e-09, 5.8573030713199996e+00, 4.1630325013750002e+02 },
    { 8.4300000000000000e-09, 8.8917875831000004e-01, 7.5010360753340001e+02 },
    { 8.1799999999999995e-09, 6.0404781382999995e-01, 1.0451548361876000e+03 },
    { 8.0399999999999995e-09, 5.2728620575100003e+00, 1.1553611574070001e+03 },
    { 8.0299999999999998e-09, 2.0354761141600002e+00, 1.3757737998458001e+03 },
    { 7.6000000000000002e-09, 4.3000555572500003e+00, 1.2584539316256000e+03 },
    { 7.3799999999999997e-09, 4.2559002016400003e+00, 1.0294940738499999e+01 },
    { 7.2799999999999997e-09, 3.5727963790100001e+00, 1.8023719907218001e+03 },
    { 6.8400000000000004e-09, 3.5262262380500000e+00, 6.6123792731640003e+02 },
};

static const VSOP87Term VSOP87A_SATURN_Y5[32] = {
    { 5.9011400000000004e-06, 3.0895374329699998e+00, 2.2041264243879999e+02 },
    { 5.8908000000000004e-06, 4.8491038698600004e+00, 2.0618554843720000e+02 },
    { 2.2644800000000000e-06, 3.0383208029300000e+00, 2.2752618943959999e+02 },
    { 1.3151300000000001e-06, 2.0496756081599998e+00, 4.3371173787679999e+02 },
    { 9.4101999999999999e-07, 5.0238240752700003e+00, 1.9907200143640000e+02 },
    { 3.9692999999999999e-07, 4.3234699840699999e+00, 1.4227094001599999e+01 },
    { 3.4027000000000002e-07, 2.0852564694500000e+00, 4.4082528487759998e+02 },
    { 2.2889000000000001e-07, 1.0922921769000000e+00, 6.4701083331480004e+02 },
    { 1.3218000000000001e-07, 5.1629962918699999e+00, 4.1948464387519999e+02 },
    { 1.1166000000000000e-07, 7.4664450299999996e-01, 6.3989728631399998e+02 },
    { 1.0307999999999999e-07, 2.9461113540700001e+00, 2.3463973644039999e+02 },
    { 8.6999999999999998e-08, 1.9338257454000001e+00, 1.9195845443560000e+02 },
    { 7.2829999999999998e-08, 8.5862173948999998e-01, 4.2659819087599999e+02 },
    { 4.8680000000000003e-08, 6.2410530506299997e+00, 8.8865680217000005e+01 },
    { 4.7780000000000002e-08, 3.4702751944600001e+00, 7.1135470007999997e+00 },
    { 4.3740000000000000e-08, 1.1943191880099999e+00, 6.5412438031559998e+02 },
    { 4.1099999999999997e-08, 2.2105340513399998e+00, 2.1329909543799999e+02 },
    { 3.8430000000000001e-08, 7.6786462039000003e-01, 4.0525754987360000e+02 },
    { 3.3300000000000000e-08, 6.1274587854700000e+00, 8.5319638175199998e+02 },
    { 3.2600000000000001e-08, 2.3288481268700001e+00, 1.2443341522100000e+02 },
    { 2.7039999999999999e-08, 1.4435501671800000e+00, 5.1546387109299997e+02 },
    { 2.6779999999999999e-08, 1.7889143297000001e-01, 8.6030992875280003e+02 },
    { 2.5650000000000001e-08, 1.2025030521300000e+00, 1.1020632121940000e+02 },
    { 2.1380000000000000e-08, 2.0159925539299999e+00, 4.4793883187839998e+02 },
    { 1.8889999999999999e-08, 4.9025110230000002e-01, 9.5979227217800002e+01 },
    { 1.6359999999999999e-08, 3.9362704173000002e-01, 3.0216477565500003e+02 },
    { 1.4540000000000000e-08, 1.4310762167999999e-01, 3.0927832265580003e+02 },
    { 9.8400000000000008e-09, 2.3849959141300001e+00, 2.0300415469949999e+02 },
    { 9.8400000000000008e-09, 2.5046302524600002e+00, 2.2359403617650000e+02 },
    { 9.7200000000000003e-09, 4.1350304986299999e+00, 1.1731986822019999e+02 },
    { 7.8600000000000006e-09, 1.6154185600400000e+00, 5.2257741809380002e+02 },
    { 7.8000000000000004e-09, 5.5796214439000003e-01, 3.3061896365820002e+02 },
};

static const VSOP87Term VSOP87A_SATURN_Z0[420] = {
    { 4.1356950939999998e-01, 3.6023414298200001e+00, 2.1329909543799999e+02 },
    { 1.2142498670000000e-02, 0.0000000000000000e+00, 0.0000000000000000e+00 },
    { 1.1482835760000001e-02, 2.8512836746899999e+00, 4.2659819087599999e+02 },
    { 3.2928079099999999e-03, 5.7121407103999999e-01, 2.0618554843720000e+02 },
    { 2.8693404800000000e-03, 3.4807352669300000e+00, 2.2041264243879999e+02 },
    { 9.9076583999999990e-04, 4.7336951126400004e+00, 7.1135470007999997e+00 },
    { 5.7361819999999998e-04, 4.9261122509300002e+00, 1.1020632121940000e+02 },
    { 4.7738126999999998e-04, 2.1003977972799999e+00, 6.3989728631399998e+02 },
    { 4.3458803000000001e-04, 5.8490497805099997e+00, 4.1948464387519999e+02 },
    { 3.4565673000000002e-04, 5.4261422959000001e+00, 3.1639186965660002e+02 },
    { 1.6185390999999999e-04, 2.7298717367499998e+00, 4.3371173787679999e+02 },
    { 1.1433574000000001e-04, 3.7166202107199999e+00, 5.2969096509459996e+02 },
    { 9.0012700000000000e-05, 1.3814010273699999e+00, 1.0309277421860000e+02 },
    { 5.3987080000000002e-05, 5.1320489236300002e+00, 2.0225339517410001e+02 },
    { 3.9024669999999997e-05, 3.7149973879600000e+00, 3.2350541665740002e+02 },
    { 3.7092119999999998e-05, 5.0554934878499997e+00, 6.3278373931320004e+02 },
    { 3.6140999999999998e-05, 3.3521045127600000e+00, 2.2752618943959999e+02 },
    { 3.3799530000000003e-05, 2.1386891920600002e+00, 1.1045700263900001e+01 },
    { 3.0898739999999999e-05, 3.6257285708500002e+00, 2.0936694217490000e+02 },
    { 2.9634929999999999e-05, 4.6490184984999999e-01, 2.1723124870109999e+02 },
    { 2.6830640000000000e-05, 4.8768955558100000e+00, 2.2434479570190001e+02 },
    { 2.4236630000000000e-05, 2.9290709476000001e+00, 6.3735898303399999e+01 },
    { 2.3433670000000000e-05, 1.3455827834000000e+00, 8.5319638175199998e+02 },
    { 1.9901449999999999e-05, 7.3166053611000004e-01, 1.9907200143640000e+02 },
    { 1.9412050000000000e-05, 4.5942131466199996e+00, 1.4227094001599999e+01 },
    { 1.7019159999999999e-05, 1.8989252565400001e+00, 7.3587651353180001e+02 },
    { 1.4602650000000000e-05, 3.1285133972399999e+00, 5.2257741809380002e+02 },
    { 1.1483410000000000e-05, 4.4113921391500002e+00, 1.1731986822019999e+02 },
    { 1.0982540000000000e-05, 9.6097709156000000e-01, 2.1011770170029999e+02 },
    { 1.0928090000000001e-05, 3.1067938120899998e+00, 2.1648048917570000e+02 },
    { 1.0287430000000000e-05, 2.1193305924299999e+00, 4.1555249061209997e+02 },
    { 1.0151789999999999e-05, 1.9789719599400000e+00, 6.4701083331480004e+02 },
    { 9.9103000000000006e-06, 2.9961002668200001e+00, 8.4608283475120004e+02 },
    { 9.0781700000000002e-06, 4.6857627860999997e+00, 3.0927832265580003e+02 },
    { 8.1809199999999996e-06, 2.9149765619600001e+00, 1.4956319713459999e+02 },
    { 7.3344300000000000e-06, 2.1001871561400001e+00, 7.4781598567299994e+01 },
    { 6.3127499999999998e-06, 1.3055725581400000e+00, 2.7703499374140000e+02 },
    { 5.7426599999999998e-06, 6.3353925381999998e-01, 4.9033408917940000e+02 },
    { 4.7164500000000004e-06, 1.6232978227699999e+00, 1.0522683831884001e+03 },
    { 4.1629200000000001e-06, 5.8817138002300000e+00, 9.5979227217800002e+01 },
    { 4.1246700000000001e-06, 2.5632227722100001e+00, 1.1624747044077999e+03 },
    { 3.8193400000000003e-06, 2.8385053221100001e+00, 8.3896928775039999e+02 },
    { 3.7090200000000001e-06, 2.2235695387400001e+00, 3.5181659230870002e+02 },
    { 3.3733800000000000e-06, 4.1248769043599998e+00, 3.9321532631000000e+00 },
    { 2.6317300000000001e-06, 2.6059432426100000e+00, 4.4082528487759998e+02 },
    { 2.5228700000000000e-06, 3.0420534526199998e+00, 1.3703302416240001e+02 },
    { 2.4541400000000002e-06, 3.8771602742400000e+00, 1.0593819301891999e+03 },
    { 2.3283499999999999e-06, 1.6942963263499999e+00, 3.8133035637799999e+01 },
    { 2.3118600000000000e-06, 2.7913751814299999e+00, 1.2747179660680000e+02 },
    { 2.2621800000000000e-06, 9.8981012262000001e-01, 2.1085141488319999e+02 },
    { 2.2392399999999999e-06, 3.0749834639100002e+00, 2.1574677599280000e+02 },
    { 2.1365600000000002e-06, 3.0004681958699999e+00, 5.3680451209540001e+02 },
    { 2.0269900000000000e-06, 3.8558189987999998e+00, 9.4917560896980001e+02 },
    { 1.8991700000000000e-06, 5.7525523310200004e+00, 7.4299006053259995e+02 },
    { 1.6450999999999999e-06, 2.4569745290500000e+00, 4.2266603761290003e+02 },
    { 1.6355099999999999e-06, 1.6578976592000000e-01, 2.1233588759150001e+02 },
    { 1.6287299999999999e-06, 3.8976269058500002e+00, 2.1426230328450001e+02 },
    { 1.5830400000000000e-06, 2.3130640723100000e+00, 1.4788665740644001e+03 },
    { 1.5645400000000001e-06, 1.3568019812300001e+00, 1.3686602528450001e+03 },
    { 1.5007900000000000e-06, 4.1404532924400002e+00, 4.3764389113990001e+02 },
    { 1.4270500000000000e-06, 2.5838394786799999e+00, 1.1553611574070001e+03 },
    { 1.4053400000000000e-06, 3.9728986174799998e+00, 6.2567019231239999e+02 },
    { 1.3537500000000000e-06, 3.5742402570100000e+00, 2.1325091132820000e+02 },
    { 1.3533499999999999e-06, 4.8883402314000002e-01, 2.1334727954780001e+02 },
    { 1.2914000000000000e-06, 2.1703571412099998e+00, 7.6266071275599998e+01 },
    { 1.2820599999999999e-06, 5.9415255592000005e-01, 1.0664954771900000e+03 },
    { 1.2156400000000001e-06, 4.5138104980799998e+00, 3.1813937377000001e+00 },
    { 1.1919000000000000e-06, 1.6139141617699999e+00, 6.2885158605009997e+02 },
    { 1.1514699999999999e-06, 3.4345904024500000e+00, 3.3061896365820002e+02 },
    { 1.1380800000000000e-06, 4.3922136440999998e-01, 2.2286032299359999e+02 },
    { 1.1003700000000001e-06, 3.5115275768499998e+00, 1.2655674786264001e+03 },
    { 1.0755999999999999e-06, 3.9983783647000002e+00, 8.5827298831199997e+01 },
    { 1.0749800000000000e-06, 3.5249856406900002e+00, 1.3851749687070000e+02 },
    { 1.0497300000000001e-06, 4.1951839491699996e+00, 9.5612275556000004e+00 },
    { 1.0372700000000000e-06, 6.1763944136799998e+00, 2.0076892246579999e+02 },
    { 1.0192800000000000e-06, 2.1047731069000000e-01, 4.1237109687440000e+02 },
    { 1.0142500000000001e-06, 1.4126238471999999e-01, 4.3053034413910001e+02 },
    { 9.4919999999999995e-07, 2.2622253602800000e+00, 3.8846515523820000e+02 },
    { 9.0986000000000002e-07, 6.2819848071999997e+00, 4.2341679713830001e+02 },
    { 8.3620000000000004e-07, 4.8201359177500001e+00, 1.4844727083000000e+00 },
    { 7.7047999999999995e-07, 8.0511279722999995e-01, 5.2690198039499997e+01 },
    { 7.6181000000000002e-07, 2.6743575974099998e+00, 1.9120769491019999e+02 },
    { 7.1103999999999999e-07, 4.9274130604700002e+00, 2.1277783057619999e+02 },
    { 7.0958999999999999e-07, 5.4197994427899996e+00, 2.1382036029980000e+02 },
    { 7.0149999999999998e-07, 3.8257437373499998e+00, 9.4206206196899996e+02 },
    { 7.0088999999999997e-07, 1.0510802525900000e+00, 1.2530172972200001e+01 },
    { 6.6662000000000002e-07, 1.2261715289499999e+00, 8.6030992875280003e+02 },
    { 6.3466999999999995e-07, 2.3695937226199999e+00, 4.2977958461370002e+02 },
    { 6.3389000000000002e-07, 1.8155617758900000e+00, 3.5033211960040001e+02 },
    { 6.2745999999999998e-07, 2.0610589324399999e+00, 1.7952584437210000e+03 },
    { 5.8999000000000000e-07, 2.3188265073699998e+00, 1.4717530270636000e+03 },
    { 5.7836999999999996e-07, 3.2904342007200000e+00, 5.1546387109299997e+02 },
    { 5.6601000000000002e-07, 1.1130395770499999e+00, 1.6850521225016000e+03 },
    { 5.4298000000000001e-07, 3.1807601314600000e+00, 1.5819593482830001e+03 },
    { 5.1666000000000002e-07, 3.4911134407000000e+00, 2.0373786788240000e+02 },
    { 5.1149000000000004e-07, 3.8603013168999998e+00, 1.7394221952280000e+02 },
    { 5.0128000000000005e-07, 1.9827369393700001e+00, 1.9513984817330001e+02 },
    { 4.9760999999999997e-07, 6.1880815663800002e+00, 7.0363318461740005e+02 },
    { 4.8090999999999998e-07, 3.2097133454800000e+00, 2.3463973644039999e+02 },
    { 4.7483000000000002e-07, 1.5365509348499999e+00, 2.6598929347749998e+02 },
    { 4.6488000000000002e-07, 3.8076851879800002e+00, 2.2582926841020000e+02 },
    { 4.2785000000000001e-07, 1.1273484212000000e-01, 2.5265597135319999e+02 },
    { 4.1716000000000001e-07, 1.0511122148500001e+00, 5.6363121503840000e+02 },
    { 4.0506000000000000e-07, 2.2007375914399998e+00, 3.6286229257259998e+02 },
    { 3.9602000000000002e-07, 2.2750774606699999e+00, 2.8808069400530002e+02 },
    { 3.9354000000000002e-07, 2.1053729596799999e+00, 7.2876296653099996e+02 },
    { 3.8184999999999999e-07, 4.1999781687900004e+00, 2.0910630974399999e+02 },
    { 3.7967000000000001e-07, 4.5707529492800001e+00, 3.0216477565500003e+02 },
    { 3.7861999999999999e-07, 4.0077386996900000e+00, 6.2251425595100002e+01 },
    { 3.7717999999999999e-07, 5.9633365399300002e+00, 2.1037833413120001e+02 },
    { 3.7688999999999998e-07, 4.3797741828700003e+00, 2.1621985674480001e+02 },
    { 3.7613999999999998e-07, 6.1417907460199999e+00, 2.1749188113200000e+02 },
    { 3.6791999999999998e-07, 1.3529239346899999e+00, 5.6511568774670002e+02 },
    { 3.6057000000000001e-07, 3.3089995375300001e+00, 2.1254833591260001e+02 },
    { 3.5830000000000003e-07, 7.5230512172999997e-01, 2.1404985496340001e+02 },
    { 3.5460000000000001e-07, 5.0110583024600004e+00, 6.5220371011699996e+01 },
    { 3.4852000000000000e-07, 4.7544550464600004e+00, 2.0788246946660001e+02 },
    { 3.4252999999999998e-07, 2.9683515747400002e+00, 2.2359403617650000e+02 },
    { 3.3335000000000000e-07, 5.6240162269500003e+00, 2.1871572140940000e+02 },
    { 3.2939000000000001e-07, 4.4514471257599997e+00, 2.1340641002400002e+01 },
    { 3.2818000000000002e-07, 5.7953878568499997e+00, 9.9160620955499994e+01 },
    { 3.2560000000000000e-07, 6.1480630184300002e+00, 4.1703696332039999e+02 },
    { 3.2547999999999997e-07, 2.9574047496200002e+00, 7.0849445304200003e+01 },
    { 3.2421000000000002e-07, 5.7855975682900000e+00, 1.6060889739850001e+02 },
    { 3.2329999999999999e-07, 4.2279494409199998e+00, 2.2091400527800001e+01 },
    { 3.1282999999999998e-07, 3.5488106256299998e+00, 1.2584539316256000e+03 },
    { 3.0744000000000001e-07, 3.6431730229400001e+00, 2.0767002114549999e+02 },
    { 3.0191999999999999e-07, 4.2929684302999999e-01, 2.1892816973050000e+02 },
    { 2.9913000000000001e-07, 3.3118230710300001e+00, 1.2443341522100000e+02 },
    { 2.9522999999999998e-07, 3.7961341544999999e-01, 5.5406998748280000e+02 },
    { 2.9455000000000000e-07, 5.8304204724300002e+00, 9.5628915597059995e+02 },
    { 2.8921999999999999e-07, 5.5885898073800000e+00, 4.2511371816769997e+02 },
    { 2.8901999999999998e-07, 5.8152914912400000e+00, 8.8865680217000005e+01 },
    { 2.7692999999999998e-07, 5.5561895021299996e+00, 1.8159247264699999e+01 },
    { 2.6647999999999999e-07, 8.6666603224000005e-01, 2.1796496188399999e+02 },
    { 2.6495000000000000e-07, 2.8628811546600001e+00, 1.8983512179396000e+03 },
    { 2.6011000000000002e-07, 2.0558918193300002e+00, 1.7881448967202000e+03 },
    { 2.5662999999999999e-07, 1.8156475663500000e+00, 2.1116503133776000e+03 },
    { 2.5659000000000000e-07, 5.1086989467399997e+00, 4.7928838891549998e+02 },
    { 2.5652000000000002e-07, 4.8021515605399996e+00, 2.3145834270270001e+02 },
    { 2.3743999999999999e-07, 3.2004626501100000e+00, 2.0863322899200000e+02 },
    { 2.2977000000000000e-07, 5.4403783973800000e+00, 1.4244965013379999e+02 },
    { 2.2228000000000000e-07, 5.4814250950399996e+00, 4.0843894361129998e+02 },
    { 2.1685000000000001e-07, 3.1787148275999999e+00, 4.1406801790380001e+02 },
    { 2.1236999999999999e-07, 8.7139429027000004e-01, 2.0014439921582000e+03 },
    { 2.0521000000000000e-07, 6.1611741456800004e+00, 5.6622351302600002e+01 },
    { 2.0174000000000001e-07, 1.8538064737000000e+00, 6.5412438031559998e+02 },
    { 1.9856000000000000e-07, 3.1385757274000001e+00, 1.7516605980020000e+02 },
    { 1.8246999999999999e-07, 3.0012799534200001e+00, 8.3185574074960005e+02 },
    { 1.7924999999999999e-07, 1.1455634066000000e+00, 2.0300415469949999e+02 },
    { 1.7378000000000001e-07, 7.4127655201999998e-01, 1.7787437278589999e+02 },
    { 1.7347000000000000e-07, 5.9694616805100003e+00, 2.1318722085339999e+02 },
    { 1.7347000000000000e-07, 4.3793041789700000e+00, 2.1341097002260000e+02 },
    { 1.6969000000000001e-07, 1.2772172224100000e+00, 2.1478356814630001e+02 },
    { 1.6401000000000000e-07, 1.3640297187799999e+00, 2.1181462272970001e+02 },
    { 1.6227999999999999e-07, 3.1758219478099998e+00, 9.8899988524600005e+01 },
    { 1.6107000000000000e-07, 5.4971362868999996e-01, 2.5143213107579999e+02 },
    { 1.5545999999999999e-07, 2.2910858690799998e+00, 7.5010360753340001e+02 },
    { 1.5435999999999999e-07, 1.4816103360399999e+00, 1.9832124191099999e+02 },
    { 1.5320000000000000e-07, 3.8029637333999999e+00, 4.8322054217860000e+02 },
    { 1.4861000000000001e-07, 3.2634376200999999e+00, 1.5748458012822000e+03 },
    { 1.4819999999999999e-07, 3.3495742839600000e+00, 2.4872381809010000e+02 },
    { 1.4801000000000001e-07, 2.0111341190000001e-01, 1.4807872442630000e+02 },
    { 1.4798000000000001e-07, 1.0305697858100000e+00, 2.4476805547999998e+00 },
    { 1.4576000000000000e-07, 3.7596621366699998e+00, 2.1351154375909999e+02 },
    { 1.4576000000000000e-07, 3.0591841563000000e-01, 2.1308664711690000e+02 },
    { 1.4467000000000001e-07, 1.3254944912400000e+00, 1.1413847448250000e+02 },
    { 1.4310000000000000e-07, 2.3466140728300000e+00, 1.3757737998458001e+03 },
    { 1.4146000000000000e-07, 4.8527666880900000e+00, 3.2743756992049998e+02 },
    { 1.3755000000000001e-07, 2.5973347414900001e+00, 2.2827694896500000e+02 },
    { 1.3743000000000001e-07, 4.7727601193800000e+00, 1.0627416795630000e+02 },
    { 1.3257999999999999e-07, 2.5672417888700001e+00, 2.2147430875962000e+03 },
    { 1.3040999999999999e-07, 1.8712626505700001e+00, 1.5104766984290001e+02 },
    { 1.2945000000000001e-07, 1.8455264554699999e+00, 1.0451548361876000e+03 },
    { 1.2884999999999999e-07, 4.8609570031000002e+00, 3.9356875915200000e+01 },
    { 1.2834000000000000e-07, 4.2459746388999999e+00, 1.0294940738499999e+01 },
    { 1.2706000000000000e-07, 3.1024660308000002e-01, 4.2415051032119999e+02 },
    { 1.2557999999999999e-07, 3.1283578669999999e-02, 1.2726810256271999e+03 },
    { 1.2129000000000000e-07, 4.6378321997300000e+00, 2.6992144674060000e+02 },
    { 1.1978000000000000e-07, 6.2248680110899999e+00, 1.2125202148330000e+02 },
    { 1.1901000000000000e-07, 1.5884649607200001e+00, 6.3596513305090002e+02 },
    { 1.1808000000000000e-07, 1.7961761405900001e+00, 2.1045367663768002e+03 },
    { 1.1521000000000000e-07, 2.1157925516699998e+00, 2.0522234059069999e+02 },
    { 1.1158000000000000e-07, 4.5949551763000002e-01, 4.9744763618019999e+02 },
    { 1.1080000000000000e-07, 5.3704130593499997e+00, 1.1695882514085999e+03 },
    { 1.0959000000000000e-07, 1.3157022241700000e+00, 2.3539049596580000e+02 },
    { 1.0959000000000000e-07, 4.4051235822099999e+00, 2.1355972786890001e+02 },
    { 1.0959000000000000e-07, 5.9436396920399996e+00, 2.1303846300710001e+02 },
    { 1.0673000000000000e-07, 1.6014103594600000e+00, 2.4280421830342002e+03 },
    { 1.0474999999999999e-07, 2.1936828873300001e+00, 2.2137585028530000e+02 },
    { 1.0284000000000000e-07, 1.9302342980000001e-02, 2.1944943459230001e+02 },
    { 1.0229000000000000e-07, 9.6548776504000000e-01, 8.4215068148809996e+02 },
    { 1.0228000000000000e-07, 2.4134251682799999e+00, 2.7555052103309998e+02 },
    { 1.0063000000000000e-07, 4.2149869212000004e+00, 6.1855664531160005e+02 },
    { 9.9789999999999996e-08, 3.9725630324400001e+00, 2.7851946644970002e+02 },
    { 9.4520000000000002e-08, 2.5927790812899998e+00, 4.2904587143079999e+02 },
    { 9.4240000000000002e-08, 1.6811270123800000e+00, 4.8884961647109998e+02 },
    { 9.4059999999999997e-08, 3.3974054875599999e+00, 6.5094298657790000e+02 },
    { 9.3349999999999999e-08, 5.7077939679199998e+00, 4.2563498302950001e+02 },
    { 9.2749999999999993e-08, 5.1664287547100001e+00, 8.0767548473000002e+00 },
    { 9.1069999999999994e-08, 3.3774330802000002e-01, 1.0213285546211000e+04 },
    { 9.0439999999999995e-08, 2.3054654773000000e-01, 3.1219908396260001e+02 },
    { 9.0170000000000006e-08, 2.7086468366899998e+00, 1.6921656695024001e+03 },
    { 8.9389999999999995e-08, 5.2869727762499998e+00, 3.4077089204480001e+02 },
    { 8.8870000000000001e-08, 4.8784223722300002e+00, 1.5890728952838001e+03 },
    { 8.6700000000000002e-08, 2.7560107905000000e+00, 4.9181856188770001e+02 },
    { 8.3850000000000000e-08, 6.4196694647999997e-01, 1.8892007304980001e+02 },
    { 8.3560000000000002e-08, 6.5143790392000001e-01, 2.3178358618148000e+03 },
    { 8.2819999999999996e-08, 1.9002275035900000e+00, 1.3458534360760001e+02 },
    { 8.1689999999999999e-08, 3.2395703687899999e+00, 4.2756139872249997e+02 },
    { 8.0999999999999997e-08, 4.0603903028400001e+00, 2.0714875628370001e+02 },
    { 8.0330000000000005e-08, 4.2006588502900000e+00, 7.6736908292080000e+02 },
    { 7.9269999999999994e-08, 3.9753999771799999e+00, 2.1059078245230000e+02 },
    { 7.7960000000000005e-08, 8.8135814239999999e-02, 2.1600740842370001e+02 },
    { 7.7379999999999995e-08, 1.5536139507900000e+00, 6.0176425067620005e+02 },
    { 7.6630000000000005e-08, 2.7339978435800001e+00, 1.1482476104062000e+03 },
    { 7.6380000000000000e-08, 2.7047622955200001e+00, 5.4391805909619995e+02 },
    { 7.5769999999999996e-08, 6.0211438042600003e+00, 4.2664637498579998e+02 },
    { 7.5759999999999998e-08, 2.8233490263599998e+00, 4.2655000676620000e+02 },
    { 7.4130000000000003e-08, 2.9859584721500001e+00, 1.8912376709388000e+03 },
    { 7.3490000000000005e-08, 7.3079523904999999e-01, 3.5574874557179999e+02 },
    { 7.1579999999999996e-08, 5.8403922961500001e+00, 8.4342826122899993e+01 },
    { 6.9609999999999999e-08, 3.3482631844199999e+00, 2.3767811782620001e+02 },
    { 6.8420000000000000e-08, 6.1542992659399998e+00, 1.7935884549420001e+02 },
    { 6.8219999999999998e-08, 6.1204759110299998e+00, 1.2797945726280000e+03 },
    { 6.6289999999999993e-08, 1.5373220694500001e+00, 2.8956516671359998e+02 },
    { 6.6259999999999999e-08, 3.3750624195199999e+00, 3.1245971639350000e+02 },
    { 6.5869999999999994e-08, 5.8291546592999999e-01, 2.1345915413239999e+02 },
    { 6.5869999999999994e-08, 3.4806280850100002e+00, 2.1313903674360000e+02 },
    { 6.5799999999999994e-08, 5.8187433975899996e+00, 2.8414854074220000e+02 },
    { 6.5760000000000001e-08, 3.5050721605000001e-01, 2.2046082654860001e+02 },
    { 6.5649999999999996e-08, 5.1842352798000002e+00, 6.7668051566499997e+01 },
    { 6.4879999999999996e-08, 2.2903801998200000e+00, 2.5311349572528002e+03 },
    { 6.3780000000000006e-08, 6.1430960131900001e+00, 7.3297125859000005e+01 },
    { 6.3670000000000000e-08, 3.4578549615199998e+00, 2.2036445832900000e+02 },
    { 6.2830000000000001e-08, 3.8479622175900001e+00, 1.2598732389849999e+02 },
    { 6.2670000000000005e-08, 5.4387770923599996e+00, 6.3671589257630001e+02 },
    { 6.2309999999999993e-08, 5.9375986587899998e+00, 4.3615941843159999e+02 },
    { 5.9880000000000004e-08, 5.4755116935399997e+00, 3.5424722652100002e+01 },
    { 5.8199999999999998e-08, 2.9819977779700002e+00, 5.4166259714000002e+00 },
    { 5.8050000000000000e-08, 2.1074083640499999e+00, 4.6658664459999999e+00 },
    { 5.7229999999999997e-08, 5.2293854621199998e+00, 2.1202707105080000e+02 },
    { 5.7040000000000000e-08, 5.3116403770999998e+00, 1.4859801210651999e+03 },
    { 5.6760000000000000e-08, 5.1170600793399998e+00, 2.1457111982519999e+02 },
    { 5.6220000000000003e-08, 1.4590188232000001e+00, 5.0137978944330001e+02 },
    { 5.5899999999999998e-08, 4.1908221609399998e+00, 2.4723934538180001e+02 },
    { 5.4910000000000000e-08, 2.7052799915599999e+00, 2.0085575391590000e+03 },
    { 5.4730000000000001e-08, 8.1641753389000005e-01, 1.8105576652360000e+02 },
    { 5.4539999999999997e-08, 6.1221162434099998e+00, 6.4382943957709995e+02 },
    { 5.4420000000000000e-08, 2.3680019140000001e-02, 4.0450679034820001e+02 },
    { 5.4359999999999998e-08, 4.6797784019600002e+00, 6.2173803904930003e+02 },
    { 5.4070000000000000e-08, 1.5306653615400001e+00, 2.4209286360333999e+03 },
    { 5.3099999999999999e-08, 6.6732715963000000e-01, 1.1642609634290000e+02 },
    { 5.1830000000000002e-08, 2.1884126058100000e+00, 1.0702492748170000e+02 },
    { 5.1720000000000003e-08, 4.9679969593199997e+00, 3.4225536475310003e+02 },
    { 5.1410000000000002e-08, 2.4299859901800001e+00, 6.9258748435350003e+02 },
    { 5.0699999999999997e-08, 3.2753682331400000e+00, 1.4414657116320001e+02 },
    { 5.0680000000000001e-08, 4.3397890475000001e+00, 1.1338771495710000e+02 },
    { 4.9129999999999997e-08, 1.0328862944599999e+00, 3.2243328914400003e+01 },
    { 4.8650000000000002e-08, 5.1080984324500003e+00, 6.9152524274800001e+01 },
    { 4.8000000000000000e-08, 5.5404118389899999e+00, 3.4470304530790003e+02 },
    { 4.7559999999999997e-08, 3.7328708618799999e+00, 2.0613736432740001e+02 },
    { 4.7430000000000002e-08, 5.3760335914000001e-01, 1.2895626931510000e+02 },
    { 4.7379999999999998e-08, 5.3669462341999996e-01, 1.0736090241908000e+03 },
    { 4.7309999999999998e-08, 1.4464198594000000e+00, 2.7444340526907999e+03 },
    { 4.6859999999999997e-08, 3.9738303078600001e+00, 9.3494851496820002e+02 },
    { 4.6390000000000000e-08, 1.6382136467599999e+00, 5.6290742925000004e+00 },
    { 4.6030000000000003e-08, 5.5256269377800002e+00, 7.8713751830400000e+01 },
    { 4.5960000000000003e-08, 4.1850494434699996e+00, 2.1989137757699999e+02 },
    { 4.5830000000000001e-08, 1.8509797918699999e+00, 1.1439910691340000e+02 },
    { 4.5410000000000001e-08, 6.1777794179000001e-01, 2.0623373254699999e+02 },
    { 4.5279999999999999e-08, 1.6575659630999999e+00, 1.3615467058442000e+03 },
    { 4.3950000000000000e-08, 8.5418906453999999e-01, 5.2549817940059995e+02 },
    { 4.3219999999999999e-08, 4.1515313881200004e+00, 4.2607692601420001e+02 },
    { 4.2279999999999999e-08, 4.2836055893999996e+00, 4.1927856940000003e+00 },
    { 4.1850000000000001e-08, 3.3240534218100000e+00, 2.4554242435239999e+02 },
    { 4.1670000000000002e-08, 1.6524180235099999e+00, 1.9195845443560000e+02 },
    { 4.1640000000000001e-08, 2.4791220830300000e+00, 4.4793883187839998e+02 },
    { 4.1299999999999999e-08, 1.6153804282599999e+00, 6.4307868005169996e+02 },
    { 4.0429999999999999e-08, 2.1881971012400001e+00, 3.5893013930950002e+02 },
    { 4.0200000000000003e-08, 4.8807338674400000e+00, 5.4174670747800000e+01 },
    { 3.9750000000000002e-08, 5.8015720637100001e+00, 1.4977853527000001e+01 },
    { 3.9659999999999999e-08, 5.0866470980000002e-02, 2.0670681329900000e+02 },
    { 3.9659999999999999e-08, 4.6666291198599996e+00, 4.2711945573780002e+02 },
    { 3.9060000000000000e-08, 5.1997062334299997e+00, 6.3033605875839999e+02 },
    { 3.8829999999999998e-08, 5.3914766529099998e+00, 3.2032402291969998e+02 },
    { 3.8710000000000001e-08, 2.7134846273200002e+00, 2.2076295405954002e+03 },
    { 3.8019999999999999e-08, 4.3628874694000003e-01, 6.1780588578619995e+02 },
    { 3.6890000000000001e-08, 5.4582205539900004e+00, 9.1693228005540004e+02 },
    { 3.5520000000000003e-08, 5.9800573912699999e+00, 2.0606927819500001e+01 },
    { 3.5080000000000000e-08, 2.8526521350200000e+00, 8.7311771539500000e+01 },
    { 3.4749999999999997e-08, 2.4717896096800001e+00, 1.4646394800628000e+03 },
    { 3.4520000000000001e-08, 6.9033243447000003e-01, 7.7693031047640000e+02 },
    { 3.2220000000000000e-08, 3.6080923359299999e+00, 1.2513403846248000e+03 },
    { 3.1870000000000000e-08, 2.6114835306100002e+00, 2.3249494088155998e+03 },
    { 3.1870000000000000e-08, 2.0077201218799998e+00, 2.8475268269093999e+03 },
    { 3.1690000000000001e-08, 1.3449948846599999e+00, 1.5912442469019999e+02 },
    { 3.1429999999999998e-08, 1.7142014792800000e+00, 9.6320784650000002e-01 },
    { 3.1240000000000000e-08, 1.0655817305000000e-01, 1.4014645680499999e+01 },
    { 3.1039999999999999e-08, 3.8849805992000003e-01, 2.6342277314714001e+03 },
    { 2.9790000000000001e-08, 3.0408483012400001e+00, 4.2096911658350001e+02 },
    { 2.8900000000000001e-08, 1.2941532000600000e+00, 2.7373205056900001e+03 },
    { 2.8419999999999999e-08, 2.8817147811699999e+00, 3.3773251065900001e+02 },
    { 2.8310000000000000e-08, 4.4559576462999999e+00, 5.1864526483069994e+02 },
    { 2.8279999999999999e-08, 3.0445731776099998e+00, 2.1160217440860001e+02 },
    { 2.8139999999999999e-08, 5.2649713799999998e-01, 7.7841478318470001e+02 },
    { 2.7710000000000001e-08, 2.5436003877200002e+00, 3.1957326339430000e+02 },
    { 2.7700000000000000e-08, 4.5755972964099998e+00, 2.4379022388199999e+01 },
    { 2.7570000000000001e-08, 2.0537143135000000e+00, 2.1452293571540000e+02 },
    { 2.7570000000000001e-08, 2.8359118530899998e+00, 2.1207525516059999e+02 },
    { 2.7120000000000001e-08, 5.2749206059600002e+00, 3.0534616939270001e+02 },
    { 2.7050000000000001e-08, 4.3249199391299999e+00, 4.6470422915999997e+01 },
    { 2.6840000000000001e-08, 5.4897204782499998e+00, 3.9117346822389999e+02 },
    { 2.6530000000000000e-08, 5.7025527009000001e+00, 2.2093390730060000e+02 },
    { 2.6499999999999999e-08, 2.7184998779399998e+00, 1.4563104387150000e+02 },
    { 2.6460000000000000e-08, 3.3018500991600002e+00, 6.9651963761659999e+02 },
    { 2.6300000000000001e-08, 5.5012417987799997e+00, 4.2367742956920000e+02 },
    { 2.6260000000000002e-08, 4.9212317679000001e-01, 1.5349535039770001e+02 },
    { 2.6090000000000001e-08, 1.4192291817700000e+00, 3.2972519178089999e+02 },
    { 2.5889999999999999e-08, 5.1824823965000002e+00, 1.8023719907218001e+03 },
    { 2.5790000000000001e-08, 1.6878992479499999e+00, 6.1503391543000001e+00 },
    { 2.5340000000000001e-08, 2.6227254967999998e+00, 4.2193232442999999e+02 },
    { 2.5270000000000001e-08, 3.4738478600199998e+00, 1.8498791978669999e+02 },
    { 2.4960000000000000e-08, 5.9621838610999998e+00, 2.9207613067999998e+00 },
    { 2.4809999999999998e-08, 1.5531871205900001e+00, 5.7616138801060004e+02 },
    { 2.4439999999999999e-08, 3.8797266634600001e+00, 1.8227960680100000e+02 },
    { 2.4350000000000000e-08, 2.7182969789999998e+00, 6.2736711334179995e+02 },
    { 2.3700000000000000e-08, 1.5466057592600000e+00, 5.1205725331200000e+01 },
    { 2.3619999999999999e-08, 1.0821349524000000e-01, 1.3140394986990000e+02 },
    { 2.3590000000000001e-08, 3.9553471174900001e+00, 2.9689454166000000e+00 },
    { 2.3510000000000000e-08, 2.6849523535000003e-01, 1.4000196957899999e+02 },
    { 2.3490000000000001e-08, 2.4560276990099998e+00, 2.5240214102519999e+03 },
    { 2.3479999999999999e-08, 1.8368990207800000e+00, 2.1499601646740001e+02 },
    { 2.3129999999999999e-08, 2.8151728164900001e+00, 7.3879727483859995e+02 },
    { 2.3099999999999998e-08, 6.3846192911999999e-01, 4.1630325013750002e+02 },
    { 2.2539999999999999e-08, 5.9962317474000004e+00, 1.9054647649404001e+03 },
    { 2.2360000000000000e-08, 1.3100350164500001e+00, 2.4161027108930000e+02 },
    { 2.2250000000000001e-08, 5.2277454655200000e+00, 6.3841281360569997e+02 },
    { 2.2160000000000001e-08, 4.4637837332099997e+00, 3.5330106501699998e+02 },
    { 2.2110000000000001e-08, 4.7269918330199996e+00, 3.8724131496080003e+02 },
    { 2.2110000000000001e-08, 1.8608301411699999e+00, 2.2116340196420001e+02 },
    { 2.2069999999999998e-08, 5.6719760778000001e+00, 9.0588657979150003e+02 },
    { 2.1970000000000001e-08, 2.6392912991999999e+00, 4.3689313161450002e+02 },
    { 2.1950000000000001e-08, 1.5812514236299999e+00, 1.6779385755008000e+03 },
    { 2.1889999999999999e-08, 5.7584331725600002e+00, 4.1852143602870001e+02 },
    { 2.1820000000000000e-08, 3.1526381683700002e+00, 4.3912836384820002e+02 },
    { 2.1279999999999999e-08, 3.9537578111400000e+00, 8.3110498122419995e+02 },
    { 2.1179999999999998e-08, 2.9013357762299998e+00, 2.0543478891180001e+02 },
    { 2.1109999999999998e-08, 5.5103487927100003e+00, 4.2808266358430001e+02 },
    { 2.1060000000000001e-08, 1.8488384148999999e+00, 2.9912639426919998e+02 },
    { 2.0840000000000000e-08, 4.9965615466699997e+00, 2.5494359321360000e+02 },
    { 2.0829999999999999e-08, 6.1444343207600003e+00, 5.9803745040300001e+01 },
    { 2.0790000000000000e-08, 5.0800711447999998e+00, 7.0653628910000004e+00 },
    { 2.0650000000000000e-08, 1.1608121433000000e+00, 1.9269216761850001e+02 },
    { 2.0599999999999999e-08, 5.7891952883400002e+00, 1.2035824960600000e+02 },
    { 2.0520000000000001e-08, 1.2264915618200001e+00, 9.2047073954699997e+01 },
    { 2.0490000000000000e-08, 1.8855901493900000e+00, 1.9928444975750000e+02 },
    { 2.0459999999999999e-08, 5.1917337377599999e+00, 3.4788443904560000e+02 },
    { 2.0359999999999998e-08, 2.8897126243100000e+00, 9.8066817835879999e+02 },
    { 2.0260000000000001e-08, 2.4223572425100000e+00, 2.6413412784722000e+03 },
    { 2.0240000000000001e-08, 4.3293528994399999e+00, 3.1019488636999998e+01 },
    { 2.0070000000000000e-08, 4.3516782891999997e+00, 4.4475743814070000e+02 },
    { 2.0059999999999999e-08, 2.0113885504900000e+00, 2.1186280683950000e+02 },
    { 2.0059999999999999e-08, 2.8782376160999998e+00, 2.1473538403649999e+02 },
    { 1.9740000000000000e-08, 5.7863707967000000e+00, 4.3079097657000000e+02 },
    { 1.9379999999999999e-08, 2.4801660290999998e+00, 4.8736514376280002e+02 },
    { 1.9379999999999999e-08, 3.6221231448800002e+00, 2.6747376618580000e+02 },
    { 1.9370000000000001e-08, 4.0356110062899999e+00, 4.2951895218279998e+02 },
    { 1.9350000000000001e-08, 2.0108099190700002e+00, 7.1617311105999999e+00 },
    { 1.9329999999999998e-08, 3.3301628129599998e+00, 5.3362311835770004e+02 },
    { 1.9219999999999999e-08, 5.9822314826199996e+00, 9.6872999095099999e+01 },
    { 1.9090000000000001e-08, 1.1323125399800000e+00, 2.8245161971279998e+02 },
    { 1.9020000000000001e-08, 1.2460000663599999e+00, 1.6682867252200001e+02 },
    { 1.8979999999999999e-08, 3.5345492714300000e+00, 2.1966188291340001e+02 },
    { 1.8940000000000000e-08, 4.7470269707900004e+00, 5.5800214074589996e+02 },
    { 1.8800000000000000e-08, 3.5905805305300000e+00, 3.3406124266997999e+03 },
    { 1.8580000000000002e-08, 5.8360560737600000e+00, 2.8096714700450002e+02 },
    { 1.8439999999999998e-08, 6.2501264781100003e+00, 4.3222726516850003e+02 },
    { 1.8380000000000000e-08, 2.9700427041299999e+00, 4.2584743135060000e+02 },
    { 1.8340000000000001e-08, 3.9801725678400000e+00, 2.1240532356070000e+02 },
    { 1.8340000000000001e-08, 9.0945359874999998e-01, 2.1419286731529999e+02 },
    { 1.8279999999999999e-08, 2.2314705040899998e+00, 1.7810313497193999e+03 },
    { 1.8209999999999999e-08, 5.4789288946900001e+00, 2.0566428357539999e+02 },
    { 1.7940000000000001e-08, 2.9229986848100000e+00, 2.2731374111849999e+02 },
    { 1.7780000000000001e-08, 2.4563726151000001e-01, 6.5922821389999999e+00 },
    { 1.7680000000000000e-08, 4.8605983967500004e+00, 2.1187638603783998e+03 },
    { 1.7520000000000001e-08, 1.7721513622999999e-01, 9.6962247809489998e+02 },
    { 1.7369999999999999e-08, 2.5943223877600001e+00, 2.6450482076920002e+02 },
    { 1.7120000000000000e-08, 2.9747748966400001e+00, 2.7310284047829998e+02 },
    { 1.6980000000000001e-08, 2.0938725003699998e+00, 2.9201284726839998e+02 },
    { 1.6969999999999999e-08, 3.7468043042399999e+00, 4.2118156490460001e+02 },
    { 1.6919999999999999e-08, 1.2967382169999999e+00, 2.8404132799086001e+03 },
    { 1.6880000000000000e-08, 1.5092827479299999e+00, 8.1506334611420004e+02 },
    { 1.6759999999999999e-08, 2.7373754637499998e+00, 3.4803105662225998e+03 },
    { 1.6479999999999999e-08, 5.0076607321399997e+00, 2.6484548254729998e+03 },
    { 1.6079999999999999e-08, 3.5913812209299998e+00, 2.3390602325750001e+02 },
    { 1.6050000000000001e-08, 2.5725152083600000e+00, 3.1273133312618002e+03 },
    { 1.6020000000000001e-08, 4.8101781993900001e+00, 1.3828873468465999e+03 },
    { 1.5889999999999999e-08, 8.5622467383000000e-01, 1.0444040766621999e+03 },
    { 1.5889999999999999e-08, 5.4352741640700000e+00, 2.5382485042536000e+03 },
    { 1.5880000000000001e-08, 1.6398843881100000e+00, 3.1639186965660001e+03 },
    { 1.5650000000000001e-08, 5.0306445364399996e+00, 4.5884151979040001e+02 },
    { 1.5620000000000000e-08, 4.6230409889299997e+00, 5.4302428721889999e+02 },
    { 1.5469999999999999e-08, 1.3419340099999999e-01, 4.6595506679120001e+02 },
    { 1.5290000000000000e-08, 2.8537077135500000e+00, 2.0915449385380001e+02 },
    { 1.5020000000000001e-08, 4.9634845976999999e+00, 3.7390799283650000e+02 },
    { 1.4940000000000000e-08, 3.6684540883399999e+00, 1.1829215735329001e+03 },
    { 1.4899999999999999e-08, 2.0360144012900001e+00, 2.1744369702220001e+02 },
    { 1.4810000000000000e-08, 9.2457754587999996e-01, 1.9943304451573999e+03 },
    { 1.4769999999999999e-08, 1.3666973199900001e+00, 2.0693630796260001e+02 },
    { 1.4740000000000000e-08, 4.9777214297799999e+00, 1.6992792165031999e+03 },
    { 1.4680000000000000e-08, 4.6791289513200001e+00, 3.3928641933649999e+02 },
    { 1.4620000000000000e-08, 5.0574776433400004e+00, 3.3940249943799998e+01 },
    { 1.4600000000000000e-08, 1.1781840134499999e+00, 9.2940845831999994e+01 },
    { 1.4570000000000001e-08, 5.2628058703000002e-01, 4.2734895040139997e+02 },
    { 1.4510000000000001e-08, 3.5943861162299999e+00, 4.9426624244250002e+02 },
    { 1.4349999999999999e-08, 1.5139569936899999e+00, 8.6742347575359997e+02 },
    { 1.4040000000000000e-08, 4.0234383390100001e+00, 3.0061086697750000e+02 },
    { 1.3900000000000000e-08, 4.0815538465100003e+00, 8.9484087952760001e+02 },
    { 1.3879999999999999e-08, 4.7784783340599999e+00, 2.8546403739102002e+03 },
};

static const VSOP87Term VSOP87A_SATURN_Z1[217] = {
    { 1.9065032829999998e-02, 4.9454474611599997e+00, 2.1329909543799999e+02 },
    { 5.2830126500000001e-03, 3.1415926535900001e+00, 0.0000000000000000e+00 },
    { 1.3026228399999999e-03, 2.2614098087899999e+00, 2.0618554843720000e+02 },
    { 1.0146633200000001e-03, 1.7909582954500001e+00, 2.2041264243879999e+02 },
    { 8.5947577999999998e-04, 5.1612788496999995e-01, 4.2659819087599999e+02 },
    { 2.2257445999999999e-04, 3.0768401565599999e+00, 7.1135470007999997e+00 },
    { 1.6179945999999999e-04, 1.1998751750600001e+00, 4.1948464387519999e+02 },
    { 9.1174020000000006e-05, 6.1720562681400004e+00, 6.3989728631399998e+02 },
    { 7.4707030000000004e-05, 9.3135621171000005e-01, 4.3371173787679999e+02 },
    { 4.9666680000000002e-05, 1.9044864213000001e-01, 3.1639186965660002e+02 },
    { 3.8165640000000002e-05, 4.3828456524500004e+00, 1.1020632121940000e+02 },
    { 2.7241199999999999e-05, 1.6558013866500001e+00, 2.2752618943959999e+02 },
    { 1.7345400000000002e-05, 3.5162807563600000e+00, 1.0309277421860000e+02 },
    { 1.5419950000000001e-05, 2.4232357281199999e+00, 1.9907200143640000e+02 },
    { 1.2093020000000001e-05, 2.9114008909300000e+00, 1.4227094001599999e+01 },
    { 8.7904100000000004e-06, 2.4433850907900001e+00, 2.1723124870109999e+02 },
    { 8.6683800000000002e-06, 7.3856780280000001e-01, 6.3278373931320004e+02 },
    { 7.3437200000000002e-06, 2.7347898541700002e+00, 2.1011770170029999e+02 },
    { 7.2277500000000004e-06, 5.4472616781100003e+00, 8.5319638175199998e+02 },
    { 6.5794500000000002e-06, 1.3144218162100001e+00, 2.1648048917570000e+02 },
    { 6.5136300000000002e-06, 1.4896885555399999e+00, 2.0936694217490000e+02 },
    { 5.8246100000000000e-06, 1.1224949820000001e-01, 6.4701083331480004e+02 },
    { 5.1020399999999999e-06, 1.7557967865999999e+00, 2.0225339517410001e+02 },
    { 4.6782600000000003e-06, 2.0496730031500001e+00, 3.2350541665740002e+02 },
    { 4.4910800000000004e-06, 2.8233393048800002e+00, 1.1731986822019999e+02 },
    { 3.0997100000000000e-06, 2.2038030005499998e+00, 2.2434479570190001e+02 },
    { 2.2619200000000000e-06, 8.5233380252000002e-01, 4.4082528487759998e+02 },
    { 2.1434600000000001e-06, 4.6928813292899996e+00, 1.1045700263900001e+01 },
    { 1.8575600000000000e-06, 1.2762912823499999e+00, 5.2969096509459996e+02 },
    { 1.7743000000000001e-06, 6.7755587852999999e-01, 3.0927832265580003e+02 },
    { 1.4228899999999999e-06, 1.3444404774600001e+00, 9.5979227217800002e+01 },
    { 1.2696200000000000e-06, 1.4056039750600000e+00, 4.1237109687440000e+02 },
    { 1.2473200000000000e-06, 4.7889715126099999e+00, 5.2257741809380002e+02 },
    { 1.2173300000000000e-06, 1.5473388258700000e+00, 6.3735898303399999e+01 },
    { 1.1812900000000000e-06, 4.6283875221399997e+00, 8.4608283475120004e+02 },
    { 1.1704799999999999e-06, 5.0415140377000001e-01, 7.3587651353180001e+02 },
    { 7.8293000000000004e-07, 3.2603110219100002e+00, 2.7703499374140000e+02 },
    { 6.7952000000000004e-07, 4.6047489990999999e+00, 2.1574677599280000e+02 },
    { 6.7805999999999997e-07, 4.0102898433000000e-01, 1.4956319713459999e+02 },
    { 6.5604999999999995e-07, 4.0831715963999998e-01, 4.2266603761290003e+02 },
    { 5.8446999999999999e-07, 2.7067441897900002e+00, 3.1813937377000001e+00 },
    { 5.7884000000000001e-07, 1.7283235255500000e+00, 3.3061896365820002e+02 },
    { 5.7495000000000000e-07, 4.1524608151000004e+00, 4.1555249061209997e+02 },
    { 5.6148999999999998e-07, 4.1105840469899997e+00, 1.1624747044077999e+03 },
    { 5.5408000000000000e-07, 1.5135416385500000e+00, 2.3463973644039999e+02 },
    { 5.4483000000000004e-07, 4.7248208570400001e+00, 1.0664954771900000e+03 },
    { 5.2397999999999996e-07, 1.8575121707200000e+00, 4.2341679713830001e+02 },
    { 5.1422000000000001e-07, 3.0321248079899998e+00, 7.4299006053259995e+02 },
    { 5.1032000000000003e-07, 5.0978380735500002e+00, 3.9321532631000000e+00 },
    { 4.8311000000000004e-07, 5.7677456257799999e+00, 2.1085141488319999e+02 },
    { 4.5825000000000002e-07, 5.6001260977399996e+00, 8.6030992875280003e+02 },
    { 4.4934999999999998e-07, 5.1847195684000003e-01, 4.2977958461370002e+02 },
    { 4.2865999999999998e-07, 1.0143450906900000e+00, 5.3680451209540001e+02 },
    { 4.2398000000000002e-07, 2.5086899346000002e-01, 1.0522683831884001e+03 },
    { 3.7632999999999998e-07, 5.5062942167699997e+00, 9.4917560896980001e+02 },
    { 3.4042999999999999e-07, 1.1917443892999999e+00, 2.2359403617650000e+02 },
    { 3.3541000000000001e-07, 2.7620465196600001e+00, 2.1340641002400002e+01 },
    { 3.3430000000000002e-07, 2.4570774208500001e+00, 4.3053034413910001e+02 },
    { 3.3424000000000001e-07, 1.7475237353499999e+00, 4.3764389113990001e+02 },
    { 3.0208000000000002e-07, 4.4368076231700000e+00, 8.3896928775039999e+02 },
    { 2.9335000000000002e-07, 4.9902540489299998e+00, 5.1546387109299997e+02 },
    { 2.8027000000000001e-07, 2.7374454615000001e+00, 1.0593819301891999e+03 },
    { 2.7259999999999999e-07, 2.1981536480399999e+00, 4.9033408917940000e+02 },
    { 2.4186999999999998e-07, 1.5637311411399999e+00, 1.2443341522100000e+02 },
    { 2.3456000000000000e-07, 3.1384506024799999e+00, 7.4781598567299994e+01 },
    { 2.3381000000000000e-07, 1.2066055877399999e+00, 8.8865680217000005e+01 },
    { 2.1789000000000000e-07, 3.8566954518599998e+00, 1.4788665740644001e+03 },
    { 2.0025999999999999e-07, 2.9074776448000001e+00, 2.0300415469949999e+02 },
    { 1.9633000000000001e-07, 6.1587421189999998e-02, 6.5412438031559998e+02 },
    { 1.8953999999999999e-07, 5.5900214043899998e+00, 2.0373786788240000e+02 },
    { 1.7769000000000000e-07, 1.6323849630000001e-02, 3.0216477565500003e+02 },
    { 1.7709000000000001e-07, 1.8761725032900001e+00, 1.3703302416240001e+02 },
    { 1.7168000000000001e-07, 1.7382564383900001e+00, 1.3851749687070000e+02 },
    { 1.6759000000000001e-07, 3.5928972665999998e-01, 1.2747179660680000e+02 },
    { 1.6625000000000000e-07, 5.3168921752599996e+00, 1.2655674786264001e+03 },
    { 1.6432000000000000e-07, 5.5618159671900003e+00, 9.4206206196899996e+02 },
    { 1.6147999999999999e-07, 6.2482437071900003e+00, 1.3686602528450001e+03 },
    { 1.4146000000000000e-07, 3.9864381214600000e+00, 1.9513984817330001e+02 },
    { 1.3972999999999999e-07, 4.8621821581900004e+00, 2.0863322899200000e+02 },
    { 1.3815999999999999e-07, 5.0658403563099998e+00, 6.2567019231239999e+02 },
    { 1.3738000000000000e-07, 4.3109792823799999e+00, 1.1553611574070001e+03 },
    { 1.3727000000000001e-07, 6.6439765545999996e-01, 8.5827298831199997e+01 },
    { 1.3533000000000001e-07, 5.4765874703300002e+00, 2.1796496188399999e+02 },
    { 1.3269999999999999e-07, 3.4387785735100000e+00, 3.5181659230870002e+02 },
    { 1.3265000000000001e-07, 3.8059121254799999e+00, 7.2876296653099996e+02 },
    { 1.2548000000000001e-07, 2.8896002200500002e+00, 2.3145834270270001e+02 },
    { 1.1510000000000000e-07, 4.6392308597099996e+00, 6.2885158605009997e+02 },
    { 1.1184000000000000e-07, 6.0246170568700004e+00, 1.9120769491019999e+02 },
    { 1.0571000000000000e-07, 2.4582046788600000e+00, 1.0294940738499999e+01 },
    { 1.0509000000000000e-07, 3.0361143745399999e+00, 3.8133035637799999e+01 },
    { 1.0439000000000000e-07, 6.2755667135400000e+00, 7.6266071275599998e+01 },
    { 1.0173000000000000e-07, 3.5819731269399999e+00, 1.8159247264699999e+01 },
    { 1.0071000000000001e-07, 1.4888994091200001e+00, 2.1871572140940000e+02 },
    { 9.8080000000000002e-08, 3.7199435921799999e+00, 3.5033211960040001e+02 },
    { 9.6940000000000006e-08, 8.6976616979999999e-01, 4.0843894361129998e+02 },
    { 9.6110000000000005e-08, 2.4176156371699999e+00, 3.4077089204480001e+02 },
    { 9.5840000000000003e-08, 2.4712094030500000e+00, 2.0076892246579999e+02 },
    { 9.3540000000000002e-08, 1.3209260871099999e+00, 2.1037833413120001e+02 },
    { 8.7260000000000002e-08, 5.3038975242699999e+00, 2.0767002114549999e+02 },
    { 8.7240000000000006e-08, 4.7027339764900002e+00, 8.3185574074960005e+02 },
    { 8.5910000000000006e-08, 4.2332932678299997e+00, 5.2690198039499997e+01 },
    { 8.2870000000000000e-08, 3.5807149591499998e+00, 1.7952584437210000e+03 },
    { 8.2860000000000002e-08, 4.3255697070399997e+00, 2.2286032299359999e+02 },
    { 8.0379999999999996e-08, 2.8552657527299998e+00, 1.9195845443560000e+02 },
    { 7.9300000000000002e-08, 2.3884514253400000e+00, 2.0788246946660001e+02 },
    { 7.8950000000000002e-08, 5.2898347692399996e+00, 1.2584539316256000e+03 },
    { 7.7750000000000005e-08, 6.2027785149400003e+00, 3.6286229257259998e+02 },
    { 7.5199999999999998e-08, 1.5123893830599999e+00, 2.1233588759150001e+02 },
    { 7.4869999999999995e-08, 5.2439323168699996e+00, 2.1426230328450001e+02 },
    { 7.3049999999999996e-08, 5.1023008383099997e+00, 1.5819593482830001e+03 },
    { 6.9670000000000001e-08, 6.1476478039600000e+00, 2.6598929347749998e+02 },
    { 6.5960000000000003e-08, 6.0695528646400003e+00, 6.1855664531160005e+02 },
    { 6.5260000000000003e-08, 5.9735587689100003e+00, 1.6850521225016000e+03 },
    { 6.5099999999999994e-08, 2.5876604723500001e+00, 1.6060889739850001e+02 },
    { 6.3959999999999998e-08, 4.0517512819399997e+00, 1.4717530270636000e+03 },
    { 6.3829999999999997e-08, 7.6399546198000001e-01, 1.4244965013379999e+02 },
    { 6.2699999999999999e-08, 1.5503910225199999e+00, 5.6622351302600002e+01 },
    { 6.2229999999999995e-08, 4.9207319215799998e+00, 2.1325091132820000e+02 },
    { 6.2219999999999997e-08, 1.8353424108700001e+00, 2.1334727954780001e+02 },
    { 6.0279999999999994e-08, 5.8483575156000001e+00, 6.3596513305090002e+02 },
    { 6.0100000000000002e-08, 5.5938034070900002e+00, 4.8322054217860000e+02 },
    { 5.9890000000000002e-08, 2.6895251400200002e+00, 2.1621985674480001e+02 },
    { 5.9670000000000004e-08, 6.1056128776499996e+00, 2.0910630974399999e+02 },
    { 5.7560000000000001e-08, 1.1722011199900000e+00, 7.0849445304200003e+01 },
    { 5.7359999999999999e-08, 4.9812469532700003e+00, 2.1892816973050000e+02 },
    { 5.6820000000000002e-08, 1.2527174964100001e+00, 2.2582926841020000e+02 },
    { 5.5080000000000001e-08, 2.3756513534000001e-01, 7.5010360753340001e+02 },
    { 5.4300000000000003e-08, 7.3930957949999998e-01, 4.4793883187839998e+02 },
    { 5.4219999999999998e-08, 8.9394666012000001e-01, 2.2091400527800001e+01 },
    { 5.2560000000000002e-08, 1.9699642510000000e-02, 2.6992144674060000e+02 },
    { 4.8839999999999999e-08, 8.6865640572000002e-01, 5.4391805909619995e+02 },
    { 4.8779999999999997e-08, 5.0492260465800003e+00, 1.5748458012822000e+03 },
    { 4.5830000000000001e-08, 3.1424725082900000e+00, 1.3757737998458001e+03 },
    { 4.5750000000000003e-08, 6.2080508856899996e+00, 2.8808069400530002e+02 },
    { 4.3229999999999997e-08, 4.1479027245899998e+00, 1.2797945726280000e+03 },
    { 4.3070000000000001e-08, 4.7097853303699999e+00, 4.2415051032119999e+02 },
    { 4.2699999999999999e-08, 3.3547953167200002e+00, 1.0451548361876000e+03 },
    { 4.1659999999999997e-08, 6.0976676360799997e+00, 1.4844727083000000e+00 },
    { 4.1239999999999997e-08, 4.5506087904900001e+00, 2.3539049596580000e+02 },
    { 4.0690000000000002e-08, 2.6010000547800001e+00, 5.5406998748280000e+02 },
    { 4.0250000000000000e-08, 3.7930065194700000e+00, 1.2530172972200001e+01 },
    { 3.9150000000000003e-08, 2.9737950636699999e+00, 1.0627416795630000e+02 },
    { 3.9150000000000003e-08, 4.4502437816800002e+00, 1.1482476104062000e+03 },
    { 3.8820000000000000e-08, 3.7468268673599998e+00, 7.0363318461740005e+02 },
    { 3.8430000000000001e-08, 4.9816244210700003e+00, 4.9744763618019999e+02 },
    { 3.8239999999999997e-08, 3.4524610701600000e+00, 3.8846515523820000e+02 },
    { 3.7230000000000003e-08, 4.8530992786800002e+00, 1.0736090241908000e+03 },
    { 3.6629999999999998e-08, 2.6191974956199999e+00, 1.1338771495710000e+02 },
    { 3.6099999999999999e-08, 5.9405875136299997e+00, 9.3494851496820002e+02 },
    { 3.5810000000000001e-08, 4.1234457509600002e+00, 9.5612275556000004e+00 },
    { 3.5740000000000001e-08, 6.0140321333499998e+00, 6.4307868005169996e+02 },
    { 3.3570000000000002e-08, 4.4322506069200003e+00, 2.1944943459230001e+02 },
    { 3.3139999999999998e-08, 3.4162567657600000e+00, 2.1116503133776000e+03 },
    { 3.2700000000000002e-08, 6.2741070080099997e+00, 2.1277783057619999e+02 },
    { 3.2609999999999999e-08, 4.8253112370000001e-01, 2.1382036029980000e+02 },
    { 3.1739999999999998e-08, 1.1906321356700000e+00, 3.3773251065900001e+02 },
    { 3.1410000000000001e-08, 1.4556813367999999e+00, 6.5094298657790000e+02 },
    { 3.1030000000000001e-08, 5.6504668360499997e+00, 2.0014439921582000e+03 },
    { 3.0580000000000000e-08, 4.7795418222699997e+00, 2.1254833591260001e+02 },
    { 3.0279999999999997e-08, 3.4251871566599998e+00, 9.5628915597059995e+02 },
    { 3.0180000000000003e-08, 4.9570780256000004e+00, 1.8983512179396000e+03 },
    { 3.0120000000000001e-08, 4.4582726960099999e+00, 1.0702492748170000e+02 },
    { 2.9720000000000001e-08, 4.7144792879799997e+00, 1.8912376709388000e+03 },
    { 2.9510000000000001e-08, 1.3108897365300001e+00, 3.4470304530790003e+02 },
    { 2.9360000000000000e-08, 3.8429033874899998e+00, 2.1749188113200000e+02 },
    { 2.9169999999999999e-08, 8.9207515266000004e-01, 4.1703696332039999e+02 },
    { 2.8769999999999999e-08, 3.7845649425799999e+00, 1.7881448967202000e+03 },
    { 2.7430000000000001e-08, 5.3297800828400002e+00, 5.6511568774670002e+02 },
    { 2.6490000000000001e-08, 4.0890106888700002e+00, 2.8414854074220000e+02 },
    { 2.6230000000000001e-08, 1.6971352826199999e+00, 6.3671589257630001e+02 },
    { 2.6230000000000001e-08, 1.6476059550000002e-02, 2.0714875628370001e+02 },
    { 2.5640000000000000e-08, 8.4277674835000005e-01, 3.5893013930950002e+02 },
    { 2.5069999999999999e-08, 8.3753919585000003e-01, 4.3689313161450002e+02 },
    { 2.4089999999999999e-08, 3.5626776188800000e+00, 1.1695882514085999e+03 },
    { 2.3980000000000000e-08, 2.6043064477600000e+00, 4.0450679034820001e+02 },
    { 2.2620000000000000e-08, 4.2582083068900003e+00, 9.8899988524600005e+01 },
    { 2.2090000000000001e-08, 4.8029263130900004e+00, 2.2147430875962000e+03 },
    { 2.1870000000000000e-08, 9.5383665392000005e-01, 1.1439910691340000e+02 },
    { 2.1570000000000001e-08, 3.0244251048400002e+00, 1.6921656695024001e+03 },
    { 2.1410000000000001e-08, 1.8285377864900001e+00, 4.7928838891549998e+02 },
    { 2.1400000000000000e-08, 5.3866667280599998e+00, 1.7516605980020000e+02 },
    { 2.1360000000000001e-08, 2.9671583994400001e+00, 1.3615467058442000e+03 },
    { 2.0929999999999999e-08, 8.7904326629999996e-01, 4.6658664459999999e+00 },
    { 2.0879999999999999e-08, 5.0546583625399997e+00, 2.5143213107579999e+02 },
    { 2.0850000000000001e-08, 4.0927714903899997e+00, 1.2726810256271999e+03 },
    { 2.0820000000000000e-08, 4.6360311169899999e+00, 1.2125202148330000e+02 },
    { 2.0689999999999999e-08, 5.1530462120799996e+00, 2.2046082654860001e+02 },
    { 2.0560000000000000e-08, 2.3961809335100002e+00, 2.4351557300350000e+03 },
    { 2.0450000000000001e-08, 3.7382001577500001e+00, 1.1413847448250000e+02 },
    { 2.0219999999999998e-08, 5.8021482426500004e+00, 1.8105576652360000e+02 },
    { 2.0050000000000001e-08, 4.6625468960900003e+00, 4.2904587143079999e+02 },
    { 2.0030000000000001e-08, 2.4756427487499999e+00, 4.1630325013750002e+02 },
    { 1.9900000000000000e-08, 2.0878555006199999e+00, 2.4209286360333999e+03 },
    { 1.9390000000000000e-08, 2.2512468171300002e+00, 1.6779385755008000e+03 },
    { 1.8810000000000001e-08, 1.1306227724800000e+00, 7.5721715453419995e+02 },
    { 1.8609999999999999e-08, 4.4540297058800000e+00, 5.0835032409220003e+02 },
    { 1.8270000000000001e-08, 5.0443469443900000e+00, 8.4215068148809996e+02 },
    { 1.7949999999999999e-08, 2.9531651945599999e+00, 1.7810313497193999e+03 },
    { 1.7859999999999999e-08, 5.4487979164300002e+00, 2.4554242435239999e+02 },
    { 1.7479999999999998e-08, 5.8508622621799997e+00, 1.7394221952280000e+02 },
    { 1.7459999999999999e-08, 3.0645557577000000e+00, 2.0085575391590000e+03 },
    { 1.7140000000000000e-08, 4.9960118268500002e+00, 2.3178358618148000e+03 },
    { 1.7070000000000000e-08, 1.5793598750100000e+00, 6.0697767545533998e+03 },
    { 1.6639999999999999e-08, 1.4097949424000000e-01, 1.5104766984290001e+02 },
    { 1.6519999999999998e-08, 2.7613922525799999e+00, 6.4382943957709995e+02 },
    { 1.6420000000000001e-08, 1.8765237703300000e+00, 1.7787437278589999e+02 },
    { 1.6230000000000000e-08, 6.3294345223000004e-01, 6.2173803904930003e+02 },
    { 1.6140000000000001e-08, 5.9646152776900001e+00, 8.6742347575359997e+02 },
    { 1.5990000000000000e-08, 8.6145918975000002e-01, 8.1752133216199994e+01 },
    { 1.5530000000000001e-08, 1.3235036837300000e+00, 1.3154696222179999e+02 },
    { 1.5130000000000000e-08, 3.0141573275400000e+00, 2.4280421830342002e+03 },
    { 1.5110000000000001e-08, 3.2265070214799998e+00, 9.9160620955499994e+01 },
    { 1.4800000000000000e-08, 2.8923019793200000e+00, 3.5574874557179999e+02 },
    { 1.4640000000000001e-08, 6.1198560166200000e+00, 2.0613736432740001e+02 },
    { 1.4580000000000001e-08, 5.3705512898399999e+00, 1.4807872442630000e+02 },
    { 1.4250000000000000e-08, 3.0267999310599998e+00, 2.4872381809010000e+02 },
    { 1.3840000000000000e-08, 2.3395962885100001e+00, 4.4475743814070000e+02 },
};

static const VSOP87Term VSOP87A_SATURN_Z2[87] = {
    { 1.3127515500000000e-03, 8.8689981010000002e-02, 2.1329909543799999e+02 },
    { 3.0147649000000001e-04, 3.9139620388699998e+00, 2.0618554843720000e+02 },
    { 1.9322173000000000e-04, 9.2287486239999994e-02, 2.2041264243879999e+02 },
    { 6.8689259999999997e-05, 5.4842025539500003e+00, 4.2659819087599999e+02 },
    { 3.1382330000000001e-05, 0.0000000000000000e+00, 0.0000000000000000e+00 },
    { 2.8261070000000001e-05, 1.3658331855500001e+00, 7.1135470007999997e+00 },
    { 2.6463319999999999e-05, 2.9460739595500001e+00, 4.1948464387519999e+02 },
    { 1.8447979999999999e-05, 5.4361206285600003e+00, 4.3371173787679999e+02 },
    { 1.0553829999999999e-05, 6.2389078517899996e+00, 2.2752618943959999e+02 },
    { 1.0364350000000000e-05, 4.3391630855200001e+00, 6.3989728631399998e+02 },
    { 6.3461100000000000e-06, 4.1041382198300003e+00, 1.9907200143640000e+02 },
    { 4.9312700000000001e-06, 1.7451698308400001e+00, 3.1639186965660002e+02 },
    { 3.8499299999999997e-06, 1.2206682953100001e+00, 1.4227094001599999e+01 },
    { 2.3947600000000001e-06, 4.4793592147399997e+00, 2.1011770170029999e+02 },
    { 2.0519400000000000e-06, 1.7672812687699999e+00, 1.1020632121940000e+02 },
    { 1.8473500000000000e-06, 5.8172967911900004e+00, 2.1648048917570000e+02 },
    { 1.7630800000000000e-06, 4.5631776192000002e+00, 6.4701083331480004e+02 },
    { 1.5003300000000000e-06, 4.3364971118100000e+00, 1.0309277421860000e+02 },
    { 1.4802300000000000e-06, 4.2818811204599996e+00, 2.1723124870109999e+02 },
    { 1.2887700000000001e-06, 2.6282736382800000e+00, 6.3278373931320004e+02 },
    { 1.2133200000000000e-06, 3.4778047871900002e+00, 8.5319638175199998e+02 },
    { 1.0002000000000000e-06, 5.3853585400300004e+00, 4.4082528487759998e+02 },
    { 9.2666000000000001e-07, 1.2062258944099999e+00, 1.1731986822019999e+02 },
    { 8.6967999999999999e-07, 5.9261969575900002e+00, 2.0936694217490000e+02 },
    { 5.6695999999999999e-07, 4.6733705639000001e-01, 3.2350541665740002e+02 },
    { 5.4657999999999996e-07, 3.0102112087499999e+00, 4.1237109687440000e+02 },
    { 5.1546000000000004e-07, 3.1492665985900001e+00, 2.0225339517410001e+02 },
    { 3.7586999999999999e-07, 2.3046794939000002e+00, 5.2969096509459996e+02 },
    { 3.2681000000000000e-07, 3.0189308023599999e+00, 3.0927832265580003e+02 },
    { 3.2217999999999999e-07, 6.0945793415800003e+00, 2.3463973644039999e+02 },
    { 2.7739999999999998e-07, 3.1873528679600001e+00, 9.5979227217800002e+01 },
    { 1.9282000000000000e-07, 2.7555296167999999e-01, 2.2434479570190001e+02 },
    { 1.7380999999999999e-07, 1.0792422172500000e+00, 2.1340641002400002e+01 },
    { 1.6304000000000001e-07, 3.7189750809900000e+00, 8.6030992875280003e+02 },
    { 1.5895999999999999e-07, 5.6818198779899998e+00, 2.2359403617650000e+02 },
    { 1.5146999999999999e-07, 4.9384358574000000e+00, 4.2977958461370002e+02 },
    { 1.5102000000000001e-07, 2.3469873790000002e-02, 3.3061896365820002e+02 },
    { 1.4105000000000001e-07, 2.4960426678000000e-01, 5.2257741809380002e+02 },
    { 1.3773999999999999e-07, 2.5334550410000001e-01, 3.9321532631000000e+00 },
    { 1.3435999999999999e-07, 3.7387993640300001e+00, 4.2341679713830001e+02 },
    { 1.2373999999999999e-07, 2.7274278802400000e+00, 1.0664954771900000e+03 },
    { 1.2326999999999999e-07, 4.8017946898900004e+00, 4.2266603761290003e+02 },
    { 1.1165000000000000e-07, 4.4746912910800001e+00, 1.1045700263900001e+01 },
    { 1.0457000000000000e-07, 8.9680214250000001e-01, 3.1813937377000001e+00 },
    { 1.0434000000000001e-07, 4.6986367532899997e+00, 2.0300415469949999e+02 },
    { 1.0092000000000001e-07, 6.0864633976200002e+00, 1.2443341522100000e+02 },
    { 9.6750000000000003e-08, 4.5486512379799997e+00, 6.5412438031559998e+02 },
    { 9.3660000000000006e-08, 4.8872288698800004e+00, 2.7703499374140000e+02 },
    { 9.3159999999999995e-08, 2.9266021465900001e+00, 8.8865680217000005e+01 },
    { 9.1790000000000003e-08, 1.5631499082400000e+00, 7.4299006053259995e+02 },
    { 8.1549999999999999e-08, 6.1938266553999997e+00, 4.1555249061209997e+02 },
    { 6.9349999999999996e-08, 7.7796862891999996e-01, 9.4917560896980001e+02 },
    { 6.4930000000000000e-08, 4.4337718018399999e+00, 1.9195845443560000e+02 },
    { 6.2849999999999997e-08, 3.4693933457999998e-01, 5.1546387109299997e+02 },
    { 6.0199999999999996e-08, 5.8835983755200001e+00, 7.3587651353180001e+02 },
    { 5.9699999999999999e-08, 6.1322711168800002e+00, 6.3735898303399999e+01 },
    { 5.7429999999999999e-08, 5.4864809284799998e+00, 1.4956319713459999e+02 },
    { 5.2590000000000003e-08, 4.4639598477900000e+00, 4.3053034413910001e+02 },
    { 4.8620000000000001e-08, 5.5243855404000000e+00, 5.3680451209540001e+02 },
    { 4.4729999999999998e-08, 1.8218535818799999e+00, 1.0593819301891999e+03 },
    { 4.1120000000000000e-08, 5.1336285961200003e+00, 6.2567019231239999e+02 },
    { 3.9419999999999998e-08, 6.3354697458999998e-01, 1.0294940738499999e+01 },
    { 3.8750000000000000e-08, 5.9148424450499997e+00, 4.3764389113990001e+02 },
    { 3.7340000000000002e-08, 6.1528397859900004e+00, 8.3896928775039999e+02 },
    { 3.5889999999999999e-08, 5.2830219879899998e+00, 4.4793883187839998e+02 },
    { 3.5880000000000001e-08, 1.0408784049900000e+00, 9.4206206196899996e+02 },
    { 3.4580000000000003e-08, 5.4836396763800002e+00, 7.2876296653099996e+02 },
    { 3.3449999999999998e-08, 5.9783874666400001e+00, 2.1574677599280000e+02 },
    { 3.3050000000000001e-08, 4.8315382150100001e+00, 7.4781598567299994e+01 },
    { 2.9810000000000000e-08, 1.3802153347699999e+00, 2.3145834270270001e+02 },
    { 2.8559999999999999e-08, 1.8666586671300001e+00, 3.0216477565500003e+02 },
    { 2.7660000000000001e-08, 4.0134293559000001e-01, 8.4608283475120004e+02 },
    { 2.2690000000000000e-08, 2.1008926468900002e+00, 1.8159247264699999e+01 },
    { 2.1069999999999999e-08, 5.9861697224999999e-01, 1.2584539316256000e+03 },
    { 2.0879999999999999e-08, 1.8254119374600000e+00, 1.4717530270636000e+03 },
    { 2.0400000000000001e-08, 5.8076053683500000e+00, 1.1624747044077999e+03 },
    { 1.9900000000000000e-08, 7.9685389889000002e-01, 2.1085141488319999e+02 },
    { 1.9490000000000001e-08, 8.3703762065999998e-01, 1.2655674786264001e+03 },
    { 1.9340000000000000e-08, 4.3191894480000000e-02, 1.9513984817330001e+02 },
    { 1.8740000000000001e-08, 8.4722174142999995e-01, 1.3757737998458001e+03 },
    { 1.8679999999999999e-08, 2.3989474559000001e-01, 1.8983512179396000e+03 },
    { 1.8399999999999999e-08, 3.1488053246400001e+00, 4.0843894361129998e+02 },
    { 1.8240000000000000e-08, 8.2941277839999994e-02, 8.3185574074960005e+02 },
    { 1.7509999999999999e-08, 1.2206305582300001e+00, 2.0373786788240000e+02 },
    { 1.5959999999999999e-08, 6.4727508404000000e-01, 6.1855664531160005e+02 },
    { 1.5889999999999999e-08, 3.3356367658299999e+00, 1.0736090241908000e+03 },
    { 1.5040000000000001e-08, 2.1319055183700000e+00, 1.3686602528450001e+03 },
};

static const VSOP87Term VSOP87A_SATURN_Z3[44] = {
    { 4.7790739999999999e-05, 5.5772375632999998e+00, 2.0618554843720000e+02 },
    { 4.5594189999999997e-05, 1.7064687150100000e+00, 2.1329909543799999e+02 },
    { 3.9654019999999998e-05, 0.0000000000000000e+00, 0.0000000000000000e+00 },
    { 2.5082420000000000e-05, 4.6495905631300003e+00, 2.2041264243879999e+02 },
    { 3.2762800000000002e-06, 3.3159727546500002e+00, 4.2659819087599999e+02 },
    { 3.2270199999999999e-06, 4.7075702421600001e+00, 4.1948464387519999e+02 },
    { 3.1433399999999999e-06, 3.6597836489400000e+00, 4.3371173787679999e+02 },
    { 2.7613199999999998e-06, 4.5305155090099998e+00, 2.2752618943959999e+02 },
    { 2.4374499999999998e-06, 5.8770667817900000e+00, 7.1135470007999997e+00 },
    { 1.7495700000000000e-06, 5.7859263504999996e+00, 1.9907200143640000e+02 },
    { 9.3819999999999997e-07, 2.4963795572000000e+00, 6.3989728631399998e+02 },
    { 8.3895000000000003e-07, 5.7990559110700000e+00, 1.4227094001599999e+01 },
    { 4.4019999999999998e-07, 6.2148001348399999e+00, 2.1011770170029999e+02 },
    { 3.7044999999999999e-07, 2.7453267318400001e+00, 6.4701083331480004e+02 },
    { 3.6917000000000002e-07, 3.2072682016199998e+00, 3.1639186965660002e+02 },
    { 2.9900999999999998e-07, 3.6370094442299998e+00, 4.4082528487759998e+02 },
    { 2.7315999999999999e-07, 4.0219205547700003e+00, 2.1648048917570000e+02 },
    { 2.1812000000000000e-07, 2.9602454753000002e-01, 1.0309277421860000e+02 },
    { 1.5029000000000000e-07, 1.6128730871900001e+00, 8.5319638175199998e+02 },
    { 1.4205999999999999e-07, 4.6695934959600001e+00, 4.1237109687440000e+02 },
    { 1.3278000000000000e-07, 5.8333238627900004e+00, 1.1731986822019999e+02 },
    { 1.2912999999999999e-07, 4.4219002697100001e+00, 6.3278373931320004e+02 },
    { 1.2456000000000000e-07, 4.3913035371999998e+00, 2.3463973644039999e+02 },
    { 1.2442000000000000e-07, 6.2890480503000001e-01, 1.1020632121940000e+02 },
    { 1.2017000000000000e-07, 5.8701915878199999e+00, 2.1723124870109999e+02 },
    { 5.8150000000000001e-08, 5.6203324389600002e+00, 2.1340641002400002e+01 },
    { 5.0169999999999999e-08, 5.5322738128899998e+00, 3.0927832265580003e+02 },
    { 4.5699999999999999e-08, 4.9362443690199997e+00, 3.2350541665740002e+02 },
    { 4.5499999999999997e-08, 5.3421683276999996e+00, 2.0225339517410001e+02 },
    { 4.2330000000000003e-08, 5.0544380420200001e+00, 9.5979227217800002e+01 },
    { 4.1670000000000002e-08, 3.8946473496000000e+00, 2.2359403617650000e+02 },
    { 3.9809999999999997e-08, 1.8466317606200000e+00, 8.6030992875280003e+02 },
    { 3.4529999999999999e-08, 6.0860963714500000e+00, 1.9195845443560000e+02 },
    { 3.2100000000000003e-08, 2.7640920095600001e+00, 6.5412438031559998e+02 },
    { 2.9959999999999998e-08, 3.0835160426199999e+00, 4.2977958461370002e+02 },
    { 2.8640000000000000e-08, 4.2920644488499997e+00, 1.2443341522100000e+02 },
    { 2.7809999999999999e-08, 4.6343666123500000e+00, 3.3061896365820002e+02 },
    { 2.7190000000000001e-08, 4.6559417917500001e+00, 2.0936694217490000e+02 },
    { 2.3370000000000000e-08, 4.6166957236600004e+00, 8.8865680217000005e+01 },
    { 2.2869999999999999e-08, 7.0623098744000001e-01, 2.0300415469949999e+02 },
    { 2.1340000000000001e-08, 1.3758122445700001e+00, 1.0664954771900000e+03 },
    { 1.6930000000000000e-08, 4.7090266165200001e+00, 2.2434479570190001e+02 },
    { 1.4600000000000000e-08, 5.3276861405800000e+00, 3.0216477565500003e+02 },
    { 1.4600000000000000e-08, 3.9456233074700000e+00, 4.4793883187839998e+02 },
};

static const VSOP87Term VSOP87A_SATURN_Z4[19] = {
    { 5.7430599999999999e-06, 9.6387396085999999e-01, 2.0618554843720000e+02 },
    { 2.5251600000000001e-06, 2.9018894635499999e+00, 2.2041264243879999e+02 },
    { 2.4487500000000001e-06, 2.9649229660900001e+00, 2.1329909543799999e+02 },
    { 5.4532000000000002e-07, 2.8156501439700001e+00, 2.2752618943959999e+02 },
    { 4.0987000000000000e-07, 1.8832993239500000e+00, 4.3371173787679999e+02 },
    { 3.8219000000000001e-07, 1.1898163011400000e+00, 1.9907200143640000e+02 },
    { 3.6668000000000000e-07, 3.1415926535900001e+00, 0.0000000000000000e+00 },
    { 2.9531999999999997e-07, 2.1869783243999999e-01, 4.1948464387519999e+02 },
    { 1.6647999999999999e-07, 1.6808612679599999e+00, 4.2659819087599999e+02 },
    { 1.5829000000000000e-07, 4.0851820292200003e+00, 7.1135470007999997e+00 },
    { 1.3086000000000000e-07, 4.0950623376399999e+00, 1.4227094001599999e+01 },
    { 6.4860000000000000e-08, 1.8912194822499999e+00, 4.4082528487759998e+02 },
    { 6.4589999999999998e-08, 6.2293627130999996e-01, 6.3989728631399998e+02 },
    { 6.0269999999999996e-08, 9.3640230433000005e-01, 6.4701083331480004e+02 },
    { 3.0530000000000003e-08, 2.6631127915400001e+00, 2.3463973644039999e+02 },
    { 2.1050000000000000e-08, 4.6469400266999999e-01, 8.5319638175199998e+02 },
    { 1.7800000000000001e-08, 4.7785599892399997e+00, 1.1731986822019999e+02 },
    { 1.5920000000000000e-08, 4.9445731601999998e-01, 4.1237109687440000e+02 },
    { 1.5860000000000001e-08, 5.7995590886399997e+00, 1.1020632121940000e+02 },
};

static const VSOP87Term VSOP87A_SATURN_Z5[6] = {
    { 5.3399999999999999e-07, 2.6140386139100000e+00, 2.0618554843720000e+02 },
    { 1.8015000000000001e-07, 1.1581954231200000e+00, 2.2041264243879999e+02 },
    { 1.0802000000000000e-07, 3.1415926535900001e+00, 0.0000000000000000e+00 },
    { 7.7830000000000003e-08, 1.0795366684600001e+00, 2.2752618943959999e+02 },
    { 6.0160000000000004e-08, 2.8826766620000002e+00, 1.9907200143640000e+02 },
    { 4.4239999999999998e-08, 1.1691197681000000e-01, 4.3371173787679999e+02 },
};

static const VSOP87Block VSOP87A_SATURN_X[6] = {
    { VSOP87A_SATURN_X0, 1652 },
    { VSOP87A_SATURN_X1, 892 },
    { VSOP87A_SATURN_X2, 481 },
    { VSOP87A_SATURN_X3, 215 },
    { VSOP87A_SATURN_X4, 87 },
    { VSOP87A_SATURN_X5, 31 },
};

static const VSOP87Block VSOP87A_SATURN_Y[6] = {
    { VSOP87A_SATURN_Y0, 1658 },
    { VSOP87A_SATURN_Y1, 917 },
    { VSOP87A_SATURN_Y2, 465 },
    { VSOP87A_SATURN_Y3, 201 },
    { VSOP87A_SATURN_Y4, 88 },
    { VSOP87A_SATURN_Y5, 32 },
};

static const VSOP87Block VSOP87A_SATURN_Z[6] = {
    { VSOP87A_SATURN_Z0, 420 },
    { VSOP87A_SATURN_Z1, 217 },
    { VSOP87A_SATURN_Z2, 87 },
    { VSOP87A_SATURN_Z3, 44 },
    { VSOP87A_SATURN_Z4, 19 },
    { VSOP87A_SATURN_Z5, 6 },
};

static const int32 VSOP87A_SATURN_TRUNC_COUNT[11][3][6] = {
    {
        { 1652, 892, 481, 215, 87, 31 },
        { 1658, 917, 465, 201, 88, 32 },
        { 420, 217, 87, 44, 19, 6 },
    },
    {
        { 1434, 773, 402, 182, 76, 27 },
        { 1439, 790, 393, 172, 75, 27 },
        { 420, 217, 87, 44, 19, 6 },
    },
    {
        { 784, 411, 181, 81, 41, 13 },
        { 778, 413, 173, 81, 40, 13 },
        { 258, 130, 58, 27, 14, 5 },
    },
    {
        { 622, 320, 145, 59, 30, 11 },
        { 624, 313, 143, 58, 33, 11 },
        { 194, 93, 46, 25, 11, 3 },
    },
    {
        { 349, 165, 68, 42, 15, 5 },
        { 344, 162, 65, 43, 15, 5 },
        { 97, 49, 27, 12, 4, 1 },
    },
    {
        { 261, 123, 59, 30, 14, 4 },
        { 258, 121, 58, 32, 13, 4 },
        { 77, 36, 22, 10, 3, 0 },
    },
    {
        { 138, 60, 30, 16, 7, 2 },
        { 137, 61, 31, 15, 6, 2 },
        { 38, 23, 11, 4, 1, 0 },
    },
    {
        { 96, 48, 26, 13, 4, 0 },
        { 95, 49, 27, 12, 4, 0 },
        { 32, 15, 10, 4, 0, 0 },
    },
    {
        { 50, 28, 13, 7, 2, 0 },
        { 49, 29, 13, 7, 2, 0 },
        { 14, 9, 4, 0, 0, 0 },
    },
    {
        { 38, 24, 12, 4, 0, 0 },
        { 39, 24, 12, 4, 0, 0 },
        { 12, 7, 3, 0, 0, 0 },
    },
    {
        { 13, 10, 5, 0, 0, 0 },
        { 13, 10, 5, 0, 0, 0 },
        { 5, 4, 1, 0, 0, 0 },
    },
};

