// Copyright Epic Games, Inc. All Rights Reserved.

// Auto-generated VSOP87A header for URANUS (rectangular XYZ, J2000)
// Source: IMCCE/BDL VSOP87A.uranus file
// Generated by vsop87a_header_gen_imcce.py

#pragma once

#include "VSOP87A_Common.h"

static const VSOP87Term VSOP87A_URANUS_X0[1464] = {
    { 1.9173707303590000e+01, 5.4813341648899998e+00, 7.4781598567299994e+01 },
    { 1.3227252387199999e+00, 0.0000000000000000e+00, 0.0000000000000000e+00 },
    { 4.4402496796000002e-01, 1.6596751958600000e+00, 1.4956319713459999e+02 },
    { 1.4668209481000000e-01, 3.4239586280399998e+00, 7.3297125859000005e+01 },
    { 1.4130269478999999e-01, 4.3957292793400002e+00, 7.6266071275599998e+01 },
    { 6.2011061780000001e-02, 5.1404357412500001e+00, 1.4844727083000000e+00 },
    { 1.5429513430000000e-02, 4.1212183807200002e+00, 2.2434479570190001e+02 },
    { 1.4442166600000000e-02, 2.6511711520099999e+00, 1.4807872442630000e+02 },
    { 9.4499556299999996e-03, 1.6586933875700001e+00, 1.1045700263900001e+01 },
    { 6.5752481500000000e-03, 5.7595170636000004e-01, 1.5104766984290001e+02 },
    { 6.3400026999999998e-03, 4.0955658972400002e+00, 6.3735898303399999e+01 },
    { 6.2162467600000004e-03, 3.0588224663800001e+00, 7.7750543983900002e+01 },
    { 5.8518254199999998e-03, 4.7993477967800002e+00, 7.1812653150700001e+01 },
    { 5.4769905599999997e-03, 3.6312772505600002e+00, 8.5827298831199997e+01 },
    { 4.9608764900000001e-03, 5.9947400861000000e-01, 5.2969096509459996e+02 },
    { 4.5821998399999996e-03, 3.9078828411200002e+00, 2.9689454166000000e+00 },
    { 3.8362553500000001e-03, 6.1876201057599998e+00, 1.3851749687070000e+02 },
    { 2.6793815600000001e-03, 9.6885660136999996e-01, 2.1329909543799999e+02 },
    { 2.1536800500000001e-03, 5.3087764142799996e+00, 3.8133035637799999e+01 },
    { 1.4550538899999999e-03, 2.3175975708499998e+00, 7.0849445304200003e+01 },
    { 1.3534003200000001e-03, 5.5106246081599997e+00, 7.8713751830400000e+01 },
    { 1.2510568600000000e-03, 2.5145527306300002e+00, 1.1143016149680000e+02 },
    { 1.1959385899999999e-03, 4.1013854426699998e+00, 3.9617508346100003e+01 },
    { 1.1126024400000001e-03, 5.1225278432500003e+00, 2.2286032299359999e+02 },
    { 1.1012538699999999e-03, 4.4547352872400001e+00, 3.5164090221199999e+01 },
    { 1.0461982699999999e-03, 3.9053891633400002e+00, 1.4659425171800001e+02 },
    { 6.5066905000000000e-04, 3.7300845290600000e+00, 1.0994568878849999e+02 },
    { 6.3584587999999996e-04, 2.9966233158000000e-01, 2.9912639426919998e+02 },
    { 5.3904040999999995e-04, 3.9259042250700000e+00, 3.9321532631000000e+00 },
    { 3.9181661999999998e-04, 2.6884128076899998e+00, 4.4534181248999998e+00 },
    { 3.5943348000000002e-04, 4.0875454301599996e+00, 2.0225339517410001e+02 },
    { 3.4555651999999999e-04, 1.8469932925699999e+00, 7.9235016692200006e+01 },
    { 3.4341683000000002e-04, 3.0378166192800000e+00, 2.2582926841020000e+02 },
    { 3.3867050000000002e-04, 5.9841843610299996e+00, 7.0328180442399997e+01 },
    { 3.3134636000000002e-04, 2.5420159121800001e+00, 6.5220371011699996e+01 },
    { 3.1823951000000002e-04, 5.5394858324399996e+00, 1.5253214255120000e+02 },
    { 2.8384953000000002e-04, 6.0178543030600000e+00, 1.8472728735580000e+02 },
    { 2.8371613999999997e-04, 2.5802665712300001e+00, 1.2747179660680000e+02 },
    { 2.6657176000000001e-04, 6.1102793972700002e+00, 1.6060889739850001e+02 },
    { 2.5208833000000001e-04, 5.3027214465699997e+00, 9.5612275556000004e+00 },
    { 2.3467801999999999e-04, 4.0972986032199996e+00, 1.4563104387150000e+02 },
    { 2.2963939000000000e-04, 5.5147507365499999e+00, 8.4342826122899993e+01 },
    { 2.0084756000000000e-04, 4.4729748847100002e+00, 2.2091400527800001e+01 },
    { 1.9954280000000001e-04, 5.7450958036999999e-01, 1.2530172972200001e+01 },
    { 1.9926329000000001e-04, 1.3987819470800000e+00, 1.1291463420510000e+02 },
    { 1.9676762000000000e-04, 5.5343139833199997e+00, 7.4669723982700006e+01 },
    { 1.9653872999999999e-04, 2.2866091342099999e+00, 7.4893473151899997e+01 },
    { 1.8575632000000000e-04, 5.7021747578999999e+00, 3.3679617512900002e+01 },
    { 1.8565066999999999e-04, 6.2225019017000005e-01, 5.2690198039499997e+01 },
    { 1.6587869999999999e-04, 4.8692030916300002e+00, 1.0846121608020000e+02 },
    { 1.5171194000000000e-04, 2.8841545339899999e+00, 4.1101981054399999e+01 },
    { 1.3948521000000001e-04, 6.2754569415999999e+00, 2.2137585028530000e+02 },
    { 1.3593955000000001e-04, 2.5540782063299998e+00, 8.7311771539500000e+01 },
    { 1.2884351000000001e-04, 5.0873799947000000e+00, 1.4510977900969999e+02 },
    { 1.2394785999999999e-04, 6.2189287885000004e+00, 7.2333918012500007e+01 },
    { 1.2253318000000001e-04, 1.9452856525000001e-01, 3.6648562929500002e+01 },
    { 1.1997848000000000e-04, 9.4875212305000001e-01, 1.0593819301891999e+03 },
    { 1.1538642000000000e-04, 1.7724179453900000e+00, 7.7229279122099996e+01 },
    { 1.1245800000000000e-04, 6.1159701614599999e+00, 7.1600204829600003e+01 },
    { 1.0798349999999999e-04, 1.7003185707799999e+00, 7.7962992305000000e+01 },
    { 8.7384089999999995e-05, 4.9695680845199997e+00, 1.8621176006409999e+02 },
    { 7.7989739999999999e-05, 5.8241037258699997e+00, 3.4077089204480001e+02 },
    { 7.4873020000000001e-05, 1.1408470666999999e-01, 4.2659819087599999e+02 },
    { 7.0956079999999995e-05, 1.3038475004400001e+00, 2.9764192156090002e+02 },
    { 6.6692489999999998e-05, 5.0862658961199996e+00, 6.2251425595100002e+01 },
    { 6.2626019999999994e-05, 1.7138598378300001e+00, 1.5349535039770001e+02 },
    { 5.5053580000000003e-05, 3.3128210802500000e+00, 1.4000196957899999e+02 },
    { 5.3729269999999999e-05, 4.1249828286300003e+00, 7.5302863429100000e+01 },
    { 5.3542419999999999e-05, 3.6926397344700002e+00, 7.4260333705500003e+01 },
    { 5.0383529999999998e-05, 4.6866437691799998e+00, 1.8159247264699999e+01 },
    { 4.7513250000000001e-05, 4.9576239533699997e+00, 7.3818390720799997e+01 },
    { 4.5704700000000003e-05, 9.7536665750999996e-01, 1.8324281464750001e+02 },
    { 4.5159520000000002e-05, 2.8857630312000002e+00, 7.5744806413800006e+01 },
    { 4.4781230000000000e-05, 1.1183819147900000e+00, 6.6704843720000000e+01 },
    { 4.4486510000000002e-05, 2.9436142982000002e-01, 1.1439910691340000e+02 },
    { 4.2330750000000001e-05, 3.9491360818399999e+00, 2.6598929347749998e+02 },
    { 3.5782350000000001e-05, 1.1152890320800000e+00, 1.3703302416240001e+02 },
    { 3.5592760000000000e-05, 1.4762760750299999e+00, 5.9378908332000000e+00 },
    { 3.3123400000000002e-05, 5.2418923788000005e-01, 8.2858353414600003e+01 },
    { 3.2681169999999999e-05, 5.1827231333000001e-01, 2.2041264243879999e+02 },
    { 3.0047370000000001e-05, 5.1212213205100001e+00, 7.1135470007999997e+00 },
    { 2.8823920000000002e-05, 2.7613658389900002e+00, 3.7390799283650000e+02 },
    { 2.7227450000000000e-05, 6.0945617501599996e+00, 1.0697674337190000e+02 },
    { 2.6311379999999999e-05, 4.8315525310000003e-02, 3.0534616939270001e+02 },
    { 2.5977650000000000e-05, 2.2409539936000000e-01, 9.6872999095099999e+01 },
    { 2.5794540000000001e-05, 3.8478433033299999e+00, 2.7703499374140000e+02 },
    { 2.5607440000000001e-05, 4.4423622345000000e+00, 8.0198224538700003e+01 },
    { 2.5417159999999998e-05, 6.4495056482000002e-01, 3.2195144804599998e+01 },
    { 2.5280249999999999e-05, 8.9508396542000002e-01, 6.8843707734099993e+01 },
    { 2.2411289999999999e-05, 5.2237769750099998e+00, 3.1813937377000001e+00 },
    { 2.2320220000000002e-05, 6.3571664756000001e-01, 8.0719489400499995e+01 },
    { 2.2155160000000002e-05, 2.1861311287500000e+00, 2.5950888592310002e+02 },
    { 2.1829010000000001e-05, 1.2375547834500000e+00, 4.7928838891549998e+02 },
    { 2.1383910000000001e-05, 4.2089742992200003e+00, 7.4520966136400006e+01 },
    { 2.1264270000000000e-05, 3.6117146543600001e+00, 7.5042230998199997e+01 },
    { 2.0755910000000000e-05, 3.1558693346400002e+00, 1.3140394986990000e+02 },
    { 2.0641730000000001e-05, 3.6020860640999999e+00, 8.3503713448730002e+02 },
    { 1.9651450000000000e-05, 9.2075266320000002e-02, 2.0606927819500001e+01 },
    { 1.9338169999999999e-05, 2.0655758539500000e+00, 7.4829782677099999e+01 },
    { 1.9338140000000000e-05, 5.7549003386399997e+00, 7.4733414457500004e+01 },
    { 1.9276790000000001e-05, 1.2922802193200000e+00, 1.5912442469019999e+02 },
    { 1.9125820000000000e-05, 5.8309191869600001e+00, 1.4977853527000001e+01 },
    { 1.8923589999999999e-05, 4.3212862184700001e+00, 1.5401661525950001e+02 },
    { 1.8912130000000000e-05, 5.4994142424800003e+00, 3.0061086697750000e+02 },
    { 1.8735420000000002e-05, 4.2339186716899997e+00, 2.0618554843720000e+02 },
    { 1.8605910000000002e-05, 1.6753671171600000e+00, 4.2586453762700003e+01 },
    { 1.7970470000000001e-05, 1.7341746559400000e+00, 2.2731374111849999e+02 },
    { 1.7965580000000002e-05, 5.7327154333500001e+00, 7.4621539872900001e+01 },
    { 1.7925220000000000e-05, 2.0878916698399999e+00, 7.4941657261700001e+01 },
    { 1.7687629999999999e-05, 1.1925448162000001e+00, 2.1989137757699999e+02 },
    { 1.7445550000000001e-05, 8.2022450312999995e-01, 5.6622351302600002e+01 },
    { 1.6776590000000001e-05, 4.2525121333999999e-01, 8.0767548473000002e+00 },
    { 1.6728930000000000e-05, 1.2326461860000000e-02, 1.4362530630140000e+02 },
    { 1.5749050000000001e-05, 5.7229780026299997e+00, 3.9356875915200000e+01 },
    { 1.5584890000000000e-05, 6.1689107048900000e+00, 5.9803745040300001e+01 },
    { 1.5291800000000002e-05, 2.0520410482000000e+00, 1.9120769491019999e+02 },
    { 1.4672680000000000e-05, 2.1097557875800002e+00, 2.4476805547999998e+00 },
    { 1.4388689999999999e-05, 7.2633739717000001e-01, 5.2257741809380002e+02 },
    { 1.4271799999999999e-05, 3.8786175669000000e-01, 9.2940845831999994e+01 },
    { 1.4183000000000000e-05, 2.2971871201199998e+00, 2.3539049596580000e+02 },
    { 1.4089970000000000e-05, 3.6175190435600002e+00, 5.3680451209540001e+02 },
    { 1.3473030000000000e-05, 5.4676314022400003e+00, 2.8808069400530002e+02 },
    { 1.3343710000000000e-05, 2.1762174368900000e+00, 3.8012776796000003e+02 },
    { 1.2696859999999999e-05, 2.6633010403099999e+00, 1.4244965013379999e+02 },
    { 1.2630320000000000e-05, 5.6368959685300002e+00, 4.0450679034820001e+02 },
    { 1.2568670000000000e-05, 5.6168473642499999e+00, 6.7668051566499997e+01 },
    { 1.2415890000000000e-05, 1.8128296235700001e+00, 1.3458534360760001e+02 },
    { 1.2236480000000001e-05, 3.8437351463999998e+00, 1.8769623277240001e+02 },
    { 1.2215060000000000e-05, 5.2001245589399998e+00, 5.4174670747800000e+01 },
    { 1.2120250000000000e-05, 3.4157783265999999e+00, 2.1181462272970001e+02 },
    { 1.1922740000000000e-05, 5.5866199023299998e+00, 5.0402576179100002e+01 },
    { 1.1668980000000000e-05, 2.0295584854299999e+00, 1.1020632121940000e+02 },
    { 1.1642180000000000e-05, 2.0830263754099998e+00, 8.1895145568100006e+01 },
    { 1.1445649999999999e-05, 2.4214884523900002e+00, 2.9615744885260000e+02 },
    { 1.0936780000000000e-05, 3.9445181223299999e+00, 2.3056457082540001e+02 },
    { 1.0779160000000001e-05, 5.0141774002100004e+00, 1.8175834193919999e+02 },
    { 1.0778099999999999e-05, 1.0568590092000001e+00, 1.2895626931510000e+02 },
    { 1.0568480000000001e-05, 5.5344085416400004e+00, 1.4014645680499999e+01 },
    { 1.0409479999999999e-05, 2.7464416550099999e+00, 5.4166259714000002e+00 },
    { 1.0092520000000000e-05, 2.0232051703699998e+00, 7.4030839041899995e+01 },
    { 9.9539199999999994e-06, 5.7970344104100002e+00, 7.5532358092699994e+01 },
    { 9.8795600000000000e-06, 6.2025945394399997e+00, 2.0373786788240000e+02 },
    { 9.6409100000000002e-06, 2.6615143613200001e+00, 3.5424722652100002e+01 },
    { 9.1347700000000006e-06, 1.6939273380500000e+00, 1.4945132254999999e+02 },
    { 9.0978999999999997e-06, 4.7291954378199996e+00, 1.4967507171919999e+02 },
    { 8.9503400000000008e-06, 3.3867821982100001e+00, 2.3575873236100001e+01 },
    { 8.7020400000000001e-06, 5.3726329616399999e+00, 8.1001373690799994e+01 },
    { 8.6672799999999995e-06, 3.6563898067300000e+00, 1.2598732389849999e+02 },
    { 8.6204999999999999e-06, 5.0307700876399997e+00, 1.6209337010679999e+02 },
    { 8.5860299999999992e-06, 6.0609524060500002e+00, 6.8189316428300003e+01 },
    { 8.5833499999999997e-06, 1.5045480766699999e+00, 8.9759452094300002e+01 },
    { 8.1143699999999994e-06, 5.1763809920600004e+00, 2.0076892246579999e+02 },
    { 7.6681200000000000e-06, 2.4643137048999999e+00, 6.8561823443799994e+01 },
    { 7.4356100000000003e-06, 1.2266499859900000e+00, 7.6025553592000006e+02 },
    { 7.2797900000000003e-06, 1.6845096624800000e+00, 5.1205725331200000e+01 },
    { 7.0455700000000004e-06, 1.7866724838400001e+00, 8.1373880706300000e+01 },
    { 6.9954199999999997e-06, 2.9356690556900000e+00, 6.1780588578619995e+02 },
    { 6.9635100000000002e-06, 4.5928521045999998e-01, 1.1642609634290000e+02 },
    { 6.7453900000000000e-06, 3.0600550891000000e+00, 6.2197751234999998e+00 },
    { 6.4379800000000003e-06, 1.2717314605700001e+00, 8.8796244247800004e+01 },
    { 6.2050900000000003e-06, 3.3234007310500000e+00, 6.9364972595899999e+01 },
    { 5.9853799999999998e-06, 5.0473581905499998e+00, 2.4379022388199999e+01 },
    { 5.9769399999999997e-06, 5.2170163729599999e+00, 2.1840690486870000e+02 },
    { 5.8745100000000001e-06, 3.9080317425400000e+00, 1.5201087768939999e+02 },
    { 5.8079599999999999e-06, 6.1869274305099999e+00, 2.5802441321480001e+02 },
    { 5.7864299999999998e-06, 2.5559769926899998e+00, 2.8571808082200000e+01 },
    { 5.5188700000000002e-06, 1.1442038303299999e+00, 2.6099335863139999e+02 },
    { 5.4200099999999996e-06, 1.0488371803800001e+00, 1.9513984817330001e+02 },
    { 5.2819700000000000e-06, 2.2804392051800000e+00, 1.4638180339690001e+02 },
    { 5.2433700000000001e-06, 2.2960583551400000e+00, 3.5181659230870002e+02 },
    { 5.1415499999999999e-06, 5.4082001798999997e+00, 1.1588357962169999e+02 },
    { 5.0486500000000004e-06, 2.5358571834400001e+00, 1.5763995198190000e+02 },
    { 4.8482700000000003e-06, 3.3092998607000002e-01, 7.3409000443599993e+01 },
    { 4.7632499999999997e-06, 4.0817994982199997e+00, 4.1555249061209997e+02 },
    { 4.4904000000000002e-06, 1.0350814665000001e+00, 5.4302428721889999e+02 },
    { 4.4514000000000003e-06, 5.8418465046200003e+00, 1.2035824960600000e+02 },
    { 4.4404199999999996e-06, 4.7485443612199996e+00, 4.1644497775600001e+01 },
    { 4.4283499999999997e-06, 3.7540085588099998e+00, 3.2972519178089999e+02 },
    { 4.3849199999999999e-06, 1.2995310534100000e+00, 1.5890728952838001e+03 },
    { 4.3520799999999997e-06, 3.7651815406200000e+00, 3.7242352012819998e+02 },
    { 4.3354600000000000e-06, 4.1124835757899998e+00, 2.0936694217490000e+02 },
    { 4.2740799999999997e-06, 2.4703562473199998e+00, 1.4148644228730001e+02 },
    { 4.1916300000000002e-06, 1.9740567745900000e+00, 9.5388526386799995e+01 },
    { 4.1625299999999999e-06, 4.1757843814599998e+00, 4.9033408917940000e+02 },
    { 4.0849900000000000e-06, 3.4275615609200001e+00, 9.6320784650000002e-01 },
    { 4.0432200000000002e-06, 2.1300176432800000e+00, 9.9160620955499994e+01 },
    { 3.8737500000000000e-06, 5.9686983975799999e+00, 1.5274459087229999e+02 },
    { 3.8441299999999997e-06, 5.2154659717699996e+00, 1.1413847448250000e+02 },
    { 3.7473000000000001e-06, 5.6783188074400002e+00, 1.4414657116320001e+02 },
    { 3.7141300000000001e-06, 3.4715727494599999e+00, 7.3185251274400002e+01 },
    { 3.7124299999999998e-06, 1.0374881640400000e+00, 1.0549227066360000e+02 },
    { 3.6810500000000001e-06, 5.4579982410600003e+00, 2.5505546779820000e+02 },
    { 3.5160100000000002e-06, 6.2385403730100002e+00, 2.1478356814630001e+02 },
    { 3.4614800000000001e-06, 4.5174781013700001e+00, 4.5490936652729999e+02 },
    { 3.3731899999999999e-06, 5.1783313695500004e+00, 1.5052640498110000e+02 },
    { 3.2761799999999999e-06, 7.5068049755999999e-01, 4.6209790485100001e+01 },
    { 3.2665200000000001e-06, 2.6933343605999999e-01, 7.4223635415000002e+00 },
    { 3.2600199999999998e-06, 4.1891833205199998e+00, 2.2827694896500000e+02 },
    { 3.2311800000000000e-06, 1.8618583032800000e+00, 3.0710672096300001e+01 },
    { 3.2180800000000000e-06, 8.7890940660000000e-02, 1.8027386923090000e+02 },
    { 3.1835699999999999e-06, 2.2505788912999999e-01, 3.3137100791700000e+01 },
    { 3.1438599999999998e-06, 3.8246119697499998e+00, 4.5576651038700000e+01 },
    { 3.1241200000000000e-06, 3.0288422273800002e+00, 4.1948464387519999e+02 },
    { 3.1177199999999999e-06, 2.3823512841999999e+00, 1.4711551657979999e+02 },
    { 3.0858799999999998e-06, 5.6359985074400001e+00, 6.3989728631399998e+02 },
    { 3.0582900000000002e-06, 1.7524281399699999e+00, 6.2830758499914000e+03 },
    { 3.0021999999999998e-06, 2.1265191813400000e+00, 6.5922821389999999e+00 },
    { 2.9924700000000002e-06, 1.9979982584200000e+00, 9.8460033162189995e+02 },
    { 2.9585900000000001e-06, 2.6288167422200002e+00, 1.0309277421860000e+02 },
    { 2.7870400000000000e-06, 1.1727197392499999e+00, 2.9204947528600002e+01 },
    { 2.6686300000000001e-06, 5.7300504514200004e+00, 2.8311175651300001e+01 },
    { 2.5842300000000002e-06, 5.0169223293699998e+00, 1.2899465010146000e+03 },
    { 2.5240299999999999e-06, 2.9915395892000002e-01, 1.5008446199639999e+02 },
    { 2.5193100000000000e-06, 1.5005345249399999e+00, 1.9122455111200001e+01 },
    { 2.4131100000000001e-06, 4.6409215099500001e+00, 7.5632338265689998e+02 },
    { 2.4016000000000000e-06, 1.5747224780200000e+00, 3.4225536475310003e+02 },
    { 2.3880499999999999e-06, 5.5257706374100000e+00, 5.5406998748280000e+02 },
    { 2.3124300000000001e-06, 4.3496218720700002e+00, 4.6470422915999997e+01 },
    { 2.3111700000000000e-06, 5.9598761085999996e-01, 3.3928641933649999e+02 },
    { 2.2627099999999998e-06, 5.3571932933099999e+00, 1.2099138905240000e+02 },
    { 2.2262500000000001e-06, 1.6349924759800001e+00, 1.5578297225809999e+02 },
    { 2.2039900000000001e-06, 4.6944876871000002e-01, 4.4070926471000000e+01 },
    { 2.1952300000000002e-06, 5.0108692813699998e+00, 4.7694263193399998e+01 },
    { 2.1683900000000000e-06, 2.6452878815799998e+00, 3.1639186965660002e+02 },
    { 2.1505000000000000e-06, 2.1000001702700000e+00, 6.7359235025800004e+01 },
    { 2.1440199999999998e-06, 2.7510685864000002e+00, 1.8918070548070000e+02 },
    { 2.1314000000000000e-06, 4.9043768170600002e+00, 5.2126486179999998e-01 },
    { 2.1221000000000000e-06, 2.4824992209799999e+00, 1.3554855145409999e+02 },
    { 2.0442799999999999e-06, 1.2304355643900000e+00, 1.4214083359310001e+02 },
    { 2.0348699999999998e-06, 1.1169584360400000e+00, 1.4859998928810001e+02 },
    { 1.9805700000000002e-06, 4.7636112749999997e+00, 6.9673789136600007e+01 },
    { 1.9731100000000001e-06, 1.2176536583499999e+00, 2.5653994050649999e+02 },
    { 1.9353200000000000e-06, 6.2123493383900001e+00, 6.0766952886799999e+01 },
    { 1.8943000000000000e-06, 5.6304305552800002e+00, 2.9170403072770000e+02 },
    { 1.8762400000000000e-06, 6.1949133725000003e+00, 1.4904193227280001e+02 },
    { 1.8582000000000001e-06, 1.0476969569500001e+00, 7.6154196690999996e+01 },
    { 1.8580200000000000e-06, 5.5663995350400004e+00, 7.3887826689999997e+01 },
    { 1.8580200000000000e-06, 2.2545955545499998e+00, 7.5675370444600006e+01 },
    { 1.8560999999999999e-06, 4.1465864150499998e+00, 5.5659143456099997e+01 },
    { 1.8262500000000001e-06, 2.7259358853999999e-01, 2.1692243216040001e+02 },
    { 1.7747700000000000e-06, 3.1785665019899998e+00, 1.0213285546211000e+04 },
    { 1.7744999999999999e-06, 5.1037250369000002e+00, 6.8547393735269998e+02 },
    { 1.7573500000000000e-06, 3.1061772957299998e+00, 5.7255490749000003e+01 },
    { 1.7547800000000000e-06, 2.8327004039600001e+00, 3.3365734504400001e+02 },
    { 1.7477600000000000e-06, 6.4896858671000002e-01, 1.0335340664950000e+02 },
    { 1.7350699999999999e-06, 3.0714393775400000e+00, 7.9889407997999996e+01 },
    { 1.7305199999999999e-06, 2.7190138286800001e+00, 6.8154178408960001e+02 },
    { 1.6688499999999999e-06, 9.2704904427000001e-01, 3.0056280790500001e+01 },
    { 1.6525500000000000e-06, 6.5702015843999995e-01, 1.5497982310600000e+02 },
    { 1.6403199999999999e-06, 5.4243891316599999e+00, 4.6824268865160002e+02 },
    { 1.6006100000000000e-06, 5.0182358485799998e+00, 2.6450482076920002e+02 },
    { 1.5790499999999999e-06, 5.7073446692000003e+00, 8.2203962108799999e+01 },
    { 1.5693000000000000e-06, 4.4920065671199998e+00, 9.2307706385600000e+01 },
    { 1.5338499999999999e-06, 5.7302438499099999e+00, 3.4788443904560000e+02 },
    { 1.5260200000000000e-06, 3.6041776148600002e+00, 2.3390602325750001e+02 },
    { 1.5031099999999999e-06, 6.1685489140799996e+00, 1.7394221952280000e+02 },
    { 1.4791800000000000e-06, 1.8228257046900000e+00, 1.7526107818300002e+01 },
    { 1.4636200000000000e-06, 4.6349610647799997e+00, 3.3429048449039999e+02 },
    { 1.4603100000000001e-06, 6.2132013061300002e+00, 2.9467297614429998e+02 },
    { 1.4151199999999999e-06, 6.2831413797000000e+00, 1.3044074202339999e+02 },
    { 1.4116899999999999e-06, 3.1524754697900002e+00, 2.4118389957300000e+01 },
    { 1.3890000000000000e-06, 6.0925853937600003e+00, 2.8659622129700000e+02 },
    { 1.3778899999999999e-06, 5.2229527742100004e+00, 4.4868959140380002e+02 },
    { 1.3682300000000001e-06, 2.0640342937899998e+00, 1.6674774556399999e+01 },
    { 1.3511700000000001e-06, 2.7254825246799999e+00, 1.7165459766239999e+02 },
    { 1.3401000000000001e-06, 3.9707539656500002e+00, 1.5615547927360001e+02 },
    { 1.2904500000000001e-06, 6.1343222116699998e+00, 7.6478519596699996e+01 },
    { 1.2879100000000000e-06, 3.1089963029600001e+00, 1.5550108796780000e+02 },
    { 1.2851100000000001e-06, 1.8787317631500000e+00, 2.5494359321360000e+02 },
    { 1.2786900000000001e-06, 1.2045069057300000e+00, 6.1448276442999997e+01 },
    { 1.2700800000000000e-06, 5.8370683129299996e+00, 6.2885158605009997e+02 },
    { 1.2571400000000000e-06, 5.4304050145299998e+00, 9.8357471803400003e+01 },
    { 1.2108699999999999e-06, 1.7422963984000001e-01, 7.9447465013300004e+01 },
    { 1.1588400000000001e-06, 5.5575528351100001e+00, 6.3278373931320004e+02 },
    { 1.1566399999999999e-06, 2.6330686821599998e+00, 1.2151649024472999e+03 },
    { 1.1549400000000000e-06, 2.7938683862200002e+00, 6.0447256366190004e+02 },
    { 1.1543600000000001e-06, 3.6421584103100000e+00, 1.3333322124300000e+01 },
    { 1.1534900000000000e-06, 8.8899033683999995e-01, 6.9258748435350003e+02 },
    { 1.1232399999999999e-06, 5.1418380422999999e-01, 2.2879821382680001e+02 },
    { 1.1179100000000000e-06, 3.5990662219199998e+00, 1.9643719973000000e+01 },
    { 1.1129299999999999e-06, 1.0667309735000000e-01, 8.8114920691600005e+01 },
    { 1.0994600000000000e-06, 1.2354734735499999e+00, 1.0038446123289999e+02 },
    { 1.0951300000000001e-06, 1.3756505675399999e+00, 4.4725317776799997e+01 },
    { 1.0927200000000001e-06, 3.1877482482400001e+00, 2.7310284047829998e+02 },
    { 1.0865699999999999e-06, 2.3419995877999999e-01, 6.6917292041099998e+01 },
    { 1.0838200000000000e-06, 1.6379875600000000e-02, 1.1624747044077999e+03 },
    { 1.0804399999999999e-06, 4.7218816751999998e-01, 2.5357099508990001e+02 },
    { 1.0685799999999999e-06, 3.1397317531300000e+00, 8.1682697246999993e+01 },
    { 1.0615699999999999e-06, 4.2110814426700003e+00, 3.0209533968580001e+02 },
    { 1.0582900000000000e-06, 1.6799998037799999e+00, 3.7539246554480002e+02 },
    { 1.0471699999999999e-06, 9.1018749600000007e-03, 4.3371173787679999e+02 },
    { 1.0443300000000001e-06, 1.6654012735899999e+00, 5.4334729442200000e+01 },
    { 1.0215800000000000e-06, 1.4249135247800000e+00, 1.1187458459999999e-01 },
    { 1.0082100000000000e-06, 4.6906619258499997e+00, 2.1033015002139999e+02 },
    { 1.0041000000000001e-06, 1.9260676148200000e+00, 3.6286229257259998e+02 },
    { 9.9273000000000002e-07, 5.7987691799299999e+00, 3.2835259365719998e+02 },
    { 9.9119999999999993e-07, 3.8719985959999997e-01, 1.4930256470370000e+02 },
    { 9.9119999999999993e-07, 6.0733830647299998e+00, 1.4982382956550001e+02 },
    { 9.7686000000000008e-07, 3.0853813158099999e+00, 7.6787336137400004e+01 },
    { 9.7419999999999992e-07, 6.1781706889899999e+00, 1.0522683831884001e+03 },
    { 9.6332999999999993e-07, 2.4313803190500001e+00, 9.8899988524600005e+01 },
    { 9.5969000000000004e-07, 9.5437290533999997e-01, 2.5887574647669999e+02 },
    { 9.4924000000000000e-07, 1.2238225058500001e+00, 2.6747376618580000e+02 },
    { 9.3122000000000000e-07, 3.0538677662099998e+00, 4.3289029178299998e+01 },
    { 9.2951999999999997e-07, 3.3745937040600000e+00, 9.3904053678500006e+01 },
    { 9.1016000000000004e-07, 4.2072801067199999e+00, 8.2485846399099998e+01 },
    { 9.0851999999999999e-07, 6.1790968943199998e+00, 1.7265475387399999e+01 },
    { 8.9933999999999995e-07, 4.5272214776000004e+00, 1.4961138124440001e+02 },
    { 8.9933999999999995e-07, 1.9333614467300000e+00, 1.4951501302480000e+02 },
    { 8.9642999999999999e-07, 4.7004677741699998e+00, 2.1723124870109999e+02 },
    { 8.9563000000000003e-07, 2.6156567548399998e+00, 2.9519424100610001e+02 },
    { 8.8790999999999996e-07, 1.3681313901300001e+00, 2.9318850343600002e+02 },
    { 8.8764999999999998e-07, 4.6826856918399997e+00, 2.7087335373900000e+01 },
    { 8.8716000000000001e-07, 4.7760177218599997e+00, 2.7555052103309998e+02 },
    { 8.7996999999999996e-07, 7.0645172869999995e-02, 8.9484087952760001e+02 },
    { 8.7921000000000005e-07, 5.8975028123200000e+00, 7.4148459120900000e+01 },
    { 8.7921000000000005e-07, 1.9234922772700001e+00, 7.5414738013700003e+01 },
    { 8.6240999999999995e-07, 6.1014293878999997e+00, 7.6426129970000005e+01 },
    { 8.6043000000000004e-07, 4.5834685726300002e+00, 1.7368158709190001e+02 },
    { 8.3936000000000002e-07, 4.8562871784299997e+00, 3.7093904741990002e+02 },
    { 8.3725000000000001e-07, 3.1947345157000000e+00, 1.7516605980020000e+02 },
    { 8.3480000000000004e-07, 4.5495275637199999e+00, 1.4972325582900001e+02 },
    { 8.3480000000000004e-07, 1.9110553606100000e+00, 1.4940313844020000e+02 },
    { 8.3195000000000003e-07, 3.6448071624399998e+00, 5.1078094306999997e+00 },
    { 8.2020000000000000e-07, 1.1203029448199999e+00, 8.2645905093500005e+01 },
    { 8.1902000000000004e-07, 3.2347430581999999e+00, 4.8077286162380000e+02 },
    { 8.0963999999999999e-07, 3.4694152648999999e-01, 3.3132153907380001e+02 },
    { 8.0708000000000005e-07, 4.6913706133500002e+00, 1.6462326235300001e+01 },
    { 8.0075999999999998e-07, 2.0379082893699998e+00, 7.3137067164599998e+01 },
    { 7.9910999999999996e-07, 2.4172707400300002e+00, 3.3280601178209997e+02 },
    { 7.9594000000000002e-07, 6.1935705491700004e+00, 1.9269216761850001e+02 },
    { 7.8846999999999997e-07, 1.3220410009999999e-02, 2.6247783133970000e+02 },
    { 7.8477999999999996e-07, 1.1347886886500000e+00, 6.9010986796999996e+00 },
    { 7.8342000000000001e-07, 5.9029485965699999e+00, 5.8106824010899999e+01 },
    { 7.8311000000000003e-07, 5.4121557855700004e+00, 5.0663208609999998e+01 },
    { 7.8118999999999997e-07, 2.2809856204500001e+00, 4.7780391620720002e+02 },
    { 7.7986000000000000e-07, 3.2736851265000002e+00, 7.0115732121299999e+01 },
    { 7.7639999999999999e-07, 3.0632876520600001e+00, 7.1158261844899997e+01 },
    { 7.7423999999999998e-07, 1.4663858416900000e+00, 7.0046296152099998e+01 },
    { 7.7380000000000001e-07, 4.7649203259800004e+00, 3.1017209453309999e+02 },
    { 7.7268999999999997e-07, 1.5913379013100000e+00, 2.6023553790899999e+01 },
    { 7.6677000000000003e-07, 1.1269799911100000e+00, 7.3557758289899994e+01 },
    { 7.6487999999999995e-07, 7.3635514691000004e-01, 2.3767811782620001e+02 },
    { 7.6021000000000000e-07, 1.3897886504900001e+00, 2.1892816973050000e+02 },
    { 7.5512000000000000e-07, 3.6382470254000001e+00, 6.7077350735500005e+01 },
    { 7.5092999999999996e-07, 4.7390492202600001e+00, 7.8404935289700006e+01 },
    { 7.2806999999999998e-07, 5.9531396931199998e+00, 7.8643065262000000e+00 },
    { 7.2762000000000005e-07, 6.1779161472900004e+00, 9.5228467692400002e+01 },
    { 7.2396999999999999e-07, 4.3180691747299997e+00, 6.1288217748599997e+01 },
    { 7.2381000000000002e-07, 4.2522327780099998e+00, 1.5499118388799999e+01 },
    { 7.0498000000000000e-07, 4.3718323860800004e+00, 7.6377945860200001e+01 },
    { 7.0434000000000002e-07, 5.0082437387400001e+00, 1.1791056905120000e+02 },
    { 7.0212000000000004e-07, 4.4130471094799999e+00, 8.2005928096030004e+02 },
    { 7.0060000000000001e-07, 2.8549319870600001e+00, 1.6772244439930000e+02 },
    { 6.9882000000000001e-07, 3.9676860363299999e+00, 1.0664954771900000e+03 },
    { 6.9179000000000003e-07, 3.1195198082000002e+00, 1.8972322220190000e+02 },
    { 6.8354000000000004e-07, 8.0387037859999996e-02, 7.9516900982500005e+01 },
    { 6.7841000000000000e-07, 8.2180485113000001e-01, 6.0676018552230005e+02 },
    { 6.7827000000000005e-07, 3.4101040881600002e+00, 1.6041635110000001e+01 },
    { 6.6792999999999997e-07, 4.8109959354400003e+00, 1.0627416795630000e+02 },
    { 6.6499999999999999e-07, 1.5336556412600000e+00, 1.2518417474640000e+02 },
    { 6.5828999999999995e-07, 1.9301807105800000e+00, 7.3978449415200004e+01 },
    { 6.5828999999999995e-07, 5.8908143790100000e+00, 7.5584747719399999e+01 },
    { 6.4911999999999998e-07, 4.5398853341100001e+00, 4.7217484191469998e+02 },
    { 6.3900000000000004e-07, 2.1811231993300000e+00, 9.2047073954699997e+01 },
    { 6.3890000000000004e-07, 7.8467782617000004e-01, 1.1950691634410001e+02 },
    { 6.3430000000000001e-07, 3.4417922932499998e+00, 8.4608283475120004e+02 },
    { 6.0317000000000002e-07, 5.8593111708699999e+00, 1.2353964334370001e+02 },
    { 5.9454999999999999e-07, 6.3514957815999995e-01, 2.9021955801939998e+02 },
    { 5.9398999999999999e-07, 1.1364688487200001e+00, 4.8640193591629998e+02 },
    { 5.9365999999999998e-07, 4.2145520867000004e+00, 1.1736805233000000e+02 },
    { 5.8627999999999998e-07, 4.0805425491000002e-01, 4.0302231763990000e+02 },
    { 5.7520999999999997e-07, 1.5931763167499999e+00, 1.4297091499560000e+02 },
    { 5.7214000000000004e-07, 1.1384944487699999e+00, 3.0386169668439999e+02 },
    { 5.7136000000000000e-07, 2.0713566047400001e+00, 3.1652628083400000e+01 },
    { 5.6884999999999996e-07, 2.1091110682799998e+00, 2.5060345944400002e+01 },
    { 5.6604000000000000e-07, 4.4874557593700004e+00, 9.4917560896980001e+02 },
    { 5.6471999999999999e-07, 1.9658516995999999e+00, 7.7042478318000001e+00 },
    { 5.6128000000000001e-07, 4.1113880564900001e+00, 6.5380429706100003e+01 },
    { 5.5685000000000001e-07, 2.9430217088899999e+00, 4.3128970483899998e+01 },
    { 5.5443999999999998e-07, 1.3628677931299999e+00, 1.7878939652259999e+02 },
    { 5.5249000000000005e-07, 2.7729925019000001e-01, 7.4409091551800003e+01 },
    { 5.5249000000000005e-07, 1.2605105322200001e+00, 7.5154105582800000e+01 },
    { 5.4636000000000003e-07, 1.4733349923000001e+00, 7.1860837260500006e+01 },
    { 5.4468000000000003e-07, 1.2036036678699999e+00, 2.3687496867409999e+02 },
    { 5.4466000000000001e-07, 3.7489973558399998e+00, 7.4127207261500004e+01 },
    { 5.4466000000000001e-07, 4.0719977337500000e+00, 7.5435989873099999e+01 },
    { 5.4087999999999996e-07, 2.0256206565800001e+00, 1.3493380818700000e+01 },
    { 5.3990999999999997e-07, 4.5822633672500004e+00, 1.4881243760920000e+02 },
    { 5.3865000000000002e-07, 5.8378277306699999e+00, 9.0823233677299996e+01 },
    { 5.3837999999999998e-07, 9.3176659975999998e-01, 2.6992144674060000e+02 },
    { 5.3830000000000000e-07, 2.5406235772399999e+00, 8.3110498122419995e+02 },
    { 5.2985000000000000e-07, 2.2217182134600000e+00, 1.0400779795530001e+02 },
    { 5.2967000000000000e-07, 5.9699190437500000e+00, 3.6500115658670001e+02 },
    { 5.2404999999999998e-07, 1.6338002962100000e+00, 5.8739963457300000e+01 },
    { 5.2188999999999997e-07, 3.6542970803900001e+00, 4.8758044776399998e+01 },
    { 5.1865999999999996e-07, 6.4474704250000001e-02, 7.7702359874099997e+01 },
    { 5.1623999999999997e-07, 4.7221895320599998e+00, 1.8498791978669999e+02 },
    { 5.0910000000000003e-07, 9.1148399106000000e-01, 5.8319272331999997e+01 },
    { 5.0137000000000000e-07, 1.4850607088800001e+00, 2.5602862665600000e+01 },
    { 5.0050999999999997e-07, 1.5664219669999999e+00, 3.2983706636549999e+02 },
    { 4.9941999999999995e-07, 4.8987983520900000e+00, 1.2450285119020000e+02 },
    { 4.9928000000000001e-07, 4.5311946793100004e+00, 5.1887048887399999e+01 },
    { 4.9301000000000005e-07, 6.8505345474000001e-01, 7.6005438844699995e+01 },
    { 4.9024999999999999e-07, 6.2208293150899996e+00, 3.3406124266997999e+03 },
    { 4.8854999999999996e-07, 2.6836020284200002e+00, 1.0186893394120000e+02 },
    { 4.8818999999999998e-07, 4.9665135549000000e-01, 1.2247586176070000e+02 },
    { 4.8683000000000003e-07, 2.6121392864900002e+00, 4.8918103470799998e+01 },
    { 4.8357000000000003e-07, 5.4495272776999997e+00, 1.6289651925890001e+02 },
    { 4.7899000000000003e-07, 2.2691450677699998e+00, 9.1456373123700004e+01 },
    { 4.7746000000000004e-07, 1.9666882806399999e+00, 5.5771018040700000e+01 },
    { 4.7736000000000003e-07, 3.7099203826700000e+00, 8.4182767428500000e+01 },
    { 4.7698999999999999e-07, 8.3638374580999997e-01, 1.7787437278589999e+02 },
    { 4.7665000000000002e-07, 5.6322101798800004e+00, 7.3084677537900006e+01 },
    { 4.7590000000000002e-07, 4.1572576989299996e+00, 2.2423292111730001e+02 },
    { 4.7477000000000001e-07, 9.0974768311999998e-01, 2.2445667028650001e+02 },
    { 4.7398000000000001e-07, 9.0624036754000004e-01, 7.5632931829200004e+01 },
    { 4.7398000000000001e-07, 6.3156941486999996e-01, 7.3930265305399999e+01 },
    { 4.6580000000000000e-07, 5.6677923332300004e+00, 8.3652160719560004e+02 },
    { 4.6176999999999999e-07, 4.3991557808300001e+00, 1.4227094001599999e+01 },
    { 4.6007000000000002e-07, 2.9648548452100001e+00, 4.9721252622900003e+01 },
    { 4.5419000000000002e-07, 3.5478046947699999e+00, 3.9346109008430000e+02 },
    { 4.5107999999999999e-07, 1.9046922294199999e+00, 3.1492569389000000e+01 },
    { 4.4854999999999998e-07, 1.9275049757000000e+00, 2.7851946644970002e+02 },
    { 4.4633000000000000e-07, 5.1055999537599996e+00, 5.7516123179899999e+01 },
    { 4.4202999999999999e-07, 6.0748318140700004e+00, 7.0588812873300000e+01 },
    { 4.3663000000000001e-07, 4.7067905851800003e+00, 8.3355266177900000e+02 },
    { 4.3618999999999999e-07, 1.8982086814999999e-01, 1.9928444975750000e+02 },
    { 4.3296999999999999e-07, 3.0865042979299999e+00, 2.8414854074220000e+02 },
    { 4.3014000000000001e-07, 9.9655679001999997e-01, 1.9832124191099999e+02 },
    { 4.2747000000000000e-07, 4.4376859999100002e+00, 2.2633917249000000e+01 },
    { 4.2464000000000002e-07, 1.9184104903600001e+00, 1.5031395666000000e+02 },
    { 4.2243000000000000e-07, 4.8413806526699998e+00, 2.3242155054919999e+02 },
    { 4.1909999999999999e-07, 1.6558914244100000e+00, 1.4169889060840001e+02 },
    { 4.1871999999999998e-07, 2.2302180303700001e+00, 3.0683064210100002e+02 },
    { 4.1559999999999999e-07, 3.7704285552800001e+00, 1.6357784281510001e+02 },
    { 4.1400999999999999e-07, 1.5921923539600000e+00, 2.5863495096499999e+01 },
    { 4.1143000000000003e-07, 4.0282309073800002e+00, 1.3647280995819001e+03 },
    { 4.1073000000000003e-07, 5.0141179828600002e+00, 4.1162033734900001e+02 },
    { 4.1020000000000001e-07, 1.7461630994499999e+00, 7.8974384261300003e+01 },
    { 4.0362000000000001e-07, 3.0708757398500000e+00, 2.9226199388000001e+01 },
    { 4.0107999999999999e-07, 4.0263179235099997e+00, 1.6454105066160000e+02 },
    { 4.0056999999999999e-07, 1.4187991318000001e-01, 1.7448558675418999e+03 },
    { 3.9924999999999998e-07, 4.8014365338199996e+00, 2.0522234059069999e+02 },
    { 3.9720999999999999e-07, 3.7719755577899998e+00, 8.8274979385999998e+01 },
    { 3.9643000000000001e-07, 5.2079245616299996e+00, 6.9152524274800001e+01 },
    { 3.9625999999999998e-07, 3.1926563222400000e+00, 7.2493976706900000e+01 },
    { 3.9560999999999998e-07, 1.1535517515700000e+00, 5.8628088872699998e+01 },
    { 3.8450999999999999e-07, 2.9408516799400002e+00, 1.3310087089929999e+02 },
    { 3.8174999999999999e-07, 1.5175993702699999e+00, 1.1494162363460001e+02 },
    { 3.7780000000000001e-07, 4.5490819267999996e+00, 7.7069220427700003e+01 },
    { 3.7773999999999999e-07, 2.6586057821300000e+00, 3.9739324334740002e+02 },
    { 3.7772999999999998e-07, 3.8995132313999997e-01, 9.1243924802600006e+01 },
    { 3.7505000000000001e-07, 5.5917730076799996e+00, 9.4425318540299997e+01 },
    { 3.7356000000000002e-07, 3.0732057377999999e+00, 8.0314915210000004e-01 },
    { 3.7166999999999999e-07, 6.0638238567600000e+00, 2.2679247625670001e+02 },
    { 3.6880999999999998e-07, 4.3716177676200001e+00, 7.5372299398300001e+01 },
    { 3.6880999999999998e-07, 3.4493773219700001e+00, 7.4190897736300002e+01 },
    { 3.5171000000000002e-07, 3.5683548120999999e+00, 9.7415515816300001e+01 },
    { 3.5107999999999999e-07, 1.2148255191299999e+00, 7.6736908292080000e+02 },
    { 3.4980000000000002e-07, 4.1979668357300000e+00, 4.7353024151999996e+00 },
    { 3.4894000000000000e-07, 4.9266881424399998e+00, 1.4334342201109999e+02 },
    { 3.4854999999999998e-07, 5.3110960243600003e+00, 1.8409414790939999e+02 },
    { 3.4498000000000001e-07, 6.2282674024000002e+00, 1.8016199464630000e+02 },
    { 3.4238000000000003e-07, 4.4137658840800000e+00, 2.0446869125100001e+01 },
    { 3.4236000000000000e-07, 1.8746853264999999e-01, 5.4286545332400003e+01 },
    { 3.4210000000000003e-07, 7.7552752107000000e-01, 3.2686812094890001e+02 },
    { 3.4189000000000000e-07, 3.6011844789600000e+00, 3.3577495719870001e+02 },
    { 3.4130000000000002e-07, 2.4906965850000001e-02, 3.5685355082999997e+01 },
    { 3.4037000000000002e-07, 7.7937092279999998e-01, 1.4266209845489999e+02 },
    { 3.3848000000000000e-07, 1.3377240364000001e-01, 1.3288842257819999e+02 },
    { 3.3551999999999998e-07, 5.0395437700699999e+00, 2.2908009811709999e+02 },
    { 3.3489000000000000e-07, 1.7187378916300000e+00, 2.2752618943959999e+02 },
    { 3.3281999999999999e-07, 3.1734178097800001e+00, 7.5075952540000002e-01 },
    { 3.3052999999999998e-07, 2.7134750984000000e-01, 1.1403833038800001e+03 },
    { 3.3000000000000002e-07, 1.0818199637399999e+00, 2.4872381809010000e+02 },
    { 3.2921999999999998e-07, 1.5464032493600000e+00, 2.1543795945209999e+02 },
    { 3.2854000000000000e-07, 1.8018534098200001e+00, 1.0333583763982999e+03 },
    { 3.2772000000000003e-07, 5.8587934290100003e+00, 4.2538269652899999e+01 },
    { 3.2459999999999998e-07, 3.4022655357199998e+00, 1.4395096981491999e+03 },
    { 3.2427999999999999e-07, 2.2655770792099998e+00, 8.0410672859800002e+01 },
    { 3.2376999999999999e-07, 2.2729866524500002e+00, 8.5980197091000008e+00 },
    { 3.1995000000000000e-07, 5.0722189097600001e+00, 2.1626804085460000e+02 },
    { 3.1365000000000001e-07, 9.6344013620000002e-02, 3.2243328914400003e+01 },
    { 3.1148999999999999e-07, 2.4059950278200000e+00, 1.4065636088479999e+02 },
    { 3.1103000000000000e-07, 1.6543382104700000e+00, 5.6290742925000004e+00 },
    { 3.0833000000000001e-07, 5.3452599717600000e+00, 8.9068362497999995e+00 },
    { 3.0776000000000000e-07, 3.5661012394600000e+00, 6.6183578858199994e+01 },
    { 3.0713000000000002e-07, 2.5238867539699998e+00, 7.4527768239299996e+02 },
    { 3.0624999999999998e-07, 3.8543308891000000e-01, 9.0935108261899998e+01 },
    { 3.0461999999999998e-07, 5.3671855669099999e+00, 1.5467100656529999e+02 },
    { 3.0311999999999998e-07, 4.9165791815400004e+00, 6.1929035849449997e+02 },
    { 3.0162000000000002e-07, 5.9222119951799996e+00, 3.1540753498800001e+01 },
    { 2.9983000000000001e-07, 3.2803807820000003e-02, 9.0280716956099994e+01 },
    { 2.9782000000000000e-07, 2.6634046966499998e+00, 4.3240845068500001e+01 },
    { 2.9774000000000002e-07, 3.4714344727799999e+00, 1.6708930495289999e+02 },
    { 2.9686000000000003e-07, 6.1349871920699997e+00, 4.0164971951619998e+02 },
    { 2.9628000000000000e-07, 6.1112753072299997e+00, 9.5857677783099996e+02 },
    { 2.9506000000000000e-07, 5.1511362512000003e-01, 3.6797010200329998e+02 },
    { 2.9499000000000002e-07, 1.8995571682700001e+00, 5.6511568774670002e+02 },
    { 2.9395000000000001e-07, 5.7842927093999996e-01, 5.3065417294110000e+02 },
    { 2.9382000000000002e-07, 1.6680358431000000e-01, 1.3606981631590000e+02 },
    { 2.8766000000000002e-07, 5.9903932707700003e+00, 9.3792179093900003e+01 },
    { 2.8495000000000002e-07, 3.9829905193999999e+00, 6.1632141307790005e+02 },
    { 2.8332999999999998e-07, 3.7587921271200000e+00, 5.2872775724810003e+02 },
    { 2.7412000000000002e-07, 4.7440747111399997e+00, 2.2116340196420001e+02 },
    { 2.7346000000000001e-07, 1.7284757302900000e+00, 3.6648562929500002e+02 },
    { 2.7052000000000002e-07, 2.2398687474000001e+00, 6.7928683997400000e+01 },
    { 2.7035999999999999e-07, 3.0312278608300001e+00, 2.7720474820300002e+01 },
    { 2.7006000000000002e-07, 5.6633901182899997e+00, 1.0483787935780001e+02 },
    { 2.6911000000000001e-07, 2.5517648824700001e+00, 4.0599126305649997e+02 },
    { 2.6862999999999999e-07, 4.4651355735999996e+00, 2.5272794265500000e+01 },
    { 2.6777999999999998e-07, 1.2895086256699999e+00, 2.2876218603999998e+00 },
    { 2.6441000000000002e-07, 4.8888561114500000e+00, 1.5126011816400000e+02 },
    { 2.6389999999999997e-07, 2.5386843608300000e+00, 3.6945457471160000e+02 },
    { 2.6208999999999999e-07, 5.8281176851699996e+00, 4.0160025067299998e+01 },
    { 2.6189000000000002e-07, 1.3771407103100000e+00, 1.0933825679300000e+01 },
    { 2.6053999999999998e-07, 4.4230863014699997e+00, 1.1157574848499999e+01 },
    { 2.6014000000000000e-07, 6.2322938345500001e+00, 4.4720511869550000e+02 },
    { 2.6007999999999998e-07, 4.6400409731699996e+00, 4.0841348623499997e+01 },
    { 2.5835999999999999e-07, 5.1113519354700001e+00, 6.8301191012900006e+01 },
    { 2.5791000000000001e-07, 3.3016590300600002e+00, 6.5874762317500000e+01 },
    { 2.5701000000000000e-07, 2.1973163321700002e+00, 7.5893564260100007e+01 },
    { 2.5701000000000000e-07, 5.6236787574199996e+00, 7.3669632874499996e+01 },
    { 2.5534000000000001e-07, 5.2331995977399997e+00, 3.6233367223999999e+00 },
    { 2.5450000000000001e-07, 5.8589908136400002e+00, 1.0064509366380000e+02 },
    { 2.5236999999999998e-07, 2.5830816595399999e+00, 8.1262006121699997e+01 },
    { 2.5078000000000003e-07, 2.2006834910799999e+00, 9.2419580970200002e+01 },
    { 2.4843000000000001e-07, 5.6801315767900000e+00, 5.7143616164400001e+01 },
    { 2.4777000000000000e-07, 3.8255854555000002e+00, 1.3406407874580000e+02 },
    { 2.4737999999999998e-07, 5.5944837929400002e+00, 8.1634513137200003e+01 },
    { 2.4513999999999998e-07, 5.8328280941399999e+00, 4.1406801790380001e+02 },
    { 2.4493000000000001e-07, 2.2829425196000000e-01, 4.9178735901700001e+01 },
    { 2.4335999999999998e-07, 4.2322377978199999e+00, 9.6962247809489998e+02 },
    { 2.4236000000000001e-07, 2.2741024101899998e+00, 7.5877106321170004e+02 },
    { 2.4081000000000000e-07, 4.2791086995500001e+00, 1.7017012495410000e+02 },
    { 2.4027000000000002e-07, 4.7827583814199999e+00, 3.2824071907259997e+02 },
    { 2.3718000000000001e-07, 5.5366082215299999e+00, 4.5555399179299997e+01 },
    { 2.3404000000000000e-07, 2.0857830956600001e+00, 5.4153981451059997e+02 },
    { 2.3248000000000000e-07, 1.6995384962000000e+00, 1.6682867252200001e+02 },
    { 2.3236000000000000e-07, 4.2718622511600000e+00, 5.2147681318300002e+01 },
    { 2.2931000000000001e-07, 1.3752396849699999e+00, 1.1731986822019999e+02 },
    { 2.2917000000000001e-07, 1.6906464679700000e+00, 1.1802244363580000e+02 },
    { 2.2853000000000000e-07, 3.2972610804800002e+00, 1.0398654609590000e+02 },
    { 2.2852000000000001e-07, 2.8909672755900000e+00, 6.2491943278700001e+02 },
    { 2.2784000000000001e-07, 2.8900298013999998e-01, 6.4486657828800006e+01 },
    { 2.2571000000000000e-07, 3.7376711329900001e+00, 7.0636996983100005e+01 },
    { 2.2471000000000001e-07, 5.8316987607000004e-01, 2.4846318565920001e+02 },
    { 2.2284999999999999e-07, 2.3659595075900000e+00, 1.0791869935900000e+02 },
    { 2.2242000000000000e-07, 3.1041708817000002e-01, 8.5939173415799999e+01 },
    { 2.2092000000000000e-07, 5.5223911085399999e+00, 1.0749800823370001e+02 },
    { 2.1818000000000001e-07, 5.4366811561299997e+00, 3.4621573499999997e+01 },
    { 2.1659999999999999e-07, 1.7057315106800000e+00, 5.5137878594299998e+01 },
    { 2.1610000000000001e-07, 1.2475385383699999e+00, 3.1231936958100000e+01 },
    { 2.1567000000000000e-07, 1.1240932971599999e+00, 2.2530800354839999e+02 },
    { 2.1549999999999999e-07, 3.8029070385000002e-01, 1.4096517742550000e+02 },
    { 2.1477000000000001e-07, 4.0550364149200000e+00, 1.6700742689746000e+03 },
    { 2.1304999999999999e-07, 3.5642953835900002e+00, 3.7872403206900003e+01 },
    { 2.1297000000000001e-07, 1.3085946381600000e+00, 9.5276651802200007e+01 },
    { 2.1208000000000000e-07, 1.6516376265899999e+00, 1.8892007304980001e+02 },
    { 2.1107000000000000e-07, 1.6924011271899999e+00, 2.5208652238159999e+02 },
    { 2.1061999999999999e-07, 1.5793301169000000e+00, 3.3120966448920001e+02 },
    { 2.0956999999999999e-07, 4.0835283136799996e+00, 7.8926200151499998e+01 },
    { 2.0916000000000000e-07, 9.7601965891999998e-01, 1.0082492417399999e+01 },
    { 2.0821999999999999e-07, 5.2308704917100002e+00, 7.7489911552999999e+01 },
    { 2.0821999999999999e-07, 2.5901245978800000e+00, 7.2073285581600004e+01 },
    { 2.0493000000000000e-07, 2.6716492977000001e+00, 1.4796684984170000e+02 },
    { 2.0414000000000000e-07, 5.0063222764999997e+00, 2.2189711514710001e+02 },
    { 2.0289000000000001e-07, 1.4163730743800000e+00, 1.4393412284210001e+02 },
    { 2.0130999999999999e-07, 4.4305431811300000e+00, 1.5667674413540001e+02 },
    { 1.9766000000000001e-07, 9.5451045484999997e-01, 9.4011688612000004e+00 },
    { 1.9656999999999999e-07, 4.5310853849999999e-01, 4.0843894361129998e+02 },
    { 1.9634000000000000e-07, 2.4328455554500001e+00, 3.7619561469690001e+02 },
    { 1.9612000000000001e-07, 5.2101119391599999e+00, 5.3197858695499997e+02 },
    { 1.9467000000000001e-07, 4.7075581635000002e-01, 5.5013783421970004e+02 },
    { 1.9327999999999999e-07, 9.5373832885999998e-01, 3.6351668387839999e+02 },
    { 1.9238000000000001e-07, 3.4356732435700001e+00, 1.4489733068859999e+02 },
    { 1.9109000000000000e-07, 5.4636319782899996e+00, 6.0554504565700000e+01 },
    { 1.9012000000000001e-07, 6.2398737204000003e+00, 1.6560483224460000e+02 },
    { 1.8883999999999999e-07, 6.2729454751700002e+00, 2.2894549679899999e+01 },
    { 1.8855999999999999e-07, 2.8092510850400001e+00, 2.8956516671359998e+02 },
    { 1.8741000000000001e-07, 1.2459668914900000e+00, 8.5076539305799997e+01 },
    { 1.8734000000000001e-07, 2.2412458582000001e+00, 3.1245971639350000e+02 },
    { 1.8715000000000000e-07, 2.0007270114000000e+00, 1.0702492748170000e+02 },
    { 1.8685000000000001e-07, 4.7675686005099998e+00, 6.4699106149900004e+01 },
    { 1.8599000000000001e-07, 5.8314645745900000e+00, 1.4819059901090000e+02 },
    { 1.8468000000000001e-07, 2.2098213156400002e+00, 3.7916456011349999e+02 },
    { 1.8454000000000001e-07, 5.3251540380400000e+00, 3.8109097580650001e+02 },
    { 1.8449999999999999e-07, 3.2453052785700001e+00, 1.3533610313299999e+02 },
    { 1.8353999999999999e-07, 4.9232196610100001e+00, 1.4557162401699999e+01 },
    { 1.8297000000000000e-07, 3.5719058741900001e+00, 1.9010580526599998e+01 },
    { 1.8190999999999999e-07, 4.1743136080700003e+00, 3.8161224066829999e+02 },
    { 1.8122999999999999e-07, 2.2904578695000000e-01, 1.1807062774560001e+02 },
    { 1.7994999999999999e-07, 3.1585729271699998e+00, 3.7864329525170001e+02 },
    { 1.7882000000000001e-07, 1.6618816169799999e+00, 1.9066517818899999e+02 },
    { 1.7692000000000000e-07, 1.9614122466199999e+00, 4.7061123747000003e+01 },
    { 1.7559000000000000e-07, 3.5908279201999999e-01, 3.0305854753230000e+02 },
    { 1.7391000000000000e-07, 4.4313220258600001e+00, 7.2775860997199999e+01 },
    { 1.6980000000000000e-07, 5.0893929494999997e+00, 7.5239172954300003e+01 },
    { 1.6980000000000000e-07, 2.7316021400900001e+00, 7.4324024180300000e+01 },
    { 1.6918999999999999e-07, 3.4561287317999998e+00, 7.0363318461740005e+02 },
    { 1.6801000000000001e-07, 3.3038999664399999e+00, 1.9662432088160000e+02 },
    { 1.6646000000000000e-07, 2.8631541441200001e+00, 3.3683873878169999e+02 },
    { 1.6591999999999999e-07, 9.6055193272999995e-01, 2.8511174858869998e+02 },
    { 1.6467000000000000e-07, 1.6192505783800000e+00, 2.1187638603783998e+03 },
    { 1.6441000000000000e-07, 4.3202372109600002e+00, 8.3379618276399995e+01 },
    { 1.6220999999999999e-07, 4.3822184282599999e+00, 1.2991947716160001e+02 },
    { 1.6129000000000001e-07, 5.7091473038600000e+00, 7.3587651353180001e+02 },
    { 1.6082000000000001e-07, 2.1058603967899998e+00, 7.4299006053259995e+02 },
    { 1.6061999999999999e-07, 2.1525931068399999e+00, 7.3036493428100002e+01 },
    { 1.6061999999999999e-07, 2.5268093291599998e+00, 7.6526703706500001e+01 },
    { 1.6059000000000001e-07, 2.1282416698400000e+00, 6.1069233878540001e+02 },
    { 1.6009000000000000e-07, 2.3025248636600000e+00, 7.5611679969799994e+01 },
    { 1.6009000000000000e-07, 5.5184702259299998e+00, 7.3951517164799995e+01 },
    { 1.6005000000000001e-07, 8.1396049171999996e-01, 1.7665053250849999e+02 },
    { 1.5997000000000000e-07, 6.5650708975000005e-01, 1.1887377689770000e+02 },
    { 1.5867999999999999e-07, 5.0431030027499997e+00, 8.5319638175199998e+02 },
    { 1.5706999999999999e-07, 5.5008100514000002e-01, 1.0601353552540000e+02 },
    { 1.5650999999999999e-07, 1.1445188624600000e+00, 5.5258551477449998e+02 },
    { 1.5358000000000000e-07, 7.5466203124999998e-01, 6.3847772888000002e+01 },
    { 1.5351999999999999e-07, 8.0879086995000005e-01, 5.1546387109299997e+02 },
    { 1.5264000000000000e-07, 3.7021821758900000e+00, 1.0931254934210000e+02 },
    { 1.5032000000000001e-07, 1.8710429029600000e+00, 4.0313419222450000e+02 },
    { 1.4945999999999999e-07, 1.9336315940000001e-02, 7.3345309968799995e+01 },
    { 1.4945999999999999e-07, 4.6600661200599998e+00, 7.6217887165799993e+01 },
    { 1.4831000000000001e-07, 2.4776569830800002e+00, 1.1536231475990000e+02 },
    { 1.4777000000000001e-07, 4.0647430840299998e+00, 7.9777533413399993e+01 },
    { 1.4677000000000001e-07, 5.9913819815300000e+00, 2.0884567731310000e+02 },
    { 1.4623000000000001e-07, 3.6064939162999998e+00, 2.8096714700450002e+02 },
    { 1.4573000000000000e-07, 9.8064774203000005e-01, 7.6314255385400003e+01 },
    { 1.4573000000000000e-07, 3.6987546939699998e+00, 7.3248941749200000e+01 },
    { 1.4481000000000001e-07, 1.4509021338000001e+00, 1.0970942747017000e+03 },
    { 1.4396000000000000e-07, 4.2066463613599998e+00, 1.2184272231430000e+02 },
    { 1.4394999999999999e-07, 2.8245023793100001e+00, 2.3204904353370000e+02 },
    { 1.4380999999999999e-07, 5.2199270520200001e+00, 2.0788246946660001e+02 },
    { 1.4378000000000001e-07, 2.1557822413899999e+00, 1.9365537546499999e+02 },
    { 1.4249000000000000e-07, 3.8667056838899998e+00, 9.0981873305459999e+02 },
    { 1.4226999999999999e-07, 3.1833169718600001e+00, 8.4864090984699999e+01 },
    { 1.4158000000000000e-07, 8.9623255370999999e-01, 1.4755745956449999e+02 },
    { 1.4137000000000000e-07, 5.8479900612000002e-01, 1.7637982402900001e+01 },
    { 1.3958999999999999e-07, 6.0567260378999999e+00, 6.0112561581000001e+01 },
    { 1.3776000000000001e-07, 1.2404915101800000e+00, 6.3624023718799997e+01 },
    { 1.3647000000000000e-07, 6.1091705990999996e+00, 8.4215068148809996e+02 },
    { 1.3549000000000000e-07, 5.9586098099599996e+00, 1.7219711438360000e+02 },
    { 1.3526999999999999e-07, 3.1735503780000000e-02, 7.3457184553399998e+01 },
    { 1.3526999999999999e-07, 4.6476669322199999e+00, 7.6106012581200005e+01 },
    { 1.3475000000000001e-07, 6.2503226044800000e+00, 4.3829828244570001e+02 },
    { 1.3441000000000000e-07, 4.6474394235399998e+00, 5.3591074021810005e+02 },
    { 1.3418000000000000e-07, 3.6364496377800002e+00, 6.9785663721199995e+01 },
    { 1.3362000000000000e-07, 4.2257910707299997e+00, 8.8379517926369999e+02 },
    { 1.3241999999999999e-07, 4.6484785418400003e+00, 1.0538039607899999e+02 },
    { 1.3222999999999999e-07, 3.2046984985800000e+00, 9.7676148247200004e+01 },
    { 1.3152999999999999e-07, 2.7687897266800001e+00, 2.2486606056369999e+02 },
    { 1.3092000000000001e-07, 6.1650637504199999e+00, 3.2350541665740002e+02 },
    { 1.3052999999999999e-07, 4.1613006442699998e+00, 1.2008908110400000e+01 },
    { 1.3031000000000001e-07, 4.0010415471999999e-01, 7.5780785536520000e+02 },
    { 1.3012000000000000e-07, 2.6969385453100001e+00, 9.0588657979150003e+02 },
    { 1.2986000000000000e-07, 2.7081769601199999e+00, 1.9428851491140000e+02 },
    { 1.2912000000000001e-07, 1.8801497503900000e+00, 2.8873508531110002e+02 },
    { 1.2793000000000001e-07, 2.2409399837600001e+00, 6.2772690456900001e+01 },
    { 1.2777000000000001e-07, 1.9575909783000001e+00, 7.4920983565610004e+02 },
    { 1.2662999999999999e-07, 2.7800805196599998e+00, 4.0610313764109998e+02 },
    { 1.2599000000000001e-07, 2.0571633961800000e+00, 1.1387784205160000e+02 },
    { 1.2512000000000000e-07, 3.5473320087800002e+00, 1.1718758732690001e+03 },
    { 1.2475000000000000e-07, 3.4799302762600002e+00, 1.9577298761969999e+02 },
    { 1.2459000000000001e-07, 6.4812677657999995e-01, 3.2261164478009999e+02 },
    { 1.2330000000000000e-07, 4.4934835395999997e+00, 8.3688434817100003e+01 },
    { 1.2276000000000000e-07, 1.8773818616600000e+00, 3.3780194662820003e+02 },
    { 1.2242000000000001e-07, 3.7791888025199998e+00, 8.2792358748649997e+02 },
    { 1.2045000000000000e-07, 4.8343140652200001e+00, 4.4092178330400003e+01 },
    { 1.2011000000000001e-07, 5.8626932385700004e+00, 2.1085141488319999e+02 },
    { 1.1922000000000000e-07, 4.4576128115799998e+00, 5.0137978944330001e+02 },
    { 1.1794000000000000e-07, 1.7759977405800000e+00, 3.4530950774799997e+01 },
    { 1.1715000000000000e-07, 4.1521610434599996e+00, 5.4450875992720000e+02 },
    { 1.1682999999999999e-07, 5.0140212860400002e+00, 6.7925416222920001e+02 },
    { 1.1670000000000001e-07, 5.9933871044300000e+00, 3.0868762182479998e+02 },
    { 1.1622000000000000e-07, 4.2576635036999999e+00, 3.9096243484299997e+01 },
    { 1.1579000000000001e-07, 2.2479110488199998e+00, 6.5332245596299998e+01 },
    { 1.1550000000000000e-07, 5.9040234187799996e+00, 2.5642806592189999e+02 },
    { 1.1539000000000000e-07, 5.6262925906000005e-01, 7.4212149595699998e+01 },
    { 1.1539000000000000e-07, 9.7518052335000005e-01, 7.5351047538900005e+01 },
    { 1.1508999999999999e-07, 4.3530113865700004e+00, 3.5330106501699998e+02 },
    { 1.1480000000000000e-07, 1.7058154622099999e+00, 3.1867949151699997e+02 },
    { 1.1426000000000001e-07, 3.3278244962099999e+00, 1.2429040286910001e+02 },
    { 1.1319000000000000e-07, 1.0601562990100000e+00, 4.3549661609200001e+01 },
    { 1.1241000000000000e-07, 5.7598638023500000e+00, 7.5483890994859996e+02 },
    { 1.1184000000000000e-07, 4.5401632936600000e+00, 1.5036634628670001e+02 },
    { 1.1131000000000000e-07, 4.6085855471499997e+00, 7.6343692965770003e+02 },
    { 1.1100000000000000e-07, 6.2528741926900002e+00, 4.1703696332039999e+02 },
    { 1.1089999999999999e-07, 3.7354312732000001e+00, 5.5871591777200003e+01 },
    { 1.1079000000000000e-07, 3.5985425641100002e+00, 5.2575881183150000e+02 },
    { 1.0969000000000000e-07, 3.8003992087800000e+00, 6.8005731138130000e+02 },
    { 1.0893999999999999e-07, 2.6091992203099998e+00, 1.7730492381430000e+02 },
    { 1.0863000000000000e-07, 3.1422426468700002e+00, 2.2976142167329999e+02 },
    { 1.0858000000000000e-07, 3.6579772669400001e+00, 6.3996530734300002e+01 },
    { 1.0811000000000001e-07, 2.0345110952700001e+00, 8.9008692568900003e+01 },
    { 1.0733000000000000e-07, 1.7642668555200001e+00, 8.9450635553599994e+01 },
    { 1.0715000000000000e-07, 5.9563347398800000e+00, 1.1081399749656000e+03 },
    { 1.0704000000000000e-07, 1.1216767700700001e+00, 4.0016524680790002e+02 },
    { 1.0681000000000000e-07, 5.4142679476300000e+00, 1.8196374661092000e+03 },
    { 1.0655000000000000e-07, 5.2208912948700004e+00, 2.6396230404800002e+02 },
    { 1.0593000000000000e-07, 4.0575251687599998e+00, 7.1700778566099999e+01 },
    { 1.0576000000000000e-07, 5.0692275694099997e+00, 1.1848849416000000e+01 },
    { 1.0565999999999999e-07, 5.3106664555000005e-01, 5.3362311835770004e+02 },
    { 1.0522000000000000e-07, 2.5522992872399999e+00, 1.5429849954980000e+02 },
    { 1.0377000000000000e-07, 2.7167417993599998e+00, 3.0598797511699999e+01 },
    { 1.0353000000000000e-07, 1.4713330526999999e-01, 4.6675821594330000e+02 },
    { 1.0279000000000000e-07, 2.1427864463500002e+00, 5.4695644048199995e+02 },
    { 1.0265000000000000e-07, 3.4951471368300000e+00, 5.4391805909619995e+02 },
    { 1.0262000000000000e-07, 2.3756103949299998e+00, 3.6137781986430002e+02 },
    { 1.0253000000000001e-07, 1.6951480000000001e-02, 3.0689420236899998e+01 },
    { 1.0129000000000000e-07, 1.2763119692800000e+00, 4.4386366626339998e+02 },
    { 9.9820000000000003e-08, 3.2145746270600002e+00, 1.7542669223109999e+02 },
    { 9.9540000000000003e-08, 4.0697067708000001e-01, 7.9607523707699997e+01 },
    { 9.8910000000000004e-08, 6.2867977206000003e-01, 4.0461866493280002e+02 },
    { 9.8280000000000004e-08, 1.0934249514000001e+00, 6.9955673426900006e+01 },
    { 9.7810000000000000e-08, 5.5730831275600003e+00, 8.4230951538300005e+01 },
    { 9.5809999999999996e-08, 5.1983008233300003e+00, 2.4187090352019999e+02 },
    { 9.5710000000000001e-08, 1.0725282156000000e-01, 2.9201284726839998e+02 },
    { 9.5239999999999998e-08, 3.9187288663600000e+00, 2.1233588759150001e+02 },
    { 9.5020000000000000e-08, 1.5567975492899999e+00, 3.5033211960040001e+02 },
    { 9.5010000000000002e-08, 5.5519670136099997e+00, 1.5646429581429999e+02 },
    { 9.4810000000000000e-08, 4.6720938584300002e+00, 3.6909195360399998e+01 },
    { 9.4199999999999996e-08, 3.1850409006499998e+00, 1.1265400177420000e+02 },
    { 9.4100000000000002e-08, 1.6029550199999999e-02, 2.9823262239190001e+02 },
    { 9.4059999999999997e-08, 1.0118231728600000e+00, 6.9312582969199994e+01 },
    { 9.4059999999999997e-08, 5.2598660954999998e-01, 8.0250614165399995e+01 },
    { 9.3470000000000003e-08, 1.9167507468700000e+00, 1.5698556067609999e+02 },
    { 9.3380000000000006e-08, 2.7631188879400002e+00, 2.1744369702220001e+02 },
    { 9.3299999999999995e-08, 1.5622739408299999e+00, 5.2802072624099999e+01 },
    { 9.2729999999999997e-08, 6.2387008250199996e+00, 2.6302034806090001e+02 },
    { 9.2360000000000001e-08, 4.1241891064400003e+00, 4.2266603761290003e+02 },
    { 9.2259999999999994e-08, 9.9071863539000005e-01, 7.5696622304000002e+01 },
    { 9.2259999999999994e-08, 5.4709114701999995e-01, 7.3866574830600001e+01 },
    { 9.2039999999999996e-08, 3.6024514480300001e+00, 2.2338158785540000e+02 },
    { 9.2019999999999999e-08, 1.6479917748199999e+00, 5.5555446019110002e+02 },
    { 9.1839999999999994e-08, 3.5669056703000002e+00, 1.0080515235820000e+02 },
    { 9.0559999999999998e-08, 1.1525943257000000e-01, 2.3093707784090000e+02 },
    { 9.0320000000000004e-08, 9.5177465041999998e-01, 8.9338760969000006e+01 },
    { 9.0060000000000000e-08, 5.9784568319999998e-01, 6.7049608382570000e+02 },
    { 8.9050000000000007e-08, 3.4113432905800001e+00, 5.9963803734700001e+01 },
    { 8.8940000000000001e-08, 4.5376471787700003e+00, 2.1011770170029999e+02 },
    { 8.8829999999999995e-08, 1.6049938154400001e+00, 7.4958234267160003e+02 },
    { 8.7820000000000002e-08, 5.2152219851400003e+00, 4.8884961647109998e+02 },
    { 8.7139999999999998e-08, 6.1138009126600004e+00, 6.8398946464439996e+02 },
    { 8.6430000000000000e-08, 1.1980340030200001e+00, 8.6630447983300002e+01 },
    { 8.6169999999999997e-08, 4.1681026443900002e+00, 8.5566666400299994e+01 },
    { 8.6169999999999997e-08, 4.7159821953099996e+00, 1.5045696901189999e+02 },
    { 8.6070000000000002e-08, 6.2678880856400001e+00, 6.2932749151300001e+01 },
    { 8.5119999999999997e-08, 3.0349245616200000e+00, 5.3287235883230005e+02 },
    { 8.4979999999999997e-08, 5.4398645943200004e+00, 2.0495053234899999e+01 },
    { 8.4950000000000003e-08, 5.6299607071000004e+00, 2.4994765836750000e+02 },
    { 8.4859999999999994e-08, 4.7331068207599998e+00, 7.7862418568500004e+01 },
    { 8.4730000000000005e-08, 4.6898463469799996e+00, 7.1009503998599996e+01 },
    { 8.4730000000000005e-08, 3.1311487426100002e+00, 7.8553693136000007e+01 },
    { 8.4289999999999996e-08, 6.9056403770999997e-01, 1.5318653385700000e+02 },
    { 8.1930000000000006e-08, 5.3774878944999998e+00, 5.9282480178500002e+01 },
    { 8.1890000000000001e-08, 2.8916198889300002e+00, 2.5345912050530001e+02 },
    { 8.1660000000000005e-08, 2.0552979964400002e+00, 3.2538364824060000e+02 },
    { 8.1250000000000003e-08, 4.1072494691200001e+00, 1.1562294719080001e+02 },
    { 8.1219999999999995e-08, 5.9311768597599999e+00, 2.7406604832480002e+02 },
    { 8.0700000000000001e-08, 5.1023981336300004e+00, 3.7169827791300001e+01 },
    { 8.0550000000000003e-08, 4.5154180414000002e+00, 5.9591296719200002e+01 },
    { 8.0519999999999996e-08, 6.1733678989199996e+00, 6.7442823708879996e+02 },
    { 8.0439999999999998e-08, 2.4024596882600000e+00, 7.7117404537499993e+01 },
    { 8.0439999999999998e-08, 5.4185354013299998e+00, 7.2445792597099995e+01 },
    { 8.0319999999999994e-08, 2.4092589023500000e+00, 2.2382353084010001e+02 },
    { 7.9759999999999994e-08, 5.6290575698899996e+00, 1.4788665740644001e+03 },
    { 7.9539999999999996e-08, 5.6916978262100004e+00, 1.0223126761344000e+03 },
    { 7.9529999999999998e-08, 5.6330144278900001e+00, 8.6790506677699994e+01 },
    { 7.9520000000000000e-08, 1.8680435468600001e+00, 3.4200882374700001e+01 },
    { 7.9420000000000006e-08, 3.6397816018800002e+00, 1.0898248094200000e+02 },
    { 7.8930000000000006e-08, 4.0855601037199998e+00, 9.3691605357399993e+01 },
    { 7.8629999999999997e-08, 4.4112086970000002e+00, 4.3764389113990001e+02 },
    { 7.7709999999999999e-08, 4.2902703529100004e+00, 1.9907200143640000e+02 },
    { 7.7529999999999993e-08, 3.8925189480900002e+00, 1.9280404220310001e+02 },
    { 7.7210000000000001e-08, 2.8136571781600002e+00, 1.8079513409270001e+02 },
    { 7.6759999999999994e-08, 1.2205330938000000e-01, 6.0224436165599997e+01 },
    { 7.6669999999999998e-08, 2.2656187417400000e+00, 6.8865533109039995e+02 },
    { 7.6399999999999996e-08, 1.1514597803900000e+00, 2.1979525943199999e+01 },
    { 7.6160000000000002e-08, 1.9823929024000000e-01, 3.4373983746139999e+02 },
    { 7.6119999999999996e-08, 6.8976665906000001e-01, 1.3948070471720001e+02 },
    { 7.6080000000000004e-08, 7.6491107529000002e-01, 4.0907208305770001e+02 },
    { 7.5940000000000004e-08, 2.8040362269200001e+00, 2.1395348674380000e+02 },
    { 7.5779999999999994e-08, 1.0439834460699999e+00, 2.1426230328450001e+02 },
    { 7.5759999999999998e-08, 1.6397699839800000e+00, 1.4866942525729999e+02 },
    { 7.5429999999999995e-08, 4.7119319292500004e+00, 7.7016830800999998e+01 },
    { 7.5429999999999995e-08, 6.2506558139299999e+00, 7.2546366333600005e+01 },
    { 7.5429999999999995e-08, 3.7407136048899998e+00, 7.4047885384400004e+01 },
    { 7.5429999999999995e-08, 9.3868883111000001e-01, 7.5515311750199999e+01 },
    { 7.5149999999999995e-08, 3.9935293345599998e+00, 5.1471311156759998e+02 },
    { 7.4900000000000002e-08, 8.3410715646000000e-01, 1.2914309737229000e+03 },
    { 7.4880000000000006e-08, 3.6669168528100000e+00, 8.9971900415400000e+01 },
    { 7.4429999999999999e-08, 1.8299996128100000e+00, 3.9654191008549998e+02 },
    { 7.3430000000000003e-08, 4.0893171081000002e-01, 1.6921656695024001e+03 },
    { 7.3350000000000005e-08, 1.4196381197300001e+00, 5.2650957135689998e+02 },
    { 7.3329999999999996e-08, 4.1502927929700002e+00, 4.9744763618019999e+02 },
    { 7.3140000000000005e-08, 6.0202911516400004e+00, 1.8865944061889999e+02 },
    { 7.2860000000000005e-08, 5.6156547677599997e+00, 2.4161027108930000e+02 },
    { 7.2069999999999996e-08, 3.8636687466700002e+00, 2.6721313375490001e+02 },
    { 7.1760000000000002e-08, 9.1191120786000002e-01, 4.4572064598719999e+02 },
    { 7.1540000000000004e-08, 3.8367300876999999e-01, 2.5976951835400001e+02 },
    { 7.1530000000000006e-08, 9.7472079949000001e-01, 4.4985950207700000e+01 },
    { 7.1130000000000002e-08, 1.8122516383800000e+00, 7.3194436026870005e+02 },
    { 7.1070000000000000e-08, 4.4125004552000000e+00, 3.3025226207099998e+01 },
    { 7.1070000000000000e-08, 5.6868719363100002e+00, 6.4568800576977001e+03 },
    { 7.0860000000000001e-08, 2.5669884015000002e+00, 9.5979227217800002e+01 },
    { 7.0840000000000004e-08, 3.4405941990000000e+00, 1.0252332524700000e+02 },
    { 7.0309999999999999e-08, 4.1133975950500004e+00, 4.1756372360199997e+01 },
    { 7.0059999999999993e-08, 5.4836901329899996e+00, 2.3741748539529999e+02 },
    { 6.9959999999999999e-08, 8.9646725407000005e-01, 1.4445538770389999e+02 },
    { 6.9790000000000005e-08, 1.4652950609299999e+00, 1.9779997704920001e+02 },
    { 6.9769999999999995e-08, 8.0654770180000002e-01, 1.2655674786264001e+03 },
    { 6.9699999999999995e-08, 3.5318963199100000e+00, 1.3055261660799999e+02 },
    { 6.9320000000000001e-08, 1.1487768495699999e+00, 4.8322054217860000e+02 },
    { 6.8900000000000002e-08, 1.2812285418700000e+00, 1.6412035953629999e+02 },
    { 6.8649999999999996e-08, 4.8535423265300004e+00, 4.0758761034939999e+02 },
    { 6.8009999999999998e-08, 1.0141234771800001e+00, 3.0141401612959999e+02 },
    { 6.7799999999999998e-08, 5.6273381686799997e+00, 6.4701083331480004e+02 },
    { 6.7350000000000004e-08, 1.5758016744400001e+00, 5.9949951559861001e+03 },
    { 6.7169999999999999e-08, 1.0184664713000000e-01, 4.7494684537519998e+02 },
    { 6.7130000000000006e-08, 2.5330055486399998e+00, 3.5893013930950002e+02 },
    { 6.7119999999999995e-08, 2.3412186254299998e+00, 3.0412232911529998e+02 },
    { 6.6699999999999995e-08, 1.3555146868800001e+00, 4.3681380973739999e+02 },
    { 6.6629999999999995e-08, 1.3032167114500000e+00, 8.9335640681929999e+02 },
    { 6.6590000000000003e-08, 6.2431567185000003e-01, 4.9508804301799998e+01 },
    { 6.6520000000000003e-08, 1.8766299420200001e+00, 2.3619364511789999e+02 },
    { 6.6450000000000003e-08, 2.6672538683100000e+00, 6.9110301164520001e+02 },
    { 6.5690000000000001e-08, 1.4528371228900001e+00, 3.4936891175390002e+02 },
    { 6.5419999999999999e-08, 5.6282908088100001e+00, 2.1149444540699999e+01 },
    { 6.5390000000000005e-08, 3.9509951720399998e+00, 1.2911632800949999e+02 },
    { 6.5009999999999998e-08, 4.7551640281000002e-01, 3.8606565879319999e+02 },
    { 6.4939999999999998e-08, 3.0008831722000000e+00, 9.9252048522056994e+03 },
    { 6.4679999999999994e-08, 1.3714397194000001e+00, 4.4613443192200002e+01 },
    { 6.4109999999999996e-08, 6.2596855687400002e+00, 1.5819593482830001e+03 },
    { 6.4039999999999996e-08, 5.3718371712600002e+00, 1.3103933701397000e+03 },
    { 6.3689999999999997e-08, 3.2167823470000001e+00, 4.5719698838770000e+02 },
    { 6.3549999999999997e-08, 2.9187554444399999e+00, 4.2511371816769997e+02 },
    { 6.3539999999999999e-08, 5.3630807552600004e+00, 3.6997583957339998e+02 },
    { 6.3339999999999997e-08, 3.2910573991500001e+00, 1.3757737998458001e+03 },
    { 6.3339999999999997e-08, 4.1446504672000001e-01, 4.9181856188770001e+02 },
    { 6.3329999999999999e-08, 3.0433904576200002e+00, 1.5185081899500000e+02 },
    { 6.3290000000000006e-08, 5.7345052256900004e+00, 9.5628915597059995e+02 },
    { 6.3259999999999999e-08, 4.5648631955900001e+00, 1.7813500521680001e+02 },
    { 6.3069999999999995e-08, 3.3636344359499999e+00, 3.0089275126780001e+02 },
    { 6.2880000000000005e-08, 3.7545109960600000e+00, 3.1490739694830000e+02 },
    { 6.2820000000000003e-08, 2.7032610268599999e+00, 7.1144930703379998e+02 },
    { 6.2750000000000003e-08, 6.2587160919800002e+00, 9.6761124510499997e+01 },
    { 6.2569999999999997e-08, 3.4347219349999998e+00, 1.1939504175950000e+02 },
    { 6.2330000000000003e-08, 1.6806357014200000e+00, 1.5952926704073000e+03 },
    { 6.2260000000000003e-08, 1.4201211173799999e+00, 1.4817794832600001e+01 },
    { 6.2190000000000003e-08, 5.0564070107100001e+00, 9.5931043107999997e+01 },
    { 6.1739999999999996e-08, 6.2019529143899996e+00, 1.2884620283063000e+03 },
    { 6.1719999999999999e-08, 1.6408254409300000e+00, 3.5840887444769999e+02 },
    { 6.1109999999999996e-08, 5.7632522173000000e-01, 3.4639996633729999e+02 },
    { 6.0800000000000002e-08, 5.3800553499800001e+00, 3.6908206769610001e+02 },
    { 6.0300000000000004e-08, 4.6008387740999996e+00, 3.1910492295652002e+03 },
    { 6.0119999999999998e-08, 3.7632767344100002e+00, 4.7631944349890000e+02 },
    { 6.0119999999999998e-08, 6.0163758842600004e+00, 6.9503516490829998e+02 },
    { 5.9800000000000006e-08, 2.2625933470700001e+00, 3.6203221117010003e+02 },
    { 5.9629999999999999e-08, 3.7821005567500001e+00, 3.7611770776000000e+01 },
    { 5.9219999999999997e-08, 2.2262617967499998e+00, 8.9632535223590003e+02 },
    { 5.9190000000000003e-08, 4.4603275171199996e+00, 2.6087903141574199e+04 },
    { 5.9149999999999997e-08, 5.3892555610599997e+00, 8.7051139108599997e+01 },
    { 5.9149999999999997e-08, 2.4317395285400001e+00, 6.2512058025999998e+01 },
    { 5.9109999999999998e-08, 4.3601071218999996e+00, 5.6188610989920005e+02 },
    { 5.8729999999999997e-08, 1.5454055795299999e+00, 3.4793262315539999e+02 },
    { 5.8710000000000001e-08, 4.9253665044300003e+00, 5.2427433912319998e+02 },
    { 5.8680000000000000e-08, 4.2321066973899999e+00, 1.0656017053127000e+03 },
    { 5.8010000000000001e-08, 5.6094254814499998e+00, 4.5342489381899998e+02 },
    { 5.7989999999999998e-08, 3.3841742707200000e+00, 6.9407195706180005e+02 },
    { 5.7930000000000003e-08, 4.7539516052000002e+00, 1.8944190646765001e+03 },
    { 5.7889999999999998e-08, 6.2823057740300001e+00, 1.3192521473170001e+02 },
    { 5.7590000000000002e-08, 1.1226046429000000e-01, 2.1574677599280000e+02 },
    { 5.7520000000000002e-08, 4.2532661287600000e+00, 8.3800607990389994e+02 },
    { 5.7450000000000002e-08, 3.1956712544400001e+00, 2.0202226296720091e+04 },
    { 5.7429999999999999e-08, 5.1319573992500001e+00, 2.4643619622969999e+02 },
    { 5.7000000000000001e-08, 4.4202137312999996e+00, 7.7258876693610000e+02 },
    { 5.6970000000000000e-08, 1.9239889845100000e+00, 8.0901358069640003e+02 },
    { 5.6430000000000003e-08, 2.9744517562700001e+00, 1.1653797092750000e+02 },
    { 5.6430000000000003e-08, 2.4450934792000001e+00, 3.1516802937919999e+02 },
    { 5.6230000000000001e-08, 2.5708417693199999e+00, 4.2065188900899997e+01 },
    { 5.6050000000000002e-08, 2.7693491375399999e+00, 2.5938339944439598e+04 },
    { 5.5980000000000002e-08, 4.4202077243000000e+00, 1.8823874949360001e+02 },
    { 5.5910000000000002e-08, 3.3394675146299999e+00, 1.2356117715723999e+03 },
    { 5.5770000000000002e-08, 5.6756125429099997e+00, 1.6868565224579999e+02 },
    { 5.5379999999999997e-08, 9.7968779981999998e-01, 2.9674814968359999e+02 },
    { 5.5369999999999999e-08, 4.6944662191899997e+00, 4.7535623565240002e+02 },
    { 5.5180000000000002e-08, 1.7280826547200001e+00, 8.5715424246599994e+01 },
    { 5.4959999999999997e-08, 2.4009472912600001e+00, 1.5896436599579999e+02 },
    { 5.4919999999999998e-08, 3.6241554273799998e+00, 2.9370976829780000e+02 },
    { 5.4889999999999997e-08, 3.6661623659200000e+00, 1.3622987501029999e+02 },
    { 5.4739999999999999e-08, 2.8826485546499998e+00, 1.1885252503830000e+02 },
    { 5.4650000000000003e-08, 5.7547767293299996e+00, 4.9651816653700003e+01 },
    { 5.4490000000000000e-08, 2.5298896960300001e+00, 2.9747464249800000e+01 },
    { 5.4399999999999997e-08, 1.5486146504400000e+00, 4.0153784493159998e+02 },
    { 5.4210000000000000e-08, 5.8125835318999997e-01, 2.2934073054800001e+02 },
    { 5.4070000000000000e-08, 3.6895663890199999e+00, 1.4791866573190001e+02 },
    { 5.3960000000000001e-08, 1.5555739464800000e+00, 8.1777165909990003e+02 },
    { 5.3879999999999997e-08, 3.4721528682999998e+00, 6.3666770846650002e+02 },
    { 5.3869999999999999e-08, 3.6739665703000002e-01, 1.9391600789590001e+02 },
    { 5.3400000000000002e-08, 3.7484442477600002e+00, 3.3217287233569999e+02 },
    { 5.2679999999999999e-08, 3.2357532919400001e+00, 1.6438099196720000e+02 },
    { 5.2539999999999999e-08, 4.8980449971000004e+00, 3.3514181775230003e+02 },
    { 5.2509999999999998e-08, 7.7102401214000005e-01, 3.0560680182359999e+02 },
    { 5.2509999999999998e-08, 3.1712854845999998e+00, 4.6972716135989998e+02 },
    { 5.2469999999999999e-08, 3.9927330902299998e+00, 8.9599393399899995e+01 },
    { 5.2469999999999999e-08, 1.7023750178699999e+00, 2.1997652340691998e+03 },
    { 5.2409999999999997e-08, 4.9980767581699999e+00, 4.4423617327890003e+02 },
    { 5.2290000000000000e-08, 2.3745314893899998e+00, 1.5142912967165000e+03 },
    { 5.2199999999999998e-08, 2.7384932443899999e+00, 6.6396027179300006e+01 },
    { 5.2189999999999999e-08, 4.2911385067500003e+00, 2.7741726679700001e+01 },
    { 5.2120000000000000e-08, 5.8170408522299999e+00, 5.6118353448360006e+02 },
    { 5.1970000000000001e-08, 3.9261818560499999e+00, 2.2345102382459999e+02 },
    { 5.1510000000000003e-08, 5.0802088019999997e-02, 9.0195442652839995e+02 },
    { 5.1389999999999999e-08, 5.2245840189999997e-02, 1.7001006625970001e+02 },
    { 5.0860000000000000e-08, 3.0278132789000001e+00, 1.0318739036899999e+03 },
    { 5.0769999999999997e-08, 1.7746278607599999e+00, 7.8623090560109995e+02 },
    { 4.9920000000000000e-08, 1.9137912462700000e+00, 5.3653405886000002e+01 },
    { 4.9789999999999998e-08, 5.7255688234999997e+00, 6.8302625679790003e+02 },
    { 4.9619999999999997e-08, 3.2517837894900001e+00, 4.0250647792499997e+01 },
    { 4.9560000000000002e-08, 4.1447911400999997e+00, 4.5017406411209998e+02 },
    { 4.9539999999999999e-08, 4.5277916346199998e+00, 1.6611084081600001e+01 },
    { 4.9369999999999998e-08, 3.5047308596400000e+00, 7.7841478318470001e+02 },
    { 4.8660000000000000e-08, 4.1500195773700002e+00, 2.7161836777000002e+02 },
    { 4.8620000000000001e-08, 3.9844320684799999e+00, 1.5604360468900001e+02 },
    { 4.8410000000000001e-08, 1.9555277391100001e+00, 8.2287946853059998e+02 },
    { 4.8319999999999999e-08, 1.8383978780100001e+00, 4.3978275515399997e+02 },
    { 4.8310000000000000e-08, 3.3183210498800002e+00, 6.2173803904930003e+02 },
    { 4.8300000000000002e-08, 5.6892351312200002e+00, 8.1857480825200003e+02 },
    { 4.8280000000000000e-08, 6.1933574564000005e-01, 2.6205714021440002e+02 },
    { 4.8170000000000001e-08, 2.8846654266899998e+00, 3.0414860324306001e+03 },
    { 4.8149999999999998e-08, 3.6541785176300001e+00, 6.7331627139600005e+02 },
    { 4.8009999999999998e-08, 1.9165717364999999e+00, 7.6322448133659998e+02 },
    { 4.7979999999999997e-08, 1.9976011461300001e+00, 2.5739127376840003e+02 },
    { 4.7869999999999998e-08, 2.4571731199500002e+00, 4.6112914165080002e+02 },
    { 4.7759999999999999e-08, 5.0123072552999997e+00, 3.7901580226720000e+02 },
    { 4.7729999999999998e-08, 5.8720978713800003e+00, 1.2301837848190000e+02 },
    { 4.7699999999999997e-08, 5.0455018296900001e+00, 8.2399143422340001e+02 },
    { 4.7629999999999997e-08, 2.4511968376500000e+00, 1.0160830151030000e+02 },
    { 4.7559999999999997e-08, 1.4800128265700001e+00, 1.3607959463187999e+03 },
    { 4.7519999999999998e-08, 5.1331071734600000e+00, 2.3082520325630000e+02 },
    { 4.7479999999999999e-08, 4.7235264018300001e+00, 5.5121291665080003e+02 },
    { 4.7419999999999997e-08, 1.0853150963200000e+00, 2.5788776747305001e+04 },
    { 4.7309999999999998e-08, 4.5822074115599998e+00, 5.9853467282869997e+02 },
    { 4.7269999999999999e-08, 4.1912375757099998e+00, 6.8695841006100000e+02 },
    { 4.7250000000000003e-08, 5.5092150839600000e+00, 1.2222784494481000e+03 },
    { 4.7120000000000001e-08, 7.2241704208000002e-01, 2.4038643081190000e+02 },
    { 4.7099999999999998e-08, 2.2646578664999999e+00, 4.8491746320800002e+02 },
    { 4.7050000000000001e-08, 3.1835864731200001e+00, 2.3916259053450000e+02 },
    { 4.7029999999999998e-08, 1.8391535840000001e-01, 4.7365931462300000e+02 },
    { 4.6999999999999997e-08, 2.6333239349699999e+00, 2.9281599642050003e+02 },
    { 4.6490000000000001e-08, 6.1688400083900001e+00, 7.0077611378539996e+02 },
    { 4.6450000000000002e-08, 8.0030005818000005e-01, 4.4126722786229999e+02 },
    { 4.6369999999999998e-08, 1.2300886214200000e+00, 1.7867752193800001e+02 },
    { 4.6299999999999998e-08, 3.5302786349400002e+00, 6.1701612157863001e+03 },
    { 4.6280000000000001e-08, 3.4854671149999999e-01, 3.2472925693479999e+02 },
    { 4.6240000000000002e-08, 3.3672553291399998e+00, 8.8772733252679996e+02 },
    { 4.6210000000000002e-08, 1.6031518568100001e+00, 1.0452906281710000e+02 },
    { 4.6189999999999999e-08, 4.5628336859800003e+00, 1.9417664032680000e+02 },
    { 4.6040000000000001e-08, 3.2373053301399999e+00, 5.2177290755856688e+04 },
    { 4.6010000000000000e-08, 5.6848721506000000e-01, 2.0470107572890001e+02 },
    { 4.5979999999999999e-08, 3.6241216598100001e+00, 1.5519227142710000e+02 },
    { 4.5979999999999999e-08, 2.8482489625700000e+00, 2.2408416327099999e+02 },
    { 4.5979999999999999e-08, 5.1431391976100000e+00, 1.5875191767470000e+02 },
    { 4.5870000000000000e-08, 4.6059405712199997e+00, 2.5483430577912291e+04 },
    { 4.5839999999999999e-08, 3.5948242553699998e+00, 3.0168155375100000e+01 },
    { 4.5779999999999997e-08, 1.2293169591000000e-01, 2.7361398630378999e+03 },
    { 4.5639999999999997e-08, 2.9736165183600001e+00, 1.1239336934329999e+02 },
    { 4.5620000000000001e-08, 4.8449706333800000e+00, 3.5574874557179999e+02 },
    { 4.5459999999999998e-08, 2.3323515267200001e+00, 4.4182801055600002e+01 },
    { 4.5450000000000000e-08, 3.9900759419599998e+00, 1.3840562228610000e+02 },
    { 4.5359999999999997e-08, 5.1801222344399998e+00, 3.8724131496080003e+02 },
    { 4.5289999999999997e-08, 4.2105215119000000e-01, 5.2985102378900001e+02 },
    { 4.5270000000000001e-08, 1.8440066667999999e-01, 1.0005439283280000e+02 },
    { 4.5179999999999998e-08, 3.9196103251599999e+00, 5.2953090640020002e+02 },
    { 4.5179999999999998e-08, 1.2021315429000000e-01, 2.2759562540880000e+02 },
    { 4.5079999999999997e-08, 3.9614361609500000e+00, 1.0643422665070000e+02 },
    { 4.5079999999999997e-08, 3.0738654895000002e+00, 9.8311585891360005e+02 },
    { 4.5009999999999997e-08, 1.1366886282099999e+00, 5.2347118997109999e+02 },
    { 4.4780000000000001e-08, 2.5309581770400000e+00, 4.4631134681819998e+02 },
    { 4.4710000000000001e-08, 3.6750815910500001e+00, 3.3940249943799998e+01 },
    { 4.4619999999999999e-08, 9.3157251490000004e-01, 1.6141204655060000e+02 },
    { 4.4579999999999999e-08, 3.1703385792000000e-01, 1.2758367119140000e+02 },
    { 4.4540000000000000e-08, 3.4257976209800001e+00, 3.7471114198859999e+02 },
    { 4.4419999999999997e-08, 2.5571944163200002e+00, 1.5780271950199001e+03 },
    { 4.4339999999999999e-08, 7.1080738069000005e-01, 1.6850521225016000e+03 },
    { 4.4320000000000002e-08, 4.2979590396200003e+00, 7.8159617692239999e+02 },
    { 4.4099999999999998e-08, 3.9443383724699999e+00, 1.0348428491065999e+03 },
    { 4.4099999999999998e-08, 5.4683841710900003e+00, 4.6053844081979997e+02 },
    { 4.4059999999999999e-08, 5.4755190132299996e+00, 8.4756730745949994e+02 },
    { 4.3980000000000001e-08, 3.0501000036899999e+00, 1.0197644218431000e+03 },
    { 4.3740000000000000e-08, 5.3960998679400003e+00, 3.5692440173940003e+02 },
    { 4.3709999999999999e-08, 1.6176166767500000e+00, 1.0727152523135001e+03 },
    { 4.3639999999999999e-08, 2.1039389902200001e+00, 1.9683676920270000e+02 },
    { 4.3630000000000001e-08, 4.9529728333399996e+00, 1.0100370912005899e+04 },
    { 4.3499999999999999e-08, 5.8138936557999998e-01, 2.1197628650500000e+01 },
    { 4.3499999999999999e-08, 2.9208175005700001e+00, 4.4275170057060001e+02 },
    { 4.3460000000000000e-08, 2.5378681035700001e+00, 3.1957326339430000e+02 },
    { 4.3369999999999997e-08, 7.1530474065000005e-01, 3.2595309721220002e+02 },
    { 4.3359999999999999e-08, 1.5964435123100000e+00, 5.4865336151140002e+02 },
    { 4.3250000000000000e-08, 4.5098003578799997e+00, 5.0286426215159997e+02 },
    { 4.3159999999999997e-08, 5.4679542461599997e+00, 1.3257960603750001e+02 },
    { 4.3159999999999997e-08, 3.6202358673099999e+00, 3.1097524368519998e+02 },
    { 4.3070000000000001e-08, 3.9357883640100000e+00, 3.0263785640700002e+02 },
    { 4.3019999999999997e-08, 4.0748814609800004e+00, 9.9841944511700007e+01 },
    { 4.3009999999999999e-08, 3.6666294953600000e+00, 1.3755428902419999e+02 },
    { 4.2920000000000003e-08, 4.2639101283900001e+00, 9.0833426034629997e+02 },
    { 4.2790000000000001e-08, 1.3989762992999999e+00, 3.8330916169770001e+02 },
    { 4.2650000000000001e-08, 3.6293801095500000e+00, 1.3917188817650000e+02 },
    { 4.2569999999999997e-08, 4.6595320339799997e+00, 2.6907011347870002e+02 },
    { 4.2450000000000000e-08, 2.4859413109999999e-02, 2.9111907539110001e+02 },
    { 4.2209999999999999e-08, 4.8556582208299996e+00, 1.0250207338760001e+02 },
    { 4.2060000000000001e-08, 2.0894690807799998e+00, 4.3519621058510000e+02 },
    { 4.1990000000000001e-08, 6.0013294263099999e+00, 3.0545804397730001e+02 },
    { 4.1850000000000001e-08, 2.9659743730799999e+00, 3.0523429480810000e+02 },
    { 4.1850000000000001e-08, 6.2348537593100000e+00, 1.5748458012822000e+03 },
    { 4.1829999999999998e-08, 5.7402167686199999e+00, 9.9793365374619998e+02 },
    { 4.1719999999999999e-08, 2.9321162327299999e+00, 5.8115732285670003e+02 },
    { 4.1659999999999997e-08, 8.4927305009999995e-02, 6.2736711334179995e+02 },
    { 4.1540000000000000e-08, 5.6269039382499999e+00, 1.9540048060420000e+02 },
    { 4.1500000000000001e-08, 4.2970745854599999e+00, 1.8164646735459999e+02 },
    { 4.1379999999999997e-08, 2.7320239075899999e+00, 4.3615941843159999e+02 },
    { 4.1320000000000002e-08, 1.2897898810000001e-02, 7.6174000862829996e+02 },
    { 4.1239999999999997e-08, 3.3797675489599999e+00, 2.5265597135319999e+02 },
    { 4.1220000000000001e-08, 4.8038947259500002e+00, 2.8857030601725000e+03 },
    { 4.1190000000000000e-08, 3.7058994655399999e+00, 2.1202483330866999e+03 },
    { 4.1019999999999999e-08, 4.5909383613700001e+00, 1.1693276189777000e+03 },
    { 4.1019999999999999e-08, 2.9994570135199998e+00, 8.0821043154430004e+02 },
    { 4.0889999999999998e-08, 4.8867162727500002e+00, 7.7033802833740003e+02 },
    { 4.0819999999999998e-08, 2.5118427960000000e-01, 5.1159540830470002e+02 },
    { 4.0690000000000002e-08, 1.2765976522200000e+00, 2.7463549729639999e+02 },
    { 4.0670000000000000e-08, 1.3059333121400001e+00, 5.2820649238630006e+02 },
    { 4.0579999999999997e-08, 4.1718279328300003e+00, 1.0444040766621999e+03 },
    { 4.0579999999999997e-08, 1.5073685544300000e+00, 4.3222726516850003e+02 },
    { 4.0539999999999998e-08, 3.9300868223899998e+00, 1.1474968508807999e+03 },
    { 4.0250000000000000e-08, 2.9966193387100000e+00, 2.5632993775046889e+04 },
    { 4.0180000000000000e-08, 3.6686156449600000e+00, 7.5728659050340002e+02 },
    { 4.0100000000000002e-08, 9.6832089300000002e-01, 9.5709230512270005e+02 },
    { 4.0019999999999997e-08, 1.6008723002700000e+00, 3.9505743737720002e+02 },
    { 4.0000000000000001e-08, 4.2313755903999999e+00, 2.3890195810360001e+02 },
    { 3.9990000000000003e-08, 6.7919742508000003e-01, 5.5819202150499997e+01 },
    { 3.9990000000000003e-08, 8.5861235732999996e-01, 9.3743994984099999e+01 },
    { 3.9949999999999997e-08, 2.7481026978299998e+00, 6.3203297978779995e+02 },
    { 3.9729999999999999e-08, 2.0494598596400002e+00, 4.3319047301500001e+02 },
    { 3.9690000000000000e-08, 2.9734781168000000e+00, 2.5060204967330000e+02 },
    { 3.9610000000000002e-08, 5.0193058716700003e+00, 2.1648048917570000e+02 },
    { 3.9610000000000002e-08, 3.3325077824700000e+00, 1.8557862061770001e+02 },
    { 3.9589999999999999e-08, 5.5093438383400004e+00, 1.5093579525830000e+02 },
    { 3.9570000000000003e-08, 5.8873037934000001e+00, 4.0747573576479999e+02 },
    { 3.9500000000000003e-08, 5.2597334092599999e+00, 6.9780716836880003e+02 },
    { 3.9459999999999997e-08, 4.1382758470000003e-02, 6.0968640899272996e+03 },
    { 3.9419999999999998e-08, 9.0741814441000002e-01, 1.0200250542740000e+03 },
    { 3.9419999999999998e-08, 4.2827525176999997e+00, 9.2797798031929995e+02 },
    { 3.9349999999999998e-08, 3.1753512169699998e+00, 4.8788640862459999e+02 },
    { 3.9319999999999997e-08, 3.9252433858900000e+00, 2.4335537622850001e+02 },
    { 3.9239999999999999e-08, 4.4043916493799999e+00, 5.2698265210889997e+02 },
    { 3.9209999999999998e-08, 2.3123872711000000e+00, 3.6785822741869998e+02 },
    { 3.9190000000000002e-08, 2.6952670688000002e-01, 6.1374448061199000e+03 },
    { 3.9169999999999999e-08, 3.6987084848400000e+00, 6.0824465823059995e+02 },
    { 3.9139999999999998e-08, 5.0553460792399996e+00, 5.2549817940059995e+02 },
    { 3.9120000000000002e-08, 4.3148140974100002e+00, 6.6969293467360001e+02 },
    { 3.9069999999999998e-08, 3.1869771864000001e+00, 3.6175482177500001e+01 },
    { 3.9039999999999998e-08, 2.4163677358999999e-01, 4.5809076026499997e+02 },
    { 3.8959999999999999e-08, 4.6366115562999999e-01, 7.9838857155779999e+02 },
    { 3.8920000000000000e-08, 1.6927386150299999e+00, 1.2466574718362999e+03 },
    { 3.8759999999999998e-08, 5.9051017869100004e+00, 9.9022940591439999e+02 },
    { 3.8719999999999999e-08, 5.4511986823600003e+00, 4.7069036920640002e+02 },
    { 3.8700000000000002e-08, 5.5135658666299996e+00, 2.1249836355019002e+03 },
    { 3.8680000000000000e-08, 2.6490924986600000e+00, 1.1345715092630000e+02 },
    { 3.8600000000000002e-08, 4.1737150739800004e+00, 4.8236779914600000e+01 },
    { 3.8570000000000001e-08, 3.1626803203699998e+00, 2.5409225995169999e+02 },
    { 3.8540000000000000e-08, 2.6817550294800001e+00, 2.2493549653290000e+02 },
    { 3.8449999999999997e-08, 5.3509847628799996e+00, 6.1483694036960003e+02 },
    { 3.8409999999999998e-08, 2.5054270583100000e+00, 6.7480074410429995e+02 },
    { 3.8400000000000000e-08, 4.9558651953500004e+00, 6.0616999997061002e+03 },
    { 3.8330000000000000e-08, 4.5088270613699999e+00, 4.6447059408289999e+02 },
    { 3.8299999999999999e-08, 3.4432353577199999e+00, 4.2341679713830001e+02 },
    { 3.8260000000000000e-08, 3.2975071098200002e+00, 6.1295804995936996e+03 },
    { 3.8220000000000001e-08, 3.9984349362999998e+00, 8.2717282796109998e+02 },
    { 3.8089999999999999e-08, 4.3840061859699997e+00, 3.1177521037062002e+03 },
    { 3.8070000000000003e-08, 5.2678882452000000e+00, 1.7076082578510000e+02 },
    { 3.7900000000000002e-08, 8.9714826479999998e-01, 1.3862937145530000e+02 },
    { 3.7800000000000001e-08, 3.6561443302000001e+00, 9.0562601246400007e+01 },
    { 3.7779999999999998e-08, 1.0786376266900000e+00, 8.6939264523999995e+01 },
    { 3.7739999999999999e-08, 1.3988602888699999e+00, 2.5117149864490000e+02 },
    { 3.7690000000000002e-08, 6.0875044770999998e-01, 6.5004921470060003e+02 },
    { 3.7679999999999997e-08, 5.3607674174600000e+00, 1.7559015678057999e+03 },
    { 3.7639999999999998e-08, 5.5061087549399996e+00, 6.8631259412999995e+01 },
    { 3.7639999999999998e-08, 5.4564789882399998e+00, 8.0931937721599994e+01 },
    { 3.7620000000000002e-08, 2.6065517490299999e+00, 2.9430046912879999e+02 },
    { 3.7599999999999999e-08, 3.7298664338899998e+00, 7.5106681537990005e+02 },
    { 3.7580000000000003e-08, 1.4294735320999999e-01, 1.0412226829245001e+03 },
    { 3.7560000000000000e-08, 1.8003327428400000e+00, 6.6901161111739998e+02 },
    { 3.7499999999999998e-08, 3.7269417012499999e+00, 5.9854339284304997e+03 },
    { 3.7410000000000002e-08, 3.3398234764000001e-01, 8.1383950583679996e+02 },
    { 3.7389999999999999e-08, 2.4993059038299998e+00, 9.0993060763920005e+02 },
    { 3.7350000000000000e-08, 5.7469664753499998e+00, 9.0970685847000004e+02 },
    { 3.7259999999999997e-08, 2.6727774624300000e+00, 2.6542812508101491e+04 },
    { 3.7249999999999999e-08, 1.8898208558900000e+00, 4.9841084402669998e+02 },
    { 3.7170000000000001e-08, 5.6132691153799996e+00, 1.4717530270636000e+03 },
    { 3.7160000000000003e-08, 4.7136065382699996e+00, 5.2724328453980002e+02 },
    { 3.7149999999999998e-08, 1.4639835930100000e+00, 1.0027073786146901e+04 },
    { 3.7039999999999999e-08, 2.5830844810100002e+00, 2.5865042818580590e+04 },
    { 3.7020000000000003e-08, 5.1333912897899996e+00, 1.3214390704036000e+03 },
    { 3.6990000000000002e-08, 1.6922358501000001e+00, 1.0067654502339499e+04 },
    { 3.6909999999999998e-08, 2.8796103276400000e+00, 7.4809786996330001e+02 },
    { 3.6720000000000001e-08, 3.9454281112000000e-01, 3.7694637422229999e+02 },
    { 3.6680000000000002e-08, 4.0450971297200002e+00, 8.2962050851590004e+02 },
    { 3.6680000000000002e-08, 5.2257239971700002e+00, 4.1566436519669998e+02 },
    { 3.6669999999999997e-08, 3.0714328910000002e+00, 1.6969210293999999e+00 },
    { 3.6640000000000003e-08, 6.1291864845999999e+00, 3.2016396422529999e+02 },
    { 3.6519999999999999e-08, 2.4496988230699999e+00, 3.9868077409960000e+02 },
    { 3.6459999999999997e-08, 1.5548389487300001e+00, 4.7346237266690002e+02 },
    { 3.6410000000000000e-08, 1.9257534079900001e+00, 2.9795648359600001e+01 },
    { 3.6400000000000002e-08, 6.1816987372199996e+00, 4.1013586464069999e+02 },
    { 3.6389999999999997e-08, 2.4027183139499999e+00, 5.1307988101299998e+02 },
    { 3.6379999999999999e-08, 3.5091881835100001e+00, 1.0089793540101000e+03 },
    { 3.6370000000000001e-08, 4.8790334029200002e+00, 5.8786147958030006e+02 },
    { 3.6340000000000000e-08, 3.0043958211100001e+00, 7.7638669399299999e+01 },
    { 3.6340000000000000e-08, 1.6750066149000000e+00, 7.1924527735300003e+01 },
    { 3.6330000000000002e-08, 1.9092511330199999e+00, 2.1753862116810001e+03 },
    { 3.6309999999999999e-08, 7.1340218954000001e-01, 1.3364169239306000e+03 },
    { 3.6230000000000001e-08, 4.6498545629299999e+00, 4.5479749194269999e+02 },
    { 3.6190000000000002e-08, 5.3814238131000003e-01, 1.2125202148330000e+02 },
    { 3.6160000000000001e-08, 1.4020034733900000e+00, 4.5502124111190000e+02 },
    { 3.6139999999999998e-08, 9.5299266839999994e-02, 9.9919096959257004e+03 },
    { 3.6039999999999997e-08, 4.7201679739699998e+00, 1.0059790195813301e+04 },
    { 3.6029999999999999e-08, 1.5946995153800001e+00, 3.8672005009899999e+02 },
    { 3.6029999999999999e-08, 3.5521145296799999e+00, 5.8637700687200004e+02 },
    { 3.5980000000000002e-08, 2.4186259790000000e-01, 1.3203708931630001e+02 },
    { 3.5889999999999999e-08, 2.0335381576999998e+00, 4.8710451133190003e+02 },
    { 3.5880000000000001e-08, 2.7873050601999999e-01, 1.3403490771936999e+03 },
    { 3.5780000000000000e-08, 5.0811853825600002e+00, 1.5455913198069999e+02 },
    { 3.5759999999999997e-08, 4.1034597551400003e+00, 1.6857377766120001e+02 },
    { 3.5679999999999999e-08, 8.5509496232000004e-01, 1.0160929010109001e+03 },
    { 3.5609999999999999e-08, 5.0934220315400003e+00, 1.6685897962663000e+03 },
    { 3.5590000000000003e-08, 4.2999288817499997e+00, 9.0046995382010005e+02 },
    { 3.5579999999999998e-08, 1.4957844923999999e-01, 3.6611312227949998e+02 },
    { 3.5570000000000000e-08, 4.9548471586499998e+00, 1.2545217783625001e+03 },
    { 3.5530000000000001e-08, 7.7771622370000001e-02, 2.6195656647790003e+02 },
    { 3.5469999999999999e-08, 1.1895488900500000e+00, 3.8606116389800000e+01 },
    { 3.5439999999999998e-08, 1.3337912548799999e+00, 1.8802630117250001e+02 },
    { 3.5360000000000000e-08, 3.3627565452399999e+00, 7.8039364629020682e+04 },
    { 3.5350000000000002e-08, 1.0988485688999999e-01, 8.7290953769420003e+02 },
    { 3.5329999999999999e-08, 5.5185133960000003e+00, 2.7140591944890002e+02 },
    { 3.5159999999999999e-08, 7.8503214934999999e-01, 4.7049342725029999e+02 },
    { 3.5089999999999999e-08, 3.3027349220800000e+00, 6.3548885031420996e+03 },
    { 3.5019999999999999e-08, 5.1487618377300004e+00, 9.9156436246500998e+03 },
    { 3.4980000000000000e-08, 1.4578444966000001e-01, 3.1147831582896001e+03 },
    { 3.4940000000000001e-08, 1.5064263285999999e+00, 1.9131956949479999e+02 },
    { 3.4790000000000002e-08, 2.1471955910000001e-01, 2.7166655187980001e+02 },
    { 3.4590000000000001e-08, 4.5807915014000002e+00, 8.0923732986560003e+02 },
    { 3.4529999999999999e-08, 1.6112251123200001e+00, 1.2059761819072000e+03 },
    { 3.4399999999999997e-08, 2.4507851658400002e+00, 2.3650246165860000e+02 },
    { 3.4389999999999999e-08, 1.4659870471600001e+00, 8.1294573395949999e+02 },
    { 3.4270000000000002e-08, 1.7455494619400000e+00, 9.1448459950059998e+02 },
    { 3.4219999999999998e-08, 1.5373362759400000e+00, 8.8624285981850005e+02 },
    { 3.4119999999999997e-08, 4.7707903135400000e+00, 8.3258945393249996e+02 },
    { 3.3990000000000002e-08, 3.0647000545899998e+00, 4.1852143602870001e+02 },
    { 3.3979999999999997e-08, 2.3047450502000000e-01, 1.8260967520110000e+02 },
    { 3.3979999999999997e-08, 4.8543871062099999e+00, 9.9171387862270001e+02 },
    { 3.3920000000000002e-08, 2.2589968501800000e+00, 5.9091678571549000e+03 },
    { 3.3880000000000003e-08, 4.6284192304699996e+00, 2.5862073873164001e+04 },
    { 3.3860000000000000e-08, 9.7499612837000005e-01, 1.0562005364515001e+03 },
    { 3.3799999999999998e-08, 7.0280224084999998e-01, 2.0057375701000000e+00 },
    { 3.3769999999999997e-08, 4.7255953125700003e+00, 1.0285098199361701e+04 },
    { 3.3750000000000001e-08, 1.7464399325600000e+00, 6.0595703637019994e+02 },
    { 3.3729999999999998e-08, 2.5136652920000002e-01, 7.6492140236600005e+02 },
    { 3.3720000000000000e-08, 4.5014900270000002e-01, 4.0695447090300001e+02 },
    { 3.3720000000000000e-08, 8.4534442531999998e-01, 2.1325091132820000e+02 },
    { 3.3689999999999999e-08, 4.0421147884900002e+00, 2.1334727954780001e+02 },
    { 3.3589999999999998e-08, 4.4680334320500004e+00, 3.5989334715600000e+02 },
    { 3.3510000000000000e-08, 3.7628040036199999e+00, 7.4379320968469995e+02 },
    { 3.3489999999999997e-08, 5.4564364729200001e+00, 5.2375307426140000e+02 },
    { 3.3489999999999997e-08, 1.7700667217799999e+00, 6.7880499887599996e+01 },
    { 3.3470000000000001e-08, 5.1413466508700001e+00, 2.4783004621280000e+02 },
    { 3.3449999999999998e-08, 1.9350862009200001e+00, 5.8340806145540000e+02 },
    { 3.3449999999999998e-08, 5.3341822777000001e+00, 8.1065811209909998e+02 },
    { 3.3409999999999999e-08, 5.2007276373900000e+00, 5.1153171782990000e+02 },
    { 3.3400000000000001e-08, 4.4847914814900003e+00, 1.3662125722901999e+03 },
    { 3.3390000000000003e-08, 2.9514868497100002e+00, 8.3949055261219996e+02 },
    { 3.3360000000000002e-08, 3.1016259718499999e+00, 7.2509144569880004e+02 },
    { 3.3349999999999998e-08, 3.3044583361400002e+00, 7.7130123618389996e+02 },
    { 3.3339999999999999e-08, 1.5340910675000000e-01, 1.3617591541653001e+03 },
    { 3.3279999999999998e-08, 4.5515869325200002e+00, 3.9197661737599998e+02 },
    { 3.3269999999999999e-08, 8.2829766821999995e-01, 4.1258354519549999e+02 },
    { 3.3239999999999999e-08, 4.8033736753999996e+00, 6.2097787241324004e+03 },
    { 3.3180000000000003e-08, 5.4536039227600002e+00, 1.4505553984131000e+03 },
    { 3.3180000000000003e-08, 3.7391277906900000e+00, 2.6036021918500001e+02 },
    { 3.3169999999999999e-08, 2.5126983737200002e+00, 8.0775285715730001e+02 },
    { 3.3160000000000000e-08, 2.5325707758100000e+00, 3.0237722397610003e+02 },
    { 3.3139999999999998e-08, 2.8412276686200002e+00, 2.8021638747909998e+02 },
    { 3.3120000000000001e-08, 8.9488112288999999e-01, 5.1716079212240004e+02 },
    { 3.2990000000000000e-08, 2.1548133695599998e+00, 4.4089472084680000e+02 },
    { 3.2870000000000003e-08, 2.6000896186000000e-01, 4.6324675380550002e+02 },
    { 3.2730000000000003e-08, 5.7636322074799997e+00, 2.4241100297711000e+03 },
    { 3.2689999999999997e-08, 4.9817345720000003e-02, 7.3742467671489999e+02 },
    { 3.2679999999999999e-08, 4.0008014120900004e+00, 5.9000595888200003e+01 },
    { 3.2660000000000003e-08, 4.8873509465399998e+00, 2.6572866104660000e+02 },
    { 3.2660000000000003e-08, 6.5217674021000005e-01, 2.0129018732759999e+02 },
    { 3.2579999999999998e-08, 7.0846024545999997e-01, 1.0457724692690000e+02 },
    { 3.2549999999999997e-08, 1.0359183487100001e+00, 4.8236920891670002e+02 },
    { 3.2509999999999998e-08, 1.9051098453999999e+00, 8.3167169955299997e+01 },
    { 3.2500000000000000e-08, 3.5084075679100000e+00, 9.4943624140070006e+02 },
    { 3.2469999999999999e-08, 4.7960306315999999e+00, 3.1342292423999999e+02 },
    { 3.2420000000000002e-08, 1.8326079798699999e+00, 8.0524148612770000e+02 },
    { 3.2350000000000002e-08, 1.7902581407100000e+00, 3.7715882254339999e+02 },
    { 3.2320000000000001e-08, 3.9133444665899999e+00, 4.4482687410990002e+02 },
    { 3.2310000000000003e-08, 4.9594121317299997e+00, 2.7458731318660000e+02 },
    { 3.2299999999999998e-08, 5.9292739382799997e+00, 2.1951887056149999e+02 },
    { 3.2259999999999999e-08, 1.4524059990400000e+00, 3.9494556279260001e+02 },
    { 3.2250000000000001e-08, 1.4063790552600000e+00, 5.7616138801060004e+02 },
    { 3.2210000000000002e-08, 3.2142397170399999e+00, 1.4424786435658000e+03 },
    { 3.2189999999999999e-08, 2.6414374356499999e+00, 1.2749686474876000e+03 },
    { 3.2170000000000003e-08, 4.8706995475800001e+00, 1.0616695520496000e+03 },
    { 3.2110000000000001e-08, 5.5386687289800003e+00, 1.5156893470470001e+02 },
    { 3.2080000000000000e-08, 3.7986549742700002e+00, 3.8698068252989998e+02 },
    { 3.1930000000000002e-08, 3.9872096102199999e+00, 3.0289848883790000e+02 },
    { 3.1909999999999999e-08, 3.6815114493699999e+00, 9.8393775533744993e+03 },
    { 3.1900000000000001e-08, 4.2582703302300002e+00, 1.5961864422845999e+03 },
    { 3.1900000000000001e-08, 3.6975638315600001e+00, 7.6961982151949996e+02 },
    { 3.1890000000000003e-08, 2.8502763412899998e+00, 4.3993151300030001e+02 },
    { 3.1860000000000002e-08, 3.0158715822900000e+00, 5.3632154026599999e+01 },
    { 3.1849999999999997e-08, 5.7994387551599997e+00, 5.9824649830138997e+03 },
    { 3.1790000000000002e-08, 5.7635788351099997e+00, 1.8865547581502999e+03 },
    { 3.1790000000000002e-08, 2.4736216448300001e+00, 5.2943033266370003e+02 },
    { 3.1669999999999998e-08, 3.7797184952200000e+00, 1.2567636172691100e+04 },
    { 3.1650000000000002e-08, 4.7505889156099999e+00, 2.6161200267433189e+04 },
    { 3.1529999999999998e-08, 4.4615006624599998e+00, 3.7955217932270998e+03 },
    { 3.1489999999999999e-08, 1.4598442126500000e+00, 3.9510562148700001e+02 },
    { 3.1459999999999998e-08, 3.9197879867999998e+00, 6.5331461289732997e+03 },
    { 3.1459999999999998e-08, 2.7666128566300001e+00, 1.2427253185732000e+03 },
    { 3.1389999999999999e-08, 6.2239813985100003e+00, 1.0139988420352000e+04 },
    { 3.1319999999999999e-08, 5.1276748417300002e+00, 1.5536481726316999e+03 },
    { 3.1300000000000002e-08, 6.2575767524700003e+00, 2.7279717264555002e+03 },
    { 3.1289999999999998e-08, 4.4784303250699997e+00, 9.3961438141420001e+02 },
    { 3.1200000000000001e-08, 5.4436636055800003e+00, 7.6981676347560006e+02 },
    { 3.1149999999999998e-08, 2.7647241651800001e+00, 4.3675011926259998e+02 },
    { 3.1119999999999997e-08, 1.6367990391400000e+00, 4.8225733433210002e+02 },
    { 3.1090000000000002e-08, 6.0971617777800002e+00, 4.5034134317499998e+01 },
    { 3.1049999999999997e-08, 4.2212828385000001e-01, 5.5110104206619997e+02 },
    { 3.1020000000000002e-08, 2.1509429896399999e+00, 4.7643131808350000e+02 },
    { 3.1009999999999998e-08, 1.8237761602200000e+00, 5.2995159752550001e+02 },
    { 3.0979999999999997e-08, 4.0781292793699997e+00, 6.5742339413230002e+02 },
    { 3.0969999999999999e-08, 8.9535439878000000e-01, 9.1151565408399995e+02 },
    { 3.0969999999999999e-08, 4.7204718270199999e+00, 1.0063722349076401e+04 },
    { 3.0930000000000000e-08, 7.8760201101000005e-01, 9.3531546663000000e+01 },
    { 3.0909999999999997e-08, 3.3684657618500000e+00, 9.1278767847120002e+02 },
    { 3.0909999999999997e-08, 5.6153076096000003e+00, 5.5418186206739995e+02 },
    { 3.0909999999999997e-08, 2.6846308854000001e-01, 3.4139095525588000e+03 },
    { 3.0880000000000003e-08, 5.5804780721600000e+00, 2.3028268653510000e+02 },
    { 3.0790000000000000e-08, 4.4893426396300002e+00, 8.2881735936380005e+02 },
    { 3.0790000000000000e-08, 1.1324214951700000e+00, 1.0625633239269000e+03 },
    { 3.0769999999999997e-08, 4.1114564723999996e+00, 3.6759759498779999e+02 },
    { 3.0729999999999998e-08, 2.6671135495599998e+00, 1.0686486878730000e+02 },
    { 3.0729999999999998e-08, 3.5436617006999999e-01, 1.0065206821784701e+04 },
    { 3.0689999999999999e-08, 3.5750092348300000e+00, 5.3323337485899998e+01 },
    { 3.0629999999999997e-08, 1.0663223728000000e+00, 6.9970103135429997e+02 },
    { 3.0619999999999999e-08, 3.9114281567400000e+00, 7.8946048344860003e+02 },
    { 3.0610000000000001e-08, 2.2440334453399999e+00, 5.2174321810440088e+04 },
    { 3.0510000000000000e-08, 2.7836874643799998e+00, 4.2118156490460001e+02 },
    { 3.0500000000000002e-08, 2.9468648979899998e+00, 6.5301771835566997e+03 },
    { 3.0489999999999997e-08, 2.0652490496000002e+00, 1.3032798231388999e+03 },
    { 3.0470000000000001e-08, 3.7101042552600001e+00, 7.4569150246199996e+01 },
    { 3.0470000000000001e-08, 9.6929818073999996e-01, 7.4994046888400007e+01 },
    { 3.0430000000000002e-08, 5.8455328824999997e+00, 7.5255128808819995e+02 },
    { 3.0400000000000001e-08, 5.1674549279099997e+00, 4.8088473620840000e+02 },
    { 3.0290000000000002e-08, 4.0333234035199999e+00, 2.4554242435239999e+02 },
    { 3.0220000000000002e-08, 3.6797625464800001e+00, 8.2940806019479999e+02 },
    { 3.0209999999999997e-08, 1.2127727447400001e+00, 2.8000393915799998e+02 },
    { 3.0110000000000003e-08, 2.2247070456600002e+00, 1.6055876111458001e+03 },
    { 3.0110000000000003e-08, 2.3711051584799998e+00, 3.7301422095919997e+02 },
    { 3.0059999999999999e-08, 4.0439128685499997e+00, 1.5440869450760999e+03 },
    { 3.0050000000000001e-08, 3.5399213264100000e+00, 1.6263588682800000e+02 },
    { 2.9989999999999999e-08, 5.2236126027100003e+00, 6.1349971255650999e+03 },
    { 2.9970000000000003e-08, 4.2236415819999998e-02, 3.5499798604640000e+02 },
    { 2.9929999999999997e-08, 4.9951880041000002e-01, 4.5905396811150001e+02 },
    { 2.9870000000000002e-08, 1.0547395428299999e+00, 1.8227960680100000e+02 },
    { 2.9840000000000001e-08, 9.3929798504999995e-01, 9.9126746792334998e+03 },
    { 2.9840000000000001e-08, 2.4635683106000000e-01, 9.3664543599759998e+02 },
    { 2.9779999999999999e-08, 5.9458062464600001e+00, 3.7353548582100001e+02 },
    { 2.9779999999999999e-08, 1.9713653807500000e+00, 4.0073469577949999e+02 },
    { 2.9749999999999998e-08, 4.2088404497500003e+00, 2.2418473700749999e+02 },
    { 2.9729999999999999e-08, 1.6762265464899999e+00, 6.0527571281400003e+02 },
    { 2.9690000000000000e-08, 4.5160554699200004e+00, 1.5418517128424000e+03 },
    { 2.9620000000000000e-08, 2.6092079777700001e+00, 1.1585425511446999e+03 },
    { 2.9609999999999998e-08, 2.7841986626600002e+00, 1.2564667227274500e+04 },
    { 2.9569999999999999e-08, 3.8919003039400000e+00, 5.3583969916800001e+01 },
    { 2.9499999999999999e-08, 4.5971776083200000e+00, 4.7273572068100002e+01 },
    { 2.9499999999999999e-08, 5.7685918574299997e+00, 6.5323060843830001e+02 },
    { 2.9480000000000000e-08, 3.5341335849000002e+00, 7.8338491023289884e+04 },
    { 2.9469999999999999e-08, 1.1017790208800000e+00, 4.4683261168000001e+02 },
    { 2.9449999999999999e-08, 3.5171926216700000e+00, 7.1441825245040002e+02 },
    { 2.9410000000000000e-08, 1.4058577866099999e+00, 1.4786627610520000e+02 },
    { 2.9370000000000001e-08, 2.9646296116999998e-01, 5.9024546966030005e+02 },
    { 2.9370000000000001e-08, 9.9310447017000003e-01, 2.8266406803389998e+02 },
    { 2.9360000000000000e-08, 5.6836357176899996e+00, 3.8087852748540001e+02 },
    { 2.9300000000000001e-08, 1.5304289005500000e+00, 1.7313907037070001e+02 },
    { 2.9230000000000001e-08, 4.0356048488900003e+00, 1.5999585368533001e+03 },
    { 2.9220000000000000e-08, 6.1496402947999995e-01, 9.4755386940400001e+01 },
    { 2.9220000000000000e-08, 9.2284575293000004e-01, 5.4807810194200002e+01 },
    { 2.9200000000000000e-08, 4.8165563650500003e+00, 1.1907858800591000e+03 },
    { 2.9140000000000002e-08, 3.0295869246299998e+00, 7.8474643289280004e+02 },
    { 2.9119999999999999e-08, 1.7801017402399999e+00, 7.5314198891920000e+02 },
    { 2.9099999999999999e-08, 7.8044221568000005e-01, 6.5094298657790000e+02 },
    { 2.9090000000000001e-08, 1.8656194774999999e+00, 3.7937700843459999e+02 },
    { 2.9090000000000001e-08, 3.2531106420899998e+00, 1.1249394307980000e+02 },
    { 2.9060000000000000e-08, 8.0826038657999999e-01, 4.8173606947029998e+02 },
    { 2.9029999999999999e-08, 3.0718162703999998e+00, 1.5411179996594999e+03 },
    { 2.9020000000000001e-08, 1.7912968178799999e+00, 4.7221182441400003e+01 },
    { 2.9020000000000001e-08, 6.0296982717100001e+00, 1.0234201469320000e+02 },
    { 2.8909999999999999e-08, 1.6321611263999999e-01, 2.4549424024260000e+02 },
    { 2.8859999999999998e-08, 5.7523998300699999e+00, 4.2808266358430001e+02 },
    { 2.8789999999999998e-08, 4.5417898153199996e+00, 8.5712853501510006e+02 },
    { 2.8789999999999998e-08, 5.9144031480999999e+00, 9.2404582705619998e+02 },
    { 2.8749999999999999e-08, 5.6371300804500004e+00, 6.8127668150860000e+03 },
    { 2.8740000000000001e-08, 5.1211929555499998e+00, 1.8727554164710000e+02 },
    { 2.8660000000000000e-08, 3.6622274688399998e+00, 5.2964278098479997e+02 },
    { 2.8660000000000000e-08, 4.9121363572699996e+00, 2.1926516870684000e+03 },
    { 2.8640000000000000e-08, 6.7930219876999998e-01, 5.2973914920439995e+02 },
    { 2.8629999999999999e-08, 4.0003654857099997e+00, 3.2324478422649997e+02 },
    { 2.8590000000000000e-08, 3.5916291271900000e+00, 3.9590877063910000e+02 },
    { 2.8579999999999999e-08, 1.8645665730500001e+00, 4.3053034413910001e+02 },
    { 2.8559999999999999e-08, 4.4622736949000003e-01, 2.2816603796373001e+03 },
    { 2.8539999999999999e-08, 8.8172955033000000e-01, 6.4611706143749996e+02 },
    { 2.8509999999999999e-08, 2.3723052982400001e+00, 7.4772536294780002e+02 },
    { 2.8500000000000000e-08, 9.3628396612999998e-01, 1.4489108670103999e+03 },
    { 2.8460000000000001e-08, 3.2009706999300001e+00, 5.4005534180229995e+02 },
    { 2.8410000000000001e-08, 4.9971050635700003e+00, 1.3497502460548999e+03 },
    { 2.8349999999999999e-08, 5.2604040114500004e+00, 2.9456110155969998e+02 },
    { 2.8340000000000001e-08, 1.4904504733899999e+00, 4.8588067105450000e+02 },
    { 2.8310000000000000e-08, 4.2953973834100001e+00, 5.3977345751200005e+02 },
    { 2.8250000000000002e-08, 3.8049709980899999e+00, 1.6063383706712000e+03 },
    { 2.8240000000000000e-08, 2.9449763324200000e+00, 5.0862646288809998e+02 },
    { 2.8229999999999999e-08, 1.9015234079000001e-01, 1.2205517063540000e+02 },
    { 2.8100000000000000e-08, 1.2389019732800000e+00, 3.1128406022590002e+02 },
    { 2.8060000000000001e-08, 5.1754338615300002e+00, 2.0888081533243999e+03 },
    { 2.8060000000000001e-08, 5.7181480742800002e+00, 3.5170471772410002e+02 },
    { 2.8040000000000002e-08, 4.2091396475799998e+00, 9.7897125732940003e+02 },
    { 2.8030000000000000e-08, 2.0908188565099999e+00, 4.5172797278960002e+02 },
    { 2.8019999999999999e-08, 1.9794548516699999e+00, 1.5920418407003999e+03 },
    { 2.8019999999999999e-08, 3.3401727135299999e+00, 7.6440013750419996e+02 },
    { 2.7990000000000001e-08, 4.8681338825899996e+00, 1.0050472007470000e+03 },
    { 2.7949999999999999e-08, 7.7259066808999999e-01, 1.0742976511305600e+04 },
    { 2.7929999999999999e-08, 1.5001176002500001e+00, 4.6527374323499998e+02 },
    { 2.7829999999999998e-08, 1.0275508786800001e+00, 1.2687488723641000e+03 },
    { 2.7829999999999998e-08, 1.0255059970700000e+00, 1.0805795588397000e+03 },
    { 2.7820000000000000e-08, 3.1571902494400002e+00, 5.1960847267719998e+02 },
    { 2.7820000000000000e-08, 2.1759341219000001e+00, 8.8921180523509997e+02 },
    { 2.7800000000000001e-08, 1.7798796186000001e+00, 1.4117762574660000e+02 },
    { 2.7789999999999999e-08, 2.9533285284400002e+00, 3.9265794093220001e+02 },
    { 2.7739999999999999e-08, 4.3326717505900003e+00, 5.5178236562239999e+02 },
    { 2.7730000000000001e-08, 4.2197424040799998e+00, 4.3133349329120000e+02 },
    { 2.7710000000000001e-08, 5.0095067741200001e+00, 1.8211219388175000e+03 },
    { 2.7689999999999998e-08, 4.1262513354400001e+00, 2.0736120460480001e+02 },
    { 2.7680000000000000e-08, 9.0030726298999997e-01, 2.2745468326364999e+03 },
    { 2.7660000000000001e-08, 2.5384501006800000e+00, 1.6001185955476999e+03 },
    { 2.7640000000000001e-08, 3.8916459102699998e+00, 2.9104963942190000e+02 },
    { 2.7440000000000000e-08, 3.9094258682100000e+00, 4.4263982598600001e+02 },
    { 2.7430000000000001e-08, 4.2066483056199999e+00, 1.9692006632437999e+03 },
    { 2.7339999999999999e-08, 1.9130865979800000e+00, 7.9987304426610001e+02 },
    { 2.7269999999999999e-08, 4.3906543709000001e-01, 1.0638353483141000e+03 },
    { 2.7190000000000001e-08, 4.5068700830000002e-01, 5.2325369480282992e+04 },
    { 2.7170000000000001e-08, 2.4838154531000001e+00, 3.5192846689330003e+02 },
    { 2.7160000000000000e-08, 5.9439068869299998e+00, 1.5332536931333000e+03 },
    { 2.7129999999999999e-08, 1.3380107913599999e+00, 7.7055047665849997e+02 },
    { 2.7109999999999999e-08, 3.9889510344699999e+00, 6.7836039035190004e+02 },
    { 2.7050000000000001e-08, 2.1299412693900002e+00, 1.7245774681450001e+02 },
    { 2.6980000000000001e-08, 2.2512948728000000e+00, 5.6904784100979998e+02 },
    { 2.6890000000000001e-08, 3.9171962775500000e+00, 5.1742142455329997e+02 },
    { 2.6849999999999999e-08, 4.8536858168299997e+00, 2.5791253863020000e+02 },
    { 2.6849999999999999e-08, 2.9793614172399998e+00, 3.0357981239409997e+02 },
    { 2.6810000000000000e-08, 6.0092328661899996e+00, 4.1232291276460001e+02 },
    { 2.6750000000000001e-08, 1.1840953100300000e+00, 5.1901777184620005e+02 },
    { 2.6600000000000000e-08, 4.3582063761700001e+00, 9.6239859648700005e+01 },
    { 2.6569999999999999e-08, 3.1147782876900001e+00, 5.8779778910549999e+02 },
    { 2.6510000000000000e-08, 2.0447870893700002e+00, 9.6006125053929998e+02 },
    { 2.6510000000000000e-08, 2.9722893263799999e+00, 6.4382943957709995e+02 },
    { 2.6480000000000000e-08, 3.4521616712099998e+00, 6.5967413273099999e+02 },
    { 2.6420000000000001e-08, 4.8101787017400000e+00, 4.8715269544170002e+02 },
    { 2.6350000000000001e-08, 1.0842887614200001e+00, 6.1320281801484998e+03 },
    { 2.6309999999999999e-08, 2.3705219306099998e+00, 1.4749344208012999e+03 },
    { 2.6219999999999999e-08, 1.0834794159300001e+00, 3.6934270012700000e+02 },
    { 2.6219999999999999e-08, 5.7355054140700004e+00, 1.0483362299252999e+03 },
    { 2.6180000000000000e-08, 5.2783169108700001e+00, 8.7757540414020002e+02 },
    { 2.6060000000000000e-08, 2.5931012914200000e+00, 6.6109491496450005e+02 },
    { 2.5979999999999998e-08, 3.4509586350300001e+00, 2.2674332856357000e+03 },
    { 2.5979999999999998e-08, 5.9830622779000002e+00, 1.1553611574070001e+03 },
    { 2.5959999999999999e-08, 5.2516606160699997e+00, 8.2643911477819995e+02 },
    { 2.5950000000000001e-08, 3.7298234491400000e+00, 1.5861039498672001e+03 },
    { 2.5920000000000000e-08, 3.8239629377900002e+00, 1.7582045110600001e+02 },
    { 2.5920000000000000e-08, 3.4989578798699998e+00, 3.6054773846180001e+02 },
    { 2.5850000000000000e-08, 2.6737355538299998e+00, 2.1172793876700998e+03 },
    { 2.5850000000000000e-08, 1.0441065205700000e+00, 7.5403576079649997e+02 },
    { 2.5839999999999998e-08, 3.5494303967900001e+00, 2.6843697403229999e+02 },
    { 2.5780000000000000e-08, 1.2835350896800000e+00, 1.1608301730051001e+03 },
    { 2.5729999999999999e-08, 4.0960758479999998e-02, 5.8489253416370002e+02 },
    { 2.5729999999999999e-08, 2.6649824789999998e-01, 1.5362226385499000e+03 },
    { 2.5720000000000001e-08, 2.1397274865100000e+00, 5.8482884368889995e+02 },
    { 2.5699999999999999e-08, 3.4976475632000001e+00, 1.6992792165031999e+03 },
    { 2.5659999999999999e-08, 4.3955412538500003e+00, 6.3475265872500003e+01 },
    { 2.5620000000000000e-08, 5.0463198033300003e+00, 1.3741292684431000e+03 },
    { 2.5510000000000001e-08, 2.4034025821800000e+00, 1.1593457002968000e+03 },
    { 2.5329999999999999e-08, 5.9531669593199998e+00, 4.3281796599950002e+02 },
    { 2.5320000000000001e-08, 4.4622658898000003e+00, 1.5113223512999000e+03 },
    { 2.5300000000000002e-08, 2.9577351577100002e+00, 1.3307878496380999e+03 },
    { 2.5220000000000000e-08, 2.5561933673100001e+00, 4.4115535327769999e+02 },
    { 2.5089999999999998e-08, 3.5728122490200001e+00, 1.7896817590552000e+03 },
    { 2.5080000000000000e-08, 2.1145402049699999e+00, 3.4173409989129999e+02 },
    { 2.5069999999999999e-08, 1.4344380643300001e+00, 2.0278492367995688e+04 },
    { 2.5069999999999999e-08, 1.8082840217600000e+00, 3.9435486196160002e+02 },
    { 2.5060000000000001e-08, 6.2321292084600000e+00, 3.8213350553010002e+02 },
    { 2.5049999999999999e-08, 8.0744053241000002e-01, 6.7294376438050006e+02 },
    { 2.5049999999999999e-08, 3.2705752924999998e-01, 6.6812248533995999e+03 },
    { 2.4970000000000001e-08, 4.1870781244500002e+00, 5.9964663852149999e+02 },
    { 2.4739999999999999e-08, 2.8162173802299999e+00, 1.4072663692348001e+03 },
    { 2.4599999999999999e-08, 2.2728717534700000e+00, 1.2929154464312001e+03 },
    { 2.4590000000000000e-08, 5.1644781610000001e-01, 4.9426624244250002e+02 },
    { 2.4520000000000001e-08, 3.1540899055799998e+00, 2.8725061260280000e+02 },
    { 2.4509999999999999e-08, 1.7408380678800000e+00, 8.3821852822500000e+02 },
    { 2.4480000000000001e-08, 1.9861967846099999e+00, 1.3278189042214999e+03 },
    { 2.4480000000000001e-08, 5.3509794675200002e+00, 3.2032402291969998e+02 },
    { 2.4470000000000000e-08, 5.3019253154400001e+00, 2.9971709510020003e+02 },
    { 2.4369999999999999e-08, 3.8702218640000002e-01, 3.7687693825309998e+02 },
    { 2.4310000000000001e-08, 8.5207197894999998e-01, 8.3673405551669998e+02 },
    { 2.4280000000000000e-08, 4.6154118992999998e-01, 2.0275523422579088e+04 },
    { 2.4280000000000000e-08, 2.9946811754199998e+00, 2.2945260513260001e+02 },
    { 2.4270000000000002e-08, 6.0083452888100002e+00, 2.4389789294970001e+02 },
    { 2.4200000000000002e-08, 8.9948530880999999e-01, 1.7204768451537000e+03 },
    { 2.4170000000000001e-08, 3.0335527264000000e+00, 2.8362727588040002e+02 },
    { 2.4129999999999998e-08, 3.6969363591200000e+00, 6.2302556980149996e+02 },
    { 2.4109999999999999e-08, 4.7768747549299997e+00, 3.4321857259960001e+02 },
    { 2.4030000000000001e-08, 2.9470392688999998e+00, 1.1514290041439001e+03 },
    { 2.3919999999999998e-08, 2.1242741759000000e-01, 6.1462449204849997e+02 },
    { 2.3870000000000001e-08, 2.8303738359300001e+00, 7.1467888488129995e+02 },
    { 2.3840000000000000e-08, 1.7385031621000000e+00, 1.9706851359520999e+03 },
    { 2.3820000000000001e-08, 6.8810083756999996e-01, 3.2579303851780003e+02 },
    { 2.3800000000000001e-08, 9.3599236213000003e-01, 1.2584539316256000e+03 },
    { 2.3770000000000000e-08, 2.3676168500600001e+00, 1.9281992559258999e+03 },
    { 2.3680000000000001e-08, 5.9895418070700002e+00, 2.2663241756229999e+02 },
    { 2.3590000000000001e-08, 1.3044558409600000e+00, 3.7812203038989998e+02 },
    { 2.3310000000000002e-08, 5.5559756967200000e+00, 8.2081004048570003e+02 },
    { 2.3250000000000000e-08, 8.0296980817999997e-01, 2.9036831586570003e+02 },
    { 2.3190000000000001e-08, 3.3985971780600002e+00, 1.7304844764550000e+02 },
    { 2.3149999999999999e-08, 4.8559467715700002e+00, 4.2415051032119999e+02 },
    { 2.3079999999999999e-08, 4.9345410071200000e+00, 1.1703914005607000e+03 },
    { 2.2860000000000001e-08, 5.3395830819299999e+00, 5.5703893289940004e+02 },
    { 2.2830000000000000e-08, 5.6596489283600002e+00, 8.1146126125119997e+02 },
    { 2.2790000000000001e-08, 5.8421087995800001e+00, 5.9491133610630004e+02 },
    { 2.2780000000000000e-08, 5.3563285184499998e+00, 1.0262448293975001e+03 },
    { 2.2760000000000000e-08, 2.9967018556300000e+00, 1.4040849754971000e+03 },
    { 2.2740000000000000e-08, 1.8516821754799999e+00, 1.0427683096772949e+05 },
    { 2.2679999999999999e-08, 2.0518949755999999e-01, 5.3739521292639995e+02 },
    { 2.2609999999999999e-08, 4.3311050819599997e+00, 1.7369915610157000e+03 },
    { 2.2609999999999999e-08, 4.4102163363000002e-01, 8.1709033554370001e+02 },
    { 2.2600000000000001e-08, 4.9534204068400003e+00, 1.2396033446900000e+02 },
    { 2.2589999999999999e-08, 4.6806556611600003e+00, 1.1338771495710000e+02 },
    { 2.2560000000000002e-08, 1.8530020020100000e+00, 1.8117731595830001e+03 },
    { 2.2490000000000002e-08, 3.0669892461799999e+00, 1.3726447957348000e+03 },
    { 2.2309999999999999e-08, 6.0273519325200002e+00, 3.8634754308350000e+02 },
    { 2.2280000000000002e-08, 3.9266383652100001e+00, 8.4459836204290002e+02 },
    { 2.2020000000000001e-08, 4.9383673117600004e+00, 1.4365407527325999e+03 },
    { 2.1850000000000000e-08, 2.4458659836900001e+00, 2.4697871295089999e+02 },
};

static const VSOP87Term VSOP87A_URANUS_X1[649] = {
    { 7.3973002099999999e-03, 6.0106782511599999e+00, 1.4956319713459999e+02 },
    { 5.2687830599999997e-03, 3.1415926535900001e+00, 0.0000000000000000e+00 },
    { 2.3984080099999998e-03, 5.3365776270699996e+00, 7.3297125859000005e+01 },
    { 2.2967678699999999e-03, 2.4820445577500001e+00, 7.6266071275599998e+01 },
    { 1.1104515800000000e-03, 5.5715723596000002e+00, 1.1045700263900001e+01 },
    { 9.6352822000000005e-04, 3.5070389083999998e-01, 6.3735898303399999e+01 },
    { 8.1511870000000001e-04, 1.2105861803900000e+00, 8.5827298831199997e+01 },
    { 5.1382501000000005e-04, 2.1893512525999999e+00, 2.2434479570190001e+02 },
    { 4.5687564000000002e-04, 2.2921658384299999e+00, 1.3851749687070000e+02 },
    { 3.8844329999999997e-04, 3.0724575950999999e-01, 7.0849445304200003e+01 },
    { 3.6158493000000000e-04, 1.2363479875700001e+00, 7.8713751830400000e+01 },
    { 3.2333094000000002e-04, 5.0666655670400003e+00, 7.4781598567299994e+01 },
    { 2.1685656000000001e-04, 4.9371096839200002e+00, 1.5104766984290001e+02 },
    { 1.9441970000000000e-04, 1.3061749030400001e+00, 7.7750543983900002e+01 },
    { 1.7376241000000001e-04, 2.4607221230000001e-01, 7.1812653150700001e+01 },
    { 1.5211070999999999e-04, 5.5314163314000000e+00, 3.9321532631000000e+00 },
    { 7.7359839999999994e-05, 1.6134955278900001e+00, 7.1600204829600003e+01 },
    { 7.5852639999999995e-05, 2.7607421833000001e+00, 1.4807872442630000e+02 },
    { 7.4250780000000003e-05, 6.2035797711600003e+00, 7.7962992305000000e+01 },
    { 7.2916910000000000e-05, 2.2359757144399999e+00, 2.9689454166000000e+00 },
    { 6.9958569999999996e-05, 2.4063328381400000e+00, 1.4563104387150000e+02 },
    { 4.3783350000000002e-05, 3.7429632223999998e+00, 1.6060889739850001e+02 },
    { 4.1277130000000000e-05, 1.4847518130499999e+00, 2.2091400527800001e+01 },
    { 3.9335409999999997e-05, 4.7386420420800004e+00, 6.5220371011699996e+01 },
    { 3.1776139999999998e-05, 4.6522663492599996e+00, 2.9912639426919998e+02 },
    { 2.9103120000000001e-05, 5.9194133305000003e+00, 1.2747179660680000e+02 },
    { 2.8246680000000000e-05, 3.4014368567300002e+00, 8.4342826122899993e+01 },
    { 2.8023919999999999e-05, 3.7650543643400001e+00, 5.2690198039499997e+01 },
    { 2.7884339999999999e-05, 4.9011729719600003e+00, 2.1329909543799999e+02 },
    { 2.6470729999999999e-05, 4.5381317634500000e+00, 1.2530172972200001e+01 },
    { 2.5452640000000000e-05, 3.3676833762800000e+00, 9.5612275556000004e+00 },
    { 2.5252030000000000e-05, 2.8382114496100002e+00, 1.8159247264699999e+01 },
    { 2.2283959999999999e-05, 1.8087986338000001e-01, 8.7311771539500000e+01 },
    { 2.2169870000000001e-05, 4.7833890995099999e+00, 7.2333918012500007e+01 },
    { 1.9288810000000001e-05, 2.3994018031100000e+00, 3.9617508346100003e+01 },
    { 1.7472660000000000e-05, 3.2487004680900000e+00, 7.7229279122099996e+01 },
    { 1.7342280000000000e-05, 5.3961990229800003e+00, 1.4659425171800001e+02 },
    { 1.7168310000000000e-05, 3.3941566265700001e+00, 2.2286032299359999e+02 },
    { 1.7075040000000001e-05, 1.1129601210600000e+00, 2.2582926841020000e+02 },
    { 1.6244930000000000e-05, 3.7581728112700001e+00, 1.5349535039770001e+02 },
    { 1.6008650000000001e-05, 8.3762475429999997e-02, 7.9235016692200006e+01 },
    { 1.5554440000000001e-05, 4.1374166729699997e+00, 7.3818390720799997e+01 },
    { 1.5334690000000000e-05, 3.7177649804800001e+00, 1.5253214255120000e+02 },
    { 1.4766170000000001e-05, 3.5304707543900000e+00, 3.1813937377000001e+00 },
    { 1.4763170000000001e-05, 3.6728385102900001e+00, 7.5744806413800006e+01 },
    { 1.4270879999999999e-05, 1.4569075901399999e+00, 7.0328180442399997e+01 },
    { 1.2393150000000000e-05, 9.8221206501000002e-01, 4.4534181248999998e+00 },
    { 1.1796450000000000e-05, 5.1395327636700001e+00, 2.2041264243879999e+02 },
    { 1.0996910000000000e-05, 1.3513885450500001e+00, 6.2251425595100002e+01 },
    { 1.0613120000000000e-05, 5.0012510537999999e+00, 1.3140394986990000e+02 },
    { 1.0292609999999999e-05, 5.4243459786499999e+00, 1.0994568878849999e+02 },
    { 9.7712299999999993e-06, 1.0184764249500000e+00, 7.4669723982700006e+01 },
    { 9.7557099999999995e-06, 5.1903242149999995e-01, 7.4893473151899997e+01 },
    { 8.9165399999999995e-06, 4.2740512745199997e+00, 4.2659819087599999e+02 },
    { 8.6885699999999997e-06, 2.7979254480500000e+00, 5.6622351302600002e+01 },
    { 8.2826600000000001e-06, 3.3267963947900001e+00, 7.1135470007999997e+00 },
    { 7.4742000000000004e-06, 4.7845520023899999e+00, 9.2940845831999994e+01 },
    { 7.4319400000000002e-06, 5.9417939058900000e+00, 2.0618554843720000e+02 },
    { 7.2096499999999997e-06, 2.8353896717999999e-01, 1.4844727083000000e+00 },
    { 6.6611700000000003e-06, 3.4975479102799998e+00, 9.6872999095099999e+01 },
    { 6.4713600000000002e-06, 2.0307260484000000e-01, 8.0198224538700003e+01 },
    { 6.0545799999999999e-06, 1.0335604387000000e-01, 1.0593819301891999e+03 },
    { 5.7555000000000002e-06, 2.4289766254599998e+00, 5.2257741809380002e+02 },
    { 5.6379099999999997e-06, 1.9150093351799999e+00, 5.3680451209540001e+02 },
    { 4.9779100000000001e-06, 3.5005484996499998e+00, 1.3703302416240001e+02 },
    { 4.7725399999999999e-06, 1.1579116796000000e+00, 4.1101981054399999e+01 },
    { 4.5288900000000001e-06, 1.0971048212600001e+00, 6.7668051566499997e+01 },
    { 4.1257199999999999e-06, 3.7895300346799998e+00, 6.6704843720000000e+01 },
    { 4.0850300000000003e-06, 3.3951833273999998e-01, 8.1895145568100006e+01 },
    { 3.9574200000000001e-06, 1.5889396495500001e+00, 1.4977853527000001e+01 },
    { 3.7781299999999999e-06, 3.7335376142799999e+00, 2.4476805547999998e+00 },
    { 3.6926500000000001e-06, 4.7318314111099999e+00, 2.0225339517410001e+02 },
    { 3.4720899999999999e-06, 4.0974037857600001e+00, 1.4638180339690001e+02 },
    { 3.4701200000000002e-06, 6.1599315509099997e+00, 1.3458534360760001e+02 },
    { 3.4627599999999999e-06, 3.8864775038100001e+00, 5.9803745040300001e+01 },
    { 3.4371399999999999e-06, 4.4244634275500001e+00, 1.4244965013379999e+02 },
    { 3.4083000000000000e-06, 1.8489368918600000e+00, 3.6648562929500002e+01 },
    { 3.2914800000000000e-06, 5.8410328396900004e+00, 1.1291463420510000e+02 },
    { 3.0647600000000000e-06, 1.0659060072600000e+00, 3.3679617512900002e+01 },
    { 2.9415999999999999e-06, 4.4209628216899999e+00, 5.4166259714000002e+00 },
    { 2.9312500000000001e-06, 4.9846117327000000e-01, 1.4510977900969999e+02 },
    { 2.7543699999999999e-06, 4.1047979330300004e+00, 8.2858353414600003e+01 },
    { 2.5462800000000000e-06, 6.2547367854899996e+00, 2.3539049596580000e+02 },
    { 2.4994900000000001e-06, 6.7221714016000000e-01, 2.6598929347749998e+02 },
    { 2.4650499999999998e-06, 5.3202443808600002e+00, 1.5912442469019999e+02 },
    { 2.3577299999999999e-06, 3.6265375125200001e+00, 2.0606927819500001e+01 },
    { 2.3113700000000000e-06, 3.8954829748000003e-01, 1.0846121608020000e+02 },
    { 2.2803099999999999e-06, 5.7383250459099999e+00, 2.9764192156090002e+02 },
    { 2.2148400000000002e-06, 2.7193974699500001e+00, 1.9513984817330001e+02 },
    { 2.1718300000000001e-06, 4.2558321906199996e+00, 1.5274459087229999e+02 },
    { 2.1034300000000000e-06, 4.6333662262799997e+00, 1.4000196957899999e+02 },
    { 1.9282500000000001e-06, 4.5904824372000003e-01, 2.3575873236100001e+01 },
    { 1.9177800000000000e-06, 8.3177362607000005e-01, 3.7390799283650000e+02 },
    { 1.8391999999999999e-06, 5.0522848654999999e+00, 1.9120769491019999e+02 },
    { 1.8176000000000000e-06, 1.5082081942600001e+00, 1.2035824960600000e+02 },
    { 1.7781700000000000e-06, 2.3832727962600000e+00, 2.0936694217490000e+02 },
    { 1.6498200000000000e-06, 6.0320986718700000e+00, 5.9378908332000000e+00 },
    { 1.6092700000000000e-06, 4.7761445809299996e+00, 8.0767548473000002e+00 },
    { 1.5647100000000001e-06, 3.6204994671500002e+00, 1.1642609634290000e+02 },
    { 1.5438299999999999e-06, 9.0699803576000004e-01, 2.8808069400530002e+02 },
    { 1.5408999999999999e-06, 3.2570848689699998e+00, 1.4014645680499999e+01 },
    { 1.5343800000000000e-06, 2.7008822313700001e+00, 1.6209337010679999e+02 },
    { 1.5091600000000000e-06, 1.3081718247100000e+00, 6.9364972595899999e+01 },
    { 1.4800499999999999e-06, 3.0190183659500001e+00, 1.8621176006409999e+02 },
    { 1.4351900000000000e-06, 5.8083938274499998e+00, 2.1181462272970001e+02 },
    { 1.4203199999999999e-06, 4.7259948335199997e+00, 1.1439910691340000e+02 },
    { 1.4170299999999999e-06, 2.6487429828800000e+00, 6.8843707734099993e+01 },
    { 1.4119500000000001e-06, 5.8353201180900003e+00, 4.5576651038700000e+01 },
    { 1.4061200000000001e-06, 5.3593169975399997e+00, 2.9204947528600002e+01 },
    { 1.4027500000000000e-06, 5.1592511092700004e+00, 8.0719489400499995e+01 },
    { 1.3843499999999999e-06, 3.8546052988000001e-01, 8.1001373690799994e+01 },
    { 1.3461400000000000e-06, 4.1866243812999998e-01, 3.9356875915200000e+01 },
    { 1.3297900000000001e-06, 1.1691955656099999e+00, 6.8561823443799994e+01 },
    { 1.3227099999999999e-06, 3.9310240059899999e+00, 8.9759452094300002e+01 },
    { 1.2406900000000000e-06, 1.4858988807800000e+00, 4.1644497775600001e+01 },
    { 1.2291499999999999e-06, 3.5718897223599999e+00, 3.0061086697750000e+02 },
    { 1.2104099999999999e-06, 2.5171459220100001e+00, 1.5401661525950001e+02 },
    { 1.1999200000000001e-06, 2.8840786667099998e+00, 5.0402576179100002e+01 },
    { 1.1866700000000000e-06, 9.7921601471999997e-01, 3.0534616939270001e+02 },
    { 1.1764500000000000e-06, 1.0457724156999999e+00, 2.7703499374140000e+02 },
    { 1.1665599999999999e-06, 4.7776862257600001e+00, 9.9160620955499994e+01 },
    { 1.1627900000000001e-06, 6.1626357831299998e+00, 2.2731374111849999e+02 },
    { 1.1585499999999999e-06, 3.5060831458499999e+00, 3.3137100791700000e+01 },
    { 1.1090099999999999e-06, 4.6927738771399996e+00, 5.1205725331200000e+01 },
    { 1.0953999999999999e-06, 5.2311635727799999e+00, 8.8796244247800004e+01 },
    { 1.0377600000000000e-06, 2.1028785403599999e+00, 5.4174670747800000e+01 },
    { 1.0321400000000001e-06, 3.7162926907000000e+00, 1.4414657116320001e+02 },
    { 1.0273600000000001e-06, 5.3476316547099998e+00, 1.5201087768939999e+02 },
    { 9.1187000000000003e-07, 1.1048119314700000e+00, 1.1020632121940000e+02 },
    { 9.0202000000000003e-07, 5.6709477474000003e-01, 1.2598732389849999e+02 },
    { 8.7580000000000005e-07, 4.8126511918699997e+00, 4.1948464387519999e+02 },
    { 8.4840000000000000e-07, 6.2106679380300003e+00, 4.2586453762700003e+01 },
    { 8.1722999999999997e-07, 6.2707929781900003e+00, 2.2827694896500000e+02 },
    { 8.1515000000000004e-07, 2.3337944895999998e+00, 3.2195144804599998e+01 },
    { 8.0037000000000001e-07, 4.4174326982499998e+00, 6.2197751234999998e+00 },
    { 7.4651000000000003e-07, 4.4051987559199999e+00, 1.2895626931510000e+02 },
    { 7.4140999999999997e-07, 6.1265141356399999e+00, 1.5052640498110000e+02 },
    { 7.4094999999999997e-07, 2.5759427545000002e-01, 1.4859998928810001e+02 },
    { 7.3275999999999995e-07, 5.3584067574200001e+00, 9.5388526386799995e+01 },
    { 7.2987999999999997e-07, 3.5194544812699999e+00, 6.3989728631399998e+02 },
    { 7.2236000000000002e-07, 1.9438572716100000e+00, 2.4379022388199999e+01 },
    { 7.1854999999999999e-07, 2.4025635640299998e+00, 4.0450679034820001e+02 },
    { 7.1445000000000000e-07, 1.5241442474000000e+00, 1.0697674337190000e+02 },
    { 6.9920000000000001e-07, 5.0669526659499997e+00, 2.3056457082540001e+02 },
    { 6.9373000000000001e-07, 4.5036551185700002e+00, 3.3365734504400001e+02 },
    { 6.8988999999999999e-07, 2.6836716008199999e+00, 1.8324281464750001e+02 },
    { 6.8100999999999998e-07, 4.8671149857099998e+00, 2.8571808082200000e+01 },
    { 6.7741000000000003e-07, 2.3367851706000001e-01, 1.6674774556399999e+01 },
    { 6.0943000000000002e-07, 4.0442703934199997e+00, 3.4788443904560000e+02 },
    { 6.0704999999999997e-07, 5.8473821984900001e+00, 7.0115732121299999e+01 },
    { 6.0413000000000005e-07, 2.9387944866300000e+00, 1.4967507171919999e+02 },
    { 5.6769999999999998e-07, 1.7600162132599999e+00, 1.9643719973000000e+01 },
    { 5.6672999999999999e-07, 5.4290172829000005e-01, 7.3084677537900006e+01 },
    { 5.5718000000000002e-07, 5.1010875325000002e+00, 7.9447465013300004e+01 },
    { 5.5708999999999997e-07, 1.0599541457000000e-01, 1.5763995198190000e+02 },
    { 5.4799999999999998e-07, 4.1225852781199999e+00, 7.6478519596699996e+01 },
    { 5.3588000000000000e-07, 5.4527702807000000e-01, 3.2972519178089999e+02 },
    { 5.3425000000000001e-07, 4.4880695755900000e+00, 4.3371173787679999e+02 },
    { 5.2956000000000004e-07, 4.2430407701200004e+00, 6.6917292041099998e+01 },
    { 5.2758000000000002e-07, 1.7137946551000001e+00, 1.4362530630140000e+02 },
    { 5.2262999999999996e-07, 1.2277041673600000e+00, 1.4711551657979999e+02 },
    { 4.8630999999999997e-07, 3.0875032957399999e+00, 6.8189316428300003e+01 },
    { 4.6174000000000000e-07, 1.8353738470500001e+00, 7.8643065262000000e+00 },
    { 4.3940999999999999e-07, 4.2228992459699999e+00, 7.5532358092699994e+01 },
    { 4.3779000000000000e-07, 1.3486243711699999e+00, 4.1555249061209997e+02 },
    { 4.3331000000000001e-07, 4.5599585461999997e-01, 1.5890728952838001e+03 },
    { 4.2882000000000000e-07, 3.6095327495200000e+00, 7.4030839041899995e+01 },
    { 4.1612000000000000e-07, 1.4136111259699999e+00, 5.8106824010899999e+01 },
    { 4.1250999999999998e-07, 1.5714466465900001e+00, 2.7310284047829998e+02 },
    { 4.0824000000000001e-07, 5.9730322581799999e+00, 3.4077089204480001e+02 },
    { 4.0256999999999998e-07, 2.7086694630700001e+00, 1.5497982310600000e+02 },
    { 4.0028999999999999e-07, 1.9623996824300001e+00, 1.8769623277240001e+02 },
    { 3.9553000000000000e-07, 2.7895355699200000e+00, 2.5887574647669999e+02 },
    { 3.7479999999999999e-07, 4.6969053260500004e+00, 8.1373880706300000e+01 },
    { 3.7429000000000000e-07, 7.9282523593999998e-01, 7.3887826689999997e+01 },
    { 3.7429000000000000e-07, 7.4498454647000001e-01, 7.5675370444600006e+01 },
    { 3.7371999999999999e-07, 2.1122306999000001e-01, 2.5950888592310002e+02 },
    { 3.7066000000000001e-07, 2.1770810520600001e+00, 4.6470422915999997e+01 },
    { 3.6050999999999999e-07, 4.8141749417300002e+00, 1.3554855145409999e+02 },
    { 3.5447000000000002e-07, 9.8545964790999996e-01, 1.6772244439930000e+02 },
    { 3.5363000000000002e-07, 3.4222762797800002e+00, 8.2645905093500005e+01 },
    { 3.4620000000000002e-07, 2.5430186885700001e+00, 1.4227094001599999e+01 },
    { 3.3926999999999999e-07, 6.0271325840400003e+00, 1.7165459766239999e+02 },
    { 3.3444000000000002e-07, 3.7241130213600000e+00, 1.4945132254999999e+02 },
    { 3.1512999999999999e-07, 3.0822049569900001e+00, 2.2137585028530000e+02 },
    { 3.1192000000000000e-07, 4.1227933942300004e+00, 5.2969096509459996e+02 },
    { 3.1110000000000003e-07, 1.7303149137100000e+00, 1.0522683831884001e+03 },
    { 3.1094000000000000e-07, 2.3933468627200001e+00, 9.8357471803400003e+01 },
    { 3.1064999999999999e-07, 1.7182769587100000e+00, 3.8133035637799999e+01 },
    { 3.0737999999999999e-07, 5.9077470634499996e+00, 1.1143016149680000e+02 },
    { 3.0708000000000002e-07, 2.4881030437000001e-01, 7.2493976706900000e+01 },
    { 3.0708000000000002e-07, 1.2889994780400000e+00, 7.7069220427700003e+01 },
    { 2.9956000000000002e-07, 2.5708201522700000e+00, 6.0766952886799999e+01 },
    { 2.9829000000000001e-07, 2.3428321306600002e+00, 1.0664954771900000e+03 },
    { 2.9560999999999998e-07, 5.6953803771800002e+00, 1.0309277421860000e+02 },
    { 2.9458999999999999e-07, 3.0742608348800000e+00, 1.2099138905240000e+02 },
    { 2.9298000000000002e-07, 5.9488223989400000e+00, 1.4169889060840001e+02 },
    { 2.9097999999999998e-07, 2.9409756727000000e+00, 1.8472728735580000e+02 },
    { 2.8874999999999999e-07, 5.4752160833999997e-01, 2.8311175651300001e+01 },
    { 2.8112999999999998e-07, 9.0670132999999997e-01, 2.9519424100610001e+02 },
    { 2.7631000000000002e-07, 5.9583025449799996e+00, 6.5922821389999999e+00 },
    { 2.6884000000000002e-07, 3.0405723004600000e+00, 2.1989137757699999e+02 },
    { 2.6708999999999999e-07, 4.7559891637599998e+00, 4.3289029178299998e+01 },
    { 2.6658000000000000e-07, 3.7099096338999998e-01, 2.1723124870109999e+02 },
    { 2.6063999999999998e-07, 6.2596384983800002e+00, 4.7217484191469998e+02 },
    { 2.6035000000000003e-07, 5.1844176177000001e-01, 9.1456373123700004e+01 },
    { 2.4906999999999999e-07, 5.7671873005299998e+00, 4.8640193591629998e+02 },
    { 2.4834000000000001e-07, 2.6878086986500000e+00, 2.5272794265500000e+01 },
    { 2.3975000000000001e-07, 5.9167713711700003e+00, 2.1892816973050000e+02 },
    { 2.3887999999999998e-07, 6.0348813517500002e+00, 3.5164090221199999e+01 },
    { 2.3868999999999998e-07, 4.2363028400000001e-03, 5.7255490749000003e+01 },
    { 2.3509000000000000e-07, 2.6162679940400002e+00, 4.7694263193399998e+01 },
    { 2.3342999999999999e-07, 4.4288819927900001e+00, 6.1448276442999997e+01 },
    { 2.3216000000000001e-07, 4.0942867491200001e+00, 5.4302428721889999e+02 },
    { 2.2980000000000001e-07, 4.9093754166699997e+00, 2.5494359321360000e+02 },
    { 2.2926000000000000e-07, 1.3016085291999999e+00, 2.3390602325750001e+02 },
    { 2.2287999999999999e-07, 6.1479505021899996e+00, 2.2752618943959999e+02 },
    { 2.1446000000000000e-07, 3.1722461883099999e+00, 1.0627416795630000e+02 },
    { 2.1432999999999999e-07, 4.6412361829600002e+00, 1.9122455111200001e+01 },
    { 2.1268000000000000e-07, 2.9792221271099999e+00, 5.5406998748280000e+02 },
    { 2.1162000000000001e-07, 1.8708889025300000e+00, 3.7242352012819998e+02 },
    { 2.0924000000000001e-07, 2.7661493195800002e+00, 2.6992144674060000e+02 },
    { 2.0909000000000000e-07, 3.8597420386999999e+00, 6.7359235025800004e+01 },
    { 2.0847000000000001e-07, 4.8140990456499999e+00, 7.4223635415000002e+00 },
    { 2.0526000000000000e-07, 3.5588347511400000e+00, 1.1588357962169999e+02 },
    { 2.0200999999999999e-07, 2.1241784464700002e+00, 4.6824268865160002e+02 },
    { 2.0130999999999999e-07, 3.7734988080599998e+00, 3.1639186965660002e+02 },
    { 1.9679999999999999e-07, 3.7692395289000002e-01, 2.9615744885260000e+02 },
    { 1.9604000000000000e-07, 2.4579731068899999e+00, 1.7265475387399999e+01 },
    { 1.9458000000000001e-07, 4.7679260392399998e+00, 9.8899988524600005e+01 },
    { 1.9282999999999999e-07, 1.1009667875400000e+00, 4.7928838891549998e+02 },
    { 1.8795000000000001e-07, 8.6065812507999995e-01, 6.0554504565700000e+01 },
    { 1.8710000000000000e-07, 2.0700210269200001e+00, 6.9673789136600007e+01 },
    { 1.8460000000000000e-07, 3.1125119729200001e+00, 3.5424722652100002e+01 },
    { 1.8374000000000000e-07, 3.1297282894800000e+00, 8.8114920691600005e+01 },
    { 1.8185000000000000e-07, 1.5354922268800000e+00, 9.2307706385600000e+01 },
    { 1.7982999999999999e-07, 3.2307609192600002e+00, 1.5126011816400000e+02 },
    { 1.7966999999999999e-07, 3.8822214485300002e+00, 1.8175834193919999e+02 },
    { 1.7819000000000001e-07, 2.7595854332999997e-01, 2.2116340196420001e+02 },
    { 1.7763999999999999e-07, 2.7554934165199998e+00, 1.0549227066360000e+02 },
    { 1.7442000000000000e-07, 2.8080008381599999e+00, 7.6025553592000006e+02 },
    { 1.7345000000000001e-07, 5.4923348473500004e+00, 2.6099335863139999e+02 },
    { 1.7317999999999999e-07, 3.5818381404299999e+00, 1.0038446123289999e+02 },
    { 1.7226999999999999e-07, 2.9402089847000001e-01, 1.3333322124300000e+01 },
    { 1.7081999999999999e-07, 5.7509738013999998e+00, 7.9889407997999996e+01 },
    { 1.7060999999999999e-07, 9.3145915427000003e-01, 1.8409414790939999e+02 },
    { 1.6920000000000001e-07, 5.7627710121600000e+00, 6.7880499887599996e+01 },
    { 1.6772000000000000e-07, 6.9703195541999996e-01, 2.1033015002139999e+02 },
    { 1.6749000000000000e-07, 1.4588844906700000e+00, 2.8414854074220000e+02 },
    { 1.6712000000000000e-07, 1.2264770471999999e+00, 5.5659143456099997e+01 },
    { 1.6556000000000001e-07, 5.5851487905299999e+00, 1.9832124191099999e+02 },
    { 1.6506000000000000e-07, 3.1274801692700001e+00, 7.6426129970000005e+01 },
    { 1.6506000000000000e-07, 4.6935149203200002e+00, 7.3137067164599998e+01 },
    { 1.6331000000000000e-07, 3.6571626273300000e+00, 9.4425318540299997e+01 },
    { 1.6285000000000001e-07, 5.5773221064799996e+00, 2.4118389957300000e+01 },
    { 1.6262000000000001e-07, 1.9991252346699999e+00, 6.4486657828800006e+01 },
    { 1.5825999999999999e-07, 3.1249952755099999e+00, 1.3493380818700000e+01 },
    { 1.5636000000000001e-07, 3.3259279823900001e+00, 1.7394221952280000e+02 },
    { 1.5543999999999999e-07, 4.9642345932899996e+00, 5.8319272331999997e+01 },
    { 1.5379999999999999e-07, 4.5042165283000002e-01, 1.6462326235300001e+01 },
    { 1.5260999999999999e-07, 4.5218070979300000e+00, 3.9739324334740002e+02 },
    { 1.5207000000000001e-07, 3.3496234340500002e+00, 4.1162033734900001e+02 },
    { 1.5036000000000000e-07, 1.8676512357399999e+00, 1.3288842257819999e+02 },
    { 1.4998000000000000e-07, 9.3967600270000007e-02, 2.0076892246579999e+02 },
    { 1.4970000000000000e-07, 3.7216302662200000e+00, 3.0056280790500001e+01 },
    { 1.4873000000000001e-07, 3.5684608560800002e+00, 3.0710672096300001e+01 },
    { 1.4725000000000000e-07, 5.1231296658199996e+00, 1.3533610313299999e+02 },
    { 1.4560999999999999e-07, 3.1138496966100000e+00, 5.0663208609999998e+01 },
    { 1.4306000000000001e-07, 2.4306036163600000e+00, 3.1017209453309999e+02 },
    { 1.4154999999999999e-07, 2.8654310692000000e+00, 1.5578297225809999e+02 },
    { 1.3967999999999999e-07, 2.6177295540399999e+00, 6.1288217748599997e+01 },
    { 1.3827999999999999e-07, 5.3649350891500003e+00, 3.1652628083400000e+01 },
    { 1.3824000000000000e-07, 4.2370796234599997e+00, 2.6023553790899999e+01 },
    { 1.3682999999999999e-07, 3.9222111634200001e+00, 8.2203962108799999e+01 },
    { 1.3680999999999999e-07, 4.9792942574600003e+00, 4.4070926471000000e+01 },
    { 1.3533000000000001e-07, 5.8219260570799998e+00, 8.5076539305799997e+01 },
    { 1.3283000000000001e-07, 2.8219182438399999e+00, 6.9010986796999996e+00 },
    { 1.3204000000000001e-07, 3.5082463716099999e+00, 1.2353964334370001e+02 },
    { 1.3099000000000001e-07, 2.0141742289900000e+00, 2.8659622129700000e+02 },
    { 1.2982999999999999e-07, 7.5249502704000004e-01, 1.7368158709190001e+02 },
    { 1.2959000000000001e-07, 9.3518082352999998e-01, 1.3310087089929999e+02 },
    { 1.2872000000000000e-07, 3.9580401426699998e+00, 4.6209790485100001e+01 },
    { 1.2734000000000000e-07, 1.7020891303900001e+00, 7.4260333705500003e+01 },
    { 1.2734000000000000e-07, 2.9773133056100001e+00, 7.5302863429100000e+01 },
    { 1.2701000000000000e-07, 1.7102796389499999e+00, 5.7516123179899999e+01 },
    { 1.2678000000000000e-07, 5.2125576959400002e+00, 8.1682697246999993e+01 },
    { 1.2526000000000000e-07, 3.6195836863799999e+00, 3.5181659230870002e+02 },
    { 1.2333999999999999e-07, 6.2625977447999999e+00, 6.3278373931320004e+02 },
    { 1.2255000000000000e-07, 2.5242887085499999e+00, 5.1546387109299997e+02 },
    { 1.2080000000000000e-07, 5.1641582377399997e+00, 7.4127207261500004e+01 },
    { 1.2080000000000000e-07, 2.6568368518500001e+00, 7.5435989873099999e+01 },
    { 1.1855000000000000e-07, 2.9701973028199999e+00, 2.6747376618580000e+02 },
    { 1.1679000000000000e-07, 1.3306012832000000e+00, 4.8758044776399998e+01 },
    { 1.1542000000000000e-07, 4.1898877618499997e+00, 4.2538269652899999e+01 },
    { 1.1443000000000000e-07, 1.7805162022400001e+00, 3.2243328914400003e+01 },
    { 1.1401000000000000e-07, 1.5944553129600001e+00, 2.6450482076920002e+02 },
    { 1.1303000000000000e-07, 9.3310421001999999e-01, 3.1492569389000000e+01 },
    { 1.0953000000000000e-07, 5.6341673986999996e-01, 8.9008692568900003e+01 },
    { 1.0910000000000000e-07, 2.9100418248200000e+00, 1.4214083359310001e+02 },
    { 1.0787000000000000e-07, 6.0114246431899998e+00, 1.8972322220190000e+02 },
    { 1.0751000000000000e-07, 1.1981906654400001e+00, 6.2491943278700001e+02 },
    { 1.0738000000000000e-07, 2.4602380812999999e-01, 9.3904053678500006e+01 },
    { 1.0652000000000000e-07, 5.0748694254900002e+00, 1.2429040286910001e+02 },
    { 1.0460000000000000e-07, 4.8159010386199999e+00, 6.9258748435350003e+02 },
    { 1.0459000000000000e-07, 5.1219132115399999e+00, 1.4489733068859999e+02 },
    { 1.0123000000000000e-07, 5.1857132070200000e+00, 2.3687496867409999e+02 },
    { 1.0035000000000000e-07, 3.2968632002199998e+00, 4.4868959140380002e+02 },
    { 1.0000999999999999e-07, 1.2559588436500000e+00, 5.3287235883230005e+02 },
    { 9.9779999999999997e-08, 3.9778465770000002e+00, 2.1478356814630001e+02 },
    { 9.9490000000000000e-08, 6.0006605778899997e+00, 1.2991947716160001e+02 },
    { 9.8360000000000002e-08, 5.9524449257600001e+00, 6.8154178408960001e+02 },
    { 9.8319999999999996e-08, 2.8313198823099999e+00, 5.4334729442200000e+01 },
    { 9.7780000000000006e-08, 5.7283679628400002e+00, 4.3128970483899998e+01 },
    { 9.7390000000000001e-08, 5.2815818866399997e+00, 7.3185251274400002e+01 },
    { 9.7390000000000001e-08, 5.6810058565399997e+00, 7.6377945860200001e+01 },
    { 9.5560000000000003e-08, 4.9481332433800000e+00, 1.9662432088160000e+02 },
    { 9.5519999999999998e-08, 1.1913230696899999e+00, 1.5615547927360001e+02 },
    { 9.2970000000000005e-08, 3.7921785794899998e+00, 2.0373786788240000e+02 },
    { 9.1699999999999994e-08, 1.8882338324000000e-01, 6.8547393735269998e+02 },
    { 9.1440000000000004e-08, 4.9625312048300003e+00, 4.4725317776799997e+01 },
    { 9.1370000000000004e-08, 2.6652161111599999e+00, 6.1780588578619995e+02 },
    { 8.9999999999999999e-08, 4.3241005010400002e+00, 2.1085141488319999e+02 },
    { 8.9719999999999999e-08, 1.0072096939000000e-01, 2.7087335373900000e+01 },
    { 8.9470000000000006e-08, 4.9097448473999999e-01, 1.4148644228730001e+02 },
    { 8.9309999999999997e-08, 1.7546419896200001e+00, 1.1791056905120000e+02 },
    { 8.8440000000000003e-08, 5.1369162239000001e+00, 8.8274979385999998e+01 },
    { 8.8090000000000003e-08, 4.6047911145600002e+00, 1.1494162363460001e+02 },
    { 8.7410000000000000e-08, 4.7864008403199998e+00, 5.8739963457300000e+01 },
    { 8.6949999999999994e-08, 3.8677379612400000e+00, 1.9365537546499999e+02 },
    { 8.5240000000000001e-08, 4.9396392981700004e+00, 1.7526107818300002e+01 },
    { 8.5209999999999993e-08, 1.3062302475900001e+00, 1.5550108796780000e+02 },
    { 8.4919999999999995e-08, 1.0466908901600001e+00, 1.0398654609590000e+02 },
    { 8.4390000000000003e-08, 2.9729894976600000e+00, 1.8016199464630000e+02 },
    { 8.4019999999999994e-08, 1.6081220686000000e+00, 7.9516900982500005e+01 },
    { 8.4019999999999994e-08, 6.2128730209900001e+00, 7.0046296152099998e+01 },
    { 8.3960000000000005e-08, 1.8527407828500000e+00, 6.1069233878540001e+02 },
    { 8.3719999999999998e-08, 4.1553289431099998e+00, 2.1840690486870000e+02 },
    { 8.2630000000000006e-08, 1.7938277112300001e+00, 5.4391805909619995e+02 },
    { 8.2020000000000003e-08, 1.7013703997600000e+00, 1.9269216761850001e+02 },
    { 8.1569999999999995e-08, 1.4128034661300000e+00, 1.6357784281510001e+02 },
    { 8.0449999999999996e-08, 2.5649980645900001e+00, 4.0160025067299998e+01 },
    { 7.8079999999999995e-08, 3.5008742637800001e+00, 4.7061123747000003e+01 },
    { 7.7849999999999999e-08, 1.9160038735700000e+00, 1.7787437278589999e+02 },
    { 7.6199999999999994e-08, 3.7353833564899999e+00, 5.5137878594299998e+01 },
    { 7.4429999999999999e-08, 5.6378602661999997e+00, 1.7516605980020000e+02 },
    { 7.4200000000000003e-08, 3.4107683707400001e+00, 1.3044074202339999e+02 },
    { 7.3780000000000003e-08, 5.8427020908799996e+00, 1.0931254934210000e+02 },
    { 7.1589999999999995e-08, 3.9750871568999999e+00, 6.0676018552230005e+02 },
    { 7.0930000000000001e-08, 2.5272225400899999e+00, 1.1887377689770000e+02 },
    { 7.0679999999999995e-08, 3.3028218908700002e+00, 5.2650957135689998e+02 },
    { 7.0669999999999997e-08, 6.1274640100399997e+00, 9.4917560896980001e+02 },
    { 7.0399999999999995e-08, 1.6766290094000000e-01, 3.9346109008430000e+02 },
    { 6.9969999999999997e-08, 4.1136701921999999e+00, 1.1950691634410001e+02 },
    { 6.9730000000000003e-08, 2.3810565254100000e+00, 1.6289651925890001e+02 },
    { 6.8770000000000000e-08, 3.4495714272400000e+00, 2.0788246946660001e+02 },
    { 6.8369999999999996e-08, 6.0615756326200003e+00, 3.7539246554480002e+02 },
    { 6.8070000000000000e-08, 6.0040366307299999e+00, 1.9907200143640000e+02 },
    { 6.8029999999999994e-08, 4.9896708449700000e+00, 9.5228467692400002e+01 },
    { 6.7799999999999998e-08, 8.8290624841999998e-01, 7.1158261844899997e+01 },
    { 6.7799999999999998e-08, 6.5490353399000001e-01, 7.8404935289700006e+01 },
    { 6.7440000000000000e-08, 1.0807370671000001e+00, 2.7720474820300002e+01 },
    { 6.6510000000000005e-08, 3.5693343194999998e+00, 1.0749800823370001e+02 },
    { 6.6500000000000007e-08, 4.2806337965000001e+00, 4.5490936652729999e+02 },
    { 6.6310000000000003e-08, 3.9912484696799999e+00, 3.4530950774799997e+01 },
    { 6.6040000000000001e-08, 3.0352752605100002e+00, 9.0823233677299996e+01 },
    { 6.5620000000000001e-08, 5.8246247424200002e+00, 1.1731986822019999e+02 },
    { 6.5369999999999996e-08, 2.9978515707400000e+00, 3.6286229257259998e+02 },
    { 6.5140000000000000e-08, 2.3552085991900000e+00, 3.0209533968580001e+02 },
    { 6.5120000000000003e-08, 5.9613574109699998e+00, 9.7415515816300001e+01 },
    { 6.3909999999999995e-08, 5.7043963033400003e+00, 4.9721252622900003e+01 },
    { 6.3450000000000002e-08, 2.7175517280100001e+00, 9.1243924802600006e+01 },
    { 6.3189999999999999e-08, 3.5771162084900001e+00, 1.4334342201109999e+02 },
    { 6.3080000000000006e-08, 3.4505692022000001e-01, 8.3110498122419995e+02 },
    { 6.2999999999999995e-08, 5.0786828642200001e+00, 5.5013783421970004e+02 },
    { 6.2219999999999997e-08, 1.1639128621499999e+00, 3.3683873878169999e+02 },
    { 6.1959999999999994e-08, 3.5442158729700002e+00, 3.8109097580650001e+02 },
    { 6.1959999999999994e-08, 5.5417607462899996e+00, 5.2872775724810003e+02 },
    { 6.0800000000000002e-08, 5.0766936598100001e+00, 5.3065417294110000e+02 },
    { 6.0800000000000002e-08, 4.0092824286499997e+00, 3.7916456011349999e+02 },
    { 6.0220000000000006e-08, 1.3129705354600001e+00, 1.0538039607899999e+02 },
    { 5.9690000000000000e-08, 3.1129028270000001e-02, 5.3591074021810005e+02 },
    { 5.8810000000000002e-08, 4.6027344078599999e+00, 4.0599126305649997e+02 },
    { 5.8740000000000002e-08, 9.6815943251000003e-01, 4.0610313764109998e+02 },
    { 5.8630000000000003e-08, 5.6046270277100003e+00, 1.0791869935900000e+02 },
    { 5.8400000000000000e-08, 4.7427599474999997e+00, 2.9226199388000001e+01 },
    { 5.8220000000000001e-08, 4.3126073661800000e+00, 3.0141401612959999e+02 },
    { 5.7569999999999999e-08, 2.2867163841800000e+00, 4.0843894361129998e+02 },
    { 5.6550000000000000e-08, 5.8180438061999995e-01, 4.4092178330400003e+01 },
    { 5.5950000000000001e-08, 2.8232330980300002e+00, 3.2261164478009999e+02 },
    { 5.5869999999999997e-08, 5.4084634638100004e+00, 1.2518417474640000e+02 },
    { 5.5770000000000002e-08, 5.5454338836500003e+00, 2.3767811782620001e+02 },
    { 5.5479999999999998e-08, 3.2581030915600002e+00, 4.0302231763990000e+02 },
    { 5.5320000000000002e-08, 3.1891843881999998e-01, 1.6454105066160000e+02 },
    { 5.5159999999999999e-08, 2.0833968029399998e+00, 2.3242155054919999e+02 },
    { 5.4980000000000000e-08, 5.0140083202500003e+00, 1.4266209845489999e+02 },
    { 5.4529999999999999e-08, 4.0689487025600002e+00, 2.3204904353370000e+02 },
    { 5.4249999999999999e-08, 1.6380669642000001e-01, 7.6154196690999996e+01 },
    { 5.4249999999999999e-08, 4.5155957395800002e+00, 7.3409000443599993e+01 },
    { 5.4210000000000000e-08, 2.4248299087800000e+00, 4.9508804301799998e+01 },
    { 5.4140000000000000e-08, 2.3481115779000000e-01, 1.6041635110000001e+01 },
    { 5.3710000000000003e-08, 2.4693205143500001e+00, 2.9370976829780000e+02 },
    { 5.2899999999999997e-08, 6.2520677302500003e+00, 2.1011770170029999e+02 },
    { 5.2590000000000003e-08, 5.5308987035700001e+00, 3.1231936958100000e+01 },
    { 5.2520000000000003e-08, 4.3746136394499997e+00, 4.9178735901700001e+01 },
    { 5.1770000000000000e-08, 2.4240891570700001e+00, 3.4621573499999997e+01 },
    { 5.1679999999999997e-08, 2.9765826537800000e+00, 1.4786627610520000e+02 },
    { 5.1669999999999999e-08, 3.4959072057200000e+00, 1.0702492748170000e+02 },
    { 5.1170000000000001e-08, 4.2750349147300000e+00, 1.1413847448250000e+02 },
    { 5.0920000000000002e-08, 1.8369723363299999e+00, 3.7694637422229999e+02 },
    { 5.0390000000000003e-08, 4.9473492002000002e-01, 2.2633917249000000e+01 },
    { 5.0330000000000001e-08, 2.5586916535399999e+00, 2.3449672408850000e+02 },
    { 5.0290000000000002e-08, 8.5996015449999996e-01, 1.8918070548070000e+02 },
    { 4.9840000000000002e-08, 8.6380095340999996e-01, 3.1245971639350000e+02 },
    { 4.9789999999999998e-08, 1.2086892230299999e+00, 9.1375088631769995e+02 },
    { 4.8720000000000002e-08, 4.6958705633799998e+00, 3.5685355082999997e+01 },
    { 4.8100000000000001e-08, 6.9226746482000001e-01, 5.9282480178500002e+01 },
    { 4.8079999999999998e-08, 5.1781327746099999e+00, 8.4215068148809996e+02 },
    { 4.7829999999999999e-08, 2.9491075634700001e+00, 7.6343692965770003e+02 },
    { 4.7670000000000003e-08, 4.9687331699300001e+00, 2.8096714700450002e+02 },
    { 4.7400000000000001e-08, 2.8957740938900001e+00, 8.5319638175199998e+02 },
    { 4.6509999999999998e-08, 5.3676435318799998e+00, 8.2792358748649997e+02 },
    { 4.6229999999999998e-08, 6.1866849692999999e+00, 3.4470304530790003e+02 },
    { 4.5900000000000001e-08, 2.5074321700100000e+00, 7.6736908292080000e+02 },
    { 4.5699999999999999e-08, 2.7475223191999998e-01, 3.3514181775230003e+02 },
    { 4.5699999999999999e-08, 5.4830813614300000e+00, 2.5060345944400002e+01 },
    { 4.5450000000000000e-08, 1.8821467205900000e+00, 5.2147681318300002e+01 },
    { 4.5410000000000001e-08, 4.5373169020799997e+00, 9.2047073954699997e+01 },
    { 4.5200000000000001e-08, 2.6686563503600000e+00, 2.1197628650500000e+01 },
    { 4.5079999999999997e-08, 1.2363788873699999e+00, 2.7555052103309998e+02 },
    { 4.5060000000000001e-08, 4.6811842639199996e+00, 1.0005439283280000e+02 },
    { 4.5030000000000000e-08, 5.5419525701400003e+00, 2.2345102382459999e+02 },
    { 4.4799999999999997e-08, 5.8314421608200000e+00, 3.0598797511699999e+01 },
    { 4.4619999999999999e-08, 4.1070702329600000e+00, 1.6682867252200001e+02 },
    { 4.4309999999999998e-08, 1.6676110903900001e+00, 1.8027386923090000e+02 },
    { 4.4239999999999998e-08, 5.0463290085199999e+00, 1.0186893394120000e+02 },
    { 4.4169999999999998e-08, 3.9510773270700001e+00, 3.0237722397610003e+02 },
    { 4.3910000000000001e-08, 5.2662007809300002e+00, 4.4928029223480002e+02 },
    { 4.3830000000000003e-08, 3.9719574759700000e+00, 1.0335340664950000e+02 },
    { 4.3740000000000000e-08, 3.4274274178200002e+00, 3.2350541665740002e+02 },
    { 4.3719999999999997e-08, 5.9021052429100003e+00, 5.9587454395279997e+02 },
    { 4.3450000000000002e-08, 4.0625804288099996e+00, 4.9651816653700003e+01 },
    { 4.3259999999999998e-08, 2.4645099579799998e+00, 4.2266603761290003e+02 },
    { 4.3189999999999998e-08, 5.3844657768399999e+00, 6.1010163795439996e+02 },
    { 4.3070000000000001e-08, 4.5630954211499999e+00, 1.2247586176070000e+02 },
    { 4.3030000000000002e-08, 2.5982708928000000e+00, 3.3928641933649999e+02 },
    { 4.2909999999999998e-08, 4.8006024986000000e+00, 4.6350738623640001e+02 },
    { 4.2900000000000000e-08, 3.9122094404799999e+00, 2.5739127376840003e+02 },
    { 4.2540000000000002e-08, 1.6279881750700000e+00, 3.2824071907259997e+02 },
    { 4.2540000000000002e-08, 3.6211425175700001e+00, 7.4920983565610004e+02 },
    { 4.2510000000000001e-08, 1.7351063134000000e-01, 2.4872381809010000e+02 },
    { 4.2279999999999999e-08, 6.0602313680800002e+00, 3.4936891175390002e+02 },
    { 4.2270000000000001e-08, 1.6406949431200000e+00, 4.8077286162380000e+02 },
    { 4.2230000000000002e-08, 5.8460833518099999e+00, 3.3217287233569999e+02 },
    { 4.2209999999999999e-08, 5.7136507618300003e+00, 4.0580716192600001e+01 },
    { 4.2170000000000000e-08, 1.5151533293099999e+00, 1.2450285119020000e+02 },
    { 4.2090000000000002e-08, 2.5982919650400000e+00, 2.7161836777000002e+02 },
    { 4.1960000000000000e-08, 3.7528152976000002e+00, 2.2189711514710001e+02 },
    { 4.1950000000000002e-08, 5.1067045680899996e+00, 3.8330916169770001e+02 },
    { 4.1929999999999999e-08, 3.8266407467599999e+00, 7.5632338265689998e+02 },
    { 4.1880000000000002e-08, 5.4360205818200003e+00, 4.5097721326419997e+02 },
    { 4.1610000000000000e-08, 3.0817329017000001e+00, 4.3549661609200001e+01 },
    { 4.1540000000000000e-08, 1.3203566596400000e+00, 1.7017012495410000e+02 },
    { 4.1320000000000002e-08, 1.4655272544300000e+00, 5.1864526483069994e+02 },
    { 4.1299999999999999e-08, 5.9003331656800002e+00, 3.8012776796000003e+02 },
    { 4.0889999999999998e-08, 5.5540902192499999e+00, 1.2648167191010000e+03 },
    { 4.0800000000000001e-08, 8.0320966997999999e-01, 4.7069036920640002e+02 },
    { 4.0760000000000002e-08, 1.4462216907000001e+00, 4.8788640862459999e+02 },
    { 4.0730000000000001e-08, 2.3439247993699999e+00, 3.0386169668439999e+02 },
    { 4.0719999999999997e-08, 9.0732377695999999e-01, 3.1957326339430000e+02 },
    { 4.0719999999999997e-08, 3.5139168703000001e+00, 4.0221916848780000e+02 },
    { 4.0530000000000000e-08, 5.7485741536399999e+00, 1.2184272231430000e+02 },
    { 4.0530000000000000e-08, 3.6617667413999999e-01, 2.2908009811709999e+02 },
    { 4.0180000000000000e-08, 4.0646422832800004e+00, 7.5877106321170004e+02 },
    { 4.0170000000000002e-08, 1.1396254750399999e+00, 4.4275170057060001e+02 },
    { 4.0089999999999997e-08, 2.4045567121899998e+00, 1.5128068240082000e+03 },
    { 4.0019999999999997e-08, 3.5610668088400002e+00, 1.8873055176757000e+03 },
    { 3.9909999999999998e-08, 1.2831111719799999e+00, 2.2679247625670001e+02 },
    { 3.9419999999999998e-08, 1.2926231823300001e+00, 7.2876296653099996e+02 },
    { 3.9290000000000003e-08, 5.0815651093000003e-01, 1.3175069171405000e+03 },
    { 3.9249999999999997e-08, 4.9796970002899998e+00, 1.3606981631590000e+02 },
    { 3.9190000000000002e-08, 2.2592924086199999e+00, 2.0470107572890001e+02 },
    { 3.8710000000000001e-08, 1.7110702131600000e+00, 1.9577298761969999e+02 },
    { 3.8659999999999997e-08, 3.9317068402999999e-01, 5.1153171782990000e+02 },
    { 3.8460000000000002e-08, 6.0171026633900002e+00, 2.5345912050530001e+02 },
    { 3.8290000000000001e-08, 2.8094336162900002e+00, 3.3120966448920001e+02 },
    { 3.8129999999999998e-08, 2.9413995145999999e-01, 4.5787831194389997e+02 },
    { 3.7989999999999998e-08, 1.1677829072899999e+00, 7.3587651353180001e+02 },
    { 3.7989999999999998e-08, 4.5112606507199997e+00, 3.2835259365719998e+02 },
    { 3.7879999999999999e-08, 8.7966092564999998e-01, 9.0981873305459999e+02 },
    { 3.7730000000000001e-08, 4.0844701450200001e+00, 3.5803636743220000e+02 },
    { 3.7370000000000003e-08, 3.6607689745999999e-01, 5.9839494557221997e+03 },
    { 3.7160000000000003e-08, 2.2578780827300000e+00, 6.6820846196529999e+02 },
    { 3.7130000000000002e-08, 4.6711937402099997e+00, 9.0684978763799995e+02 },
    { 3.6920000000000002e-08, 1.4175860308199999e+00, 2.1951887056149999e+02 },
    { 3.6489999999999998e-08, 4.9310438660799996e+00, 4.6972716135989998e+02 },
    { 3.6470000000000002e-08, 5.4083019935600003e+00, 2.9021955801939998e+02 },
    { 3.6410000000000000e-08, 1.4362815430800000e+00, 8.2005928096030004e+02 },
    { 3.6379999999999999e-08, 6.3261201547000001e-01, 6.0150361824530000e+02 },
    { 3.6349999999999998e-08, 5.9383895530000004e+00, 3.4225536475310003e+02 },
    { 3.6270000000000000e-08, 3.2193271207300000e+00, 1.5667674413540001e+02 },
    { 3.6169999999999999e-08, 2.1067640747400000e+00, 3.1097524368519998e+02 },
    { 3.5940000000000003e-08, 2.3994452265900001e+00, 7.5780785536520000e+02 },
    { 3.5940000000000003e-08, 3.2134026487000000e-01, 7.0511765732569995e+02 },
    { 3.5910000000000002e-08, 4.7810594324000000e-01, 1.0623508756058000e+03 },
    { 3.5770000000000002e-08, 3.5195103345300001e+00, 6.4701083331480004e+02 },
    { 3.5600000000000001e-08, 2.6040446411099998e+00, 7.8159617692239999e+02 },
    { 3.5560000000000002e-08, 3.0601869839300000e+00, 5.0137978944330001e+02 },
    { 3.5560000000000002e-08, 3.6170350256499999e+00, 1.3686602528450001e+03 },
    { 3.5329999999999999e-08, 5.7957961088999999e-01, 3.3280601178209997e+02 },
    { 3.5059999999999998e-08, 1.6483953193900001e+00, 5.1887048887399999e+01 },
    { 3.5040000000000001e-08, 1.7894870499600000e+00, 9.9141591519417998e+03 },
    { 3.4970000000000001e-08, 2.1989673558000000e-01, 3.9096243484299997e+01 },
    { 3.4739999999999999e-08, 5.7377163516499996e+00, 1.0656017053127000e+03 },
    { 3.4720000000000003e-08, 2.6714686266999998e+00, 2.9674814968359999e+02 },
    { 3.4700000000000000e-08, 4.3841604017200000e+00, 1.0679799498983000e+03 },
    { 3.4420000000000000e-08, 3.6066574405599998e+00, 6.8940609061580005e+02 },
    { 3.4149999999999998e-08, 2.8163719567099998e+00, 6.3033605875839999e+02 },
    { 3.4140000000000000e-08, 3.8644073192300001e+00, 2.5602862665600000e+01 },
    { 3.4070000000000000e-08, 5.9063162565000003e+00, 2.9318850343600002e+02 },
    { 3.4000000000000000e-08, 4.6788751791600003e+00, 9.9564603188579997e+02 },
    { 3.3979999999999997e-08, 2.9637250503499999e+00, 5.4785021235930003e+02 },
    { 3.3930000000000000e-08, 1.7216464372000000e+00, 2.9111907539110001e+02 },
    { 3.3899999999999999e-08, 2.8511186642899999e+00, 1.0507839104801001e+03 },
    { 3.3899999999999999e-08, 4.6726629475800001e+00, 1.4710022675381999e+03 },
    { 3.3600000000000003e-08, 1.5525313851000000e-01, 3.7004527554260000e+02 },
    { 3.3589999999999998e-08, 4.9503546864299999e+00, 2.2879821382680001e+02 },
    { 3.3479999999999999e-08, 1.2990827406800001e+00, 6.7214061522839995e+02 },
    { 3.3479999999999999e-08, 2.1589322848500001e+00, 9.6962247809489998e+02 },
    { 3.3470000000000001e-08, 5.4329131893899998e+00, 1.0021837280099400e+04 },
    { 3.3449999999999998e-08, 2.1006120048999999e+00, 1.5984740641450001e+03 },
    { 3.3379999999999998e-08, 2.3147033018599998e+00, 8.9971900415400000e+01 },
    { 3.3020000000000001e-08, 1.2818403861900001e+00, 5.3213864564940002e+02 },
    { 3.2920000000000000e-08, 1.1169023339299999e+00, 2.4241100297711000e+03 },
    { 3.2800000000000003e-08, 5.5110264223999996e+00, 9.5628915597059995e+02 },
    { 3.2789999999999998e-08, 6.1306987564800002e+00, 2.6036021918500001e+02 },
    { 3.2749999999999999e-08, 3.6953967072900000e+00, 1.8498791978669999e+02 },
    { 3.2549999999999997e-08, 3.1246073975200002e+00, 1.1387784205160000e+02 },
    { 3.2490000000000002e-08, 2.1048309518199999e+00, 6.8398946464439996e+02 },
    { 3.2460000000000001e-08, 4.2201878546399998e+00, 3.0689420236899998e+01 },
    { 3.2220000000000000e-08, 6.5043581113000004e-01, 4.2096911658350001e+02 },
    { 3.2170000000000003e-08, 1.2820891377999999e-01, 3.4200882374700001e+01 },
    { 3.1900000000000001e-08, 4.5484604448199999e+00, 5.5555446019110002e+02 },
    { 3.1890000000000003e-08, 1.1680193531900001e+00, 3.7471114198859999e+02 },
    { 3.1860000000000002e-08, 4.0915929748700002e+00, 1.4409941708575000e+03 },
    { 3.1849999999999997e-08, 1.6294193443000000e+00, 5.8238116313410001e+02 },
    { 3.1820000000000003e-08, 2.5672179409700000e+00, 2.5642806592189999e+02 },
    { 3.1769999999999999e-08, 1.9535441493000000e-01, 1.9428851491140000e+02 },
    { 3.1629999999999999e-08, 6.4155462640000005e-01, 3.4884764689209999e+02 },
    { 3.1419999999999999e-08, 4.5729745450900001e+00, 4.3675011926259998e+02 },
    { 3.1240000000000000e-08, 5.9111211690000003e+00, 3.8427236954419999e+02 },
    { 3.1210000000000000e-08, 2.3799169518199998e+00, 6.0920786607709999e+02 },
    { 3.1079999999999998e-08, 6.2811543958699998e+00, 6.2640390549530002e+02 },
    { 3.0750000000000001e-08, 2.0141417399099999e+00, 9.8066817835879999e+02 },
    { 3.0740000000000003e-08, 3.5594582349100001e+00, 2.1648048917570000e+02 },
    { 3.0479999999999999e-08, 1.1476909801399999e+00, 1.3032798231388999e+03 },
    { 3.0339999999999999e-08, 3.8974808776900001e+00, 6.9651963761659999e+02 },
    { 3.0150000000000002e-08, 3.8941758565400000e+00, 1.0537528558967001e+03 },
    { 3.0010000000000002e-08, 3.1795737948300000e+00, 2.1543795945209999e+02 },
    { 3.0010000000000002e-08, 3.8359121300500001e+00, 1.8125239191083999e+03 },
    { 2.9980000000000001e-08, 5.1710693419700000e+00, 4.3281796599950002e+02 },
    { 2.9970000000000003e-08, 1.2652667086000000e+00, 3.7056654040439997e+02 },
    { 2.9760000000000000e-08, 4.9810651292200001e+00, 8.0410672859800002e+01 },
    { 2.9600000000000000e-08, 1.7270807126100001e+00, 1.5499118388799999e+01 },
    { 2.9510000000000001e-08, 4.9803405078300003e+00, 2.5583736509089999e+02 },
    { 2.9469999999999999e-08, 5.4619205486500002e+00, 8.1857480825200003e+02 },
    { 2.9070000000000002e-08, 2.5386607276499999e+00, 6.2025356634100001e+02 },
    { 2.9040000000000001e-08, 4.0141128755900004e+00, 9.0280716956099994e+01 },
    { 2.9000000000000002e-08, 1.8286247671800000e+00, 2.9823262239190001e+02 },
    { 2.8979999999999999e-08, 6.2168004478899999e+00, 1.0080515235820000e+02 },
    { 2.8959999999999999e-08, 4.3762055386999998e+00, 9.5979227217800002e+01 },
    { 2.8959999999999999e-08, 1.0930744630700000e+00, 6.3203297978779995e+02 },
    { 2.8930000000000002e-08, 5.6010064090000000e-01, 4.3978275515399997e+02 },
    { 2.8900000000000001e-08, 3.3156610581099999e+00, 3.6137781986430002e+02 },
    { 2.8810000000000001e-08, 3.6859058917600001e+00, 3.6997583957339998e+02 },
    { 2.8760000000000001e-08, 2.8936749907400001e+00, 6.0587310542895002e+03 },
    { 2.8730000000000000e-08, 4.1608256829699997e+00, 1.9280404220310001e+02 },
    { 2.8719999999999998e-08, 4.4174482086899998e+00, 3.8405992122309999e+02 },
    { 2.8640000000000000e-08, 2.9373200304500000e+00, 6.3744960575920004e+02 },
    { 2.8550000000000001e-08, 5.7177237410000004e+00, 4.8918103470799998e+01 },
    { 2.8500000000000000e-08, 8.3031157342999995e-01, 9.4753107756710006e+02 },
    { 2.8369999999999999e-08, 4.7528515487000000e-01, 4.8491746320800002e+02 },
    { 2.8310000000000000e-08, 4.7324329478399996e+00, 1.0601353552540000e+02 },
    { 2.8299999999999999e-08, 1.3492194434200000e+00, 1.2125202148330000e+02 },
    { 2.8110000000000002e-08, 1.6244779557700000e+00, 6.0447256366190004e+02 },
    { 2.8080000000000001e-08, 3.9241711974700002e+00, 1.0400779795530001e+02 },
    { 2.8050000000000000e-08, 2.0206019083800002e+00, 2.1116503133776000e+03 },
    { 2.8019999999999999e-08, 1.3742921143600000e+00, 6.7442823708879996e+02 },
    { 2.7920000000000001e-08, 4.7830754765599997e+00, 1.2884620283063000e+03 },
    { 2.7859999999999999e-08, 2.7339200439900000e+00, 5.9491133610630004e+02 },
    { 2.7859999999999999e-08, 5.7388832977200002e+00, 3.5033211960040001e+02 },
    { 2.7840000000000000e-08, 1.5729265085699999e+00, 5.3197858695499997e+02 },
    { 2.7770000000000000e-08, 3.9992836601500001e+00, 3.2716409666399997e+01 },
    { 2.7680000000000000e-08, 6.2350309922499996e+00, 3.8458118608490003e+02 },
    { 2.7579999999999999e-08, 1.2381396429300000e+00, 2.8021638747909998e+02 },
    { 2.7570000000000001e-08, 3.2321931195700002e+00, 4.1703696332039999e+02 },
    { 2.7479999999999999e-08, 3.4252932626999999e+00, 1.0650110044817000e+03 },
    { 2.7409999999999999e-08, 5.9255119076799998e+00, 5.4844091319029997e+02 },
    { 2.7400000000000001e-08, 5.5850031509100004e+00, 3.9590877063910000e+02 },
    { 2.7190000000000001e-08, 2.9779548252399999e+00, 1.8802630117250001e+02 },
    { 2.7179999999999999e-08, 1.3114531686999999e+00, 4.7940026350009998e+02 },
    { 2.7140000000000000e-08, 3.4980714047000000e-01, 8.3355266177900000e+02 },
    { 2.6930000000000000e-08, 6.4719337286000000e-01, 1.2491370101415499e+04 },
    { 2.6799999999999998e-08, 4.3150416413399997e+00, 9.9889407505091003e+03 },
    { 2.6790000000000000e-08, 1.6139790018200000e+00, 1.3364169239306000e+03 },
    { 2.6770000000000001e-08, 2.3690735057999999e+00, 2.4161027108930000e+02 },
    { 2.6759999999999999e-08, 2.0530031323900002e+00, 4.0992341631959999e+02 },
    { 2.6639999999999999e-08, 5.0825528459399996e+00, 7.6174000862829996e+02 },
    { 2.6630000000000001e-08, 1.4619598521899999e+00, 5.9194239068970001e+02 },
    { 2.6569999999999999e-08, 3.4545854592200000e+00, 6.2322251175760005e+02 },
    { 2.6480000000000000e-08, 5.6421799907099999e+00, 7.4527768239299996e+02 },
    { 2.6480000000000000e-08, 3.9597852015699999e+00, 4.6053844081979997e+02 },
    { 2.6440000000000001e-08, 6.0342603886399999e+00, 5.2523754696970002e+02 },
    { 2.6120000000000002e-08, 9.9380221309000005e-01, 3.1162676309979001e+03 },
    { 2.6099999999999999e-08, 8.1925002649999995e-02, 1.5971512552120001e+02 },
    { 2.6060000000000000e-08, 4.1692767202700001e+00, 3.9021026037740000e+02 },
    { 2.5959999999999999e-08, 1.9197979385399999e+00, 5.9571448525840003e+02 },
    { 2.5900000000000000e-08, 3.9963029714400000e+00, 3.9117346822389999e+02 },
    { 2.5799999999999999e-08, 3.8488479584399999e+00, 4.6112914165080002e+02 },
    { 2.5620000000000000e-08, 6.2109646805400001e+00, 6.5716276170139997e+02 },
    { 2.5620000000000000e-08, 4.2035638498000000e-01, 4.5045594840240000e+02 },
    { 2.5290000000000000e-08, 2.6008347093799999e+00, 2.4988916283384001e+03 },
    { 2.5220000000000000e-08, 4.5478602817200002e+00, 1.1536231475990000e+02 },
    { 2.5110000000000001e-08, 2.7221221987400002e+00, 1.2584539316256000e+03 },
    { 2.4970000000000001e-08, 1.7858091837600001e+00, 6.2885158605009997e+02 },
    { 2.4970000000000001e-08, 2.0871234949700002e+00, 1.0898248094200000e+02 },
    { 2.4929999999999999e-08, 1.9433631304800001e+00, 9.1693228005540004e+02 },
    { 2.4739999999999999e-08, 4.4962035920599996e+00, 6.9970103135429997e+02 },
    { 2.4590000000000000e-08, 2.0561228530600002e+00, 6.3448066034260000e+02 },
    { 2.4420000000000000e-08, 2.3526085973999999e-01, 7.6588461021249998e+02 },
    { 2.4380000000000001e-08, 5.7233675632600001e+00, 8.9484087952760001e+02 },
    { 2.4200000000000002e-08, 3.1215799429500000e+00, 2.5265597135319999e+02 },
    { 2.4179999999999999e-08, 5.0604072248499996e+00, 3.4639996633729999e+02 },
    { 2.4159999999999999e-08, 3.7219548398800000e+00, 6.1217681149370003e+02 },
    { 2.4089999999999999e-08, 5.1853592883399999e+00, 1.2284982245716001e+03 },
    { 2.3899999999999999e-08, 1.9403020545700000e+00, 8.4363515419639998e+02 },
    { 2.3779999999999999e-08, 2.8152999551100000e+00, 2.2338158785540000e+02 },
    { 2.3530000000000000e-08, 5.4560565773900001e+00, 7.4569150246199996e+01 },
    { 2.3530000000000000e-08, 5.5065311657900002e+00, 7.4994046888400007e+01 },
    { 2.3520000000000001e-08, 1.3662647961000001e+00, 1.2395439248355001e+03 },
    { 2.3450000000000002e-08, 2.7773911953099999e+00, 5.3362311835770004e+02 },
    { 2.3300000000000000e-08, 6.0843432685599996e+00, 8.4608283475120004e+02 },
    { 2.3300000000000000e-08, 3.2085050738600001e+00, 8.2154375366859995e+02 },
    { 2.3199999999999999e-08, 1.1767042259600000e+00, 2.4697871295089999e+02 },
    { 2.3079999999999999e-08, 1.9561891750000001e-02, 3.6908206769610001e+02 },
    { 2.2919999999999999e-08, 4.2696242146700003e+00, 5.4695644048199995e+02 },
    { 2.2550000000000000e-08, 5.8199906669499999e+00, 4.5194042111070002e+02 },
    { 2.2410000000000000e-08, 2.2299721166799999e+00, 6.1445583531207003e+03 },
    { 2.2239999999999999e-08, 3.4516554762200000e+00, 5.3117543780289998e+02 },
    { 2.2230000000000001e-08, 1.9016544718499999e+00, 5.3583969916800001e+01 },
};

static const VSOP87Term VSOP87A_URANUS_X2[249] = {
    { 1.6015731999999999e-04, 3.8370002661900000e+00, 7.4781598567299994e+01 },
    { 1.0915299000000000e-04, 3.0298777627000000e+00, 1.4956319713459999e+02 },
    { 8.0536230000000005e-05, 2.5464614612199998e+00, 6.3735898303399999e+01 },
    { 7.4976189999999995e-05, 3.8342913666100000e+00, 1.1045700263900001e+01 },
    { 6.7173129999999998e-05, 5.3126421450099999e+00, 8.5827298831199997e+01 },
    { 5.4080330000000000e-05, 4.7803364230299996e+00, 7.0849445304200003e+01 },
    { 5.2846840000000000e-05, 2.1190194209700000e+00, 7.3297125859000005e+01 },
    { 5.0219709999999997e-05, 3.0463277292800002e+00, 7.8713751830400000e+01 },
    { 4.8749360000000002e-05, 5.6861613217600002e+00, 7.6266071275599998e+01 },
    { 3.0021239999999999e-05, 4.0794439845200001e+00, 1.3851749687070000e+02 },
    { 2.5217969999999999e-05, 3.3602825317299998e+00, 7.1600204829600003e+01 },
    { 2.4138320000000000e-05, 4.4586522568999998e+00, 7.7962992305000000e+01 },
    { 2.2917670000000000e-05, 0.0000000000000000e+00, 0.0000000000000000e+00 },
    { 2.2213729999999999e-05, 8.7427485235000002e-01, 3.9321532631000000e+00 },
    { 1.0461000000000000e-05, 5.8531191022800000e+00, 2.2434479570190001e+02 },
    { 1.0402500000000001e-05, 7.3133408837000002e-01, 1.4563104387150000e+02 },
    { 6.6158600000000002e-06, 1.0154450534499999e+00, 1.8159247264699999e+01 },
    { 5.0247100000000002e-06, 2.2251421471000001e+00, 1.5104766984290001e+02 },
    { 4.9722900000000000e-06, 5.2773321418299997e+00, 2.2091400527800001e+01 },
    { 4.7834800000000003e-06, 3.1910794121900001e+00, 7.2333918012500007e+01 },
    { 4.6976700000000004e-06, 1.8248120224200000e+00, 3.1813937377000001e+00 },
    { 4.0317400000000001e-06, 1.5621517827200000e+00, 1.6060889739850001e+02 },
    { 3.8474099999999996e-06, 4.7323932475000001e+00, 7.7229279122099996e+01 },
    { 3.8245399999999998e-06, 5.2015953177299998e+00, 7.7750543983900002e+01 },
    { 3.3593100000000000e-06, 5.3778689599999999e-02, 6.5220371011699996e+01 },
    { 3.2945500000000001e-06, 2.5533363409400001e+00, 7.1812653150700001e+01 },
    { 3.0102500000000001e-06, 1.7767760794500000e+00, 8.4342826122899993e+01 },
    { 2.8448500000000000e-06, 5.8026905027200000e+00, 1.4807872442630000e+02 },
    { 2.8138000000000001e-06, 5.4117101232999998e-01, 1.3140394986990000e+02 },
    { 2.6686000000000002e-06, 1.6574457840000001e-02, 5.2690198039499997e+01 },
    { 2.4776100000000001e-06, 2.1793804965399999e+00, 1.2747179660680000e+02 },
    { 2.3487600000000000e-06, 4.7537949523399998e+00, 5.6622351302600002e+01 },
    { 2.1937799999999999e-06, 3.4707174253600002e+00, 2.2041264243879999e+02 },
    { 2.1122700000000001e-06, 5.5820166473999997e+00, 1.5349535039770001e+02 },
    { 2.0587500000000000e-06, 2.9004445065600000e+00, 9.2940845831999994e+01 },
    { 2.0474300000000001e-06, 4.2502772201700001e+00, 8.7311771539500000e+01 },
    { 1.9806900000000001e-06, 2.7123199700499998e+00, 1.2530172972200001e+01 },
    { 1.5280300000000000e-06, 1.3751383108999999e+00, 2.0618554843720000e+02 },
    { 1.3984700000000001e-06, 1.6773038501599999e+00, 9.5612275556000004e+00 },
    { 1.3845099999999999e-06, 1.0380976130499999e+00, 2.1329909543799999e+02 },
    { 1.2143800000000001e-06, 1.5242242877200001e+00, 7.1135470007999997e+00 },
    { 1.1767699999999999e-06, 4.1270274457199996e+00, 5.2257741809380002e+02 },
    { 1.1460300000000001e-06, 2.1128941925000000e-01, 5.3680451209540001e+02 },
    { 1.0632200000000000e-06, 5.8938348652099997e+00, 1.4638180339690001e+02 },
    { 1.0383399999999999e-06, 3.5930177837500001e+00, 6.2251425595100002e+01 },
    { 9.8993000000000002e-07, 2.0242880902599998e+00, 8.0198224538700003e+01 },
    { 9.7004000000000008e-07, 8.6929640948999998e-01, 9.6872999095099999e+01 },
    { 9.0521999999999995e-07, 2.7439544745300002e+00, 6.7668051566499997e+01 },
    { 8.6287000000000005e-07, 2.2301510595299998e+00, 2.9912639426919998e+02 },
    { 8.5667999999999997e-07, 5.6537042423599999e+00, 2.9689454166000000e+00 },
    { 7.7436000000000001e-07, 4.9863644313500002e+00, 8.1895145568100006e+01 },
    { 7.4595999999999999e-07, 5.3081848403800000e+00, 5.2969096509459996e+02 },
    { 7.0849000000000001e-07, 5.0708845681900003e+00, 2.4476805547999998e+00 },
    { 6.5861999999999995e-07, 3.5542495299999999e+00, 1.4977853527000001e+01 },
    { 6.5260000000000001e-07, 2.5108056282200000e+00, 4.2659819087599999e+02 },
    { 6.1521999999999997e-07, 2.5666248573299999e+00, 1.5274459087229999e+02 },
    { 5.8821000000000000e-07, 4.3224516489499996e+00, 1.3458534360760001e+02 },
    { 5.1450000000000001e-07, 4.4076562780400002e+00, 1.9513984817330001e+02 },
    { 5.1315000000000001e-07, 3.7375957599999998e-03, 2.0225339517410001e+02 },
    { 5.0856000000000005e-07, 4.1502680462899999e+00, 7.9235016692200006e+01 },
    { 4.9991999999999999e-07, 1.9801892781199999e+00, 5.9803745040300001e+01 },
    { 4.9174999999999999e-07, 4.9078684627299998e+00, 2.2582926841020000e+02 },
    { 4.8281000000000002e-07, 6.1886846677499996e+00, 1.4244965013379999e+02 },
    { 4.4331999999999998e-07, 3.6673784320700000e+00, 7.0328180442399997e+01 },
    { 4.1618000000000001e-07, 1.4311744196900000e+00, 1.5253214255120000e+02 },
    { 3.9803000000000002e-07, 3.2173851457899998e+00, 2.9204947528600002e+01 },
    { 3.9411999999999998e-07, 3.5455840928700000e+00, 1.2035824960600000e+02 },
    { 3.8820000000000000e-07, 5.1134040994500003e+00, 1.0593819301891999e+03 },
    { 3.7711000000000002e-07, 7.0882593116000003e-01, 2.0936694217490000e+02 },
    { 3.7700000000000000e-07, 1.7025462977300001e+00, 4.5576651038700000e+01 },
    { 3.6505000000000000e-07, 5.8441834180300001e+00, 6.9364972595899999e+01 },
    { 3.6469999999999998e-07, 5.4548335107800003e+00, 1.3703302416240001e+02 },
    { 3.2468000000000002e-07, 6.0976698725300000e+00, 5.4166259714000002e+00 },
    { 2.7599999999999998e-07, 6.5039122586999998e-01, 1.5201087768939999e+02 },
    { 2.6156000000000002e-07, 4.9779058359000000e-01, 1.4844727083000000e+00 },
    { 2.5904999999999998e-07, 8.0510379906999996e-01, 2.2286032299359999e+02 },
    { 2.5328000000000000e-07, 4.0771349602900004e+00, 2.3539049596580000e+02 },
    { 2.5082999999999998e-07, 4.1674791271800000e+00, 2.3575873236100001e+01 },
    { 2.4866000000000000e-07, 1.4502344247900001e+00, 2.0606927819500001e+01 },
    { 2.2886000000000000e-07, 5.8034945567399996e+00, 3.9617508346100003e+01 },
    { 2.1959000000000000e-07, 7.8915450301000001e-01, 3.3137100791700000e+01 },
    { 2.0344000000000000e-07, 1.1080561319600000e+00, 1.4227094001599999e+01 },
    { 2.0188999999999999e-07, 1.3124221518800001e+00, 7.0115732121299999e+01 },
    { 1.9964000000000001e-07, 5.0220161880000003e-02, 1.1642609634290000e+02 },
    { 1.9179000000000000e-07, 4.2322518573999997e+00, 4.1644497775600001e+01 },
    { 1.9159000000000001e-07, 1.7225729178600000e+00, 1.4659425171800001e+02 },
    { 1.8948000000000000e-07, 4.9373783234699999e+00, 4.4534181248999998e+00 },
    { 1.8655000000000001e-07, 3.3584328071799998e+00, 7.9447465013300004e+01 },
    { 1.8566000000000001e-07, 3.5120155453600002e+00, 1.5912442469019999e+02 },
    { 1.8559000000000001e-07, 5.8365304766700001e+00, 8.9759452094300002e+01 },
    { 1.7966000000000001e-07, 2.3015452064900002e+00, 7.3084677537900006e+01 },
    { 1.7436999999999999e-07, 2.3685071412899998e+00, 7.6478519596699996e+01 },
    { 1.6178999999999999e-07, 4.9103806615999996e+00, 1.6674774556399999e+01 },
    { 1.6133999999999999e-07, 9.5473971870999996e-01, 1.9120769491019999e+02 },
    { 1.5341000000000000e-07, 2.5913254143600000e+00, 2.6598929347749998e+02 },
    { 1.5321000000000001e-07, 5.5078914228100002e+00, 1.4711551657979999e+02 },
    { 1.4945000000000000e-07, 5.5877569735999999e-01, 1.6209337010679999e+02 },
    { 1.4406999999999999e-07, 6.2220502902800003e+00, 3.3365734504400001e+02 },
    { 1.4327000000000001e-07, 6.0770853992999996e+00, 1.9643719973000000e+01 },
    { 1.3773999999999999e-07, 2.8191940743799999e+00, 4.3371173787679999e+02 },
    { 1.3526999999999999e-07, 2.3675508637900000e+00, 3.4788443904560000e+02 },
    { 1.2898000000000001e-07, 5.0208480683000001e-01, 4.1948464387519999e+02 },
    { 1.2693000000000001e-07, 2.0053782925000001e+00, 1.0994568878849999e+02 },
    { 1.1934000000000001e-07, 1.1359736822099999e+00, 5.1205725331200000e+01 },
    { 1.1585000000000000e-07, 1.7248586198100000e+00, 1.4414657116320001e+02 },
    { 1.1487000000000000e-07, 2.9287491087599999e+00, 8.8796244247800004e+01 },
    { 1.1019000000000000e-07, 8.9138177474000002e-01, 2.5272794265500000e+01 },
    { 1.0994000000000000e-07, 3.1074518421200001e+00, 6.0554504565700000e+01 },
    { 1.0330000000000001e-07, 6.8034253830000002e-02, 1.4000196957899999e+02 },
    { 1.0122000000000000e-07, 1.2661074740600000e+00, 2.1181462272970001e+02 },
    { 9.3880000000000004e-08, 1.2749474272500001e+00, 3.4077089204480001e+02 },
    { 9.2620000000000005e-08, 1.8246057489100000e+00, 2.2827694896500000e+02 },
    { 9.2599999999999995e-08, 5.3938129860200004e+00, 1.6772244439930000e+02 },
    { 9.2410000000000005e-08, 1.6384628423500001e+00, 6.3989728631399998e+02 },
    { 9.0489999999999998e-08, 4.5290039295100000e+00, 2.5887574647669999e+02 },
    { 8.9599999999999995e-08, 3.0366696817999999e+00, 1.2598732389849999e+02 },
    { 8.9579999999999999e-08, 5.5298388592399998e+00, 6.6704843720000000e+01 },
    { 8.6710000000000000e-08, 4.3017672535699996e+00, 5.4174670747800000e+01 },
    { 8.6109999999999995e-08, 2.3739011437799999e+00, 2.7703499374140000e+02 },
    { 8.5959999999999997e-08, 3.2214179937299998e+00, 2.8808069400530002e+02 },
    { 7.7900000000000003e-08, 5.0609341204199998e+00, 4.1101981054399999e+01 },
    { 7.7050000000000005e-08, 3.3698050817300000e+00, 5.8106824010899999e+01 },
    { 7.5320000000000002e-08, 4.8417145895999996e+00, 8.9008692568900003e+01 },
    { 7.4480000000000003e-08, 1.8702725084999999e-01, 1.2895626931510000e+02 },
    { 7.2170000000000004e-08, 2.8638221541700002e+00, 9.5388526386799995e+01 },
    { 6.8869999999999994e-08, 6.1423876012500003e+00, 2.7310284047829998e+02 },
    { 6.7659999999999998e-08, 1.5373799241099999e+00, 1.5126011816400000e+02 },
    { 6.3559999999999995e-08, 4.5211191701000004e+00, 1.5497982310600000e+02 },
    { 6.3520000000000002e-08, 4.2703062753100003e+00, 2.2752618943959999e+02 },
    { 6.2830000000000001e-08, 2.2663529531000002e+00, 3.0534616939270001e+02 },
    { 6.2649999999999995e-08, 2.7988190810600000e+00, 1.4510977900969999e+02 },
    { 6.1900000000000005e-08, 4.9647095933900003e+00, 2.6992144674060000e+02 },
    { 6.1109999999999996e-08, 7.7279065409000003e-01, 1.0664954771900000e+03 },
    { 6.1070000000000004e-08, 9.0480185525000001e-01, 2.4379022388199999e+01 },
    { 6.0150000000000006e-08, 4.7718457116200002e+00, 3.6648562929500002e+01 },
    { 5.9810000000000004e-08, 4.1637475396000001e-01, 1.3533610313299999e+02 },
    { 5.8290000000000001e-08, 2.1180773516200002e+00, 3.9356875915200000e+01 },
    { 5.8269999999999998e-08, 1.0065943856899999e+00, 1.0309277421860000e+02 },
    { 5.8239999999999997e-08, 6.9450297478000000e-01, 4.6470422915999997e+01 },
    { 5.4849999999999998e-08, 3.8293806566100002e+00, 1.4169889060840001e+02 },
    { 5.4230000000000003e-08, 4.5280511637399998e+00, 9.1456373123700004e+01 },
    { 5.4090000000000003e-08, 4.0992481102099996e+00, 4.8640193591629998e+02 },
    { 5.3640000000000003e-08, 2.5324195377600001e+00, 7.5744806413800006e+01 },
    { 5.3640000000000003e-08, 2.1469828982400001e+00, 7.3818390720799997e+01 },
    { 5.2490000000000002e-08, 5.1124310426999999e+00, 6.0766952886799999e+01 },
    { 5.2490000000000002e-08, 6.1090743340699998e+00, 9.0195442652839995e+02 },
    { 5.2089999999999999e-08, 3.5580919351300002e+00, 1.0522683831884001e+03 },
    { 5.1910000000000000e-08, 1.6618885419200000e+00, 4.7217484191469998e+02 },
    { 5.1620000000000002e-08, 4.1816465142999997e+00, 3.8330916169770001e+02 },
    { 5.0080000000000002e-08, 4.2358642918399996e+00, 5.1546387109299997e+02 },
    { 4.8210000000000000e-08, 4.9566263140600002e+00, 3.7390799283650000e+02 },
    { 4.7780000000000002e-08, 5.8618173972000003e+00, 5.5137878594299998e+01 },
    { 4.7470000000000001e-08, 5.9419304202800003e+00, 9.8357471803400003e+01 },
    { 4.6509999999999998e-08, 1.5071340362900001e+00, 3.2350541665740002e+02 },
    { 4.6140000000000002e-08, 1.0553135310999999e+00, 2.3056457082540001e+02 },
    { 4.6040000000000001e-08, 5.8801017680200003e+00, 4.1555249061209997e+02 },
    { 4.5720000000000002e-08, 7.5390834385000005e-01, 1.2429040286910001e+02 },
    { 4.5660000000000000e-08, 8.1568008467999997e-01, 2.2345102382459999e+02 },
    { 4.4029999999999998e-08, 3.4790985470200000e+00, 1.7165459766239999e+02 },
    { 4.2869999999999999e-08, 2.7699488348900001e+00, 1.0846121608020000e+02 },
    { 4.2330000000000003e-08, 2.4461247117800000e+00, 3.2972519178089999e+02 },
    { 4.0790000000000003e-08, 1.2768123841600001e+00, 1.9692006632437999e+03 },
    { 4.0620000000000002e-08, 3.0065508821400000e+00, 1.1291463420510000e+02 },
    { 4.0009999999999999e-08, 5.7166635539000001e+00, 1.1020632121940000e+02 },
    { 3.9410000000000000e-08, 3.6949970347300001e+00, 5.0456118318099999e+02 },
    { 3.9290000000000003e-08, 5.6585065873499998e+00, 9.0588657979150003e+02 },
    { 3.8810000000000001e-08, 2.9844496851600000e+00, 1.8409414790939999e+02 },
    { 3.8549999999999998e-08, 4.0536791636600000e+00, 1.5578297225809999e+02 },
    { 3.8509999999999999e-08, 2.8065674408899999e+00, 8.2858353414600003e+01 },
    { 3.8120000000000000e-08, 1.5070617776499999e+00, 1.9907200143640000e+02 },
    { 3.8050000000000000e-08, 2.8941381863100002e+00, 4.7928838891549998e+02 },
    { 3.7949999999999999e-08, 6.1001845151499996e+00, 2.9430046912879999e+02 },
    { 3.7440000000000003e-08, 1.5880085294999999e-01, 4.3289029178299998e+01 },
    { 3.7189999999999998e-08, 4.3682351149000000e+00, 4.0450679034820001e+02 },
    { 3.7090000000000003e-08, 1.7313585990900000e+00, 9.4425318540299997e+01 },
    { 3.6850000000000002e-08, 3.6706221353199999e+00, 1.3288842257819999e+02 },
    { 3.6809999999999997e-08, 2.1547507570200000e+00, 4.6053844081979997e+02 },
    { 3.6729999999999999e-08, 5.1352928114100003e+00, 6.9152524274800001e+01 },
    { 3.6729999999999999e-08, 2.6857022781800000e+00, 8.0410672859800002e+01 },
    { 3.6589999999999999e-08, 4.7921365211399998e+00, 8.4926422848890002e+02 },
    { 3.6510000000000001e-08, 6.0845510427100002e+00, 4.4631134681819998e+02 },
    { 3.6319999999999997e-08, 5.4284837095900000e+00, 5.7516123179899999e+01 },
    { 3.6169999999999999e-08, 4.9705576754300003e+00, 1.0398654609590000e+02 },
    { 3.6050000000000002e-08, 7.2583106928999996e-01, 3.4470304530790003e+02 },
    { 3.6029999999999999e-08, 3.2186436243000000e-01, 3.9739324334740002e+02 },
    { 3.5999999999999998e-08, 1.7614673538700001e+00, 6.1448276442999997e+01 },
    { 3.5420000000000002e-08, 3.6919021303299999e+00, 9.5628915597059995e+02 },
    { 3.5339999999999997e-08, 3.9246158909700002e+00, 3.3679617512900002e+01 },
    { 3.5299999999999998e-08, 2.3521707693699998e+00, 2.0470107572890001e+02 },
    { 3.4429999999999998e-08, 2.2102275917799998e+00, 3.5499798604640000e+02 },
    { 3.4289999999999998e-08, 2.8781828105299998e+00, 7.3587651353180001e+02 },
    { 3.4280000000000000e-08, 5.1440073028200004e+00, 4.4068227252570000e+02 },
    { 3.4210000000000000e-08, 7.4103984530000000e-01, 3.0927832265580003e+02 },
    { 3.3670000000000003e-08, 9.5172267680000006e-02, 5.4391805909619995e+02 },
    { 3.3360000000000002e-08, 5.8747209243800000e+00, 5.0402576179100002e+01 },
    { 3.3320000000000003e-08, 5.6729346574599999e+00, 3.3683873878169999e+02 },
    { 3.2950000000000001e-08, 1.5788154259600000e+00, 3.2243328914400003e+01 },
    { 3.2800000000000003e-08, 5.4670374879299999e+00, 2.9519424100610001e+02 },
    { 3.2670000000000001e-08, 5.5559668681999996e+00, 1.5890728952838001e+03 },
    { 3.2660000000000003e-08, 2.8719252229199999e+00, 6.1069233878540001e+02 },
    { 3.2439999999999998e-08, 4.4687056303199997e+00, 4.2808266358430001e+02 },
    { 3.2250000000000001e-08, 6.8793280970000004e-01, 1.0444040766621999e+03 },
    { 3.2110000000000001e-08, 6.2780121950599996e+00, 1.0578974574808999e+03 },
    { 3.2100000000000003e-08, 2.9948618467500001e+00, 2.3767811782620001e+02 },
    { 3.1639999999999997e-08, 6.1608804493099996e+00, 1.8196374661092000e+03 },
    { 3.1480000000000001e-08, 6.0050387936999998e+00, 1.1695882514085999e+03 },
    { 3.1280000000000000e-08, 5.4125539982699999e+00, 1.2828329540138000e+03 },
    { 3.0960000000000001e-08, 1.5725032974900000e+00, 1.0931254934210000e+02 },
    { 3.0930000000000000e-08, 1.8751847171200000e+00, 2.1723124870109999e+02 },
    { 3.0519999999999998e-08, 5.1209008409000001e-01, 5.3117543780289998e+02 },
    { 3.0330000000000001e-08, 2.6866055341999999e+00, 9.7673602509570003e+02 },
    { 3.0190000000000001e-08, 2.1567891131399999e+00, 9.2404582705619998e+02 },
    { 3.0099999999999998e-08, 4.4294712802299996e+00, 2.7555052103309998e+02 },
    { 3.0069999999999997e-08, 3.4132710452300001e+00, 2.9111907539110001e+02 },
    { 2.9569999999999999e-08, 3.3866040984299999e+00, 2.0767002114549999e+02 },
    { 2.9360000000000000e-08, 2.2393017709200000e+00, 3.0683064210100002e+02 },
    { 2.9309999999999999e-08, 5.7455787658500004e+00, 1.8016199464630000e+02 },
    { 2.9280000000000001e-08, 2.4409083512600001e+00, 4.6595506679120001e+02 },
    { 2.8690000000000001e-08, 5.9599087822700003e+00, 6.0676018552230005e+02 },
    { 2.8590000000000000e-08, 2.1700779836000000e-01, 2.4161027108930000e+02 },
    { 2.8390000000000001e-08, 1.7254622130499999e+00, 9.4206206196899996e+02 },
    { 2.8360000000000001e-08, 6.0595259072500003e+00, 8.8114920691600005e+01 },
    { 2.7970000000000002e-08, 5.2777646106800002e+00, 3.8161224066829999e+02 },
    { 2.7490000000000000e-08, 2.2815038814999999e+00, 9.2047073954699997e+01 },
    { 2.7050000000000001e-08, 9.1231544668999998e-01, 4.6913646052889999e+02 },
    { 2.6550000000000000e-08, 4.6199114276400000e+00, 4.7610699517780000e+02 },
    { 2.6550000000000000e-08, 3.0809067760100000e+00, 2.9764192156090002e+02 },
    { 2.6190000000000002e-08, 3.0873583037000002e-01, 2.2137585028530000e+02 },
    { 2.6140000000000001e-08, 3.5642265547899998e+00, 1.1553611574070001e+03 },
    { 2.6020000000000001e-08, 5.6141580456999995e-01, 1.4489733068859999e+02 },
    { 2.6020000000000001e-08, 9.7639397784000004e-01, 4.6658664459999999e+00 },
    { 2.5889999999999999e-08, 3.3271647124300001e+00, 3.7694637422229999e+02 },
    { 2.5850000000000000e-08, 2.7636054790700002e+00, 9.9160620955499994e+01 },
    { 2.5760000000000000e-08, 2.9704294542999998e-01, 9.8778172535960005e+02 },
    { 2.5760000000000000e-08, 3.8965180436100000e+00, 3.4321857259960001e+02 },
    { 2.5740000000000001e-08, 1.5664610795700000e+00, 4.1162033734900001e+02 },
    { 2.5539999999999999e-08, 2.5112465978500000e+00, 2.0076892246579999e+02 },
    { 2.5419999999999999e-08, 1.7187583716999999e-01, 4.6112914165080002e+02 },
    { 2.5380000000000000e-08, 4.2975168088300002e+00, 2.1892816973050000e+02 },
    { 2.5270000000000001e-08, 6.9443718848000002e-01, 7.7841478318470001e+02 },
    { 2.4949999999999998e-08, 3.8525861720800001e+00, 4.5097721326419997e+02 },
    { 2.4270000000000002e-08, 9.2530262268999997e-01, 7.1467888488129995e+02 },
    { 2.3860000000000000e-08, 5.0421258293599998e+00, 3.4618751801619999e+02 },
    { 2.3770000000000000e-08, 3.2384016411199998e+00, 3.5893013930950002e+02 },
    { 2.3520000000000001e-08, 3.7430503847400001e+00, 1.9832124191099999e+02 },
    { 2.3380000000000002e-08, 1.9531606517100000e+00, 9.1693228005540004e+02 },
    { 2.3320000000000000e-08, 1.2878212803200000e+00, 4.2977958461370002e+02 },
    { 2.3300000000000000e-08, 3.6405517093899999e+00, 2.2731374111849999e+02 },
    { 2.2790000000000001e-08, 3.3971491993699998e+00, 1.2991947716160001e+02 },
};

static const VSOP87Term VSOP87A_URANUS_X3[84] = {
    { 1.3070490000000001e-05, 0.0000000000000000e+00, 0.0000000000000000e+00 },
    { 6.7965099999999999e-06, 2.9337508155599998e+00, 7.0849445304200003e+01 },
    { 6.3178199999999996e-06, 4.8899823061100003e+00, 7.8713751830400000e+01 },
    { 5.7853999999999997e-06, 7.8827411584999996e-01, 1.4956319713459999e+02 },
    { 5.5532399999999999e-06, 4.6755497871299996e+00, 6.3735898303399999e+01 },
    { 4.5588700000000001e-06, 3.1733198566200000e+00, 8.5827298831199997e+01 },
    { 4.4689300000000001e-06, 5.0081017950000000e+00, 7.1600204829600003e+01 },
    { 4.2831200000000003e-06, 4.0941934177199997e+00, 7.3297125859000005e+01 },
    { 4.2430200000000004e-06, 2.8132587507200002e+00, 7.7962992305000000e+01 },
    { 4.0843199999999997e-06, 3.7297192645699999e+00, 7.6266071275599998e+01 },
    { 3.4460499999999998e-06, 2.2241656468700000e+00, 1.1045700263900001e+01 },
    { 2.5300100000000002e-06, 2.5118257200800000e+00, 3.9321532631000000e+00 },
    { 1.8856499999999999e-06, 8.1232221064999999e-01, 7.4781598567299994e+01 },
    { 1.3201399999999999e-06, 5.7301278319800000e+00, 1.3851749687070000e+02 },
    { 1.2134000000000001e-06, 5.3227881474099998e+00, 1.4563104387150000e+02 },
    { 1.2051699999999999e-06, 5.4805481445500002e+00, 1.8159247264699999e+01 },
    { 8.1058999999999995e-07, 3.1859981535999998e-01, 3.1813937377000001e+00 },
    { 5.2580999999999997e-07, 3.3242671438600002e+00, 2.2434479570190001e+02 },
    { 5.0989000000000002e-07, 2.3815214511299998e+00, 1.3140394986990000e+02 },
    { 4.5947000000000002e-07, 2.9384217796800001e+00, 2.2091400527800001e+01 },
    { 4.1352000000000001e-07, 4.0988109551000002e-01, 5.6622351302600002e+01 },
    { 3.4956000000000001e-07, 6.1518007769700001e+00, 1.5104766984290001e+02 },
    { 3.4550000000000002e-07, 1.0109527021400000e+00, 9.2940845831999994e+01 },
    { 2.9013000000000002e-07, 1.7749598386100001e+00, 2.2041264243879999e+02 },
    { 2.8439000000000002e-07, 5.7106335463700004e+00, 1.6060889739850001e+02 },
    { 2.6417999999999997e-07, 1.1462701929300001e+00, 1.5349535039770001e+02 },
    { 2.0751999999999999e-07, 2.1287415066000001e-01, 8.4342826122899993e+01 },
    { 2.0538999999999999e-07, 3.0619547191700001e+00, 2.0618554843720000e+02 },
    { 1.9493000000000001e-07, 4.4080406915100001e+00, 6.7668051566499997e+01 },
    { 1.9434999999999999e-07, 1.5465636576699999e+00, 6.5220371011699996e+01 },
    { 1.9345000000000000e-07, 2.3887211805400002e+00, 5.2690198039499997e+01 },
    { 1.8708999999999999e-07, 3.4480783870399998e+00, 8.1895145568100006e+01 },
    { 1.7277000000000000e-07, 1.2454849849800000e+00, 1.4638180339690001e+02 },
    { 1.7211000000000000e-07, 4.6960886532900004e+00, 1.2747179660680000e+02 },
    { 1.5815000000000000e-07, 5.8499489346000004e+00, 5.2257741809380002e+02 },
    { 1.5606000000000001e-07, 4.7722680474599999e+00, 5.3680451209540001e+02 },
    { 1.5538999999999999e-07, 2.0347639384299998e+00, 8.7311771539500000e+01 },
    { 1.5215000000000000e-07, 6.2300290009200001e+00, 1.4807872442630000e+02 },
    { 8.7590000000000005e-08, 5.8165272383000000e+00, 6.2251425595100002e+01 },
    { 8.6630000000000002e-08, 4.7269164614400001e+00, 9.6872999095099999e+01 },
    { 7.8750000000000000e-08, 6.0684825448300002e+00, 1.9513984817330001e+02 },
    { 7.7089999999999997e-08, 1.2290316989100001e+00, 1.5274459087229999e+02 },
    { 6.8919999999999998e-08, 4.2290386095999999e-01, 6.9152524274800001e+01 },
    { 6.8919999999999998e-08, 1.1149059214499999e+00, 8.0410672859800002e+01 },
    { 6.5190000000000003e-08, 3.5135937445100001e+00, 4.5576651038700000e+01 },
    { 6.1309999999999998e-08, 2.6940800325700001e+00, 1.3458534360760001e+02 },
    { 5.7900000000000002e-08, 1.8611389816899999e+00, 7.2333918012500007e+01 },
    { 5.6920000000000003e-08, 5.2211595223799998e+00, 2.0936694217490000e+02 },
    { 5.5540000000000000e-08, 2.8417402629400002e+00, 7.7229279122099996e+01 },
    { 5.2579999999999998e-08, 5.3646732132299997e+00, 1.2035824960600000e+02 },
    { 4.9950000000000001e-08, 5.0327849416800001e+00, 2.9823262239190001e+02 },
    { 4.5559999999999999e-08, 1.0993207404400001e+00, 2.9204947528600002e+01 },
    { 4.4199999999999999e-08, 3.8885472169700002e+00, 8.0198224538700003e+01 },
    { 4.0600000000000000e-08, 3.8481685032200001e+00, 3.1245971639350000e+02 },
    { 3.8549999999999998e-08, 1.9744489674999999e-01, 5.9803745040300001e+01 },
    { 3.8040000000000002e-08, 2.0143822235000000e-01, 2.5887574647669999e+02 },
    { 3.8029999999999997e-08, 3.4422349149799998e+00, 2.0225339517410001e+02 },
    { 3.7720000000000003e-08, 2.0281604098699999e+00, 7.1812653150700001e+01 },
    { 3.7720000000000003e-08, 2.6512420261299998e+00, 7.7750543983900002e+01 },
    { 3.7009999999999999e-08, 1.6691795858100000e+00, 2.1329909543799999e+02 },
    { 3.5929999999999998e-08, 3.1240955855100001e+00, 1.9907200143640000e+02 },
    { 3.5640000000000000e-08, 2.9740398100999998e+00, 7.0115732121299999e+01 },
    { 3.5640000000000000e-08, 1.7053626259100001e+00, 7.9447465013300004e+01 },
    { 3.5339999999999997e-08, 5.9650948025999995e-01, 6.9364972595899999e+01 },
    { 3.5240000000000003e-08, 3.8429907747900001e+00, 6.3596513305090002e+02 },
    { 3.4280000000000000e-08, 4.8042331466699997e+00, 6.2173803904930003e+02 },
    { 3.3409999999999999e-08, 3.0624961177999999e+00, 4.7217484191469998e+02 },
    { 3.2880000000000001e-08, 4.6653469439000004e+00, 3.8405992122309999e+02 },
    { 3.2439999999999998e-08, 4.0542082290000003e+00, 6.1158611066269998e+02 },
    { 3.2439999999999998e-08, 2.1370277779800002e+00, 4.0843894361129998e+02 },
    { 3.1650000000000002e-08, 6.2819309138000001e-01, 4.2659819087599999e+02 },
    { 2.9980000000000001e-08, 3.9638909853400000e+00, 7.3084677537900006e+01 },
    { 2.9980000000000001e-08, 7.1551145065999999e-01, 7.6478519596699996e+01 },
    { 2.9340000000000000e-08, 4.3908095392300002e+00, 2.7310284047829998e+02 },
    { 2.8089999999999999e-08, 6.2761620481899998e+00, 6.4701083331480004e+02 },
    { 2.8050000000000000e-08, 4.4643171860199997e+00, 9.8460033162189995e+02 },
    { 2.7090000000000000e-08, 2.2825122052100002e+00, 4.8640193591629998e+02 },
    { 2.6680000000000001e-08, 2.7878617509399999e+00, 3.7619561469690001e+02 },
    { 2.6400000000000001e-08, 2.2205881776099998e+00, 2.8808069400530002e+02 },
    { 2.6029999999999999e-08, 5.7290382834600004e+00, 4.1555249061209997e+02 },
    { 2.5629999999999999e-08, 1.5484369939500000e+00, 2.1648048917570000e+02 },
    { 2.3580000000000000e-08, 8.7971471501999998e-01, 6.2491943278700001e+02 },
    { 2.3180000000000000e-08, 8.9080611599000004e-01, 4.3371173787679999e+02 },
    { 2.2189999999999999e-08, 3.0073230532699999e+00, 3.9117346822389999e+02 },
};

static const VSOP87Term VSOP87A_URANUS_X4[12] = {
    { 6.2987000000000001e-07, 3.1415926535900001e+00, 0.0000000000000000e+00 },
    { 3.4199000000000001e-07, 4.2702885820000003e-01, 6.3735898303399999e+01 },
    { 2.7042000000000001e-07, 1.1162510994799999e+00, 8.5827298831199997e+01 },
    { 2.6702000000000002e-07, 5.3449373770599999e+00, 1.4956319713459999e+02 },
    { 1.6292999999999999e-07, 3.6271471104100002e+00, 1.8159247264699999e+01 },
    { 6.8989999999999998e-08, 4.2153996190800003e+00, 1.3140394986990000e+02 },
    { 5.8519999999999997e-08, 2.3248429694400001e+00, 5.6622351302600002e+01 },
    { 5.2269999999999998e-08, 5.4107981647400001e+00, 9.2940845831999994e+01 },
    { 2.3619999999999999e-08, 9.0248563877999999e-01, 2.8414854074220000e+02 },
    { 2.2950000000000000e-08, 3.0360798094299999e+00, 3.6590067395839998e+02 },
    { 2.2819999999999999e-08, 1.4341692666000000e-01, 2.2041264243879999e+02 },
    { 2.2480000000000000e-08, 4.8109465963000000e+00, 2.0618554843720000e+02 },
};

static const VSOP87Term VSOP87A_URANUS_X5[1] = {
    { 0.0, 0.0, 0.0 }
};

static const VSOP87Term VSOP87A_URANUS_Y0[1447] = {
    { 1.9165182315839999e+01, 3.9104567700200001e+00, 7.4781598567299994e+01 },
    { 4.4390465203000001e-01, 8.8841113289999996e-02, 1.4956319713459999e+02 },
    { 1.6256125476000000e-01, 3.1415926535900001e+00, 0.0000000000000000e+00 },
    { 1.4755940185999999e-01, 1.8542328067900000e+00, 7.3297125859000005e+01 },
    { 1.4123958128000000e-01, 2.8248607654900000e+00, 7.6266071275599998e+01 },
    { 6.2500782310000003e-02, 3.5696024385700000e+00, 1.4844727083000000e+00 },
    { 1.5426682640000000e-02, 2.5504053921300001e+00, 2.2434479570190001e+02 },
    { 1.4423565750000001e-02, 1.0800454271200000e+00, 1.4807872442630000e+02 },
    { 9.3897550100000005e-03, 9.2757147609999993e-02, 1.1045700263900001e+01 },
    { 6.5734312000000003e-03, 5.2883070446899998e+00, 1.5104766984290001e+02 },
    { 6.5033184600000000e-03, 2.7614268022199999e+00, 6.3735898303399999e+01 },
    { 6.2132677000000004e-03, 1.4879581138700000e+00, 7.7750543983900002e+01 },
    { 5.4747269399999998e-03, 2.0603792457300001e+00, 8.5827298831199997e+01 },
    { 5.4196195799999999e-03, 3.2447648666100002e+00, 7.1812653150700001e+01 },
    { 4.9593610499999996e-03, 5.3120575374000003e+00, 5.2969096509459996e+02 },
    { 4.5958911999999996e-03, 2.3374553606999999e+00, 2.9689454166000000e+00 },
    { 3.8792285299999998e-03, 4.6202692388499997e+00, 1.3851749687070000e+02 },
    { 2.6836341700000000e-03, 5.6808529902000000e+00, 2.1329909543799999e+02 },
    { 2.1623962900000002e-03, 3.7380076758000000e+00, 3.8133035637799999e+01 },
    { 1.4403247500000000e-03, 7.5015700919999995e-01, 7.0849445304200003e+01 },
    { 1.3529082000000000e-03, 3.9397026061600000e+00, 7.8713751830400000e+01 },
    { 1.2486854500000000e-03, 9.4315917318999998e-01, 1.1143016149680000e+02 },
    { 1.1967061300000000e-03, 2.5305878378000002e+00, 3.9617508346100003e+01 },
    { 1.1120486000000000e-03, 3.5516321941900002e+00, 2.2286032299359999e+02 },
    { 1.0858445400000001e-03, 6.0223484838800001e+00, 3.5164090221199999e+01 },
    { 1.0450792899999999e-03, 2.3334567560299999e+00, 1.4659425171800001e+02 },
    { 6.3774261000000001e-04, 2.1560760290399998e+00, 1.0994568878849999e+02 },
    { 6.3573747000000001e-04, 5.0120496792000004e+00, 2.9912639426919998e+02 },
    { 5.3289771000000004e-04, 2.3843758787599998e+00, 3.9321532631000000e+00 },
    { 3.9256771000000001e-04, 5.7595685370299998e+00, 2.0225339517410001e+02 },
    { 3.9218598000000001e-04, 1.1184110925199999e+00, 4.4534181248999998e+00 },
    { 3.4538316000000001e-04, 2.7613780697000001e-01, 7.9235016692200006e+01 },
    { 3.4334376999999998e-04, 1.4669616984300000e+00, 2.2582926841020000e+02 },
    { 3.4205425999999999e-04, 9.2405922575999999e-01, 6.5220371011699996e+01 },
    { 3.1816302999999998e-04, 3.9686017048400002e+00, 1.5253214255120000e+02 },
    { 2.8372491000000001e-04, 4.4471462709700003e+00, 1.8472728735580000e+02 },
    { 2.6652859000000001e-04, 4.5394439534700002e+00, 1.6060889739850001e+02 },
    { 2.6157753999999999e-04, 3.7409761079799999e+00, 9.5612275556000004e+00 },
    { 2.5237175999999999e-04, 4.4514141366600004e+00, 7.0328180442399997e+01 },
    { 2.3427328000000000e-04, 2.5274012555100001e+00, 1.4563104387150000e+02 },
    { 2.2933137999999999e-04, 3.9445554034999999e+00, 8.4342826122899993e+01 },
    { 2.1523908000000000e-04, 4.9356513206799999e+00, 3.6648562929500002e+01 },
    { 2.0115100000000000e-04, 3.4547378076199999e+00, 1.2747179660680000e+02 },
    { 2.0051640999999999e-04, 2.9038635293700001e+00, 2.2091400527800001e+01 },
    { 1.9901477000000000e-04, 6.1107540243400003e+00, 1.1291463420510000e+02 },
    { 1.9838981000000000e-04, 5.2911339735400000e+00, 1.2530172972200001e+01 },
    { 1.9666207999999999e-04, 3.9635006533500001e+00, 7.4669723982700006e+01 },
    { 1.9643845000000000e-04, 7.1577796385000003e-01, 7.4893473151899997e+01 },
    { 1.8126776000000001e-04, 9.8478853786999998e-01, 3.3679617512900002e+01 },
    { 1.5537967000000001e-04, 1.8786327545999999e+00, 5.2690198039499997e+01 },
    { 1.5174962000000000e-04, 1.3131403495900000e+00, 4.1101981054399999e+01 },
    { 1.3948849000000000e-04, 4.7047494568200001e+00, 2.2137585028530000e+02 },
    { 1.3589664999999999e-04, 9.8313719929999999e-01, 8.7311771539500000e+01 },
    { 1.2407786999999999e-04, 4.6494578333999996e+00, 7.2333918012500007e+01 },
    { 1.1996771999999999e-04, 5.6612927533499997e+00, 1.0593819301891999e+03 },
    { 1.1531140000000000e-04, 2.0190074645000000e-01, 7.7229279122099996e+01 },
    { 1.1239020000000000e-04, 4.5450833401099997e+00, 7.1600204829600003e+01 },
    { 1.0819728000000000e-04, 1.2807029855999999e-01, 7.7962992305000000e+01 },
    { 8.7361500000000000e-05, 3.3987482829300002e+00, 1.8621176006409999e+02 },
    { 7.8560140000000003e-05, 1.1235425483100001e+00, 3.4077089204480001e+02 },
    { 7.4939999999999997e-05, 4.8256577138600001e+00, 4.2659819087599999e+02 },
    { 7.0935870000000004e-05, 6.0161348724500003e+00, 2.9764192156090002e+02 },
    { 6.4082450000000005e-05, 3.9324636789500000e+00, 6.2251425595100002e+01 },
    { 6.2611529999999997e-05, 1.4258542752000000e-01, 1.5349535039770001e+02 },
    { 5.5566429999999999e-05, 3.6809521506300000e+00, 1.4510977900969999e+02 },
    { 5.5160179999999999e-05, 1.7375832611900000e+00, 1.4000196957899999e+02 },
    { 5.3684050000000000e-05, 2.5542295795799999e+00, 7.5302863429100000e+01 },
    { 5.3509479999999998e-05, 2.1217149392199999e+00, 7.4260333705500003e+01 },
    { 5.0138710000000002e-05, 3.1190774926799998e+00, 1.8159247264699999e+01 },
    { 4.7500180000000002e-05, 3.3867830005399999e+00, 7.3818390720799997e+01 },
    { 4.5090540000000001e-05, 1.3125434282899999e+00, 7.5744806413800006e+01 },
    { 4.5087940000000002e-05, 5.8222406482100002e+00, 6.6704843720000000e+01 },
    { 4.4453470000000002e-05, 5.0063849030799998e+00, 1.1439910691340000e+02 },
    { 4.3261379999999997e-05, 5.7013505685299997e+00, 1.8324281464750001e+02 },
    { 4.2903740000000003e-05, 5.5449076655100002e+00, 2.6598929347749998e+02 },
    { 3.5754350000000001e-05, 5.8399484922399996e+00, 1.3703302416240001e+02 },
    { 3.5531070000000001e-05, 6.1890651684600000e+00, 5.9378908332000000e+00 },
    { 3.3141539999999997e-05, 5.2305457432900004e+00, 8.2858353414600003e+01 },
    { 3.2656339999999997e-05, 5.2306356017600004e+00, 2.2041264243879999e+02 },
    { 2.8854430000000001e-05, 3.5027999303800001e+00, 7.1135470007999997e+00 },
    { 2.8807340000000001e-05, 1.1903842433000000e+00, 3.7390799283650000e+02 },
    { 2.6766159999999999e-05, 1.6180536204400000e+00, 3.0534616939270001e+02 },
    { 2.5941549999999999e-05, 4.9369141353700003e+00, 9.6872999095099999e+01 },
    { 2.5593570000000000e-05, 2.8718423767800001e+00, 8.0198224538700003e+01 },
    { 2.4744829999999999e-05, 2.2117375111699999e+00, 3.2195144804599998e+01 },
    { 2.2752580000000002e-05, 1.5566640150500000e+00, 1.3140394986990000e+02 },
    { 2.2465300000000000e-05, 5.8008189876299996e+00, 1.0846121608020000e+02 },
    { 2.2406480000000002e-05, 4.7739127862000003e-01, 1.8175834193919999e+02 },
    { 2.2274509999999999e-05, 5.3476526455700002e+00, 8.0719489400499995e+01 },
    { 2.2128610000000001e-05, 6.1491281306000001e-01, 2.5950888592310002e+02 },
    { 2.1774370000000001e-05, 2.8043742210099998e+00, 4.7928838891549998e+02 },
    { 2.1643819999999999e-05, 4.7581392324999999e-01, 3.1813937377000001e+00 },
    { 2.1384069999999999e-05, 2.6381780433099999e+00, 7.4520966136400006e+01 },
    { 2.1309090000000002e-05, 2.0414391249500001e+00, 7.5042230998199997e+01 },
    { 2.0609910000000001e-05, 6.2417859638400000e+00, 5.6622351302600002e+01 },
    { 2.0452490000000000e-05, 5.1740078810399996e+00, 8.3503713448730002e+02 },
    { 1.9587110000000002e-05, 4.8080704581499996e+00, 2.0606927819500001e+01 },
    { 1.9378740000000001e-05, 4.9529839431000000e-01, 7.4829782677099999e+01 },
    { 1.9378709999999999e-05, 4.1846228868399997e+00, 7.4733414457500004e+01 },
    { 1.9259460000000001e-05, 6.0052747351500004e+00, 1.5912442469019999e+02 },
    { 1.9244989999999999e-05, 2.6428488049499999e+00, 2.0618554843720000e+02 },
    { 1.8961840000000001e-05, 4.2697589800299998e+00, 1.4977853527000001e+01 },
    { 1.8893690000000001e-05, 3.9285224017100000e+00, 3.0061086697750000e+02 },
    { 1.8878119999999999e-05, 2.7500023779100000e+00, 1.5401661525950001e+02 },
    { 1.8608239999999999e-05, 1.0445996392000000e-01, 4.2586453762700003e+01 },
    { 1.8024940000000000e-05, 4.1621825990200003e+00, 7.4621539872900001e+01 },
    { 1.7962919999999998e-05, 5.1761494342000003e-01, 7.4941657261700001e+01 },
    { 1.7947540000000000e-05, 1.6290844853000000e-01, 2.2731374111849999e+02 },
    { 1.7877370000000001e-05, 3.2096699926000000e-01, 2.4476805547999998e+00 },
    { 1.6261339999999999e-05, 3.7002373118400000e+00, 1.9120769491019999e+02 },
    { 1.5888970000000001e-05, 4.1511566897399996e+00, 3.9356875915200000e+01 },
    { 1.5728260000000000e-05, 5.9336781290299996e+00, 2.1989137757699999e+02 },
    { 1.4368500000000000e-05, 5.4431219835000002e+00, 5.2257741809380002e+02 },
    { 1.4248039999999999e-05, 5.1000475803300001e+00, 9.2940845831999994e+01 },
    { 1.4169169999999999e-05, 7.2597245493999996e-01, 2.3539049596580000e+02 },
    { 1.4055640000000001e-05, 2.0467739252700001e+00, 5.3680451209540001e+02 },
    { 1.3474100000000000e-05, 3.8923701169600000e+00, 2.8808069400530002e+02 },
    { 1.3367919999999999e-05, 3.7488898975599998e+00, 3.8012776796000003e+02 },
    { 1.2910650000000000e-05, 5.6742569904700000e+00, 6.8843707734099993e+01 },
    { 1.2697860000000000e-05, 1.0968572752900001e+00, 1.4244965013379999e+02 },
    { 1.2677660000000000e-05, 9.2771324395999999e-01, 4.0450679034820001e+02 },
    { 1.2228860000000000e-05, 2.2730609990200001e+00, 1.8769623277240001e+02 },
    { 1.2206960000000001e-05, 1.8498818596300000e+00, 2.1181462272970001e+02 },
    { 1.1922920000000000e-05, 6.8084398426000003e-01, 9.9160620955499994e+01 },
    { 1.1898009999999999e-05, 4.0177830613400003e+00, 5.0402576179100002e+01 },
    { 1.1749530000000000e-05, 4.5683512472999999e-01, 1.1020632121940000e+02 },
    { 1.1653150000000000e-05, 5.1071041451999999e-01, 8.1895145568100006e+01 },
    { 1.1637620000000000e-05, 5.5115778376200000e+00, 2.3056457082540001e+02 },
    { 1.1518029999999999e-05, 4.6579056125000001e-01, 1.2895626931510000e+02 },
    { 1.1512860000000000e-05, 4.0114773543800002e+00, 6.7668051566499997e+01 },
    { 1.1491140000000000e-05, 8.5101218281000002e-01, 2.9615744885260000e+02 },
    { 1.0516060000000000e-05, 3.9690764753500001e+00, 1.4014645680499999e+01 },
    { 1.0352619999999999e-05, 1.2063987645800001e+00, 5.4166259714000002e+00 },
    { 1.0159980000000000e-05, 1.0029050130699999e+00, 3.5424722652100002e+01 },
    { 1.0094540000000001e-05, 4.5375065997000003e-01, 7.4030839041899995e+01 },
    { 9.9778400000000007e-06, 4.2264078888999999e+00, 7.5532358092699994e+01 },
    { 9.7338100000000006e-06, 6.4028748776000000e-01, 2.1840690486870000e+02 },
    { 9.5279000000000007e-06, 1.2477383005300000e+00, 1.0697674337190000e+02 },
    { 9.2336499999999993e-06, 5.7687468576600001e+00, 5.4174670747800000e+01 },
    { 9.1133800000000001e-06, 1.2254293382000001e-01, 1.4945132254999999e+02 },
    { 9.0978800000000000e-06, 3.1583987832200000e+00, 1.4967507171919999e+02 },
    { 8.9676300000000007e-06, 5.1927845768000003e-01, 2.0076892246579999e+02 },
    { 8.9141399999999993e-06, 1.8200787248200001e+00, 2.3575873236100001e+01 },
    { 8.6376800000000006e-06, 3.4604186203900000e+00, 1.6209337010679999e+02 },
    { 8.5685600000000002e-06, 6.2175886502699997e+00, 8.9759452094300002e+01 },
    { 8.5555899999999992e-06, 4.4900456103000002e+00, 6.8189316428300003e+01 },
    { 8.3526399999999997e-06, 5.3443710596000003e+00, 5.9803745040300001e+01 },
    { 7.6604000000000004e-06, 8.9351020565999995e-01, 6.8561823443799994e+01 },
    { 7.4624199999999996e-06, 2.7971755279299999e+00, 7.6025553592000006e+02 },
    { 7.4285699999999999e-06, 2.4921661368799999e+00, 2.7703499374140000e+02 },
    { 7.0315399999999998e-06, 1.4985209795400001e+00, 2.0373786788240000e+02 },
    { 6.9563599999999998e-06, 4.4976743596600004e+00, 6.1780588578619995e+02 },
    { 6.6658599999999997e-06, 4.6331419017200002e+00, 6.2197751234999998e+00 },
    { 6.6275800000000002e-06, 2.5796780547999998e-01, 8.1373880706300000e+01 },
    { 6.6131600000000002e-06, 3.9123655428399999e+00, 8.0767548473000002e+00 },
    { 6.4342400000000004e-06, 5.9838323642500004e+00, 8.8796244247800004e+01 },
    { 6.2617300000000001e-06, 1.4763136288000001e-01, 2.4379022388199999e+01 },
    { 6.2301300000000003e-06, 2.9964940259500001e+00, 5.1205725331200000e+01 },
    { 6.1890100000000002e-06, 4.5254923111799998e+00, 1.2598732389849999e+02 },
    { 6.0401799999999997e-06, 2.7578075724500000e+00, 1.9513984817330001e+02 },
    { 5.9780500000000003e-06, 1.0714777061500000e+00, 1.3458534360760001e+02 },
    { 5.8888600000000002e-06, 4.6206496964200001e+00, 2.5802441321480001e+02 },
    { 5.8745600000000002e-06, 2.3374161360099999e+00, 1.5201087768939999e+02 },
    { 5.8127700000000001e-06, 8.3588836575000003e-01, 2.8571808082200000e+01 },
    { 5.6306500000000000e-06, 3.0898605960199998e+00, 1.5578297225809999e+02 },
    { 5.6034999999999998e-06, 1.7972964240000000e+00, 6.9364972595899999e+01 },
    { 5.5078199999999997e-06, 5.8562530884199999e+00, 2.6099335863139999e+02 },
    { 5.3782900000000000e-06, 5.5422801800099997e+00, 2.0936694217490000e+02 },
    { 5.3702900000000002e-06, 3.7571179541299999e+00, 8.1001373690799994e+01 },
    { 5.3617500000000000e-06, 1.5208460521000000e+00, 1.1642609634290000e+02 },
    { 5.2841300000000003e-06, 7.1044436153000001e-01, 1.4638180339690001e+02 },
    { 5.1404900000000003e-06, 3.8374245100200000e+00, 1.1588357962169999e+02 },
    { 5.1350799999999996e-06, 7.4576777969999997e-01, 3.5181659230870002e+02 },
    { 5.0597199999999997e-06, 9.5984060933000004e-01, 1.5763995198190000e+02 },
    { 4.8807900000000001e-06, 5.0440335081300001e+00, 7.3409000443599993e+01 },
    { 4.8562700000000000e-06, 5.7727769662900004e+00, 4.1555249061209997e+02 },
    { 4.7243099999999997e-06, 1.7206368379300001e+00, 1.8027386923090000e+02 },
    { 4.4958500000000003e-06, 2.6045738626800001e+00, 5.4302428721889999e+02 },
    { 4.4558099999999996e-06, 5.3327170628799996e+00, 3.2972519178089999e+02 },
    { 4.3563199999999998e-06, 6.0115247562200000e+00, 1.5890728952838001e+03 },
    { 4.3563000000000001e-06, 2.1941380371800001e+00, 3.7242352012819998e+02 },
    { 4.2626100000000003e-06, 8.9836299897000005e-01, 1.4148644228730001e+02 },
    { 4.1961299999999999e-06, 6.2160522451400002e+00, 4.1644497775600001e+01 },
    { 4.1863400000000003e-06, 4.0340567628000001e-01, 9.5388526386799995e+01 },
    { 4.0555600000000004e-06, 2.6733981996799998e+00, 4.9033408917940000e+02 },
    { 4.0506699999999996e-06, 7.6651664120999996e-01, 2.5505546779820000e+02 },
    { 3.9036399999999996e-06, 4.3931670225700001e+00, 1.5274459087229999e+02 },
    { 3.8144899999999998e-06, 3.6396124573800002e+00, 1.1413847448250000e+02 },
    { 3.7462299999999998e-06, 1.9021509345100001e+00, 7.3185251274400002e+01 },
    { 3.6901000000000000e-06, 4.1107863882600002e+00, 1.4414657116320001e+02 },
    { 3.5452599999999999e-06, 4.6641029109299996e+00, 2.1478356814630001e+02 },
    { 3.5410000000000001e-06, 5.2626415103099999e+00, 4.6209790485100001e+01 },
    { 3.3747200000000002e-06, 3.6059477361800001e+00, 1.5052640498110000e+02 },
    { 3.2597599999999999e-06, 2.6192442104200002e+00, 2.2827694896500000e+02 },
    { 3.2427800000000000e-06, 4.9770342511900001e+00, 7.4223635415000002e+00 },
    { 3.1846399999999998e-06, 4.5911675627999998e-01, 1.2035824960600000e+02 },
    { 3.1786899999999999e-06, 4.9393789102500003e+00, 3.3137100791700000e+01 },
    { 3.1635400000000001e-06, 3.4320992609299998e+00, 3.0710672096300001e+01 },
    { 3.1366799999999998e-06, 1.3941813053600001e+00, 4.1948464387519999e+02 },
    { 3.1181500000000000e-06, 7.9978809612000001e-01, 1.4711551657979999e+02 },
    { 3.0760599999999998e-06, 4.0627940354199996e+00, 6.3989728631399998e+02 },
    { 3.0670800000000001e-06, 2.9191161277000002e+00, 1.0309277421860000e+02 },
    { 3.0625700000000001e-06, 1.8368210738999999e-01, 6.2830758499914000e+03 },
    { 3.0229500000000000e-06, 4.3076728878999998e-01, 9.8460033162189995e+02 },
    { 2.8709299999999999e-06, 5.2449142736400001e+00, 4.5576651038700000e+01 },
    { 2.7804699999999999e-06, 5.8886568396900003e+00, 2.9204947528600002e+01 },
    { 2.7700400000000000e-06, 4.2162059087700001e+00, 6.5922821389999999e+00 },
    { 2.6521099999999998e-06, 1.1122645354700000e+00, 2.8311175651300001e+01 },
    { 2.5753600000000001e-06, 3.0399822269999999e-01, 1.2899465010146000e+03 },
    { 2.5571399999999999e-06, 6.1803715020299999e+00, 1.9122455111200001e+01 },
    { 2.5240299999999999e-06, 5.0115435891600004e+00, 1.5008446199639999e+02 },
    { 2.4658300000000002e-06, 1.1777443398900000e+00, 5.5406998748280000e+02 },
    { 2.3972000000000001e-06, 6.1979329770999998e+00, 7.5632338265689998e+02 },
    { 2.3412999999999999e-06, 3.1453075510700002e+00, 3.4225536475310003e+02 },
    { 2.3329700000000000e-06, 2.1860937257200002e+00, 3.3928641933649999e+02 },
    { 2.3168099999999998e-06, 2.7809332589900002e+00, 4.6470422915999997e+01 },
    { 2.3006999999999998e-06, 5.3561929597100004e+00, 6.0766952886799999e+01 },
    { 2.2769900000000001e-06, 1.8887927465800001e+00, 2.1692243216040001e+02 },
    { 2.2732699999999999e-06, 3.7856126480799999e+00, 1.2099138905240000e+02 },
    { 2.2219200000000000e-06, 3.4439351549300001e+00, 4.7694263193399998e+01 },
    { 2.1994900000000000e-06, 5.1809424914499997e+00, 4.4070926471000000e+01 },
    { 2.1675699999999998e-06, 9.0936155593000001e-01, 1.3554855145409999e+02 },
    { 2.1443700000000002e-06, 1.1801659152499999e+00, 1.8918070548070000e+02 },
    { 2.1092999999999999e-06, 1.0978636179600001e+00, 3.1639186965660002e+02 },
    { 2.1054499999999999e-06, 3.3512449112599998e+00, 5.2126486179999998e-01 },
    { 2.0349000000000002e-06, 5.8294001741399999e+00, 1.4859998928810001e+02 },
    { 2.0286700000000002e-06, 9.2934842891000002e-01, 2.9170403072770000e+02 },
    { 1.9878499999999999e-06, 3.1879719656300001e+00, 6.9673789136600007e+01 },
    { 1.9295700000000001e-06, 5.1217789873599999e+00, 9.6320784650000002e-01 },
    { 1.8713000000000000e-06, 4.6320769817700000e+00, 1.4904193227280001e+02 },
    { 1.8582099999999999e-06, 5.7600884391299996e+00, 7.6154196690999996e+01 },
    { 1.8580200000000000e-06, 3.9956032082399999e+00, 7.3887826689999997e+01 },
    { 1.8580200000000000e-06, 6.8379922776000002e-01, 7.5675370444600006e+01 },
    { 1.7958100000000001e-06, 5.3648786082699997e+00, 1.0335340664950000e+02 },
    { 1.7849200000000000e-06, 3.9921147736000001e-01, 6.8547393735269998e+02 },
    { 1.7673199999999999e-06, 1.6084537642800001e+00, 1.0213285546211000e+04 },
    { 1.7583599999999999e-06, 1.5340849182600000e+00, 5.7255490749000003e+01 },
    { 1.7556000000000000e-06, 4.4184951295800001e+00, 3.3365734504400001e+02 },
    { 1.7374900000000000e-06, 4.2862851264800002e+00, 6.8154178408960001e+02 },
    { 1.6775600000000000e-06, 5.5939899599500000e+00, 3.0056280790500001e+01 },
    { 1.6529200000000000e-06, 5.3694119881900004e+00, 1.5497982310600000e+02 },
    { 1.6458599999999999e-06, 7.0747217893000003e-01, 4.6824268865160002e+02 },
    { 1.6345000000000000e-06, 1.5320090689100001e+00, 7.9889407997999996e+01 },
    { 1.6312700000000000e-06, 3.3878389027000000e-01, 2.6450482076920002e+02 },
    { 1.6186099999999999e-06, 2.5416602018900001e+00, 1.0549227066360000e+02 },
    { 1.5917500000000000e-06, 3.5104477507800000e+00, 1.7526107818300002e+01 },
    { 1.5791200000000000e-06, 4.1365514028200003e+00, 8.2203962108799999e+01 },
    { 1.5753000000000000e-06, 1.0226022440600000e+00, 3.4788443904560000e+02 },
    { 1.5471899999999999e-06, 4.6654892516000004e+00, 2.9467297614429998e+02 },
    { 1.5263900000000001e-06, 2.0330167942499999e+00, 2.3390602325750001e+02 },
    { 1.4643699999999999e-06, 3.0641652525600001e+00, 3.3429048449039999e+02 },
    { 1.4586600000000001e-06, 4.7250834456300002e+00, 2.4118389957300000e+01 },
    { 1.4368400000000000e-06, 4.3841673194600004e+00, 6.2885158605009997e+02 },
    { 1.4197100000000000e-06, 4.6769224594200001e+00, 1.7394221952280000e+02 },
    { 1.4046700000000001e-06, 4.4986269196800004e+00, 2.8659622129700000e+02 },
    { 1.3799300000000001e-06, 4.9895867902000002e-01, 1.6674774556399999e+01 },
    { 1.3768699999999999e-06, 3.6525592260700002e+00, 4.4868959140380002e+02 },
    { 1.3654600000000001e-06, 4.4439631891999998e+00, 9.0981873305459999e+02 },
    { 1.3517400000000000e-06, 1.1541863116700000e+00, 1.7165459766239999e+02 },
    { 1.3040400000000000e-06, 4.5669690420800002e+00, 7.6478519596699996e+01 },
    { 1.2898600000000001e-06, 3.4559121430699999e+00, 2.5494359321360000e+02 },
    { 1.2879199999999999e-06, 1.5382024654299999e+00, 1.5550108796780000e+02 },
    { 1.2815500000000000e-06, 2.4513444719700002e+00, 1.5615547927360001e+02 },
    { 1.2791400000000000e-06, 5.9161272327400001e+00, 6.1448276442999997e+01 },
    { 1.2576900000000000e-06, 3.8590752677900002e+00, 9.8357471803400003e+01 },
    { 1.2189699999999999e-06, 4.8962130300200002e+00, 7.9447465013300004e+01 },
    { 1.2010499999999999e-06, 5.0480741796800004e+00, 1.3044074202339999e+02 },
    { 1.1679800000000000e-06, 2.7997501160599998e+00, 6.9258748435350003e+02 },
    { 1.1625800000000000e-06, 4.1992335469800004e+00, 1.2151649024472999e+03 },
    { 1.1435700000000000e-06, 4.8018392495800004e+00, 2.7310284047829998e+02 },
    { 1.1398699999999999e-06, 2.0505963892399999e+00, 2.5357099508990001e+02 },
    { 1.1390400000000001e-06, 5.1995021988400003e+00, 1.3333322124300000e+01 },
    { 1.1362400000000001e-06, 1.2277440377300000e+00, 6.0447256366190004e+02 },
    { 1.1360900000000000e-06, 3.9979875181400000e+00, 6.3278373931320004e+02 },
    { 1.1232399999999999e-06, 5.2265735908200002e+00, 2.2879821382680001e+02 },
    { 1.1222600000000000e-06, 3.4625393691699999e+00, 9.2307706385600000e+01 },
    { 1.1054000000000000e-06, 2.0320473244200001e+00, 1.9643719973000000e+01 },
    { 1.0980200000000000e-06, 5.8601923136899998e+00, 1.4362530630140000e+02 },
    { 1.0915500000000000e-06, 3.6928138677200000e+00, 5.5659143456099997e+01 },
    { 1.0789200000000001e-06, 1.0090598897000000e-01, 3.7539246554480002e+02 },
    { 1.0751900000000000e-06, 4.7312102040499999e+00, 1.1624747044077999e+03 },
    { 1.0686199999999999e-06, 1.5689423053700000e+00, 8.1682697246999993e+01 },
    { 1.0615800000000000e-06, 2.6402852420100000e+00, 3.0209533968580001e+02 },
    { 1.0573900000000000e-06, 5.1356349608800000e+00, 6.6917292041099998e+01 },
    { 1.0568899999999999e-06, 5.9075087134900004e+00, 1.0038446123289999e+02 },
    { 1.0511000000000000e-06, 1.0948855773499999e+00, 3.2835259365719998e+02 },
    { 1.0492799999999999e-06, 4.7162188859600001e+00, 4.3371173787679999e+02 },
    { 1.0415200000000000e-06, 3.1250084063600001e+00, 2.1033015002139999e+02 },
    { 1.0375100000000001e-06, 9.8587271820000003e-02, 5.4334729442200000e+01 },
    { 1.0165800000000000e-06, 3.5890468314000001e-01, 3.6286229257259998e+02 },
    { 9.9119999999999993e-07, 5.0995888399800000e+00, 1.4930256470370000e+02 },
    { 9.9119999999999993e-07, 4.5025867379399998e+00, 1.4982382956550001e+02 },
    { 9.8940000000000001e-07, 1.3272071637500000e+00, 2.7087335373900000e+01 },
    { 9.8662999999999999e-07, 5.2868116053999996e+00, 8.8114920691600005e+01 },
    { 9.7992000000000006e-07, 4.6131365176700001e+00, 1.0522683831884001e+03 },
    { 9.7686999999999993e-07, 1.5145861841799999e+00, 7.6787336137400004e+01 },
    { 9.6174000000000009e-07, 2.5364954585700001e+00, 2.5887574647669999e+02 },
    { 9.3010000000000000e-07, 1.8025749511300000e+00, 9.3904053678500006e+01 },
    { 9.2793999999999998e-07, 1.4702893285200000e+00, 4.3289029178299998e+01 },
    { 9.0974999999999995e-07, 8.3013329095999999e-01, 9.8899988524600005e+01 },
    { 9.0777000000000003e-07, 2.9173047031600001e+00, 2.6747376618580000e+02 },
    { 9.0373999999999996e-07, 1.4685158614500000e+00, 1.7265475387399999e+01 },
    { 8.9933999999999995e-07, 2.9564251508099999e+00, 1.4961138124440001e+02 },
    { 8.9933999999999995e-07, 3.6256511993000001e-01, 1.4951501302480000e+02 },
    { 8.9774000000000004e-07, 3.1350459425900001e+00, 2.1723124870109999e+02 },
    { 8.9602999999999996e-07, 1.0461991681800000e+00, 2.9519424100610001e+02 },
    { 8.7921000000000005e-07, 4.3267064855299999e+00, 7.4148459120900000e+01 },
    { 8.7921000000000005e-07, 3.5269595047000002e-01, 7.5414738013700003e+01 },
    { 8.7751999999999999e-07, 5.0685020561299998e+00, 3.3132153907380001e+02 },
    { 8.7611999999999999e-07, 4.0920088954000002e+00, 5.8106824010899999e+01 },
    { 8.7003999999999997e-07, 1.6228428298599999e+00, 8.9484087952760001e+02 },
    { 8.6520999999999995e-07, 5.7799002927899998e+00, 6.9010986796999996e+00 },
    { 8.6483000000000005e-07, 1.7065412572000001e+00, 1.9269216761850001e+02 },
    { 8.6240999999999995e-07, 4.5306330610999996e+00, 7.6426129970000005e+01 },
    { 8.5206999999999998e-07, 3.0461863813000001e+00, 1.7368158709190001e+02 },
    { 8.4005000000000001e-07, 3.2855328301400002e+00, 3.7093904741990002e+02 },
    { 8.3480000000000004e-07, 2.9787312369299999e+00, 1.4972325582900001e+02 },
    { 8.3480000000000004e-07, 3.4025903382000000e-01, 1.4940313844020000e+02 },
    { 8.3450000000000001e-07, 4.8213468330699998e+00, 4.8077286162380000e+02 },
    { 8.2091000000000001e-07, 1.6630306204400001e+00, 1.7516605980020000e+02 },
    { 8.2024000000000004e-07, 5.8327028236500000e+00, 8.2645905093500005e+01 },
    { 8.0994000000000001e-07, 8.4859852290000004e-01, 3.3280601178209997e+02 },
    { 8.0077000000000004e-07, 3.1311799291000000e+00, 1.6462326235300001e+01 },
    { 8.0075999999999998e-07, 4.6711196256999998e-01, 7.3137067164599998e+01 },
    { 7.9920000000000001e-07, 3.8703019288500000e+00, 4.7780391620720002e+02 },
    { 7.8846000000000001e-07, 4.7256113048700001e+00, 2.6247783133970000e+02 },
    { 7.8742999999999995e-07, 2.6545907281200001e+00, 1.5499118388799999e+01 },
    { 7.8541999999999995e-07, 3.8415458356599999e+00, 5.0663208609999998e+01 },
    { 7.7991999999999996e-07, 1.7030010361900001e+00, 7.0115732121299999e+01 },
    { 7.7639000000000003e-07, 1.4924872613100000e+00, 7.1158261844899997e+01 },
    { 7.7421999999999995e-07, 6.1787693879000001e+00, 7.0046296152099998e+01 },
    { 7.7397000000000004e-07, 3.1940974495400001e+00, 3.1017209453309999e+02 },
    { 7.7135000000000003e-07, 2.8963214300000002e-02, 2.6023553790899999e+01 },
    { 7.6677000000000003e-07, 5.8393689714999999e+00, 7.3557758289899994e+01 },
    { 7.5759999999999995e-07, 6.0996671802500000e+00, 2.1892816973050000e+02 },
    { 7.5507000000000000e-07, 2.0674397426500000e+00, 6.7077350735500005e+01 },
    { 7.5104000000000003e-07, 3.2147615358500001e+00, 2.5602862665600000e+01 },
    { 7.4870000000000003e-07, 5.3098552354199997e+00, 1.1950691634410001e+02 },
    { 7.4732000000000005e-07, 5.2529902516200000e+00, 5.1078094306999997e+00 },
    { 7.4568999999999995e-07, 4.5578980325599998e+00, 9.5228467692400002e+01 },
    { 7.4010000000000002e-07, 4.7662690643200003e+00, 1.8972322220190000e+02 },
    { 7.3252999999999996e-07, 2.9739398637200001e+00, 1.7878939652259999e+02 },
    { 7.0498000000000000e-07, 2.8010408426000000e+00, 7.6377945860200001e+01 },
    { 7.0394999999999995e-07, 3.2055454197900000e+00, 7.8404935289700006e+01 },
    { 7.0185999999999996e-07, 5.9782623372300003e+00, 8.2005928096030004e+02 },
    { 7.0111999999999997e-07, 1.2829908041800000e+00, 1.6772244439930000e+02 },
    { 6.9347000000000003e-07, 5.0148040473200002e+00, 1.6041635110000001e+01 },
    { 6.8853000000000004e-07, 2.3973367662400000e+00, 1.0664954771900000e+03 },
    { 6.7869000000000000e-07, 2.3833684627500000e+00, 6.0676018552230005e+02 },
    { 6.7640000000000000e-07, 6.4240183724000000e-01, 2.7851946644970002e+02 },
    { 6.6431999999999996e-07, 1.8361296099600000e+00, 8.4608283475120004e+02 },
    { 6.6387000000000003e-07, 6.2409550597300001e+00, 1.2518417474640000e+02 },
    { 6.5856999999999995e-07, 4.4948669890900002e+00, 7.8643065262000000e+00 },
    { 6.5828999999999995e-07, 3.5938438378999998e-01, 7.3978449415200004e+01 },
    { 6.5828999999999995e-07, 4.3200180522100000e+00, 7.5584747719399999e+01 },
    { 6.5077999999999996e-07, 2.2830709155600002e+00, 2.3767811782620001e+02 },
    { 6.4315000000000004e-07, 6.0909061221999998e+00, 4.7217484191469998e+02 },
    { 6.2224999999999995e-07, 2.6056495082600000e+00, 8.2485846399099998e+01 },
    { 6.1846000000000005e-07, 3.0135801944999998e+00, 6.1288217748599997e+01 },
    { 6.1623999999999996e-07, 2.1942628843100001e+00, 2.9021955801939998e+02 },
    { 6.1333000000000000e-07, 5.6466442315799998e+00, 1.3647280995819001e+03 },
    { 6.0264000000000000e-07, 4.3107580099699998e+00, 4.5490936652729999e+02 },
    { 5.9490999999999997e-07, 6.0872746630399996e+00, 2.5653994050649999e+02 },
    { 5.9452000000000001e-07, 2.6445910098000001e+00, 1.1736805233000000e+02 },
    { 5.9375000000000003e-07, 1.9668123068300001e+00, 4.0302231763990000e+02 },
    { 5.8963999999999998e-07, 1.8290273054199999e+00, 4.4725317776799997e+01 },
    { 5.7708000000000003e-07, 2.7118034496500001e+00, 3.0386169668439999e+02 },
    { 5.7370000000000001e-07, 2.7260767219400002e+00, 4.8640193591629998e+02 },
    { 5.7148999999999999e-07, 5.0053166486000000e-01, 3.1652628083400000e+01 },
    { 5.6912999999999996e-07, 5.3808385027000005e-01, 2.5060345944400002e+01 },
    { 5.6293999999999998e-07, 3.3031230260000002e-02, 1.4297091499560000e+02 },
    { 5.6125999999999998e-07, 2.5405814889800000e+00, 6.5380429706100003e+01 },
    { 5.5389000000000005e-07, 1.2582675187600001e+00, 3.6500115658670001e+02 },
    { 5.5354999999999998e-07, 2.7486111091200001e+00, 1.9832124191099999e+02 },
    { 5.5252000000000003e-07, 2.8777064961900001e+00, 9.4917560896980001e+02 },
    { 5.5249000000000005e-07, 4.9896882305799997e+00, 7.4409091551800003e+01 },
    { 5.5249000000000005e-07, 5.9728995125999997e+00, 7.5154105582800000e+01 },
    { 5.4636000000000003e-07, 6.1857206331799999e+00, 7.1860837260500006e+01 },
    { 5.4466000000000001e-07, 2.1782010290499998e+00, 7.4127207261500004e+01 },
    { 5.4466000000000001e-07, 2.5012014069499999e+00, 7.5435989873099999e+01 },
    { 5.4456000000000000e-07, 5.9160618543599996e+00, 2.3687496867409999e+02 },
    { 5.3990999999999997e-07, 3.0114670404499999e+00, 1.4881243760920000e+02 },
    { 5.3827000000000001e-07, 6.7127500874000001e-01, 6.7359235025800004e+01 },
    { 5.3797999999999995e-07, 5.0648861766999997e-01, 1.3493380818700000e+01 },
    { 5.3598999999999997e-07, 6.0940118567799999e+00, 7.6736908292080000e+02 },
    { 5.3190000000000004e-07, 4.7380261763600000e+00, 7.9516900982500005e+01 },
    { 5.2972999999999997e-07, 4.4611336422100001e+00, 8.3110498122419995e+02 },
    { 5.2918000000000003e-07, 5.5552923677699999e+00, 1.1791056905120000e+02 },
    { 5.2404999999999998e-07, 6.3003520059999998e-02, 5.8739963457300000e+01 },
    { 5.2229000000000000e-07, 3.1451515758499999e+00, 1.8498791978669999e+02 },
    { 5.1867000000000002e-07, 4.7768672024800001e+00, 7.7702359874099997e+01 },
    { 5.1406000000000004e-07, 1.1084324729999999e+00, 1.0186893394120000e+02 },
    { 5.0065000000000003e-07, 2.9490604568699998e+00, 5.1887048887399999e+01 },
    { 4.9969999999999995e-07, 3.5586994497800002e+00, 7.7042478318000001e+00 },
    { 4.9880999999999995e-07, 7.2603310338000004e-01, 9.1456373123700004e+01 },
    { 4.9301000000000005e-07, 5.3974424351200003e+00, 7.6005438844699995e+01 },
    { 4.9208999999999995e-07, 5.5918002011199999e+00, 1.7787437278589999e+02 },
    { 4.9065000000000002e-07, 3.2033940212000001e-01, 5.5771018040700000e+01 },
    { 4.8902999999999998e-07, 5.2079048985299998e+00, 1.2247586176070000e+02 },
    { 4.8569999999999996e-07, 1.0469611686400000e+00, 4.8918103470799998e+01 },
    { 4.8034999999999998e-07, 4.2429852935700003e+00, 4.3128970483899998e+01 },
    { 4.7665000000000002e-07, 4.0614152989400001e+00, 7.3084677537900006e+01 },
    { 4.7603000000000001e-07, 2.1390544352599998e+00, 8.4182767428500000e+01 },
    { 4.7590000000000002e-07, 2.5864613721300000e+00, 2.2423292111730001e+02 },
    { 4.7477000000000001e-07, 5.6221366635000001e+00, 2.2445667028650001e+02 },
    { 4.7398000000000001e-07, 5.6186293479199998e+00, 7.5632931829200004e+01 },
    { 4.7398000000000001e-07, 5.3439583952599996e+00, 7.3930265305399999e+01 },
    { 4.7381999999999999e-07, 2.8128128300599999e+00, 1.4227094001599999e+01 },
    { 4.7357999999999998e-07, 9.4380667353000003e-01, 8.3652160719560004e+02 },
    { 4.7287000000000002e-07, 1.8108344978599999e+00, 1.9928444975750000e+02 },
    { 4.7175999999999998e-07, 4.6154693635599999e+00, 3.3406124266997999e+03 },
    { 4.6688000000000001e-07, 3.1410408956899998e+00, 1.0627416795630000e+02 },
    { 4.6274000000000003e-07, 6.2443837502999999e+00, 8.3355266177900000e+02 },
    { 4.5466999999999998e-07, 3.5647524895900000e+00, 6.9152524274800001e+01 },
    { 4.5165000000000000e-07, 1.6213711039000001e+00, 1.6708930495289999e+02 },
    { 4.5095000000000000e-07, 5.0984044494600003e+00, 3.9346109008430000e+02 },
    { 4.4939999999999999e-07, 2.6398263479200001e+00, 1.4214083359310001e+02 },
    { 4.4201999999999998e-07, 4.5040307457099997e+00, 7.0588812873300000e+01 },
    { 4.3958000000000002e-07, 3.5186281880900001e+00, 5.7516123179899999e+01 },
    { 4.3874999999999998e-07, 4.0157811123100000e+00, 5.7143616164400001e+01 },
    { 4.3524999999999998e-07, 3.1793526045700000e+00, 2.5863495096499999e+01 },
    { 4.2464000000000002e-07, 3.4761416357000002e-01, 1.5031395666000000e+02 },
    { 4.2300000000000002e-07, 6.0149405016399999e+00, 2.2633917249000000e+01 },
    { 4.2295000000000001e-07, 3.2698082988800001e+00, 2.3242155054919999e+02 },
    { 4.2224000000000000e-07, 5.3656513583400001e+00, 3.1492569389000000e+01 },
    { 4.1562000000000001e-07, 2.1996375279099998e+00, 1.6357784281510001e+02 },
    { 4.1021000000000002e-07, 1.7537188201000001e-01, 7.8974384261300003e+01 },
    { 4.0903000000000000e-07, 2.8712839167999998e-01, 4.1162033734900001e+02 },
    { 4.0170000000000000e-07, 2.4577006035800002e+00, 1.6454105066160000e+02 },
    { 4.0050999999999998e-07, 3.7923550615799999e+00, 3.0683064210100002e+02 },
    { 3.9957000000000002e-07, 1.7149553142000000e+00, 1.7448558675418999e+03 },
    { 3.9728000000000002e-07, 2.2010514360800002e+00, 8.8274979385999998e+01 },
    { 3.9625000000000002e-07, 1.6218566260800000e+00, 7.2493976706900000e+01 },
    { 3.9558000000000000e-07, 5.8659256192499996e+00, 5.8628088872699998e+01 },
    { 3.9295999999999999e-07, 4.6477779659099996e+00, 2.9226199388000001e+01 },
    { 3.8967000000000002e-07, 5.4743035647199996e+00, 1.4266209845489999e+02 },
    { 3.7996000000000002e-07, 3.6457395256999998e+00, 1.5467100656529999e+02 },
    { 3.7767999999999998e-07, 5.1028805952800003e+00, 9.1243924802600006e+01 },
    { 3.7510999999999998e-07, 4.0211188243300002e+00, 9.4425318540299997e+01 },
    { 3.7380000000000003e-07, 1.6416776818400001e+00, 1.8016199464630000e+02 },
    { 3.7354000000000000e-07, 4.2305834723600002e+00, 4.9721252622900003e+01 },
    { 3.7309000000000001e-07, 5.7927292614299999e+00, 1.2450285119020000e+02 },
    { 3.7298000000000000e-07, 4.6464691843199999e+00, 8.0314915210000004e-01 },
    { 3.7212000000000003e-07, 4.2442384668399997e+00, 9.3792179093900003e+01 },
    { 3.7166999999999999e-07, 4.4930246091999999e+00, 2.2679247625670001e+02 },
    { 3.7062000000000002e-07, 4.2018630606600000e+00, 3.9739324334740002e+02 },
    { 3.7009000000000000e-07, 5.7760896740799996e+00, 4.7353024151999996e+00 },
    { 3.6880999999999998e-07, 2.8008214408300001e+00, 7.5372299398300001e+01 },
    { 3.6880999999999998e-07, 1.8785809951700001e+00, 7.4190897736300002e+01 },
    { 3.6838000000000002e-07, 3.1483625478100001e+00, 2.1543795945209999e+02 },
    { 3.6388000000000000e-07, 2.8479216869599999e+00, 1.6560483224460000e+02 },
    { 3.5943999999999999e-07, 6.8720378875999999e-01, 1.8409414790939999e+02 },
    { 3.5861000000000001e-07, 4.7149104691500003e+00, 1.3606981631590000e+02 },
    { 3.5018999999999999e-07, 4.7476014155400001e+00, 3.5685355082999997e+01 },
    { 3.4957000000000003e-07, 3.4059593072000000e-01, 2.2908009811709999e+02 },
    { 3.4894000000000000e-07, 3.3558894833899999e+00, 1.4334342201109999e+02 },
    { 3.4737999999999998e-07, 4.2481244227600001e+00, 4.2538269652899999e+01 },
    { 3.4665000000000000e-07, 5.2424930804600001e+00, 3.6797010200329998e+02 },
    { 3.4542999999999999e-07, 2.3605531068799999e+00, 3.2686812094890001e+02 },
    { 3.4531000000000001e-07, 4.7844380992800000e+00, 1.3288842257819999e+02 },
    { 3.4187999999999999e-07, 2.0303765909100000e+00, 3.3577495719870001e+02 },
    { 3.3709000000000001e-07, 3.1181803078999999e+00, 7.7069220427700003e+01 },
    { 3.3593999999999998e-07, 2.0138054541599999e+00, 1.5726744496640001e+02 },
    { 3.3579000000000002e-07, 2.3567012067599999e+00, 9.2047073954699997e+01 },
    { 3.3519999999999998e-07, 5.7679305538800003e+00, 2.4872381809010000e+02 },
    { 3.3481999999999998e-07, 1.4623243725000001e-01, 2.2752618943959999e+02 },
    { 3.3089999999999998e-07, 4.9465558291500002e+00, 5.4286545332400003e+01 },
    { 3.2557999999999998e-07, 5.7336674427799998e+00, 1.3725981237000000e+00 },
    { 3.2528000000000001e-07, 2.0784726930000000e-01, 6.1929035849449997e+02 },
    { 3.2230999999999998e-07, 6.9404916107000003e-01, 8.0410672859800002e+01 },
    { 3.2142999999999999e-07, 3.3400868893600002e+00, 1.0333583763982999e+03 },
    { 3.2127000000000002e-07, 4.5441987763200000e+00, 4.8758044776399998e+01 },
    { 3.2002999999999999e-07, 3.4991371449400002e+00, 2.1626804085460000e+02 },
    { 3.1552000000000001e-07, 3.7722968490400000e+00, 8.9068362497999995e+00 },
    { 3.1483999999999998e-07, 1.7391365454600001e+00, 9.7676148247200004e+01 },
    { 3.1328000000000001e-07, 2.0704362677999999e+00, 6.6183578858199994e+01 },
    { 3.1058000000000002e-07, 1.4327920659300000e+00, 4.0164971951619998e+02 },
    { 3.0838000000000001e-07, 5.3004997079300002e+00, 9.0823233677299996e+01 },
    { 3.0713000000000002e-07, 4.3332465352899998e+00, 3.1540753498800001e+01 },
    { 3.0666000000000002e-07, 1.8155055596200000e+00, 1.1403833038800001e+03 },
    { 3.0628000000000001e-07, 5.0978408833800000e+00, 9.0935108261899998e+01 },
    { 2.9984999999999998e-07, 4.7452223202599999e+00, 9.0280716956099994e+01 },
    { 2.9867000000000001e-07, 3.7397222113800002e+00, 1.0064509366380000e+02 },
    { 2.9792000000000001e-07, 5.5517198509999997e+00, 6.1632141307790005e+02 },
    { 2.9541999999999998e-07, 3.8908166418000001e-01, 5.6511568774670002e+02 },
    { 2.9396999999999998e-07, 1.3888915274000000e+00, 9.5857677783099996e+02 },
    { 2.9381000000000001e-07, 5.2919706846499999e+00, 5.3065417294110000e+02 },
    { 2.9340999999999998e-07, 4.8019760043300002e+00, 3.2243328914400003e+01 },
    { 2.9331999999999998e-07, 7.2314041834999998e-01, 8.5980197091000008e+00 },
    { 2.9320000000000000e-07, 4.0452315934399996e+00, 7.4527768239299996e+02 },
    { 2.9191999999999998e-07, 1.8465584001299999e+00, 1.4395096981491999e+03 },
    { 2.9046999999999998e-07, 2.5483207414900000e+00, 1.1494162363460001e+02 },
    { 2.8860999999999999e-07, 6.4319686645999996e-01, 6.7928683997400000e+01 },
    { 2.8415000000000001e-07, 4.1234823075099998e+00, 4.0599126305649997e+02 },
    { 2.8342999999999999e-07, 2.1882467455299999e+00, 5.2872775724810003e+02 },
    { 2.8108999999999998e-07, 6.0734355864399996e+00, 1.0933825679300000e+01 },
    { 2.8073000000000000e-07, 5.5651619896700000e+00, 1.2353964334370001e+02 },
    { 2.7968999999999999e-07, 6.1782090444000004e-01, 7.5075952540000002e-01 },
    { 2.7799000000000002e-07, 9.6209912473000003e-01, 3.6945457471160000e+02 },
    { 2.7737000000000000e-07, 7.1377469190999998e-01, 1.4169889060840001e+02 },
    { 2.7480000000000000e-07, 2.8923360218599998e+00, 2.5272794265500000e+01 },
    { 2.7412000000000002e-07, 3.1732779932600002e+00, 2.2116340196420001e+02 },
    { 2.7160999999999998e-07, 4.0902583560999997e+00, 1.0483787935780001e+02 },
    { 2.7051000000000000e-07, 1.4604888001300000e+00, 2.7720474820300002e+01 },
    { 2.6982000000000002e-07, 2.8441183309600002e+00, 1.1157574848499999e+01 },
    { 2.6861000000000002e-07, 3.3346674098300002e+00, 1.5126011816400000e+02 },
    { 2.6857999999999999e-07, 1.1513378529100000e+00, 5.5137878594299998e+01 },
    { 2.6341000000000000e-07, 1.1142669313300000e+00, 4.0160025067299998e+01 },
    { 2.6014000000000000e-07, 4.6614975077500000e+00, 4.4720511869550000e+02 },
    { 2.5931000000000001e-07, 5.0543349828999995e-01, 3.6233367223999999e+00 },
    { 2.5834999999999998e-07, 3.5405484212700000e+00, 6.8301191012900006e+01 },
    { 2.5701000000000000e-07, 6.2652000536999997e-01, 7.5893564260100007e+01 },
    { 2.5701000000000000e-07, 4.0528824306300004e+00, 7.3669632874499996e+01 },
    { 2.5568000000000002e-07, 3.0969823578000000e-01, 5.8319272331999997e+01 },
    { 2.5516000000000002e-07, 9.4690360001999996e-01, 2.8414854074220000e+02 },
    { 2.5080000000000000e-07, 6.2990903054000003e-01, 9.2419580970200002e+01 },
    { 2.4997000000000001e-07, 2.8733059867500002e+00, 2.2876218603999998e+00 },
    { 2.4959000000000000e-07, 5.4990363056799998e+00, 2.7555052103309998e+02 },
    { 2.4959000000000000e-07, 3.0847726904599999e+00, 4.0841348623499997e+01 },
    { 2.4738999999999999e-07, 4.0236960278299998e+00, 8.1634513137200003e+01 },
    { 2.4367000000000001e-07, 4.9411241548699998e+00, 4.9178735901700001e+01 },
    { 2.4086000000000000e-07, 2.7082176039700001e+00, 1.7017012495410000e+02 },
    { 2.4050999999999997e-07, 2.7169674000000001e-02, 3.2824071907259997e+02 },
    { 2.3957000000000002e-07, 3.6219265996500001e+00, 5.4153981451059997e+02 },
    { 2.3788000000000001e-07, 5.9355868896799997e+00, 1.6682867252200001e+02 },
    { 2.3755000000000001e-07, 6.1035048799000000e+00, 9.6962247809489998e+02 },
    { 2.3717000000000000e-07, 3.9657066202400002e+00, 4.5555399179299997e+01 },
    { 2.3482999999999999e-07, 3.2934378970500000e+00, 1.9010580526599998e+01 },
    { 2.3467000000000000e-07, 3.9599850769899998e+00, 1.0749800823370001e+02 },
    { 2.3389000000000001e-07, 2.7005029296299998e+00, 5.2147681318300002e+01 },
    { 2.3243000000000000e-07, 1.0635078060200001e+00, 4.1406801790380001e+02 },
    { 2.2965000000000000e-07, 1.7434191029000001e-01, 1.8892007304980001e+02 },
    { 2.2891999999999999e-07, 6.0871115386100003e+00, 1.1731986822019999e+02 },
    { 2.2861000000000001e-07, 1.7264294950100001e+00, 1.0398654609590000e+02 },
    { 2.2639000000000000e-07, 5.1791242699700000e+00, 2.4846318565920001e+02 },
    { 2.2571000000000000e-07, 2.1668700458600001e+00, 7.0636996983100005e+01 },
    { 2.2545000000000000e-07, 2.6476931798500001e+00, 1.6289651925890001e+02 },
    { 2.2426000000000000e-07, 2.0413519579099999e+00, 3.7872403206900003e+01 },
    { 2.2319999999999999e-07, 5.6628147009100003e+00, 1.0082492417399999e+01 },
    { 2.2291000000000000e-07, 7.9512014451000002e-01, 1.0791869935900000e+02 },
    { 2.2242999999999999e-07, 5.0228191272300000e+00, 8.5939173415799999e+01 },
    { 2.2130000000000000e-07, 3.2860496959900001e+00, 2.5208652238159999e+02 },
    { 2.1822000000000000e-07, 3.8658940472199999e+00, 3.4621573499999997e+01 },
    { 2.1707000000000000e-07, 2.8325396932500002e+00, 3.1231936958100000e+01 },
    { 2.1612000000000001e-07, 4.0132280037500001e+00, 7.5877106321170004e+02 },
    { 2.1565999999999999e-07, 5.8365885370799999e+00, 2.2530800354839999e+02 },
    { 2.1539000000000000e-07, 5.0435574587399996e+00, 6.4486657828800006e+01 },
    { 2.1484999999999999e-07, 5.6081768321199998e+00, 1.6700742689746000e+03 },
    { 2.1476000000000000e-07, 3.7719344665699999e+00, 1.0400779795530001e+02 },
    { 2.1388999999999999e-07, 4.3879940524199998e+00, 2.3204904353370000e+02 },
    { 2.1299999999999999e-07, 6.0210180628899996e+00, 9.5276651802200007e+01 },
    { 2.1183000000000001e-07, 2.7627931341299998e+00, 1.2991947716160001e+02 },
    { 2.1037000000000000e-07, 4.4235341804399999e+00, 6.2491943278700001e+02 },
    { 2.1012999999999999e-07, 1.2900831867100000e+00, 8.1262006121699997e+01 },
    { 2.0956999999999999e-07, 2.5127370026200002e+00, 7.8926200151499998e+01 },
    { 2.0855000000000000e-07, 3.1798646429700002e+00, 3.3120966448920001e+02 },
    { 2.0821999999999999e-07, 3.6600741649100001e+00, 7.7489911552999999e+01 },
    { 2.0821999999999999e-07, 1.0193282710900000e+00, 7.2073285581600004e+01 },
    { 2.0494000000000001e-07, 1.1008610193499999e+00, 1.4796684984170000e+02 },
    { 2.0414000000000000e-07, 3.4355623071400001e+00, 2.2189711514710001e+02 },
    { 2.0276000000000000e-07, 3.9918174583099999e+00, 4.3240845068500001e+01 },
    { 2.0122000000000000e-07, 5.0711617707699999e+00, 1.4096517742550000e+02 },
    { 2.0094000000000000e-07, 6.0807237983300002e+00, 1.4393412284210001e+02 },
    { 2.0029999999999999e-07, 2.5164752666000001e-01, 1.4557162401699999e+01 },
    { 2.0027000000000001e-07, 2.8490815349599998e+00, 1.5667674413540001e+02 },
    { 1.9987999999999999e-07, 4.0152769274600004e+00, 3.7619561469690001e+02 },
    { 1.9829000000000001e-07, 4.4940833293999999e-01, 5.3197858695499997e+02 },
    { 1.9378999999999999e-07, 1.5096773312100000e+00, 2.2894549679899999e+01 },
    { 1.9348999999999999e-07, 2.5381216411500001e+00, 3.6351668387839999e+02 },
    { 1.9238000000000001e-07, 1.8648771012800001e+00, 1.4489733068859999e+02 },
    { 1.9167000000000000e-07, 2.5119505854800002e+00, 9.4011688612000004e+00 },
    { 1.9142999999999999e-07, 4.2085620769499998e+00, 1.5429849954980000e+02 },
    { 1.9142000000000000e-07, 1.9961476768899999e+00, 4.0843894361129998e+02 },
    { 1.9107000000000000e-07, 5.9631965911200000e+00, 1.7637982402900001e+01 },
    { 1.8650999999999999e-07, 3.3014377516999999e-01, 1.0702492748170000e+02 },
    { 1.8640000000000000e-07, 1.2436944034799999e+00, 2.8956516671359998e+02 },
    { 1.8599000000000001e-07, 4.2606749637199997e+00, 1.4819059901090000e+02 },
    { 1.8484000000000001e-07, 1.6301340413700001e+00, 9.0588657979150003e+02 },
    { 1.8477000000000001e-07, 6.1233843942999999e-01, 3.8109097580650001e+02 },
    { 1.8460000000000000e-07, 1.6290161290200000e+00, 1.3533610313299999e+02 },
    { 1.8430000000000000e-07, 3.7793430690099998e+00, 3.7916456011349999e+02 },
    { 1.8321000000000001e-07, 5.2007176238900001e+00, 1.0601353552540000e+02 },
    { 1.8049000000000000e-07, 2.3427018147999998e+00, 1.3310087089929999e+02 },
    { 1.7881000000000000e-07, 9.1082733459999995e-02, 1.9066517818899999e+02 },
    { 1.7620000000000000e-07, 5.0718475788299999e+00, 3.0305854753230000e+02 },
    { 1.7407000000000000e-07, 2.0588244586600002e+00, 5.5013783421970004e+02 },
    { 1.7391000000000000e-07, 2.8605205894600001e+00, 7.2775860997199999e+01 },
    { 1.7350000000000001e-07, 4.7748994403099996e+00, 3.7864329525170001e+02 },
    { 1.7265000000000000e-07, 3.4376874392000001e-01, 2.0446869125100001e+01 },
    { 1.7090000000000000e-07, 5.8437345586499996e+00, 8.5076539305799997e+01 },
    { 1.6980000000000000e-07, 3.5185966227100001e+00, 7.5239172954300003e+01 },
    { 1.6980000000000000e-07, 1.1608058132900001e+00, 7.4324024180300000e+01 },
    { 1.6854000000000000e-07, 3.4677163680200001e+00, 8.5319638175199998e+02 },
    { 1.6814000000000000e-07, 4.4107567457599997e+00, 3.3683873878169999e+02 },
    { 1.6677000000000000e-07, 5.7608807345799997e+00, 3.8161224066829999e+02 },
    { 1.6607000000000000e-07, 1.4754608495999999e+00, 2.4161027108930000e+02 },
    { 1.6588000000000000e-07, 1.6105462141000000e+00, 4.3829828244570001e+02 },
    { 1.6509999999999999e-07, 3.7429674273400000e+00, 1.2184272231430000e+02 },
    { 1.6509999999999999e-07, 5.5223314857199997e+00, 2.8511174858869998e+02 },
    { 1.6443000000000000e-07, 2.7498755816800000e+00, 8.3379618276399995e+01 },
    { 1.6329000000000000e-07, 5.5272412771999999e+00, 1.7665053250849999e+02 },
    { 1.6233000000000000e-07, 3.9039945015000002e+00, 3.1245971639350000e+02 },
    { 1.6061999999999999e-07, 9.5601300236999998e-01, 7.6526703706500001e+01 },
    { 1.6061999999999999e-07, 5.8179678004000002e-01, 7.3036493428100002e+01 },
    { 1.6025000000000000e-07, 4.4810289720000000e-02, 2.1187638603783998e+03 },
    { 1.6009000000000000e-07, 7.3172853686999995e-01, 7.5611679969799994e+01 },
    { 1.6009000000000000e-07, 3.9476738991400002e+00, 7.3951517164799995e+01 },
    { 1.5934000000000000e-07, 2.1146632092500002e+00, 7.0363318461740005e+02 },
    { 1.5823999999999999e-07, 1.0228052209500000e+00, 1.9428851491140000e+02 },
    { 1.5772000000000001e-07, 3.9042989744300001e+00, 1.9365537546499999e+02 },
    { 1.5629999999999999e-07, 5.6286528687199997e+00, 5.1546387109299997e+02 },
    { 1.5552999999999999e-07, 2.9265415330300000e+00, 6.4699106149900004e+01 },
    { 1.5368000000000001e-07, 3.6288741940300002e+00, 6.0554504565700000e+01 },
    { 1.5316999999999999e-07, 2.1709054498699998e+00, 1.3406407874580000e+02 },
    { 1.5304000000000000e-07, 5.2215027437400003e+00, 2.1648048917570000e+02 },
    { 1.5159000000000000e-07, 4.9313009728999999e+00, 1.1807062774560001e+02 },
    { 1.5008999999999999e-07, 4.6200927423199998e+00, 3.2350541665740002e+02 },
    { 1.4945999999999999e-07, 4.7317252963299996e+00, 7.3345309968799995e+01 },
    { 1.4945999999999999e-07, 3.0892697932600002e+00, 7.6217887165799993e+01 },
    { 1.4908999999999999e-07, 5.6277089991999996e-01, 7.4299006053259995e+02 },
    { 1.4833999999999999e-07, 9.0690489444000000e-01, 1.1536231475990000e+02 },
    { 1.4826000000000001e-07, 5.2710639175200003e+00, 6.3847772888000002e+01 },
    { 1.4812999999999999e-07, 2.9991190761300000e+00, 4.7061123747000003e+01 },
    { 1.4632000000000000e-07, 3.7060591608200002e+00, 6.1069233878540001e+02 },
    { 1.4625000000000000e-07, 4.1951318956300003e+00, 7.3587651353180001e+02 },
    { 1.4583000000000000e-07, 1.1998451160000001e+00, 4.0610313764109998e+02 },
    { 1.4573000000000000e-07, 5.6930367224099996e+00, 7.6314255385400003e+01 },
    { 1.4573000000000000e-07, 2.1279583671800002e+00, 7.3248941749200000e+01 },
    { 1.4518999999999999e-07, 7.3690474457999999e-01, 1.8196374661092000e+03 },
    { 1.4515000000000000e-07, 6.1304618082799998e+00, 6.3624023718799997e+01 },
    { 1.4342000000000000e-07, 5.3828283108699999e+00, 4.0461866493280002e+02 },
    { 1.4228000000000000e-07, 1.6125332411600000e+00, 8.4864090984699999e+01 },
    { 1.4191000000000000e-07, 1.7109424526400001e+00, 1.2429040286910001e+02 },
    { 1.4158000000000000e-07, 5.6086215340900001e+00, 1.4755745956449999e+02 },
    { 1.4109000000000000e-07, 6.1504243145800004e+00, 5.3591074021810005e+02 },
    { 1.4093999999999999e-07, 6.1756127056399999e+00, 3.9096243484299997e+01 },
    { 1.3958000000000001e-07, 4.4859144907999999e+00, 6.0112561581000001e+01 },
    { 1.3843000000000000e-07, 2.8086157155000002e+00, 1.9907200143640000e+02 },
    { 1.3545000000000001e-07, 2.2052655075400001e+00, 2.8096714700450002e+02 },
    { 1.3526999999999999e-07, 4.7441244841700003e+00, 7.3457184553399998e+01 },
    { 1.3526999999999999e-07, 3.0768706054199999e+00, 7.6106012581200005e+01 },
    { 1.3418000000000000e-07, 2.0656533109800002e+00, 6.9785663721199995e+01 },
    { 1.3363000000000001e-07, 4.4397652956200000e+00, 1.7219711438360000e+02 },
    { 1.3353000000000001e-07, 3.4730269209100002e+00, 2.8873508531110002e+02 },
    { 1.3295000000000001e-07, 2.3609639549999999e+00, 3.2261164478009999e+02 },
    { 1.3243999999999999e-07, 3.0815777756500000e+00, 5.0137978944330001e+02 },
    { 1.3152999999999999e-07, 1.1979933998900001e+00, 2.2486606056369999e+02 },
    { 1.3094000000000001e-07, 2.5888240401100000e+00, 1.2008908110400000e+01 },
    { 1.3008000000000001e-07, 2.8935117490800000e+00, 1.0970942747017000e+03 },
    { 1.2895000000000000e-07, 5.7031771597000001e+00, 8.8379517926369999e+02 },
    { 1.2785000000000000e-07, 1.0611241376000000e-01, 2.0522234059069999e+02 },
    { 1.2785000000000000e-07, 2.0002206839199999e+00, 7.5780785536520000e+02 },
    { 1.2674000000000001e-07, 3.4070092251399999e+00, 3.3780194662820003e+02 },
    { 1.2655000000000000e-07, 3.4798851095000001e+00, 7.4920983565610004e+02 },
    { 1.2611999999999999e-07, 3.1405818946499999e+00, 3.5330106501699998e+02 },
    { 1.2571000000000001e-07, 5.4413812046999999e-01, 1.1387784205160000e+02 },
    { 1.2562000000000001e-07, 4.9289592136699998e+00, 1.0931254934210000e+02 },
    { 1.2513000000000001e-07, 1.9112228740099999e+00, 1.9577298761969999e+02 },
    { 1.2331000000000001e-07, 2.9226990445299998e+00, 8.3688434817100003e+01 },
    { 1.2291999999999999e-07, 2.7580634156700001e+00, 7.9777533413399993e+01 },
    { 1.2272000000000000e-07, 5.0635530322999998e+00, 1.1718758732690001e+03 },
    { 1.2234000000000000e-07, 1.5768921777900000e+00, 1.1887377689770000e+02 },
    { 1.2188999999999999e-07, 2.6553298666799998e+00, 8.4215068148809996e+02 },
    { 1.2172999999999999e-07, 1.7117716362999999e-01, 4.4092178330400003e+01 },
    { 1.1964999999999999e-07, 9.3110019198000005e-01, 7.5483890994859996e+02 },
    { 1.1943000000000000e-07, 4.2213661288999998e-01, 5.9282480178500002e+01 },
    { 1.1934999999999999e-07, 1.3810469112999999e+00, 2.5642806592189999e+02 },
    { 1.1847000000000000e-07, 1.6028976522800000e+00, 4.1703696332039999e+02 },
    { 1.1811000000000000e-07, 3.3466092980200002e+00, 3.4530950774799997e+01 },
    { 1.1684000000000001e-07, 1.2137740630900000e+00, 1.8079513409270001e+02 },
    { 1.1670000000000001e-07, 4.4225907776300000e+00, 3.0868762182479998e+02 },
    { 1.1578000000000000e-07, 6.7710449732000000e-01, 6.5332245596299998e+01 },
    { 1.1562000000000000e-07, 5.2312313470199996e+00, 8.2792358748649997e+02 },
    { 1.1539000000000000e-07, 5.6875695037299998e+00, 7.5351047538900005e+01 },
    { 1.1539000000000000e-07, 5.2750182394499996e+00, 7.4212149595699998e+01 },
    { 1.1375000000000000e-07, 3.8180886070200000e+00, 5.4695644048199995e+02 },
    { 1.1367000000000000e-07, 2.4590284981099999e+00, 5.9591296719200002e+01 },
    { 1.1318000000000000e-07, 5.7725160857600004e+00, 4.3549661609200001e+01 },
    { 1.1301000000000000e-07, 4.3313104731300003e+00, 1.6412035953629999e+02 },
    { 1.1235999999999999e-07, 3.6032288119999998e+00, 6.7925416222920001e+02 },
    { 1.1193000000000000e-07, 1.5382497631200001e+00, 1.1265400177420000e+02 },
    { 1.1184000000000000e-07, 2.9693669668700000e+00, 1.5036634628670001e+02 },
    { 1.1089999999999999e-07, 2.1646159599499999e+00, 5.5871591777200003e+01 },
    { 1.1051000000000000e-07, 5.4171669214099998e+00, 5.4450875992720000e+02 },
    { 1.1035000000000001e-07, 5.0530650135000004e+00, 2.0470107572890001e+02 },
    { 1.0928999999999999e-07, 3.1176529655400000e+00, 3.1867949151699997e+02 },
    { 1.0923000000000001e-07, 2.1964064144800002e+00, 5.2575881183150000e+02 },
    { 1.0863000000000000e-07, 1.5714463200800000e+00, 2.2976142167329999e+02 },
    { 1.0841000000000000e-07, 5.9707416977800003e+00, 4.8322054217860000e+02 },
    { 1.0812000000000000e-07, 4.6430555570999998e-01, 8.9008692568900003e+01 },
    { 1.0768000000000000e-07, 1.6822527379000001e+00, 4.6675821594330000e+02 },
    { 1.0749000000000000e-07, 1.0146116568400001e+00, 1.5185081899500000e+02 },
    { 1.0742000000000001e-07, 2.7460255075600002e+00, 4.0016524680790002e+02 },
    { 1.0740000000000000e-07, 1.6308965653100000e+00, 2.1085141488319999e+02 },
    { 1.0737000000000000e-07, 2.7118142206900000e+00, 5.5258551477449998e+02 },
    { 1.0733999999999999e-07, 1.9349032271000000e-01, 8.9450635553599994e+01 },
    { 1.0654000000000000e-07, 3.6500920961399999e+00, 2.6396230404800002e+02 },
    { 1.0623000000000001e-07, 1.7198869891899999e+00, 2.6302034806090001e+02 },
    { 1.0621000000000000e-07, 3.5908343988000002e-01, 1.1848849416000000e+01 },
    { 1.0593000000000000e-07, 2.4867288419600002e+00, 7.1700778566099999e+01 },
    { 1.0562000000000000e-07, 1.4769973475200000e+00, 1.1081399749656000e+03 },
    { 1.0485000000000000e-07, 5.8276523918600001e+00, 1.0538039607899999e+02 },
    { 1.0472000000000000e-07, 2.7867447644999999e-01, 4.9508804301799998e+01 },
    { 1.0443000000000000e-07, 8.1812110977999997e-01, 3.6137781986430002e+02 },
    { 1.0391000000000000e-07, 4.2874097908000000e+00, 3.0598797511699999e+01 },
    { 1.0379000000000000e-07, 5.5158335910199998e+00, 4.2266603761290003e+02 },
    { 1.0267000000000001e-07, 5.2046545940100000e+00, 6.8005731138130000e+02 },
    { 1.0258000000000000e-07, 2.8691457817099999e+00, 4.4386366626339998e+02 },
    { 1.0257000000000000e-07, 4.7295229559800003e+00, 3.0689420236899998e+01 },
    { 1.0251999999999999e-07, 5.2664149858100000e+00, 5.3362311835770004e+02 },
    { 1.0242000000000000e-07, 4.1747224069000000e+00, 1.7730492381430000e+02 },
    { 1.0157000000000000e-07, 1.9263597616599999e+00, 5.4391805909619995e+02 },
    { 1.0111999999999999e-07, 4.8146577055000002e+00, 1.9662432088160000e+02 },
    { 1.0094000000000000e-07, 5.8796175351000004e+00, 6.9955673426900006e+01 },
    { 1.0031000000000000e-07, 2.0766519885800001e+00, 6.3996530734300002e+01 },
    { 9.9569999999999998e-08, 3.2769067004500001e+00, 9.5979227217800002e+01 },
    { 9.9559999999999999e-08, 3.1355013950499999e+00, 3.6648562929500002e+02 },
    { 9.9540000000000003e-08, 5.1193596574600004e+00, 7.9607523707699997e+01 },
    { 9.8990000000000002e-08, 2.2320056950000000e-01, 2.0788246946660001e+02 },
    { 9.8230000000000000e-08, 5.1683230025000002e+00, 1.5318653385700000e+02 },
    { 9.7810000000000000e-08, 4.0022989042299999e+00, 8.4230951538300005e+01 },
    { 9.7100000000000003e-08, 1.2026940152000001e+00, 2.1744369702220001e+02 },
    { 9.5970000000000005e-08, 4.8779527941699996e+00, 2.9201284726839998e+02 },
    { 9.5010000000000002e-08, 3.9811706868200001e+00, 1.5646429581429999e+02 },
    { 9.4329999999999998e-08, 2.1167203179500000e+00, 6.7049608382570000e+02 },
    { 9.4059999999999997e-08, 5.7242121532399999e+00, 6.9312582969199994e+01 },
    { 9.4059999999999997e-08, 5.2383755899400004e+00, 8.0250614165399995e+01 },
    { 9.3470000000000003e-08, 3.4595654844000001e-01, 1.5698556067609999e+02 },
    { 9.3299999999999995e-08, 6.2746413573900002e+00, 5.2802072624099999e+01 },
    { 9.2259999999999994e-08, 5.2594801273999998e+00, 7.3866574830600001e+01 },
    { 9.2259999999999994e-08, 5.7031076157799996e+00, 7.5696622304000002e+01 },
    { 9.2089999999999999e-08, 1.7925539976500000e+00, 4.7494684537519998e+02 },
    { 9.2039999999999996e-08, 2.0316569094600001e+00, 2.2338158785540000e+02 },
    { 9.1860000000000003e-08, 1.9961345882599999e+00, 1.0080515235820000e+02 },
    { 9.1230000000000004e-08, 1.4737849862600001e+00, 6.8398946464439996e+02 },
    { 8.9389999999999995e-08, 3.0767340404099999e+00, 8.9599393399899995e+01 },
    { 8.9270000000000005e-08, 3.4754510689999997e-02, 7.4958234267160003e+02 },
    { 8.9200000000000005e-08, 8.5529826739000003e-01, 1.1802244363580000e+02 },
    { 8.9039999999999995e-08, 1.8405328294400001e+00, 5.9963803734700001e+01 },
    { 8.8160000000000003e-08, 2.9368996053799998e+00, 2.1011770170029999e+02 },
    { 8.7870000000000005e-08, 1.4780303188699999e+00, 5.3287235883230005e+02 },
    { 8.7320000000000004e-08, 3.1066105822800001e+00, 3.6909195360399998e+01 },
    { 8.6369999999999998e-08, 4.5625961110900004e+00, 2.5345912050530001e+02 },
    { 8.6169999999999997e-08, 2.5973203138600001e+00, 8.5566666400299994e+01 },
    { 8.6169999999999997e-08, 3.1451858685100000e+00, 1.5045696901189999e+02 },
    { 8.6070000000000002e-08, 4.6970791855099998e+00, 6.2932749151300001e+01 },
    { 8.5430000000000005e-08, 4.2157734315599997e+00, 3.0412232911529998e+02 },
    { 8.5049999999999997e-08, 3.9267440052999998e+00, 2.0495053234899999e+01 },
    { 8.4859999999999994e-08, 3.1623104939700002e+00, 7.7862418568500004e+01 },
    { 8.4730000000000005e-08, 3.1190500201900000e+00, 7.1009503998599996e+01 },
    { 8.4730000000000005e-08, 1.5603524158099999e+00, 7.8553693136000007e+01 },
    { 8.4479999999999999e-08, 4.1019239581800004e+00, 2.4994765836750000e+02 },
    { 8.4439999999999994e-08, 4.0038165092300000e+00, 3.4470304530790003e+02 },
    { 8.4009999999999996e-08, 2.7146471402999999e+00, 3.2983706636549999e+02 },
    { 8.3960000000000005e-08, 5.6575627787900000e+00, 5.1471311156759998e+02 },
    { 8.3400000000000006e-08, 3.5218635529900002e+00, 3.7169827791300001e+01 },
    { 8.2969999999999995e-08, 1.3874313530000000e+00, 6.7442823708879996e+02 },
    { 8.2389999999999999e-08, 2.5268936408900000e+00, 1.1562294719080001e+02 },
    { 8.2039999999999999e-08, 6.1194720197099999e+00, 7.6343692965770003e+02 },
    { 8.1359999999999995e-08, 2.5904789847299998e+00, 9.2797798031929995e+02 },
    { 8.0869999999999996e-08, 8.4671801172000005e-01, 1.0223126761344000e+03 },
    { 8.0550000000000003e-08, 1.5456374187299999e+00, 1.7542669223109999e+02 },
    { 8.0439999999999998e-08, 8.3166336146999997e-01, 7.7117404537499993e+01 },
    { 8.0439999999999998e-08, 3.8477390745300002e+00, 7.2445792597099995e+01 },
    { 8.0319999999999994e-08, 8.3846257555000003e-01, 2.2382353084010001e+02 },
    { 7.9980000000000005e-08, 3.3331730373999999e+00, 4.0313419222450000e+02 },
    { 7.9539999999999996e-08, 4.0622329551499998e+00, 8.6790506677699994e+01 },
    { 7.9360000000000004e-08, 2.0679533940599999e+00, 1.0898248094200000e+02 },
    { 7.9300000000000002e-08, 5.1384860681399998e+00, 2.3093707784090000e+02 },
    { 7.9300000000000002e-08, 3.9355429021499999e+00, 1.4065636088479999e+02 },
    { 7.8940000000000004e-08, 2.5147904522800002e+00, 9.3691605357399993e+01 },
    { 7.8810000000000002e-08, 2.8291886964100001e+00, 4.3764389113990001e+02 },
    { 7.8629999999999997e-08, 3.7484073932199999e+00, 6.8865533109039995e+02 },
    { 7.8400000000000001e-08, 3.4339120611000000e+00, 1.0444040766621999e+03 },
    { 7.8349999999999997e-08, 3.6466501558300002e+00, 3.2538364824060000e+02 },
    { 7.7529999999999993e-08, 4.9793150101699997e+00, 2.5976951835400001e+02 },
    { 7.7470000000000005e-08, 4.8525199305099997e+00, 2.0884567731310000e+02 },
    { 7.7019999999999997e-08, 9.3795377618999998e-01, 2.3741748539529999e+02 },
    { 7.6759999999999994e-08, 4.8344264804200003e+00, 6.0224436165599997e+01 },
    { 7.6730000000000000e-08, 3.9924689522799999e+00, 2.4187090352019999e+02 },
    { 7.6469999999999996e-08, 2.7276712122000002e-01, 1.4866942525729999e+02 },
    { 7.6259999999999996e-08, 5.8424831675700002e+00, 2.1979525943199999e+01 },
    { 7.6070000000000006e-08, 5.4772691493399996e+00, 4.0907208305770001e+02 },
    { 7.5730000000000004e-08, 5.7574035807400001e+00, 2.1426230328450001e+02 },
    { 7.5699999999999996e-08, 1.0118055760600000e+00, 3.5893013930950002e+02 },
    { 7.5569999999999994e-08, 4.0594524303100004e+00, 1.4788665740644001e+03 },
    { 7.5429999999999995e-08, 3.1411356024599999e+00, 7.7016830800999998e+01 },
    { 7.5429999999999995e-08, 4.6798594871299999e+00, 7.2546366333600005e+01 },
    { 7.5429999999999995e-08, 2.1699172780899998e+00, 7.4047885384400004e+01 },
    { 7.5429999999999995e-08, 5.6510778115000004e+00, 7.5515311750199999e+01 },
    { 7.5359999999999995e-08, 1.5904810037099999e+00, 4.2511371816769997e+02 },
    { 7.5069999999999997e-08, 1.5818595758200000e+00, 2.9823262239190001e+02 },
    { 7.4880000000000006e-08, 2.0961500528200001e+00, 8.9971900415400000e+01 },
    { 7.4670000000000006e-08, 5.7631121846199997e+00, 1.2655674786264001e+03 },
    { 7.4460000000000006e-08, 2.3319978959399998e+00, 2.1233588759150001e+02 },
    { 7.4460000000000006e-08, 2.5886416117000000e-01, 3.9654191008549998e+02 },
    { 7.3879999999999997e-08, 2.8131114299700002e+00, 3.3025226207099998e+01 },
    { 7.3119999999999996e-08, 2.0999433253999999e-01, 3.6908206769610001e+02 },
    { 7.3099999999999999e-08, 9.1958252558999998e-01, 2.1149444540699999e+01 },
    { 7.3099999999999999e-08, 4.4493919072699999e+00, 1.8865944061889999e+02 },
    { 7.2940000000000003e-08, 5.2234137333500001e+00, 1.3948070471720001e+02 },
    { 7.2909999999999996e-08, 6.1456846659200002e+00, 5.2650957135689998e+02 },
    { 7.2499999999999994e-08, 5.7191911448699999e+00, 1.0656017053127000e+03 },
    { 7.2310000000000004e-08, 5.4346631996400001e+00, 2.6721313375490001e+02 },
    { 7.2269999999999998e-08, 5.2778671416799998e+00, 1.3192521473170001e+02 },
    { 7.1740000000000006e-08, 5.6242598888100002e+00, 4.4572064598719999e+02 },
    { 7.1379999999999995e-08, 3.0963157415799998e+00, 2.1997652340691998e+03 },
    { 7.1050000000000004e-08, 9.7447995190000003e-01, 6.4568800576977001e+03 },
    { 7.0980000000000004e-08, 5.0958403961099998e+00, 1.6921656695024001e+03 },
    { 7.0970000000000006e-08, 5.5476052948999999e-01, 6.2772690456900001e+01 },
    { 7.0770000000000004e-08, 5.5174498131499998e+00, 4.4126722786229999e+02 },
    { 7.0739999999999997e-08, 5.7485189587800001e+00, 3.5033211960040001e+02 },
    { 7.0029999999999999e-08, 5.6075177862599999e+00, 1.4445538770389999e+02 },
    { 6.9709999999999994e-08, 1.9611265567900000e+00, 1.3055261660799999e+02 },
    { 6.9569999999999994e-08, 3.2286353007800002e+00, 7.5314198891920000e+02 },
    { 6.9489999999999996e-08, 3.0153273945299999e+00, 1.9779997704920001e+02 },
    { 6.9219999999999994e-08, 3.2859390107499999e+00, 4.0758761034939999e+02 },
    { 6.8509999999999996e-08, 2.7089520282300001e+00, 3.0141401612959999e+02 },
    { 6.8089999999999996e-08, 1.0261065948300001e+00, 5.6118353448360006e+02 },
    { 6.7910000000000004e-08, 5.6645377664999996e+00, 4.1756372360199997e+01 },
    { 6.7819999999999995e-08, 4.0446607450799998e+00, 6.4701083331480004e+02 },
    { 6.7360000000000002e-08, 5.9064794380899999e+00, 2.7161836777000002e+02 },
    { 6.7160000000000001e-08, 3.4119029711700000e+00, 8.1065811209909998e+02 },
    { 6.6889999999999999e-08, 5.4235100850100002e+00, 1.7071264167530001e+02 },
    { 6.6650000000000005e-08, 2.9830527544800001e+00, 4.3681380973739999e+02 },
    { 6.6489999999999995e-08, 3.1495208806499999e+00, 5.9949951559861001e+03 },
    { 6.6360000000000007e-08, 2.2973241660700001e+00, 4.9744763618019999e+02 },
    { 6.6230000000000005e-08, 2.9930511271700002e+00, 4.4613443192200002e+01 },
    { 6.6189999999999999e-08, 5.5800160779199999e+00, 1.0457724692690000e+02 },
    { 6.5929999999999995e-08, 9.2513779202000002e-01, 2.3082520325630000e+02 },
    { 6.5589999999999994e-08, 2.3761357906099998e+00, 1.2911632800949999e+02 },
    { 6.5509999999999996e-08, 8.3934641149000000e-01, 1.5142912967165000e+03 },
    { 6.4599999999999996e-08, 4.1391558193500000e+00, 9.5628915597059995e+02 },
    { 6.4449999999999998e-08, 5.6327999880600004e+00, 3.3217287233569999e+02 },
    { 6.4319999999999996e-08, 3.3562605058299999e+00, 5.2427433912319998e+02 },
    { 6.4090000000000000e-08, 4.6129582360399999e+00, 9.9252048522056994e+03 },
    { 6.3640000000000006e-08, 3.8055336963799999e+00, 8.0901358069640003e+02 },
    { 6.3619999999999997e-08, 2.1048553722099999e+00, 3.3940249943799998e+01 },
    { 6.3520000000000002e-08, 3.7920418504200000e+00, 3.6997583957339998e+02 },
    { 6.3469999999999999e-08, 4.9342190184400003e+00, 3.0089275126780001e+02 },
    { 6.2929999999999995e-08, 2.9887233626800001e+00, 1.7813500521680001e+02 },
    { 6.2849999999999997e-08, 2.5024204673599999e+00, 2.6205714021440002e+02 },
    { 6.2760000000000001e-08, 4.6879518240499998e+00, 9.6761124510499997e+01 },
    { 6.2660000000000007e-08, 3.0030538580499999e+00, 1.4817794832600001e+01 },
    { 6.2429999999999997e-08, 4.8442739627999998e-01, 8.6630447983300002e+01 },
    { 6.2139999999999999e-08, 1.3505241176000000e+00, 4.4275170057060001e+02 },
    { 6.1980000000000003e-08, 4.1061390630599996e+00, 2.9747464249800000e+01 },
    { 6.1959999999999994e-08, 3.3236668688300002e+00, 6.1217681149370003e+02 },
    { 6.1850000000000001e-08, 3.3236588140199999e+00, 4.0153784493159998e+02 },
    { 6.1669999999999996e-08, 2.2926348963700001e+00, 2.0670681329900000e+02 },
    { 6.1280000000000003e-08, 1.4121864517900000e+00, 1.2884620283063000e+03 },
    { 6.1160000000000000e-08, 2.1998910802800000e+00, 2.5265597135319999e+02 },
    { 6.0909999999999994e-08, 1.0112471385399999e+00, 1.1239336934329999e+02 },
    { 6.0450000000000002e-08, 2.7550396063800000e+00, 1.8105576652360000e+02 },
    { 6.0380000000000002e-08, 4.7964912358999996e+00, 1.5819593482830001e+03 },
    { 6.0279999999999994e-08, 1.8677023648100000e+00, 1.3757737998458001e+03 },
    { 5.9880000000000004e-08, 5.0320541583000000e-01, 1.9417664032680000e+02 },
    { 5.9660000000000006e-08, 6.2167010702800001e+00, 4.7535623565240002e+02 },
    { 5.9459999999999998e-08, 4.6566750004199999e+00, 4.5719698838770000e+02 },
    { 5.9149999999999997e-08, 3.8184592342600001e+00, 8.7051139108599997e+01 },
    { 5.9149999999999997e-08, 8.6094320174000005e-01, 6.2512058025999998e+01 },
    { 5.9030000000000000e-08, 3.1325171124800000e+00, 1.5952926704073000e+03 },
    { 5.8980000000000003e-08, 1.1716891416999999e+00, 7.1144930703379998e+02 },
    { 5.8299999999999999e-08, 4.8180093019500001e+00, 2.1574677599280000e+02 },
    { 5.7550000000000002e-08, 3.4761416057500001e+00, 8.9632535223590003e+02 },
    { 5.7429999999999999e-08, 3.5611519887199998e+00, 2.4643619622969999e+02 },
    { 5.7420000000000001e-08, 4.7666945653799999e+00, 2.0202226296720091e+04 },
    { 5.6970000000000000e-08, 2.8492893680300000e+00, 7.7258876693610000e+02 },
    { 5.6780000000000003e-08, 2.4557516369200001e+00, 1.2914309737229000e+03 },
    { 5.6750000000000002e-08, 3.9075316546900001e+00, 3.7301422095919997e+02 },
    { 5.6669999999999997e-08, 5.1083123791000000e+00, 4.7631944349890000e+02 },
    { 5.6610000000000002e-08, 8.5799990392000003e-01, 3.1516802937919999e+02 },
    { 5.6370000000000001e-08, 4.2708230393099997e+00, 4.6112914165080002e+02 },
    { 5.6309999999999999e-08, 2.9002321488599998e+00, 6.7880499887599996e+01 },
    { 5.6249999999999997e-08, 9.9982812640999996e-01, 4.2065188900899997e+01 },
    { 5.6099999999999999e-08, 4.8203120815800000e+00, 1.6438099196720000e+02 },
    { 5.5969999999999998e-08, 7.8287898332999994e-01, 7.7110429422779998e+02 },
    { 5.5770000000000002e-08, 4.1048251960600002e+00, 1.6868565224579999e+02 },
    { 5.5740000000000001e-08, 6.1747659594700002e+00, 3.5840887444769999e+02 },
    { 5.5619999999999998e-08, 1.9906627534900001e+00, 9.7415515816300001e+01 },
    { 5.5619999999999998e-08, 4.7516522649099997e+00, 4.6972716135989998e+02 },
    { 5.5490000000000003e-08, 2.5501313849599998e+00, 2.9674814968359999e+02 },
    { 5.5430000000000001e-08, 5.9674448127300002e+00, 1.8823874949360001e+02 },
    { 5.5409999999999998e-08, 2.0593123477100002e+00, 2.1197628650500000e+01 },
    { 5.5239999999999997e-08, 4.6867538372200004e+00, 4.9181856188770001e+02 },
    { 5.5169999999999997e-08, 1.5734159045000001e-01, 8.5715424246599994e+01 },
    { 5.5089999999999999e-08, 4.7823189059800004e+00, 1.2356117715723999e+03 },
    { 5.5080000000000001e-08, 4.7205141499799996e+00, 4.8788640862459999e+02 },
    { 5.4949999999999999e-08, 8.3010161211000000e-01, 1.5896436599579999e+02 },
    { 5.4919999999999998e-08, 6.9694956791999996e-01, 8.9338760969000006e+01 },
    { 5.4879999999999999e-08, 2.0953618156800000e+00, 1.3622987501029999e+02 },
    { 5.4840000000000000e-08, 2.0518611758600001e+00, 2.9370976829780000e+02 },
    { 5.4790000000000003e-08, 2.1239290131900002e+00, 2.2934073054800001e+02 },
    { 5.4730000000000001e-08, 1.3118189204399999e+00, 1.1885252503830000e+02 },
    { 5.4650000000000003e-08, 4.1839538872600004e+00, 4.9651816653700003e+01 },
    { 5.4599999999999999e-08, 5.8654453114000003e+00, 2.7741726679700001e+01 },
    { 5.4200000000000002e-08, 3.8554134049400002e+00, 1.9683676920270000e+02 },
    { 5.4149999999999999e-08, 1.9385328281500001e+00, 1.9391600789590001e+02 },
    { 5.4070000000000000e-08, 2.1187700622299999e+00, 1.4791866573190001e+02 },
    { 5.4020000000000003e-08, 4.8890230437800000e+00, 1.7001006625970001e+02 },
    { 5.3879999999999997e-08, 7.6342177995000005e-01, 2.3650246165860000e+02 },
    { 5.3830000000000000e-08, 6.2703211112600004e+00, 8.1777165909990003e+02 },
    { 5.3820000000000002e-08, 5.2569824684800004e+00, 2.6992144674060000e+02 },
    { 5.3799999999999999e-08, 3.4349088137199999e+00, 4.4423617327890003e+02 },
    { 5.2959999999999999e-08, 1.7399611167800000e+00, 6.5874762317500000e+01 },
    { 5.2560000000000002e-08, 8.2645339083000002e-01, 4.5342489381899998e+02 },
    { 5.2390000000000001e-08, 3.4883535750300001e+00, 2.3619364511789999e+02 },
    { 5.2089999999999999e-08, 5.0325710014000002e-01, 3.5499798604640000e+02 },
    { 5.1730000000000001e-08, 4.1411476579000004e+00, 6.3596513305090002e+02 },
    { 5.1230000000000003e-08, 2.0460862447000000e-01, 7.8623090560109995e+02 },
    { 5.1210000000000000e-08, 1.1939829913100000e+00, 8.2399143422340001e+02 },
    { 5.1149999999999998e-08, 4.3556044430799998e+00, 6.9110301164520001e+02 },
    { 5.1090000000000003e-08, 5.6100316109700001e+00, 4.4985950207700000e+01 },
    { 5.0810000000000003e-08, 5.5027400319600002e+00, 2.2345102382459999e+02 },
    { 5.0540000000000001e-08, 1.5277971523500000e+00, 3.4373983746139999e+02 },
    { 5.0379999999999998e-08, 2.0465531805600001e+00, 3.4639996633729999e+02 },
    { 5.0169999999999999e-08, 2.8290279430199998e+00, 3.0560680182359999e+02 },
    { 4.9889999999999999e-08, 3.4847228732500000e+00, 5.3653405886000002e+01 },
    { 4.9859999999999998e-08, 9.4681765820999997e-01, 6.8302625679790003e+02 },
    { 4.9619999999999997e-08, 1.6809572782600000e+00, 4.0250647792499997e+01 },
    { 4.9560000000000002e-08, 2.5739948133000001e+00, 4.5017406411209998e+02 },
    { 4.9150000000000000e-08, 5.9770460456899999e+00, 2.8000393915799998e+02 },
    { 4.9100000000000003e-08, 4.6528070969200002e+00, 1.0197644218431000e+03 },
    { 4.9070000000000002e-08, 1.0788675418600000e+00, 4.7069036920640002e+02 },
    { 4.8989999999999997e-08, 1.1897547800999999e+00, 1.2301837848190000e+02 },
    { 4.8940000000000000e-08, 4.9182402165300001e+00, 1.2758367119140000e+02 },
    { 4.8699999999999999e-08, 3.0191408374800002e+00, 1.0727152523135001e+03 },
    { 4.8660000000000000e-08, 3.8498763434000000e-01, 8.2287946853059998e+02 },
    { 4.8510000000000002e-08, 2.6121477278600000e+00, 8.9335640681929999e+02 },
    { 4.8359999999999998e-08, 2.6706900034699999e+00, 5.5555446019110002e+02 },
    { 4.8140000000000000e-08, 4.4556004661699999e+00, 3.0414860324306001e+03 },
    { 4.7920000000000002e-08, 3.5114439264700001e+00, 2.5739127376840003e+02 },
    { 4.7790000000000000e-08, 2.9454435598900002e+00, 1.6611084081600001e+01 },
    { 4.7740000000000003e-08, 3.4413871715600002e+00, 3.7901580226720000e+02 },
    { 4.7699999999999997e-08, 2.0902810844799999e+00, 5.8934595228859996e+02 },
    { 4.7580000000000000e-08, 1.5425960030100001e+00, 9.0195442652839995e+02 },
    { 4.7390000000000003e-08, 2.6562410421000000e+00, 2.5788776747305001e+04 },
    { 4.7339999999999999e-08, 3.1244979604799998e+00, 1.2466574718362999e+03 },
    { 4.7020000000000000e-08, 4.2045865020199997e+00, 2.9281599642050003e+02 },
    { 4.6919999999999999e-08, 2.1351515154000000e-01, 3.3514181775230003e+02 },
    { 4.6889999999999998e-08, 4.3323094266700002e+00, 2.1395348674380000e+02 },
    { 4.6819999999999998e-08, 4.9812807894500004e+00, 3.7471114198859999e+02 },
    { 4.6749999999999998e-08, 7.4793286058999997e-01, 8.1857480825200003e+02 },
    { 4.6719999999999997e-08, 2.7985905248099998e+00, 1.7867752193800001e+02 },
    { 4.6700000000000001e-08, 1.5652851270899999e+00, 6.9503516490829998e+02 },
    { 4.6579999999999998e-08, 1.0095302202700001e+00, 6.2173803904930003e+02 },
    { 4.6480000000000003e-08, 3.4173805548299998e+00, 1.9131956949479999e+02 },
    { 4.6469999999999999e-08, 4.5979194278400000e+00, 7.0077611378539996e+02 },
    { 4.6439999999999998e-08, 4.8801947824700003e+00, 6.3666770846650002e+02 },
    { 4.6430000000000000e-08, 5.1010253283500004e+00, 6.1701612157863001e+03 },
    { 4.6290000000000000e-08, 5.0609546866899997e+00, 3.2472925693479999e+02 },
    { 4.6210000000000002e-08, 4.8401276406499996e+00, 8.8772733252679996e+02 },
    { 4.5979999999999999e-08, 2.0533253330200001e+00, 1.5519227142710000e+02 },
    { 4.5979999999999999e-08, 1.2774526357700000e+00, 2.2408416327099999e+02 },
    { 4.5890000000000003e-08, 6.1765962049800001e+00, 2.5483430577912291e+04 },
    { 4.5839999999999999e-08, 2.0239904723900000e+00, 3.0168155375100000e+01 },
    { 4.5820000000000003e-08, 4.2771975264200002e+00, 1.1939504175950000e+02 },
    { 4.5800000000000000e-08, 1.6935875859000000e+00, 2.7361398630378999e+03 },
    { 4.5459999999999998e-08, 7.6152857131999996e-01, 4.4182801055600002e+01 },
    { 4.5270000000000001e-08, 4.8968193528099997e+00, 1.0005439283280000e+02 },
    { 4.5240000000000000e-08, 2.8143966809199998e+00, 5.2698265210889997e+02 },
    { 4.5179999999999998e-08, 5.1330234243500001e+00, 5.2985102378900001e+02 },
    { 4.5090000000000002e-08, 2.3906672838400000e+00, 1.0643422665070000e+02 },
    { 4.5079999999999997e-08, 5.3373158986900000e+00, 2.4038643081190000e+02 },
    { 4.5050000000000003e-08, 4.2320594676000001e-01, 1.5875191767470000e+02 },
    { 4.5030000000000000e-08, 2.3482431538199999e+00, 5.2953090640020002e+02 },
    { 4.5020000000000002e-08, 5.8489860801600004e+00, 5.2347118997109999e+02 },
    { 4.4990000000000001e-08, 1.6905473772799999e+00, 2.2759562540880000e+02 },
    { 4.4780000000000001e-08, 3.5986567835600001e+00, 4.3978275515399997e+02 },
    { 4.4719999999999999e-08, 3.9123447408300001e+00, 3.0237722397610003e+02 },
    { 4.4320000000000002e-08, 5.4230646203899999e+00, 1.6850521225016000e+03 },
    { 4.4300000000000000e-08, 2.7270309721400001e+00, 7.8159617692239999e+02 },
    { 4.4180000000000003e-08, 6.0913809344100001e+00, 6.5864723440969999e+02 },
    { 4.4150000000000002e-08, 3.7473137904699998e+00, 4.8491746320800002e+02 },
    { 4.3810000000000000e-08, 3.0023835455599999e+00, 1.9280404220310001e+02 },
    { 4.3749999999999998e-08, 2.4053416584000001e-01, 1.0100370912005899e+04 },
    { 4.3609999999999998e-08, 4.2709690123900002e+00, 1.0318739036899999e+03 },
    { 4.3590000000000002e-08, 3.9186665973500001e+00, 2.9430046912879999e+02 },
    { 4.3530000000000000e-08, 6.2609339937900002e+00, 1.3607959463187999e+03 },
    { 4.3259999999999998e-08, 6.0256744865499998e+00, 1.3253712236667000e+03 },
    { 4.3180000000000000e-08, 9.9378487571999996e-01, 1.5780271950199001e+03 },
    { 4.3079999999999999e-08, 1.7288609554800001e+00, 5.1604882642960001e+02 },
    { 4.3070000000000001e-08, 3.0843202673599999e+00, 2.6907011347870002e+02 },
    { 4.3019999999999997e-08, 2.5041069630199999e+00, 9.9841944511700007e+01 },
    { 4.3000000000000001e-08, 5.4926515882700002e+00, 3.0263785640700002e+02 },
    { 4.2860000000000001e-08, 2.9338200584099998e+00, 5.0286426215159997e+02 },
    { 4.2839999999999998e-08, 5.4857525615900000e+00, 6.8695841006100000e+02 },
    { 4.2809999999999998e-08, 6.1708876514400002e+00, 4.6913646052889999e+02 },
    { 4.2760000000000000e-08, 5.2891274824899996e+00, 1.0348428491065999e+03 },
    { 4.2739999999999998e-08, 5.5520296258800004e+00, 3.1097524368519998e+02 },
    { 4.2680000000000002e-08, 4.1195389513800000e+00, 8.4756730745949994e+02 },
    { 4.2480000000000001e-08, 1.5955771045300000e+00, 2.9111907539110001e+02 },
    { 4.2209999999999999e-08, 3.2848618940400001e+00, 1.0250207338760001e+02 },
    { 4.2160000000000002e-08, 1.8317440968000001e+00, 4.7365931462300000e+02 },
    { 4.2069999999999999e-08, 7.0498638267000002e-01, 1.1585425511446999e+03 },
    { 4.2060000000000001e-08, 5.1867441573999995e-01, 4.3519621058510000e+02 },
    { 4.1969999999999998e-08, 1.3935063439000001e-01, 6.3033605875839999e+02 },
    { 4.1960000000000000e-08, 5.8326573272999997e+00, 1.8164646735459999e+02 },
    { 4.1910000000000003e-08, 1.2895269995200001e+00, 3.0545804397730001e+02 },
    { 4.1829999999999998e-08, 4.6639256918300003e+00, 1.5748458012822000e+03 },
    { 4.1759999999999998e-08, 4.5372943889100004e+00, 3.0523429480810000e+02 },
    { 4.1570000000000001e-08, 4.3760467256100002e+00, 7.7130123618389996e+02 },
    { 4.1519999999999997e-08, 2.4043059821899999e+00, 9.5709230512270005e+02 },
    { 4.1420000000000003e-08, 4.8490269718000001e-01, 6.1483694036960003e+02 },
    { 4.1369999999999999e-08, 1.1611537152300000e+00, 4.3615941843159999e+02 },
    { 4.1330000000000000e-08, 2.6930901086799999e+00, 2.6087903141574199e+04 },
    { 4.1190000000000000e-08, 2.1351012998400001e+00, 2.1202483330866999e+03 },
    { 4.1159999999999999e-08, 9.1321363317000004e-01, 1.9540048060420000e+02 },
    { 4.1099999999999997e-08, 5.0850463100999999e-01, 5.1153171782990000e+02 },
    { 4.1080000000000001e-08, 3.8340496863700002e+00, 4.8884961647109998e+02 },
    { 4.1070000000000003e-08, 3.0204097657700002e+00, 1.1693276189777000e+03 },
    { 4.1070000000000003e-08, 4.1350856042000004e+00, 5.4972844394250001e+02 },
    { 4.0970000000000002e-08, 1.8359325913800000e+00, 6.1374448061199000e+03 },
    { 4.0970000000000002e-08, 3.2302629290899998e+00, 2.8857030601725000e+03 },
    { 4.0870000000000001e-08, 3.3157934254099999e+00, 7.7033802833740003e+02 },
    { 4.0870000000000001e-08, 5.5134315966800003e+00, 6.6632036179513998e+03 },
    { 4.0860000000000003e-08, 2.8247626011000002e+00, 1.6857377766120001e+02 },
    { 4.0740000000000000e-08, 5.3235690677100003e+00, 1.2125202148330000e+02 },
    { 4.0590000000000001e-08, 3.7186749275300000e+00, 6.0527571281400003e+02 },
    { 4.0550000000000002e-08, 5.2045650336599998e+00, 1.3917188817650000e+02 },
    { 4.0530000000000000e-08, 1.4370074948800000e+00, 2.5632993775046889e+04 },
    { 4.0520000000000002e-08, 3.2210886976199999e+00, 5.2724328453980002e+02 },
    { 4.0329999999999998e-08, 3.1041687097400001e+00, 5.4865336151140002e+02 },
    { 4.0320000000000000e-08, 2.5350570231699998e+00, 2.4335537622850001e+02 },
    { 4.0250000000000000e-08, 6.2217965152100003e+00, 4.3222726516850003e+02 },
    { 4.0200000000000003e-08, 4.1118697942400004e+00, 3.5692440173940003e+02 },
    { 3.9990000000000003e-08, 5.3915864054700000e+00, 5.5819202150499997e+01 },
    { 3.9990000000000003e-08, 5.5710013377100003e+00, 9.3743994984099999e+01 },
    { 3.9879999999999997e-08, 4.8891715938000004e+00, 8.3821852822500000e+02 },
    { 3.9850000000000003e-08, 1.7498556133900001e+00, 7.6588461021249998e+02 },
    { 3.9799999999999999e-08, 4.7806410403999999e-01, 4.3319047301500001e+02 },
    { 3.9729999999999999e-08, 3.8624706146599999e+00, 5.2375307426140000e+02 },
    { 3.9720000000000001e-08, 4.8685012019800000e+00, 1.8557862061770001e+02 },
    { 3.9720000000000001e-08, 6.2147303592000003e-01, 9.8014691349700001e+02 },
    { 3.9640000000000003e-08, 5.1643920799399998e+00, 8.9877303279069997e+02 },
    { 3.9589999999999999e-08, 3.9385475115499999e+00, 1.5093579525830000e+02 },
    { 3.9540000000000002e-08, 2.9073033612399999e+00, 3.8330916169770001e+02 },
    { 3.9480000000000000e-08, 3.4580967250299999e+00, 2.7018207917149999e+02 },
    { 3.9389999999999997e-08, 1.6126206363900000e+00, 6.0968640899272996e+03 },
    { 3.9319999999999997e-08, 5.6750336891300002e+00, 1.6153509694600000e+01 },
    { 3.9190000000000002e-08, 5.9720313112400003e+00, 3.2595309721220002e+02 },
    { 3.9179999999999997e-08, 3.8594186675300000e+00, 3.6785822741869998e+02 },
    { 3.9169999999999999e-08, 3.5810918236400000e+00, 4.8710451133190003e+02 },
    { 3.9139999999999998e-08, 1.5928620736500001e+00, 2.5409225995169999e+02 },
    { 3.9060000000000000e-08, 1.6161459748700000e+00, 3.6175482177500001e+01 },
    { 3.8980000000000002e-08, 4.8391021333899999e+00, 1.0564129847726001e+03 },
    { 3.8980000000000002e-08, 2.7428150890500000e+00, 6.6969293467360001e+02 },
    { 3.8950000000000001e-08, 5.3844334219699999e+00, 9.0833426034629997e+02 },
    { 3.8940000000000003e-08, 3.3165279404099999e+00, 1.8944190646765001e+03 },
    { 3.8870000000000003e-08, 5.5299008607799998e+00, 1.0200250542740000e+03 },
    { 3.8850000000000000e-08, 5.9579229209399998e+00, 4.4263982598600001e+02 },
    { 3.8840000000000002e-08, 6.5443948228000004e-01, 1.0593413314171001e+04 },
    { 3.8820000000000000e-08, 3.3608583935000003e-01, 5.8340806145540000e+02 },
    { 3.8799999999999997e-08, 6.1550233756299999e+00, 3.9505743737720002e+02 },
    { 3.8789999999999999e-08, 3.4772372388899999e+00, 5.2549817940059995e+02 },
    { 3.8759999999999998e-08, 5.2405033042299998e+00, 7.5728659050340002e+02 },
    { 3.8710000000000001e-08, 1.9914236615300001e+00, 5.1159540830470002e+02 },
    { 3.8700000000000002e-08, 7.4129906666000001e-01, 2.1249836355019002e+03 },
    { 3.8659999999999997e-08, 3.2586474906200000e+00, 1.0067654502339499e+04 },
    { 3.8640000000000001e-08, 4.3933156094300001e+00, 6.6109491496450005e+02 },
    { 3.8630000000000002e-08, 5.7442560775500002e+00, 4.8236779914600000e+01 },
    { 3.8509999999999999e-08, 4.2527085170500003e+00, 2.2493549653290000e+02 },
    { 3.8460000000000002e-08, 4.2408756216999999e+00, 1.1345715092630000e+02 },
    { 3.8400000000000000e-08, 5.4503177863000003e+00, 8.2717282796109998e+02 },
    { 3.8369999999999999e-08, 1.3888557829699999e+00, 7.6174000862829996e+02 },
    { 3.8360000000000001e-08, 5.8682187782400002e+00, 9.8066817835879999e+02 },
    { 3.8350000000000003e-08, 1.3280488708999999e-01, 3.1342292423999999e+02 },
    { 3.8290000000000001e-08, 1.4712587805999999e-01, 6.2736711334179995e+02 },
    { 3.8239999999999997e-08, 4.8683818278200004e+00, 6.1295804995936996e+03 },
    { 3.8019999999999999e-08, 3.1702655607000002e-01, 4.9841084402669998e+02 },
    { 3.8010000000000001e-08, 5.9554564631500000e+00, 3.1177521037062002e+03 },
    { 3.7919999999999998e-08, 1.8388059869000000e+00, 1.1474968508807999e+03 },
    { 3.7889999999999997e-08, 2.9989168020300001e+00, 1.0549285120643001e+03 },
    { 3.7809999999999999e-08, 1.7880019259400000e+00, 1.0412226829245001e+03 },
    { 3.7639999999999998e-08, 3.9353124281500000e+00, 6.8631259412999995e+01 },
    { 3.7639999999999998e-08, 3.8856826614400002e+00, 8.0931937721599994e+01 },
    { 3.7639999999999998e-08, 2.1592095902800001e+00, 7.5106681537990005e+02 },
    { 3.7599999999999999e-08, 5.2987331353200000e+00, 3.1490739694830000e+02 },
    { 3.7510000000000003e-08, 5.6092257593999995e-01, 1.2828329540138000e+03 },
    { 3.7459999999999999e-08, 5.2980487607700004e+00, 5.9854339284304997e+03 },
    { 3.7440000000000003e-08, 2.3704621175000001e-01, 1.2545217783625001e+03 },
    { 3.7370000000000003e-08, 4.0701766054300004e+00, 9.0993060763920005e+02 },
    { 3.7329999999999997e-08, 1.0346607919499999e+00, 9.0970685847000004e+02 },
    { 3.7310000000000001e-08, 2.1114466865799999e+00, 1.3840562228610000e+02 },
    { 3.7210000000000000e-08, 8.9189960469999996e-01, 1.4749344208012999e+03 },
    { 3.7149999999999998e-08, 1.7891263165600000e+00, 7.6492140236600005e+02 },
    { 3.7100000000000001e-08, 1.4843319364400001e+00, 9.0536531492970005e+02 },
    { 3.7090000000000003e-08, 3.0352215456099998e+00, 1.0027073786146901e+04 },
    { 3.7009999999999999e-08, 3.3279338104399998e+00, 6.6901161111739998e+02 },
    { 3.6960000000000001e-08, 4.1545320729000004e+00, 2.5865042818580590e+04 },
    { 3.6860000000000001e-08, 5.6789303277299998e+00, 4.7791579079180002e+02 },
    { 3.6780000000000002e-08, 2.0425933598400001e+00, 7.9838857155779999e+02 },
    { 3.6710000000000002e-08, 2.6793252104900001e+00, 6.9970103135429997e+02 },
    { 3.6659999999999999e-08, 3.6548017370800001e+00, 4.1566436519669998e+02 },
    { 3.6620000000000000e-08, 1.7934814350899999e+00, 1.3755428902419999e+02 },
    { 3.6610000000000002e-08, 1.3867471991700000e+00, 3.2016396422529999e+02 },
    { 3.6510000000000001e-08, 1.0811472444600001e+00, 2.6542812508101491e+04 },
    { 3.6470000000000002e-08, 3.7271083980000003e-01, 1.5455913198069999e+02 },
    { 3.6470000000000002e-08, 1.6667305265500001e+00, 9.9919096959257004e+03 },
    { 3.6340000000000000e-08, 1.4335994943100001e+00, 7.7638669399299999e+01 },
    { 3.6340000000000000e-08, 1.0421028810000001e-01, 7.1924527735300003e+01 },
    { 3.6309999999999999e-08, 3.4801792004999998e+00, 2.1753862116810001e+03 },
    { 3.6249999999999997e-08, 5.0393031195600004e+00, 2.4309474379759999e+02 },
    { 3.6069999999999998e-08, 2.9296038358800001e+00, 8.1294573395949999e+02 },
    { 3.6039999999999997e-08, 3.7627986495700001e+00, 3.6203221117010003e+02 },
    { 3.6020000000000001e-08, 1.8739840165899999e+00, 4.5071658083329999e+02 },
    { 3.6020000000000001e-08, 7.8573784099999992e-03, 1.0059790195813301e+04 },
    { 3.5969999999999997e-08, 4.9542828788400000e+00, 1.3203708931630001e+02 },
    { 3.5940000000000003e-08, 2.3253704183000001e-01, 6.0616999997061002e+03 },
    { 3.5920000000000000e-08, 4.5326460513200004e+00, 2.5060204967330000e+02 },
    { 3.5889999999999999e-08, 4.9911767797500000e+00, 1.3403490771936999e+03 },
    { 3.5710000000000000e-08, 1.7392470515100000e+00, 2.4554242435239999e+02 },
    { 3.5630000000000002e-08, 5.9982163964100002e+00, 1.3364169239306000e+03 },
    { 3.5600000000000001e-08, 5.5537939027099998e+00, 6.2343496007869999e+02 },
    { 3.5560000000000002e-08, 1.7205586310100001e+00, 3.6611312227949998e+02 },
    { 3.5509999999999998e-08, 5.3327532315099999e+00, 1.3851749687070001e+03 },
    { 3.5509999999999998e-08, 4.7900476477699998e+00, 2.6195656647790003e+02 },
    { 3.5469999999999999e-08, 5.9019055873199999e+00, 3.8606116389800000e+01 },
    { 3.5420000000000002e-08, 1.7731443293500000e+00, 3.7694637422229999e+02 },
    { 3.5339999999999997e-08, 4.9336564884899996e+00, 7.8039364629020682e+04 },
    { 3.5339999999999997e-08, 1.6633227749699999e+00, 2.9318850343600002e+02 },
    { 3.5269999999999998e-08, 4.4425722664200000e+00, 6.2567019231239999e+02 },
    { 3.5189999999999999e-08, 1.8575648925699999e+00, 6.1387373252309999e+02 },
    { 3.5189999999999999e-08, 1.7331085432000000e+00, 6.3548885031420996e+03 },
    { 3.5180000000000001e-08, 2.1274354923400001e+00, 1.1364511506168999e+03 },
    { 3.5159999999999999e-08, 2.3558439950699999e+00, 4.7049342725029999e+02 },
    { 3.5159999999999999e-08, 5.7336986801599998e+00, 1.0948066528413001e+03 },
    { 3.5129999999999998e-08, 2.8827875646500001e+00, 1.8802630117250001e+02 },
    { 3.5040000000000001e-08, 1.6804844040000000e-01, 6.2720301497275004e+03 },
    { 3.4980000000000000e-08, 4.3668593772999997e-01, 9.9156436246500998e+03 },
    { 3.4959999999999997e-08, 1.7166989276300000e+00, 3.1147831582896001e+03 },
    { 3.4919999999999998e-08, 3.1892455768299999e+00, 4.7346237266690002e+02 },
    { 3.4860000000000002e-08, 3.5372893322099999e+00, 3.1394418910180002e+02 },
    { 3.4739999999999999e-08, 1.8159996142299999e+00, 1.8260967520110000e+02 },
    { 3.4720000000000003e-08, 1.8750561047100001e+00, 5.5269738935910004e+02 },
    { 3.4639999999999998e-08, 4.1679489332599999e+00, 1.6141204655060000e+02 },
    { 3.4599999999999999e-08, 4.4678385610699998e+00, 1.5604360468900001e+02 },
    { 3.4569999999999998e-08, 3.0098634339200001e+00, 8.0923732986560003e+02 },
    { 3.4499999999999998e-08, 4.0297290190000001e-02, 1.2059761819072000e+03 },
    { 3.4359999999999998e-08, 2.2293051552300001e+00, 5.1742142455329997e+02 },
    { 3.4319999999999999e-08, 4.7836588824400001e+00, 6.9407195706180005e+02 },
    { 3.4300000000000003e-08, 1.3403585341299999e+00, 7.4809786996330001e+02 },
    { 3.4289999999999998e-08, 2.0360394543100000e+00, 2.4479166482700001e+02 },
    { 3.4249999999999999e-08, 1.7462176427000001e-01, 9.1448459950059998e+02 },
    { 3.3899999999999999e-08, 3.8299290033000002e+00, 5.9091678571549000e+03 },
    { 3.3869999999999998e-08, 6.1993335152000002e+00, 2.5862073873164001e+04 },
    { 3.3869999999999998e-08, 3.1559434701300000e+00, 1.0285098199361701e+04 },
    { 3.3860000000000000e-08, 5.7596770758000004e-01, 3.8724131496080003e+02 },
    { 3.3839999999999997e-08, 5.6872497525799997e+00, 1.0562005364515001e+03 },
    { 3.3820000000000001e-08, 1.7067166280199999e+00, 8.8150755740329998e+02 },
    { 3.3759999999999999e-08, 1.6735661244000000e+00, 1.0202239845947101e+04 },
    { 3.3729999999999998e-08, 5.5580347266099999e+00, 2.1325091132820000e+02 },
    { 3.3699999999999997e-08, 2.4711396357200002e+00, 2.1334727954780001e+02 },
    { 3.3670000000000003e-08, 3.7689386561999999e-01, 2.4783004621280000e+02 },
    { 3.3650000000000000e-08, 4.2134816540299997e+00, 1.3726447957348000e+03 },
    { 3.3640000000000002e-08, 5.6341323259299996e+00, 1.0498207026335999e+03 },
    { 3.3600000000000003e-08, 6.0478561926300003e+00, 3.9197661737599998e+02 },
    { 3.3589999999999998e-08, 6.1152756907199999e+00, 5.7616138801060004e+02 },
    { 3.3570000000000002e-08, 5.2659905954999999e+00, 1.1829215735329001e+03 },
    { 3.3540000000000001e-08, 4.2954822389400000e+00, 6.3203297978779995e+02 },
    { 3.3540000000000001e-08, 5.3706358243999996e-01, 9.1937996061019999e+02 },
    { 3.3519999999999998e-08, 4.5988110387999998e-01, 6.9780716836880003e+02 },
    { 3.3339999999999999e-08, 5.8742465675400002e+00, 4.0992341631959999e+02 },
    { 3.3190000000000001e-08, 5.0019400822300000e+00, 1.0252332524700000e+02 },
    { 3.3150000000000002e-08, 9.4177030624000002e-01, 8.0775285715730001e+02 },
    { 3.3099999999999999e-08, 2.2272815573500000e+00, 2.0129018732759999e+02 },
    { 3.3039999999999997e-08, 8.6856001646000003e-01, 2.7315102458809997e+02 },
    { 3.3039999999999997e-08, 1.0166070380400001e+00, 6.5323060843830001e+02 },
    { 3.2999999999999998e-08, 3.7208703243499999e+00, 4.4089472084680000e+02 },
    { 3.2980000000000002e-08, 4.9731049676100003e+00, 4.6324675380550002e+02 },
    { 3.2940000000000002e-08, 5.1160324240600001e+00, 4.1529185818119998e+02 },
    { 3.2870000000000003e-08, 5.0279289063699997e+00, 3.2176031151820001e+02 },
    { 3.2770000000000002e-08, 5.5826574542499996e+00, 3.0289848883790000e+02 },
    { 3.2759999999999997e-08, 3.0063657111900000e+00, 5.2012973753899996e+02 },
    { 3.2679999999999999e-08, 2.4300050852899999e+00, 5.9000595888200003e+01 },
    { 3.2679999999999999e-08, 3.7488240827000001e+00, 2.4241100297711000e+03 },
    { 3.2670000000000001e-08, 4.7620816832099999e+00, 7.3742467671489999e+02 },
    { 3.2649999999999998e-08, 1.7507793485000001e-01, 2.6572866104660000e+02 },
    { 3.2539999999999999e-08, 5.7482068994800004e+00, 4.8236920891670002e+02 },
    { 3.2520000000000003e-08, 5.4917196962699997e+00, 1.0160929010109001e+03 },
    { 3.2509999999999998e-08, 3.3431351861000003e-01, 8.3167169955299997e+01 },
    { 3.2369999999999998e-08, 2.5655796533299999e+00, 3.9494556279260001e+02 },
    { 3.2360000000000000e-08, 2.7207674475200001e+00, 4.1258354519549999e+02 },
    { 3.2299999999999998e-08, 4.0510192611000004e+00, 5.5418186206739995e+02 },
    { 3.2290000000000000e-08, 3.7936583883799999e+00, 4.7643131808350000e+02 },
    { 3.2189999999999999e-08, 1.5642313593499999e+00, 5.1960847267719998e+02 },
    { 3.2140000000000002e-08, 1.0495097704300000e+00, 4.1237109687440000e+02 },
    { 3.2110000000000001e-08, 3.9678724021799998e+00, 1.5156893470470001e+02 },
    { 3.2040000000000001e-08, 5.6587701573900002e+00, 1.1341635287565000e+03 },
    { 3.1979999999999999e-08, 2.4753003945400001e+00, 2.6014606015715191e+04 },
    { 3.1960000000000003e-08, 4.2750712009400003e+00, 3.2673153008407999e+03 },
    { 3.1890000000000003e-08, 5.2524455184400001e+00, 9.8393775533744993e+03 },
    { 3.1879999999999998e-08, 2.1266357640900000e+00, 7.6961982151949996e+02 },
    { 3.1870000000000000e-08, 3.0503088897100001e+00, 1.9688495331249999e+02 },
    { 3.1860000000000002e-08, 1.4450752554999999e+00, 5.3632154026599999e+01 },
    { 3.1839999999999999e-08, 1.0871610521799999e+00, 5.9824649830138997e+03 },
    { 3.1839999999999999e-08, 3.6098484853800001e+00, 4.8088473620840000e+02 },
    { 3.1800000000000000e-08, 4.3153356377499996e+00, 2.8021638747909998e+02 },
    { 3.1779999999999997e-08, 4.1926369515499999e+00, 1.8865547581502999e+03 },
    { 3.1760000000000001e-08, 9.0264657780000002e-01, 5.2943033266370003e+02 },
    { 3.1760000000000001e-08, 3.1336053340699999e+00, 3.4936891175390002e+02 },
    { 3.1669999999999998e-08, 3.1799825102499999e+00, 2.6161200267433189e+04 },
    { 3.1660000000000000e-08, 2.2089203311599999e+00, 1.2567636172691100e+04 },
    { 3.1610000000000003e-08, 4.3146630079999998e-02, 2.0029808544932000e+03 },
    { 3.1599999999999998e-08, 2.8941939041500002e+00, 3.7955217932270998e+03 },
    { 3.1540000000000003e-08, 5.4901645460099999e+00, 6.5331461289732997e+03 },
    { 3.1419999999999999e-08, 4.6715428103000001e+00, 7.2509144569880004e+02 },
    { 3.1389999999999999e-08, 3.9238375468400002e+00, 9.3812990870589999e+02 },
    { 3.1289999999999998e-08, 5.6808032199999996e+00, 3.6759759498779999e+02 },
    { 3.1240000000000000e-08, 4.1436639288799997e+00, 5.8637700687200004e+02 },
    { 3.1230000000000002e-08, 3.3395175570900002e+00, 6.2322251175760005e+02 },
    { 3.1219999999999998e-08, 2.6834012058200001e+00, 1.5961864422845999e+03 },
    { 3.1219999999999998e-08, 6.0498917300399997e+00, 9.3961438141420001e+02 },
    { 3.1189999999999997e-08, 5.1268843036999998e+00, 8.4926422848890002e+02 },
    { 3.1090000000000002e-08, 4.5263654509800002e+00, 4.5034134317499998e+01 },
    { 3.1079999999999998e-08, 3.3629751811899999e+00, 7.5929232807350002e+02 },
    { 3.1009999999999998e-08, 2.5068844771799998e+00, 6.5742339413230002e+02 },
    { 3.0969999999999999e-08, 3.5257666296400001e+00, 1.5536481726316999e+03 },
    { 3.0939999999999998e-08, 4.9810498723799999e+00, 3.4139095525588000e+03 },
    { 3.0909999999999997e-08, 5.6073608260999999e+00, 9.1151565408399995e+02 },
    { 3.0899999999999999e-08, 2.5190671198999998e-01, 5.2995159752550001e+02 },
    { 3.0880000000000003e-08, 2.9150241496399998e+00, 8.2881735936380005e+02 },
    { 3.0880000000000003e-08, 4.0096817453700000e+00, 2.3028268653510000e+02 },
    { 3.0850000000000002e-08, 2.3585965304799998e+00, 9.3531546663000000e+01 },
    { 3.0850000000000002e-08, 3.8302882268500000e+00, 1.0578974574808999e+03 },
    { 3.0820000000000001e-08, 4.3343158489700002e+00, 4.3675011926259998e+02 },
    { 3.0729999999999998e-08, 4.5106467328599997e+00, 5.3383556667879998e+02 },
    { 3.0689999999999999e-08, 2.0042129080300000e+00, 5.3323337485899998e+01 },
    { 3.0659999999999998e-08, 3.1823249018799999e+00, 9.7897125732940003e+02 },
    { 3.0619999999999999e-08, 5.8470357226000003e+00, 1.0625633239269000e+03 },
    { 3.0610000000000001e-08, 2.3406205127700002e+00, 7.8946048344860003e+02 },
    { 3.0530000000000003e-08, 1.7166820140200001e+00, 9.9793365374619998e+02 },
    { 3.0510000000000000e-08, 4.4154601815000000e-01, 4.3053034413910001e+02 },
    { 3.0479999999999999e-08, 4.5177944770699998e+00, 6.5301771835566997e+03 },
    { 3.0470000000000001e-08, 2.1393079284700001e+00, 7.4569150246199996e+01 },
    { 3.0470000000000001e-08, 5.6816871611200002e+00, 7.4994046888400007e+01 },
    { 3.0449999999999998e-08, 4.5901625314000000e-01, 1.1907858800591000e+03 },
    { 3.0409999999999999e-08, 4.2758724396399996e+00, 7.5255128808819995e+02 },
    { 3.0400000000000001e-08, 6.7193319059000001e-01, 5.2174321810440088e+04 },
    { 3.0360000000000002e-08, 2.7243444282199998e+00, 3.2892204262880000e+02 },
    { 3.0279999999999997e-08, 1.4015487155099999e+00, 4.7940026350009998e+02 },
    { 3.0199999999999999e-08, 1.7033282407100001e+00, 6.4382943957709995e+02 },
    { 3.0169999999999998e-08, 5.2494887755599997e+00, 8.2940806019479999e+02 },
    { 3.0169999999999998e-08, 7.5308623960000001e-02, 6.2640390549530002e+02 },
    { 3.0160000000000000e-08, 2.6491686528499998e+00, 1.3257960603750001e+02 },
    { 3.0139999999999997e-08, 5.6142933306299998e+00, 1.5440869450760999e+03 },
    { 3.0099999999999998e-08, 6.5377897817999997e-01, 1.6055876111458001e+03 },
    { 3.0020000000000000e-08, 1.8956003485000000e-01, 1.0616695520496000e+03 },
    { 2.9989999999999999e-08, 2.1537566629099998e+00, 3.0927832265580003e+02 },
    { 2.9970000000000003e-08, 3.5671581537099999e+00, 2.9795648359600001e+01 },
    { 2.9929999999999997e-08, 5.7694265523699997e+00, 1.8227960680100000e+02 },
    { 2.9910000000000001e-08, 2.2469538043999999e+00, 6.2599451521809999e+02 },
    { 2.9900000000000003e-08, 4.2180263255200003e+00, 1.0686486878730000e+02 },
    { 2.9880000000000000e-08, 2.0702920787400001e+00, 4.5905396811150001e+02 },
    { 2.9849999999999999e-08, 2.4864945756800001e+00, 7.5335443724029994e+02 },
    { 2.9840000000000001e-08, 5.1382122651100000e+00, 1.6263588682800000e+02 },
    { 2.9830000000000003e-08, 1.8172718518500000e+00, 9.3664543599759998e+02 },
    { 2.9819999999999998e-08, 2.5102054902500002e+00, 9.9126746792334998e+03 },
    { 2.9760000000000000e-08, 4.4087049457700003e+00, 1.0956098019934000e+03 },
    { 2.9749999999999998e-08, 2.6380441229599998e+00, 2.2418473700749999e+02 },
    { 2.9729999999999999e-08, 3.9351175472200000e+00, 3.8405992122309999e+02 },
    { 2.9690000000000000e-08, 5.7564723611100002e+00, 1.8306831663731000e+03 },
    { 2.9670000000000000e-08, 2.9451274024499998e+00, 1.5418517128424000e+03 },
    { 2.9650000000000001e-08, 2.1382474363099999e+00, 1.5861039498672001e+03 },
    { 2.9569999999999999e-08, 2.3210680606800000e+00, 5.3583969916800001e+01 },
    { 2.9480000000000000e-08, 1.9634089071600000e+00, 7.8338491023289884e+04 },
    { 2.9440000000000001e-08, 3.3104338758799998e+00, 1.2577031721001999e+03 },
    { 2.9410000000000000e-08, 6.1182467669999996e+00, 1.4786627610520000e+02 },
    { 2.9410000000000000e-08, 1.2121066562800000e+00, 1.2564667227274500e+04 },
    { 2.9410000000000000e-08, 4.1170590296000000e+00, 5.8115732285670003e+02 },
    { 2.9399999999999999e-08, 9.7064608287999998e-01, 3.8087852748540001e+02 },
    { 2.9379999999999999e-08, 5.0088988641899999e+00, 5.9024546966030005e+02 },
    { 2.9360000000000000e-08, 5.4308414160299998e+00, 1.4245318446222000e+03 },
    { 2.9350000000000001e-08, 1.9449822438600000e+00, 7.1441825245040002e+02 },
    { 2.9329999999999999e-08, 6.0649711936499999e+00, 8.2962050851590004e+02 },
    { 2.9309999999999999e-08, 6.2428269835899997e+00, 1.7313907037070001e+02 },
    { 2.9290000000000000e-08, 1.4554697618400001e+00, 3.0423420369989998e+02 },
    { 2.9280000000000001e-08, 6.1564973836899997e+00, 4.7273572068100002e+01 },
    { 2.9280000000000001e-08, 3.8494988510399999e+00, 6.0920786607709999e+02 },
    { 2.9230000000000001e-08, 4.4426337839799999e+00, 2.3916259053450000e+02 },
    { 2.9220000000000000e-08, 5.3273530098700004e+00, 9.4755386940400001e+01 },
    { 2.9220000000000000e-08, 5.6352347333099999e+00, 5.4807810194200002e+01 },
    { 2.9220000000000000e-08, 2.4646768321999999e+00, 1.5999585368533001e+03 },
    { 2.9210000000000001e-08, 1.8786399033400001e+00, 3.4088276662940001e+02 },
    { 2.9140000000000002e-08, 3.4363776717999999e+00, 3.7937700843459999e+02 },
    { 2.9119999999999999e-08, 4.1786597948499997e+00, 4.2808266358430001e+02 },
    { 2.9080000000000000e-08, 5.4927753843899998e+00, 6.5094298657790000e+02 },
    { 2.9060000000000000e-08, 2.4447880255799999e+00, 1.3229235431119000e+03 },
    { 2.9040000000000001e-08, 2.3256915600100001e+00, 1.4560244140112000e+03 },
    { 2.9029999999999999e-08, 2.4760212058100000e+00, 6.5004921470060003e+02 },
    { 2.9020000000000001e-08, 2.2050049109000000e-01, 4.7221182441400003e+01 },
    { 2.9020000000000001e-08, 4.4589019449100000e+00, 1.0234201469320000e+02 },
    { 2.9010000000000000e-08, 2.2146641054099998e+00, 4.9351548291709997e+02 },
    { 2.9010000000000000e-08, 4.6427458510199999e+00, 1.5411179996594999e+03 },
    { 2.8889999999999999e-08, 1.9724843229400000e+00, 1.5859438911728000e+03 },
    { 2.8870000000000000e-08, 5.2445232866299998e+00, 1.3862937145530000e+02 },
    { 2.8760000000000001e-08, 2.4651336615399999e+00, 9.0562601246400007e+01 },
    { 2.8760000000000001e-08, 4.0662843821700001e+00, 6.8127668150860000e+03 },
    { 2.8719999999999998e-08, 4.3522736498999999e+00, 4.2977958461370002e+02 },
    { 2.8640000000000000e-08, 1.9987911756999999e-01, 2.1926516870684000e+03 },
    { 2.8539999999999999e-08, 3.7154907276700002e+00, 2.2796813242429999e+02 },
    { 2.8500000000000000e-08, 2.0893037637699998e+00, 5.2964278098479997e+02 },
    { 2.8480000000000001e-08, 5.0713211976399997e+00, 3.9590877063910000e+02 },
    { 2.8470000000000000e-08, 8.9078483425999999e-01, 2.1951887056149999e+02 },
    { 2.8460000000000001e-08, 5.3924240162899997e+00, 5.2973914920439995e+02 },
    { 2.8430000000000000e-08, 3.5840704036600002e+00, 1.4474263943021001e+03 },
    { 2.8360000000000001e-08, 1.2034112584700001e+00, 3.7353548582100001e+02 },
    { 2.8349999999999999e-08, 4.8203705066199998e+00, 7.7841478318470001e+02 },
    { 2.8349999999999999e-08, 2.4398930787899999e+00, 3.2324478422649997e+02 },
    { 2.8340000000000001e-08, 4.6288191339100004e+00, 5.4005534180229995e+02 },
    { 2.8310000000000000e-08, 2.7245998774100002e+00, 5.3977345751200005e+02 },
    { 2.8299999999999999e-08, 4.4131453951999999e-01, 1.3103933701397000e+03 },
    { 2.8159999999999999e-08, 5.2106074112000000e-01, 2.9456110155969998e+02 },
    { 2.8120000000000000e-08, 3.0378867409999999e-01, 3.1787634236489998e+02 },
    { 2.8060000000000001e-08, 4.0223396235599997e+00, 2.8148841186630000e+02 },
    { 2.8060000000000001e-08, 4.1473969232299996e+00, 3.5170471772410002e+02 },
    { 2.8040000000000002e-08, 3.6045057940600000e+00, 2.0888081533243999e+03 },
    { 2.8019999999999999e-08, 4.0865505420999998e-01, 1.5920418407003999e+03 },
    { 2.7980000000000000e-08, 4.8024337362500003e+00, 1.1249394307980000e+02 },
    { 2.7960000000000000e-08, 3.0521313852800001e+00, 5.9853467282869997e+02 },
    { 2.7960000000000000e-08, 2.3014069170700000e+00, 2.9104963942190000e+02 },
    { 2.7949999999999999e-08, 5.4846455095100000e+00, 1.0742976511305600e+04 },
    { 2.7910000000000000e-08, 1.6824122845100000e+00, 6.9800411032490001e+02 },
    { 2.7870000000000001e-08, 5.4983461526999999e+00, 2.2945260513260001e+02 },
    { 2.7829999999999998e-08, 3.7468686726599998e+00, 8.8921180523509997e+02 },
    { 2.7809999999999999e-08, 5.7397522527599998e+00, 1.2687488723641000e+03 },
    { 2.7809999999999999e-08, 1.7182817577500000e+00, 1.2205517063540000e+02 },
    { 2.7750000000000000e-08, 2.7621060148500001e+00, 5.5178236562239999e+02 },
    { 2.7710000000000001e-08, 5.7889811616499998e+00, 4.3133349329120000e+02 },
    { 2.7660000000000001e-08, 9.6756499866000001e-01, 1.6001185955476999e+03 },
    { 2.7640000000000001e-08, 4.4882055749500003e+00, 7.3045988756040003e+02 },
    { 2.7619999999999999e-08, 3.0410001490599998e+00, 4.6527374323499998e+02 },
    { 2.7590000000000001e-08, 6.9873056680000001e-01, 2.7406604832480002e+02 },
    { 2.7579999999999999e-08, 2.9198890709200001e+00, 1.3812428154439001e+03 },
    { 2.7529999999999999e-08, 2.6737755866900002e+00, 1.1608301730051001e+03 },
    { 2.7479999999999999e-08, 3.7216674607300000e+00, 1.7559015678057999e+03 },
    { 2.7430000000000001e-08, 6.2736270472899998e+00, 6.7836039035190004e+02 },
    { 2.7319999999999999e-08, 1.8814042693899999e+00, 1.3324847706675000e+03 },
    { 2.7269999999999999e-08, 5.1514396625099996e+00, 1.0638353483141000e+03 },
    { 2.7230000000000000e-08, 5.3761039008499996e+00, 1.6063383706712000e+03 },
    { 2.7179999999999999e-08, 5.1629305808800003e+00, 5.2325369480282992e+04 },
    { 2.7179999999999999e-08, 9.1308619680000003e-01, 3.5192846689330003e+02 },
    { 2.7160000000000000e-08, 5.4769543807599996e+00, 4.4482687410990002e+02 },
    { 2.7129999999999999e-08, 4.3730309045400002e+00, 1.5332536931333000e+03 },
    { 2.7120000000000001e-08, 2.9089388588400000e+00, 7.7055047665849997e+02 },
    { 2.7100000000000001e-08, 3.4192435132999999e-01, 1.0129115072732000e+03 },
    { 2.7100000000000001e-08, 4.4559311519399998e+00, 6.5967413273099999e+02 },
    { 2.7090000000000000e-08, 3.0663917652000000e-01, 1.8727554164710000e+02 },
    { 2.7080000000000002e-08, 1.5013086261799999e+00, 4.5735704708210000e+02 },
    { 2.7039999999999999e-08, 5.2693849352899997e+00, 9.8853248488500003e+02 },
    { 2.6930000000000000e-08, 5.7305838131999998e-01, 2.9971709510020003e+02 },
    { 2.6919999999999999e-08, 2.9274074051199999e+00, 3.3939829392109999e+02 },
    { 2.6849999999999999e-08, 1.4085650904500000e+00, 3.0357981239409997e+02 },
    { 2.6790000000000000e-08, 4.6917561290000000e-02, 1.3741292684431000e+03 },
    { 2.6779999999999999e-08, 5.0610530068799999e+00, 9.5146323083020002e+02 },
    { 2.6740000000000000e-08, 5.8963739033899998e+00, 5.1901777184620005e+02 },
    { 2.6729999999999998e-08, 1.3989889337500001e+00, 9.8311585891360005e+02 },
    { 2.6720000000000000e-08, 4.2762218827399998e+00, 3.1957326339430000e+02 },
    { 2.6670000000000000e-08, 8.6050140979999995e-02, 2.5791253863020000e+02 },
    { 2.6659999999999999e-08, 2.7851258124400000e+00, 9.6239859648700005e+01 },
    { 2.6630000000000001e-08, 4.6851490824299997e+00, 5.8779778910549999e+02 },
    { 2.6630000000000001e-08, 1.2979999084800000e+00, 4.1232291276460001e+02 },
    { 2.6619999999999999e-08, 5.0723497398899999e+00, 2.7140591944890002e+02 },
    { 2.6550000000000000e-08, 5.4648352820000003e-02, 1.0452906281710000e+02 },
    { 2.6400000000000001e-08, 3.2392618970800000e+00, 4.8715269544170002e+02 },
    { 2.6340000000000000e-08, 5.9846881623500003e+00, 6.0824465823059995e+02 },
    { 2.6300000000000001e-08, 4.7627219871399999e+00, 7.4379320968469995e+02 },
    { 2.6270000000000000e-08, 2.3965357147300002e+00, 1.4528430202735001e+03 },
    { 2.6270000000000000e-08, 1.9138883724700000e+00, 7.6885355562910001e+02 },
    { 2.6090000000000001e-08, 1.3821205706699999e+00, 1.4028611352196999e+03 },
    { 2.5979999999999998e-08, 3.4390992414000001e+00, 7.6322448133659998e+02 },
    { 2.5850000000000000e-08, 5.7566812709499997e+00, 7.5403576079649997e+02 },
    { 2.5790000000000001e-08, 4.0566404706299997e+00, 1.0554497769261000e+03 },
    { 2.5710000000000000e-08, 4.9787557347800000e+00, 1.5362226385499000e+03 },
    { 2.5699999999999999e-08, 3.7106570616900001e+00, 5.8482884368889995e+02 },
    { 2.5670000000000001e-08, 5.9661221438399998e+00, 6.3475265872500003e+01 },
    { 2.5659999999999999e-08, 1.9290162693899999e+00, 1.6992792165031999e+03 },
    { 2.5620000000000000e-08, 1.5733492760800001e+00, 6.9651963761659999e+02 },
    { 2.5600000000000001e-08, 1.1005375974400000e+00, 2.1172793876700998e+03 },
    { 2.5570000000000000e-08, 1.7972357694100001e+00, 5.3739521292639995e+02 },
    { 2.5500000000000000e-08, 3.6942812142700001e+00, 3.7250701549999998e-01 },
    { 2.5500000000000000e-08, 1.5740573307800001e+00, 8.3896928775039999e+02 },
    { 2.5489999999999999e-08, 1.1152565340500000e+00, 4.1013586464069999e+02 },
    { 2.5440000000000001e-08, 2.7660776468399999e+00, 2.6280789973980001e+02 },
    { 2.5360000000000000e-08, 4.5281076713199999e+00, 1.3307878496380999e+03 },
    { 2.5360000000000000e-08, 5.0695684395200002e+00, 2.0736120460480001e+02 },
    { 2.5240000000000000e-08, 3.6842035178499999e+00, 9.6006125053929998e+02 },
    { 2.5240000000000000e-08, 4.1272085416600000e+00, 4.4115535327769999e+02 },
    { 2.5170000000000000e-08, 4.2730262504600001e+00, 5.8386563584240002e+02 },
    { 2.5130000000000001e-08, 3.0048132605200002e+00, 2.0278492367995688e+04 },
    { 2.5119999999999999e-08, 5.5220125146999999e-01, 8.2643911477819995e+02 },
    { 2.5049999999999999e-08, 1.5198719687500000e+00, 3.8213350553010002e+02 },
    { 2.4929999999999999e-08, 1.5267852747799999e+00, 3.1665250208750001e+02 },
    { 2.4769999999999999e-08, 2.4376990018500000e+00, 8.2154375366859995e+02 },
    { 2.4760000000000001e-08, 2.9895563542499999e+00, 3.4793262315539999e+02 },
    { 2.4739999999999999e-08, 1.0627925418399999e+00, 6.6820846196529999e+02 },
    { 2.4730000000000000e-08, 1.2452537116900000e+00, 1.4072663692348001e+03 },
    { 2.4579999999999999e-08, 2.2584285580799999e+00, 1.4489108670103999e+03 },
    { 2.4459999999999999e-08, 3.5571263383199998e+00, 1.3278189042214999e+03 },
    { 2.4439999999999999e-08, 5.3993070611400000e+00, 1.7582045110600001e+02 },
    { 2.4380000000000001e-08, 5.9392979362100000e+00, 3.9510562148700001e+02 },
    { 2.4369999999999999e-08, 5.0994111667800004e+00, 3.7687693825309998e+02 },
    { 2.4350000000000000e-08, 2.0304414697900000e+00, 3.5574874557179999e+02 },
    { 2.4299999999999999e-08, 5.9872327267200003e+00, 1.4040849754971000e+03 },
    { 2.4270000000000002e-08, 2.0324707676900000e+00, 2.0275523422579088e+04 },
    { 2.4260000000000000e-08, 1.1817636744500000e+00, 5.7798007470199998e+01 },
    { 2.4240000000000001e-08, 1.2550826092400000e+00, 2.4389789294970001e+02 },
    { 2.4210000000000000e-08, 3.8717479803799999e+00, 1.1593457002968000e+03 },
    { 2.4210000000000000e-08, 2.2954136676800001e+00, 6.7294376438050006e+02 },
    { 2.4210000000000000e-08, 1.4542098919899999e+00, 4.4928029223480002e+02 },
    { 2.4190000000000000e-08, 6.7634283277999996e-01, 1.0160830151030000e+02 },
    { 2.4140000000000000e-08, 5.0585838372299996e+00, 3.6054773846180001e+02 },
    { 2.4019999999999999e-08, 3.0256696013200002e+00, 4.8225733433210002e+02 },
    { 2.3759999999999999e-08, 3.9385449175299998e+00, 1.9281992559258999e+03 },
    { 2.3600000000000000e-08, 2.9330247297800001e+00, 1.2970600480154001e+03 },
    { 2.3580000000000000e-08, 2.8137033176899999e+00, 1.1177012025212000e+03 },
    { 2.3580000000000000e-08, 4.3252456916700002e+00, 5.8238116313410001e+02 },
    { 2.3580000000000000e-08, 6.2223225648699998e+00, 1.3676970449984999e+03 },
    { 2.3569999999999999e-08, 2.8753839084299999e+00, 3.7812203038989998e+02 },
    { 2.3540000000000001e-08, 3.2093968354200002e+00, 1.5222079928694000e+03 },
    { 2.3460000000000000e-08, 5.1497472502699999e+00, 1.2112327491842000e+03 },
    { 2.3450000000000002e-08, 1.7599653852799999e+00, 2.6895823889410002e+02 },
    { 2.3409999999999999e-08, 3.4893897966599998e+00, 1.8117731595830001e+03 },
    { 2.3400000000000001e-08, 2.1236020626799998e+00, 9.7207015864970003e+02 },
    { 2.3390000000000000e-08, 4.9543749317000002e-01, 3.3455111692129998e+02 },
    { 2.3330000000000001e-08, 4.9678736367100003e+00, 1.7304844764550000e+02 },
    { 2.3320000000000000e-08, 6.1837277181500001e+00, 2.7458731318660000e+02 },
    { 2.3300000000000000e-08, 8.4390251807000005e-01, 8.2081004048570003e+02 },
    { 2.3230000000000000e-08, 1.2776552001899999e+00, 3.8634754308350000e+02 },
    { 2.3149999999999999e-08, 2.3089749541700000e+00, 2.9036831586570003e+02 },
    { 2.2970000000000000e-08, 3.6473221538199998e+00, 1.1868537267960000e+03 },
    { 2.2889999999999999e-08, 1.8602310852000001e-01, 7.6418768918310002e+02 },
    { 2.2869999999999999e-08, 4.3358357605900002e+00, 1.0237971488427000e+03 },
    { 2.2819999999999999e-08, 9.4739204341000005e-01, 8.1146126125119997e+02 },
    { 2.2749999999999999e-08, 1.6959872108900000e+00, 8.7290953769420003e+02 },
    { 2.2740000000000000e-08, 2.2039738799899999e+00, 3.2579303851780003e+02 },
    { 2.2609999999999999e-08, 3.3832136045199999e+00, 1.2396033446900000e+02 },
    { 2.2589999999999999e-08, 3.1098861422300002e+00, 1.1338771495710000e+02 },
    { 2.2580000000000001e-08, 1.1132017729100001e+00, 7.1467888488129995e+02 },
    { 2.2399999999999999e-08, 5.0738293328899999e+00, 5.0366741130370002e+02 },
    { 2.2350000000000002e-08, 2.5487634847399998e+00, 1.3214390704036000e+03 },
    { 2.2300000000000001e-08, 3.6417723022400001e+00, 4.4237919355510002e+02 },
    { 2.2270000000000000e-08, 5.0922434585899996e+00, 3.2591471642584002e+03 },
    { 2.2239999999999999e-08, 5.8597292005500003e+00, 1.2584539316256000e+03 },
    { 2.2220000000000000e-08, 3.5350883945999998e+00, 5.6782400073240001e+02 },
    { 2.2189999999999999e-08, 2.9914858482400000e+00, 6.2025356634100001e+02 },
    { 2.2090000000000001e-08, 1.4120477418299999e+00, 1.1191856752295000e+03 },
};

static const VSOP87Term VSOP87A_URANUS_Y1[659] = {
    { 2.1578963850000001e-02, 0.0000000000000000e+00, 0.0000000000000000e+00 },
    { 7.3922734900000002e-03, 4.4396389093500002e+00, 1.4956319713459999e+02 },
    { 2.3854568499999998e-03, 3.7688249314500002e+00, 7.3297125859000005e+01 },
    { 2.2939642399999998e-03, 9.1090183978000006e-01, 7.6266071275599998e+01 },
    { 1.1013711100000001e-03, 4.0084444161599997e+00, 1.1045700263900001e+01 },
    { 9.4979053999999995e-04, 5.0714153706599996e+00, 6.3735898303399999e+01 },
    { 8.1474162999999999e-04, 5.9227536710599997e+00, 8.5827298831199997e+01 },
    { 5.1366973999999999e-04, 6.1844114993999999e-01, 2.2434479570190001e+02 },
    { 4.5457173999999999e-04, 7.3292241206999997e-01, 1.3851749687070000e+02 },
    { 3.8296005000000002e-04, 5.0187357867099998e+00, 7.0849445304200003e+01 },
    { 3.6146116000000000e-04, 5.9485945278700001e+00, 7.8713751830400000e+01 },
    { 3.2420558000000000e-04, 4.3261727173200004e+00, 7.4781598567299994e+01 },
    { 2.1673269000000001e-04, 3.3660726352200001e+00, 1.5104766984290001e+02 },
    { 1.9425087000000000e-04, 6.0184218778299998e+00, 7.7750543983900002e+01 },
    { 1.7393205999999999e-04, 4.9609889548800004e+00, 7.1812653150700001e+01 },
    { 1.4991169000000000e-04, 3.9717685675799999e+00, 3.9321532631000000e+00 },
    { 7.7323669999999998e-05, 4.2566301219999998e-02, 7.1600204829600003e+01 },
    { 7.5956360000000001e-05, 1.1980764348699999e+00, 1.4807872442630000e+02 },
    { 7.4384919999999995e-05, 4.6316543647800001e+00, 7.7962992305000000e+01 },
    { 7.3215589999999997e-05, 6.6348425538000000e-01, 2.9689454166000000e+00 },
    { 6.9792379999999998e-05, 8.3723520790999995e-01, 1.4563104387150000e+02 },
    { 4.3768240000000002e-05, 2.1718272401599998e+00, 1.6060889739850001e+02 },
    { 4.1172020000000000e-05, 6.1993161279000004e+00, 2.2091400527800001e+01 },
    { 3.9625680000000001e-05, 3.1804271182399999e+00, 6.5220371011699996e+01 },
    { 3.1746170000000002e-05, 3.0813163883799999e+00, 2.9912639426919998e+02 },
    { 2.8303129999999999e-05, 3.3436522227799998e+00, 2.1329909543799999e+02 },
    { 2.8212060000000001e-05, 1.8313001094700001e+00, 8.4342826122899993e+01 },
    { 2.6900650000000001e-05, 1.7894647119800000e+00, 9.5612275556000004e+00 },
    { 2.6281589999999999e-05, 2.9745906739899999e+00, 1.2530172972200001e+01 },
    { 2.5106930000000001e-05, 1.2716656185399999e+00, 1.8159247264699999e+01 },
    { 2.2275100000000000e-05, 4.8940743705500003e+00, 8.7311771539500000e+01 },
    { 2.2246550000000001e-05, 3.2154110879800002e+00, 7.2333918012500007e+01 },
    { 1.9296619999999999e-05, 4.6838396207899997e+00, 5.2690198039499997e+01 },
    { 1.9269259999999999e-05, 8.2821252139000001e-01, 3.9617508346100003e+01 },
    { 1.7445640000000001e-05, 1.6789718508400000e+00, 7.7229279122099996e+01 },
    { 1.7390190000000001e-05, 3.8245208629200000e+00, 1.4659425171800001e+02 },
    { 1.7139450000000001e-05, 1.8233497525800000e+00, 2.2286032299359999e+02 },
    { 1.7076060000000001e-05, 5.6041526060900004e+00, 1.2747179660680000e+02 },
    { 1.7051840000000001e-05, 5.8253291761100003e+00, 2.2582926841020000e+02 },
    { 1.6208190000000001e-05, 2.1867650538599999e+00, 1.5349535039770001e+02 },
    { 1.5957029999999999e-05, 4.7955805786000001e+00, 7.9235016692200006e+01 },
    { 1.5556109999999999e-05, 2.5668195482299998e+00, 7.3818390720799997e+01 },
    { 1.5313549999999998e-05, 2.1470151940700002e+00, 1.5253214255120000e+02 },
    { 1.4765499999999999e-05, 2.1011145953899999e+00, 7.5744806413800006e+01 },
    { 1.4585260000000000e-05, 5.1014712640399997e+00, 3.1813937377000001e+00 },
    { 1.2917530000000000e-05, 6.1966624354500004e+00, 7.0328180442399997e+01 },
    { 1.2349139999999999e-05, 5.6923988983099996e+00, 4.4534181248999998e+00 },
    { 1.1799660000000000e-05, 3.5680712605500000e+00, 2.2041264243879999e+02 },
    { 1.1145230000000000e-05, 3.4130466236900001e+00, 1.3140394986990000e+02 },
    { 1.0751080000000000e-05, 1.7628645203400000e+00, 5.6622351302600002e+01 },
    { 1.0501909999999999e-05, 3.8417687934700000e+00, 1.0994568878849999e+02 },
    { 1.0356610000000001e-05, 6.1264256870800002e+00, 6.2251425595100002e+01 },
    { 9.7867500000000008e-06, 5.2320223195500004e+00, 7.4893473151899997e+01 },
    { 9.7712299999999993e-06, 5.7308654053300003e+00, 7.4669723982700006e+01 },
    { 8.8893699999999996e-06, 2.7013235052700000e+00, 4.2659819087599999e+02 },
    { 7.6455600000000007e-06, 1.1734112006299999e+00, 1.4844727083000000e+00 },
    { 7.6242099999999996e-06, 1.6786431404700000e+00, 7.1135470007999997e+00 },
    { 7.5222700000000004e-06, 4.3668622900500003e+00, 2.0618554843720000e+02 },
    { 7.4623100000000004e-06, 3.2137556011699999e+00, 9.2940845831999994e+01 },
    { 6.6547300000000004e-06, 1.9273240900899999e+00, 9.6872999095099999e+01 },
    { 6.4926299999999999e-06, 4.9162163568399997e+00, 8.0198224538700003e+01 },
    { 6.0505199999999996e-06, 4.8200316809599997e+00, 1.0593819301891999e+03 },
    { 5.7620000000000001e-06, 8.5782625562000003e-01, 5.2257741809380002e+02 },
    { 5.6229099999999998e-06, 3.4610225486000001e-01, 5.3680451209540001e+02 },
    { 5.1450400000000001e-06, 2.0197695266100002e+00, 2.4476805547999998e+00 },
    { 4.9012300000000002e-06, 1.9346228388300000e+00, 1.3703302416240001e+02 },
    { 4.7748999999999998e-06, 5.8697975769999999e+00, 4.1101981054399999e+01 },
    { 4.3000800000000000e-06, 5.7637105835500000e+00, 6.7668051566499997e+01 },
    { 4.1348400000000000e-06, 2.2177645371299999e+00, 6.6704843720000000e+01 },
    { 4.0893199999999997e-06, 5.0502921272299997e+00, 8.1895145568100006e+01 },
    { 3.9096499999999999e-06, 1.6927222289999998e-02, 1.4977853527000001e+01 },
    { 3.4720800000000001e-06, 2.5266048072700000e+00, 1.4638180339690001e+02 },
    { 3.4316299999999999e-06, 2.8483236141699999e+00, 1.4244965013379999e+02 },
    { 3.2605800000000000e-06, 4.2707906779800000e+00, 1.1291463420510000e+02 },
    { 2.9592699999999998e-06, 5.2179699810500004e+00, 1.4510977900969999e+02 },
    { 2.8866800000000001e-06, 2.8441130186299999e+00, 5.4166259714000002e+00 },
    { 2.8388799999999999e-06, 2.5151627225900000e+00, 8.2858353414600003e+01 },
    { 2.8168800000000000e-06, 2.6260977971599999e+00, 3.3679617512900002e+01 },
    { 2.6203799999999998e-06, 2.3320262166700001e+00, 2.6598929347749998e+02 },
    { 2.5470299999999999e-06, 4.6829968858999997e+00, 2.3539049596580000e+02 },
    { 2.4661500000000001e-06, 3.7497406908799999e+00, 1.5912442469019999e+02 },
    { 2.4512599999999999e-06, 3.0949333026599999e+00, 2.0225339517410001e+02 },
    { 2.3873099999999998e-06, 4.4811611293200002e+00, 1.9513984817330001e+02 },
    { 2.3545999999999999e-06, 2.0652415036199998e+00, 2.0606927819500001e+01 },
    { 2.3389900000000001e-06, 5.1035302841799997e+00, 1.0846121608020000e+02 },
    { 2.2806199999999999e-06, 4.1674137707999996e+00, 2.9764192156090002e+02 },
    { 2.1741800000000002e-06, 2.6808908031900001e+00, 1.5274459087229999e+02 },
    { 2.1250899999999999e-06, 3.8524858331399998e+00, 2.0936694217490000e+02 },
    { 2.1238599999999999e-06, 5.0246377589000002e-01, 1.9120769491019999e+02 },
    { 2.0957600000000001e-06, 5.6985322872999999e-01, 3.6648562929500002e+01 },
    { 2.0780100000000001e-06, 3.0822409201099998e+00, 1.4000196957899999e+02 },
    { 1.9227299999999998e-06, 5.1722447905199997e+00, 2.3575873236100001e+01 },
    { 1.9208200000000000e-06, 5.5438913820400000e+00, 3.7390799283650000e+02 },
    { 1.7088700000000001e-06, 2.8749967349299999e+00, 1.2895626931510000e+02 },
    { 1.6621400000000000e-06, 4.4605730861300001e+00, 5.9378908332000000e+00 },
    { 1.5418300000000000e-06, 1.6818424579000000e+00, 1.4014645680499999e+01 },
    { 1.5369000000000001e-06, 5.6278823369500000e+00, 2.8808069400530002e+02 },
    { 1.5343300000000001e-06, 1.1289990321900001e+00, 1.6209337010679999e+02 },
    { 1.5131500000000001e-06, 2.2394441384600001e+00, 1.2035824960600000e+02 },
    { 1.4804799999999999e-06, 1.4477035869499999e+00, 1.8621176006409999e+02 },
    { 1.4360200000000000e-06, 4.2242238828399996e+00, 2.1181462272970001e+02 },
    { 1.4194300000000001e-06, 3.1557785340900000e+00, 1.1439910691340000e+02 },
    { 1.4078100000000000e-06, 3.7836568299100000e+00, 2.9204947528600002e+01 },
    { 1.4027999999999999e-06, 3.5884625633099998e+00, 8.0719489400499995e+01 },
    { 1.3410400000000000e-06, 5.1598344119400004e+00, 3.9356875915200000e+01 },
    { 1.3300500000000001e-06, 1.5057298592299999e+00, 5.4174670747800000e+01 },
    { 1.3297599999999999e-06, 5.8815776037600003e+00, 6.8561823443799994e+01 },
    { 1.3295299999999999e-06, 6.0256848157200000e+00, 6.9364972595899999e+01 },
    { 1.3266500000000000e-06, 9.2310657218000003e-01, 4.5576651038700000e+01 },
    { 1.3236600000000001e-06, 2.3604631184199998e+00, 8.9759452094300002e+01 },
    { 1.2967900000000000e-06, 5.1119586844600002e+00, 2.7703499374140000e+02 },
    { 1.2291100000000001e-06, 2.0010827089999998e+00, 3.0061086697750000e+02 },
    { 1.2138800000000000e-06, 2.5570876890699998e+00, 3.0534616939270001e+02 },
    { 1.2104200000000000e-06, 9.4635099828000002e-01, 1.5401661525950001e+02 },
    { 1.2018000000000001e-06, 1.3115665192799999e+00, 5.0402576179100002e+01 },
    { 1.1627900000000001e-06, 4.5918397642900004e+00, 2.2731374111849999e+02 },
    { 1.1616400000000001e-06, 1.9363392053799999e+00, 3.3137100791700000e+01 },
    { 1.1317100000000000e-06, 2.9218856017600001e+00, 4.1644497775600001e+01 },
    { 1.0965800000000001e-06, 3.6585808385599998e+00, 8.8796244247800004e+01 },
    { 1.0328699999999999e-06, 2.1411546381700002e+00, 1.4414657116320001e+02 },
    { 1.0273999999999999e-06, 3.7768264735399999e+00, 1.5201087768939999e+02 },
    { 9.9825999999999999e-07, 4.3050197268700003e+00, 1.1642609634290000e+02 },
    { 9.9168999999999990e-07, 5.0576807334799998e+00, 8.1001373690799994e+01 },
    { 9.7949000000000004e-07, 1.1374521600200000e+00, 6.8843707734099993e+01 },
    { 9.5178999999999998e-07, 5.7714317904500003e+00, 1.1020632121940000e+02 },
    { 8.8573999999999999e-07, 5.7277589333599996e+00, 5.1205725331200000e+01 },
    { 8.6171999999999996e-07, 3.2424018706900002e+00, 4.1948464387519999e+02 },
    { 8.4751000000000000e-07, 4.6394332109800001e+00, 4.2586453762700003e+01 },
    { 8.2216999999999996e-07, 3.9019965688700000e+00, 3.2195144804599998e+01 },
    { 8.1762000000000004e-07, 4.6993100305300004e+00, 2.2827694896500000e+02 },
    { 7.8217000000000002e-07, 5.9931004742400003e+00, 6.2197751234999998e+00 },
    { 7.4820999999999995e-07, 3.5383184428999997e-01, 2.3056457082540001e+02 },
    { 7.4140999999999997e-07, 4.5557181765700001e+00, 1.5052640498110000e+02 },
    { 7.4096000000000004e-07, 4.9699897430700002e+00, 1.4859998928810001e+02 },
    { 7.3664999999999997e-07, 1.9443687303399999e+00, 6.3989728631399998e+02 },
    { 7.3369000000000000e-07, 3.7861605445600000e+00, 9.5388526386799995e+01 },
    { 7.2887000000000004e-07, 1.0935502985400001e+00, 1.8324281464750001e+02 },
    { 7.2486999999999995e-07, 3.9859619729600002e+00, 4.0450679034820001e+02 },
    { 7.0370000000000003e-07, 3.3340917938799999e+00, 2.4379022388199999e+01 },
    { 6.9240999999999999e-07, 6.0941044209299999e+00, 3.3365734504400001e+02 },
    { 6.7805999999999997e-07, 4.9642450169599996e+00, 1.6674774556399999e+01 },
    { 6.6071000000000004e-07, 3.2455062094699998e+00, 2.8571808082200000e+01 },
    { 6.5371000000000005e-07, 3.6077040999999999e-01, 1.2598732389849999e+02 },
    { 6.0712000000000000e-07, 4.2765991837000001e+00, 7.0115732121299999e+01 },
    { 6.0413000000000005e-07, 1.3679981598299999e+00, 1.4967507171919999e+02 },
    { 5.9955000000000004e-07, 5.6109000751100000e+00, 3.4788443904560000e+02 },
    { 5.8100000000000003e-07, 1.5985888800000000e-01, 1.9643719973000000e+01 },
    { 5.7619999999999998e-07, 3.5537222727100000e+00, 7.9447465013300004e+01 },
    { 5.6672000000000003e-07, 5.2552901001599999e+00, 7.3084677537900006e+01 },
    { 5.6647999999999997e-07, 2.5750350405900000e+00, 7.6478519596699996e+01 },
    { 5.5713999999999997e-07, 4.8184728332200004e+00, 1.5763995198190000e+02 },
    { 5.3603000000000002e-07, 2.9118009098300002e+00, 4.3371173787679999e+02 },
    { 5.3183999999999998e-07, 2.1040404487700002e+00, 3.2972519178089999e+02 },
    { 5.1503999999999998e-07, 5.9308454839899998e+00, 1.4711551657979999e+02 },
    { 5.1134000000000002e-07, 4.3832278477199997e+00, 1.5578297225809999e+02 },
    { 5.0001000000000004e-07, 5.8443420121200003e+00, 1.3458534360760001e+02 },
    { 4.9915000000000002e-07, 3.7692669442500000e+00, 9.9160620955499994e+01 },
    { 4.9866000000000005e-07, 1.5230509594899999e+00, 6.8189316428300003e+01 },
    { 4.7515000000000002e-07, 2.9595564843900002e+00, 4.1555249061209997e+02 },
    { 4.5536000000000002e-07, 2.6368990950999999e-01, 7.8643065262000000e+00 },
    { 4.5047999999999999e-07, 3.6471173468000000e+00, 5.9803745040300001e+01 },
    { 4.5005999999999999e-07, 2.0164112124000000e+00, 7.4030839041899995e+01 },
    { 4.4433000000000001e-07, 5.9134414709899996e+00, 5.8106824010899999e+01 },
    { 4.3940999999999999e-07, 2.6521029191799999e+00, 7.5532358092699994e+01 },
    { 4.3370000000000003e-07, 1.9146397583300001e+00, 5.2969096509459996e+02 },
    { 4.2110999999999999e-07, 3.1320150674299998e+00, 2.7310284047829998e+02 },
    { 4.2101999999999999e-07, 5.1744746333100000e+00, 1.5890728952838001e+03 },
    { 4.2002000000000002e-07, 2.8934679247599999e+00, 6.6917292041099998e+01 },
    { 4.0256999999999998e-07, 1.1378748626099999e+00, 1.5497982310600000e+02 },
    { 4.0030000000000000e-07, 4.3672721140600004e+00, 2.5887574647669999e+02 },
    { 4.0026000000000001e-07, 3.9159834860999998e-01, 1.8769623277240001e+02 },
    { 3.9695999999999998e-07, 7.7265302422000004e-01, 3.8133035637799999e+01 },
    { 3.7693999999999999e-07, 5.6856881583799996e+00, 1.6772244439930000e+02 },
    { 3.7431999999999998e-07, 4.9249383539300000e+00, 2.5950888592310002e+02 },
    { 3.7429000000000000e-07, 5.5052142163299997e+00, 7.3887826689999997e+01 },
    { 3.7429000000000000e-07, 5.4573735268499997e+00, 7.5675370444600006e+01 },
    { 3.7159000000000001e-07, 1.1057735691899999e+00, 6.0766952886799999e+01 },
    { 3.6679999999999998e-07, 5.9609772195999999e-01, 4.6470422915999997e+01 },
    { 3.5592999999999998e-07, 3.0994582446800001e+00, 8.1373880706300000e+01 },
    { 3.5364999999999999e-07, 1.8514901850300001e+00, 8.2645905093500005e+01 },
    { 3.4620999999999997e-07, 3.2540344701900001e+00, 1.3554855145409999e+02 },
    { 3.3890000000000000e-07, 4.4544315891300004e+00, 1.7165459766239999e+02 },
    { 3.3444000000000002e-07, 2.1533166945600000e+00, 1.4945132254999999e+02 },
    { 3.3158000000000001e-07, 1.0244525386900001e+00, 1.4227094001599999e+01 },
    { 3.2435000000000002e-07, 9.1593232742999997e-01, 3.4077089204480001e+02 },
    { 3.2420000000000000e-07, 1.5516182374200000e+00, 2.2137585028530000e+02 },
    { 3.1959999999999998e-07, 1.5625126834700001e+00, 1.8472728735580000e+02 },
    { 3.1887000000000000e-07, 4.1301567028199999e+00, 1.0309277421860000e+02 },
    { 3.1100000000000002e-07, 8.2257402616999997e-01, 9.8357471803400003e+01 },
    { 3.0708000000000002e-07, 4.9611992847600002e+00, 7.2493976706900000e+01 },
    { 3.0708000000000002e-07, 6.0013884584200001e+00, 7.7069220427700003e+01 },
    { 3.0695999999999999e-07, 4.7000265976300000e+00, 3.5164090221199999e+01 },
    { 3.0675000000000002e-07, 1.7114169955000000e-01, 1.0522683831884001e+03 },
    { 2.9919999999999998e-07, 1.2061778264900000e+00, 6.5922821389999999e+00 },
    { 2.9752999999999999e-07, 1.5004573144100000e+00, 1.2099138905240000e+02 },
    { 2.9326000000000002e-07, 7.7139889103000003e-01, 1.0664954771900000e+03 },
    { 2.8238000000000002e-07, 5.6207767805300000e+00, 2.9519424100610001e+02 },
    { 2.8004000000000001e-07, 1.4488158520900001e+00, 2.1989137757699999e+02 },
    { 2.7533999999999998e-07, 4.0240504241600004e+00, 1.1143016149680000e+02 },
    { 2.7010000000000002e-07, 2.9668533504000000e-01, 1.4362530630140000e+02 },
    { 2.6866999999999998e-07, 5.0777831670199998e+00, 2.1723124870109999e+02 },
    { 2.6736999999999999e-07, 3.1769850992099999e+00, 4.3289029178299998e+01 },
    { 2.6245000000000002e-07, 5.2207311976700002e+00, 9.1456373123700004e+01 },
    { 2.5965999999999998e-07, 1.0327101549700000e+00, 4.7694263193399998e+01 },
    { 2.5235000000000001e-07, 4.6904287149200004e+00, 5.5406998748280000e+02 },
    { 2.4839999999999997e-07, 1.1261250367700000e+00, 2.5272794265500000e+01 },
    { 2.4476999999999999e-07, 1.4523464841900000e+00, 4.7217484191469998e+02 },
    { 2.4285999999999999e-07, 5.6747525370299998e+00, 5.4302428721889999e+02 },
    { 2.4170000000000000e-07, 2.0566753530600002e+00, 4.6209790485100001e+01 },
    { 2.3868000000000002e-07, 4.7166092165500002e+00, 5.7255490749000003e+01 },
    { 2.3863000000000001e-07, 4.3422389773800001e+00, 2.1892816973050000e+02 },
    { 2.3787000000000000e-07, 1.9814851299000000e-01, 2.5494359321360000e+02 },
    { 2.3341000000000000e-07, 2.8580725106600000e+00, 6.1448276442999997e+01 },
    { 2.2896000000000001e-07, 6.0104597433300002e+00, 2.3390602325750001e+02 },
    { 2.2231999999999999e-07, 4.5725273440500001e+00, 2.2752618943959999e+02 },
    { 2.2153000000000000e-07, 3.0236078430500002e+00, 1.9122455111200001e+01 },
    { 2.2058000000000001e-07, 3.5468323380000000e-02, 2.0373786788240000e+02 },
    { 2.1880000000000000e-07, 2.1275781126000002e+00, 3.1639186965660002e+02 },
    { 2.1857999999999999e-07, 3.2386864266700002e+00, 7.4223635415000002e+00 },
    { 2.1183000000000001e-07, 1.0172230019499999e+00, 4.8640193591629998e+02 },
    { 2.1162000000000001e-07, 3.0009257572999998e-01, 3.7242352012819998e+02 },
    { 2.0451000000000000e-07, 1.9890800543700000e+00, 1.1588357962169999e+02 },
    { 2.0433999999999999e-07, 3.6125400583200000e+00, 4.6824268865160002e+02 },
    { 1.9975000000000000e-07, 4.7342136709700000e+00, 9.2047073954699997e+01 },
    { 1.9781000000000000e-07, 9.1026787025000000e-01, 1.9832124191099999e+02 },
    { 1.9763999999999999e-07, 5.0899694443600003e+00, 2.9615744885260000e+02 },
    { 1.9752000000000001e-07, 2.6897486673400000e+00, 4.7928838891549998e+02 },
    { 1.9105999999999999e-07, 2.5699981549699999e+00, 2.0076892246579999e+02 },
    { 1.9100999999999999e-07, 4.0371268989600004e+00, 1.7265475387399999e+01 },
    { 1.9041000000000000e-07, 2.0595605397200001e+00, 8.8114920691600005e+01 },
    { 1.8710000000000000e-07, 4.9921896875999999e-01, 6.9673789136600007e+01 },
    { 1.8336000000000000e-07, 1.6787082294400000e+00, 1.5126011816400000e+02 },
    { 1.8185999999999999e-07, 6.2479011310799999e+00, 9.2307706385600000e+01 },
    { 1.8127999999999999e-07, 2.1874963050299998e+00, 2.8311175651300001e+01 },
    { 1.7915000000000001e-07, 2.2577053741799999e+00, 1.8175834193919999e+02 },
    { 1.7819000000000001e-07, 4.9883475237199999e+00, 2.2116340196420001e+02 },
    { 1.7484000000000000e-07, 1.9607349984000000e+00, 1.3333322124300000e+01 },
    { 1.7415999999999999e-07, 2.6220605410000002e+00, 1.8409414790939999e+02 },
    { 1.7344000000000000e-07, 3.9215060735299998e+00, 2.6099335863139999e+02 },
    { 1.7326000000000000e-07, 2.0103840298300000e+00, 1.0038446123289999e+02 },
    { 1.7183000000000000e-07, 5.9266073725900004e+00, 3.9739324334740002e+02 },
    { 1.7081999999999999e-07, 4.1801837521799996e+00, 7.9889407997999996e+01 },
    { 1.6971000000000000e-07, 3.2345770561999998e-01, 1.3288842257819999e+02 },
    { 1.6745000000000001e-07, 5.4108665569700003e+00, 2.1033015002139999e+02 },
    { 1.6506000000000000e-07, 1.5566838424700000e+00, 7.6426129970000005e+01 },
    { 1.6506000000000000e-07, 3.1227185935300001e+00, 7.3137067164599998e+01 },
    { 1.6332999999999999e-07, 2.0864068331099999e+00, 9.4425318540299997e+01 },
    { 1.6315000000000000e-07, 8.7407041749000003e-01, 2.4118389957300000e+01 },
    { 1.6315000000000000e-07, 1.5578026771800000e+00, 1.7394221952280000e+02 },
    { 1.6313000000000001e-07, 5.3451027506799997e+00, 6.0554504565700000e+01 },
    { 1.6296000000000000e-07, 1.8492752908300001e+00, 3.5181659230870002e+02 },
    { 1.6261000000000000e-07, 4.2831784500999998e-01, 6.4486657828800006e+01 },
    { 1.6068000000000001e-07, 4.7944220093000001e-01, 1.0697674337190000e+02 },
    { 1.5851000000000001e-07, 1.5548179878599999e+00, 1.3493380818700000e+01 },
    { 1.5664000000000001e-07, 4.3305895264099998e+00, 7.6025553592000006e+02 },
    { 1.5606000000000001e-07, 3.4593308284500002e+00, 1.3533610313299999e+02 },
    { 1.5562000000000001e-07, 4.8835867429500004e+00, 4.1162033734900001e+02 },
    { 1.5501999999999999e-07, 2.1564499469800000e+00, 3.0056280790500001e+01 },
    { 1.5313000000000000e-07, 5.1626031973500002e+00, 1.6462326235300001e+01 },
    { 1.5298000000000001e-07, 6.1105015065500004e+00, 5.5659143456099997e+01 },
    { 1.4861000000000001e-07, 3.0798785411199998e+00, 9.8899988524600005e+01 },
    { 1.4562000000000000e-07, 1.5430924427999999e+00, 5.0663208609999998e+01 },
    { 1.4352000000000000e-07, 5.1413270619100002e+00, 3.0710672096300001e+01 },
    { 1.4296999999999999e-07, 8.5832234274999997e-01, 3.1017209453309999e+02 },
    { 1.4149000000000001e-07, 5.5502939857000000e-01, 2.8659622129700000e+02 },
    { 1.3983000000000000e-07, 5.4426905719999996e-01, 8.0767548473000002e+00 },
    { 1.3953999999999999e-07, 4.8532331849999997e+00, 1.4169889060840001e+02 },
    { 1.3850999999999999e-07, 3.4076142028500001e+00, 4.4070926471000000e+01 },
    { 1.3832000000000001e-07, 3.7941108026700001e+00, 3.1652628083400000e+01 },
    { 1.3782000000000000e-07, 2.6679491575099998e+00, 2.6023553790899999e+01 },
    { 1.3682999999999999e-07, 2.3514244314699999e+00, 8.2203962108799999e+01 },
    { 1.3631999999999999e-07, 4.2595863838600003e+00, 6.7880499887599996e+01 },
    { 1.3526999999999999e-07, 4.2509930473799997e+00, 8.5076539305799997e+01 },
    { 1.3286000000000001e-07, 1.4377661443800001e+00, 1.0627416795630000e+02 },
    { 1.3285000000000000e-07, 1.2511525559000001e+00, 6.9010986796999996e+00 },
    { 1.2991999999999999e-07, 5.4663495506200004e+00, 1.7368158709190001e+02 },
    { 1.2989999999999999e-07, 4.6303966997200003e+00, 4.5490936652729999e+02 },
    { 1.2917000000000001e-07, 2.8995051592299999e+00, 5.5137878594299998e+01 },
    { 1.2734000000000000e-07, 1.3129280359000001e-01, 7.4260333705500003e+01 },
    { 1.2734000000000000e-07, 1.4065169788200000e+00, 7.5302863429100000e+01 },
    { 1.2700000000000001e-07, 1.3947590032000001e-01, 5.7516123179899999e+01 },
    { 1.2678000000000000e-07, 3.6417670366500001e+00, 8.1682697246999993e+01 },
    { 1.2664000000000000e-07, 3.5218470607499999e+00, 2.1648048917570000e+02 },
    { 1.2513000000000001e-07, 3.5010629107500000e+00, 1.2429040286910001e+02 },
    { 1.2499000000000001e-07, 1.9514494663000001e-01, 3.2243328914400003e+01 },
    { 1.2477000000000000e-07, 1.0290235327399999e+00, 5.1546387109299997e+02 },
    { 1.2403999999999999e-07, 2.5670643736900001e+00, 4.2538269652899999e+01 },
    { 1.2328999999999999e-07, 3.1142801968799998e+00, 2.6450482076920002e+02 },
    { 1.2080000000000000e-07, 3.5933619109400001e+00, 7.4127207261500004e+01 },
    { 1.2080000000000000e-07, 1.0860405250600000e+00, 7.5435989873099999e+01 },
    { 1.2006999999999999e-07, 1.5721778880499999e+00, 1.3044074202339999e+02 },
    { 1.1969999999999999e-07, 1.4614844861600000e+00, 1.8972322220190000e+02 },
    { 1.1836000000000000e-07, 4.5881291331399998e+00, 6.3278373931320004e+02 },
    { 1.1022000000000000e-07, 1.8872299794400000e+00, 6.8547393735269998e+02 },
    { 1.0954000000000000e-07, 5.2762450712900000e+00, 8.9008692568900003e+01 },
    { 1.0739000000000000e-07, 4.9584411828899997e+00, 9.3904053678500006e+01 },
    { 1.0694000000000000e-07, 5.8650383406200000e+00, 1.5615547927360001e+02 },
    { 1.0638000000000000e-07, 2.3995874846800000e+00, 6.7359235025800004e+01 },
    { 1.0526000000000000e-07, 6.2653745460700003e+00, 6.9258748435350003e+02 },
    { 1.0514000000000000e-07, 2.9339072082800000e+00, 3.1492569389000000e+01 },
    { 1.0459000000000000e-07, 3.5511168847499999e+00, 1.4489733068859999e+02 },
    { 1.0412000000000000e-07, 4.5358102570799996e+00, 1.9907200143640000e+02 },
    { 1.0335000000000000e-07, 4.5767100484299998e+00, 2.6747376618580000e+02 },
    { 1.0244999999999999e-07, 2.7139152736600001e+00, 6.2491943278700001e+02 },
    { 1.0209000000000000e-07, 2.3741310698400002e+00, 2.1478356814630001e+02 },
    { 1.0195999999999999e-07, 5.9794869569399998e+00, 5.3287235883230005e+02 },
    { 1.0123000000000000e-07, 3.6149168802200000e+00, 2.3687496867409999e+02 },
    { 1.0052000000000000e-07, 1.9389799358199999e+00, 4.9508804301799998e+01 },
    { 1.0035000000000000e-07, 1.7260668734200000e+00, 4.4868959140380002e+02 },
    { 9.9999999999999995e-08, 1.0366833512400000e+00, 4.3128970483899998e+01 },
    { 9.9099999999999994e-08, 4.7476543353300000e+00, 1.2991947716160001e+02 },
    { 9.8900000000000005e-08, 1.8442522635100000e+00, 3.9346109008430000e+02 },
    { 9.8309999999999998e-08, 1.2605032637600000e+00, 5.4334729442200000e+01 },
    { 9.7390000000000001e-08, 3.7107855598400001e+00, 7.3185251274400002e+01 },
    { 9.7390000000000001e-08, 4.1102095297499996e+00, 7.6377945860200001e+01 },
    { 9.5640000000000001e-08, 4.7531848338999998e+00, 2.7087335373900000e+01 },
    { 9.3989999999999997e-08, 3.6630405976000002e-01, 1.7526107818300002e+01 },
    { 9.0909999999999998e-08, 4.6663625810999996e+00, 1.8016199464630000e+02 },
    { 9.0600000000000004e-08, 5.6020049962799998e+00, 1.9365537546499999e+02 },
    { 9.0489999999999998e-08, 1.6202915623200000e+00, 1.1791056905120000e+02 },
    { 8.9879999999999995e-08, 4.1608584208000003e+00, 6.1780588578619995e+02 },
    { 8.9830000000000004e-08, 5.5049594553699999e+00, 2.8414854074220000e+02 },
    { 8.9200000000000005e-08, 4.9395414987199997e+00, 2.0788246946660001e+02 },
    { 8.8450000000000001e-08, 3.5661367243600002e+00, 8.8274979385999998e+01 },
    { 8.7429999999999996e-08, 2.4358648631800000e+00, 1.1950691634410001e+02 },
    { 8.7410000000000000e-08, 3.2155880471399998e+00, 5.8739963457300000e+01 },
    { 8.5829999999999995e-08, 3.3897210496499999e+00, 1.8027386923090000e+02 },
    { 8.5739999999999999e-08, 2.9847821906099998e+00, 1.3606981631590000e+02 },
    { 8.5710000000000004e-08, 2.5289077208999999e-01, 4.4725317776799997e+01 },
    { 8.5339999999999995e-08, 1.2199920739100001e+00, 6.1288217748599997e+01 },
    { 8.5209999999999993e-08, 6.0186192279800004e+00, 1.5550108796780000e+02 },
    { 8.4940000000000005e-08, 5.7590679288900004e+00, 1.0398654609590000e+02 },
    { 8.4019999999999994e-08, 3.7325741799999999e-02, 7.9516900982500005e+01 },
    { 8.4019999999999994e-08, 4.6420766942000000e+00, 7.0046296152099998e+01 },
    { 8.2380000000000000e-08, 4.1746686216300004e+00, 4.0160025067299998e+01 },
    { 8.1870000000000005e-08, 1.1306038317600000e+00, 2.3767811782620001e+02 },
    { 8.1680000000000001e-08, 2.2449118725000000e-01, 5.4391805909619995e+02 },
    { 8.1569999999999995e-08, 6.1251948260300004e+00, 1.6357784281510001e+02 },
    { 7.8909999999999997e-08, 5.1557087700700004e+00, 4.7061123747000003e+01 },
    { 7.7140000000000001e-08, 5.3984155415500004e+00, 6.0676018552230005e+02 },
    { 7.4639999999999999e-08, 4.0683226348800003e+00, 1.7516605980020000e+02 },
    { 7.4110000000000007e-08, 6.0102995875499996e+00, 7.5632338265689998e+02 },
    { 7.1649999999999996e-08, 2.0283141508000000e+00, 1.0749800823370001e+02 },
    { 7.1089999999999997e-08, 9.5985317203999998e-01, 6.8154178408960001e+02 },
    { 7.0860000000000001e-08, 3.4907168874000000e-01, 1.7787437278589999e+02 },
    { 7.0679999999999995e-08, 5.1537468828600002e+00, 1.4148644228730001e+02 },
    { 7.0449999999999999e-08, 1.7430021273300000e+00, 5.2650957135689998e+02 },
    { 7.0319999999999997e-08, 4.6263512213200002e+00, 9.4917560896980001e+02 },
    { 6.9469999999999999e-08, 1.4453916711399999e+00, 3.6286229257259998e+02 },
    { 6.8369999999999996e-08, 1.5692886100400001e+00, 5.3591074021810005e+02 },
    { 6.8369999999999996e-08, 4.4907793058300003e+00, 3.7539246554480002e+02 },
    { 6.8210000000000000e-08, 3.5140445468100001e+00, 1.4266209845489999e+02 },
    { 6.8029999999999994e-08, 3.4189038439099999e+00, 9.5228467692400002e+01 },
    { 6.7799999999999998e-08, 5.5952952288000004e+00, 7.1158261844899997e+01 },
    { 6.7799999999999998e-08, 5.3672925143799999e+00, 7.8404935289700006e+01 },
    { 6.7480000000000006e-08, 4.0962868670799999e+00, 2.4872381809010000e+02 },
    { 6.7440000000000000e-08, 5.7930920015299998e+00, 2.7720474820300002e+01 },
    { 6.7290000000000002e-08, 3.0990216922000002e-01, 5.5013783421970004e+02 },
    { 6.6460000000000001e-08, 3.1302964415400001e+00, 3.5685355082999997e+01 },
    { 6.6390000000000001e-08, 5.5618575631900002e+00, 3.4530950774799997e+01 },
    { 6.5589999999999994e-08, 4.2536966749399996e+00, 1.1731986822019999e+02 },
    { 6.5140000000000000e-08, 7.8441227240000000e-01, 3.0209533968580001e+02 },
    { 6.5050000000000003e-08, 5.7365356774300000e+00, 1.2184272231430000e+02 },
    { 6.5009999999999998e-08, 1.0650212446800000e+00, 4.9721252622900003e+01 },
    { 6.4249999999999996e-08, 5.9368343578099996e+00, 2.1085141488319999e+02 },
    { 6.3889999999999998e-08, 5.3958601192400000e+00, 1.1494162363460001e+02 },
    { 6.3500000000000006e-08, 1.4637096320900000e+00, 9.0823233677299996e+01 },
    { 6.3460000000000000e-08, 1.1467769306000000e+00, 9.1243924802600006e+01 },
    { 6.3430000000000006e-08, 3.5313554098500002e+00, 1.1887377689770000e+02 },
    { 6.3189999999999999e-08, 2.0063198817000001e+00, 1.4334342201109999e+02 },
    { 6.3139999999999995e-08, 6.9574142677999995e-01, 1.0931254934210000e+02 },
    { 6.2999999999999995e-08, 3.2268432413500001e+00, 2.8096714700450002e+02 },
    { 6.2289999999999997e-08, 4.7146143374399996e+00, 4.0302231763990000e+02 },
    { 6.1889999999999994e-08, 5.1152879779099996e+00, 3.8109097580650001e+02 },
    { 6.1889999999999994e-08, 3.9706886118700000e+00, 5.2872775724810003e+02 },
    { 6.1599999999999996e-08, 2.2747128990699998e+00, 1.6682867252200001e+02 },
    { 6.1350000000000003e-08, 6.2776793500499997e+00, 1.9662432088160000e+02 },
    { 6.0829999999999996e-08, 3.5053408012100000e+00, 5.3065417294110000e+02 },
    { 6.0829999999999996e-08, 5.5806353054500004e+00, 3.7916456011349999e+02 },
    { 6.0730000000000002e-08, 3.6592469330000002e-02, 2.9226199388000001e+01 },
    { 6.0339999999999996e-08, 3.2868260994399998e+00, 2.0439822618111000e+03 },
    { 5.8700000000000003e-08, 5.6804085295500002e+00, 4.0610313764109998e+02 },
    { 5.8640000000000001e-08, 4.0338174207600002e+00, 1.0791869935900000e+02 },
    { 5.8339999999999998e-08, 6.2650496950600001e+00, 4.0599126305649997e+02 },
    { 5.7860000000000003e-08, 2.0625381622900001e+00, 2.2633917249000000e+01 },
    { 5.7460000000000000e-08, 2.2097133034300001e+00, 4.4092178330400003e+01 },
    { 5.7439999999999997e-08, 3.4409917146400000e+00, 6.1069233878540001e+02 },
    { 5.7310000000000002e-08, 4.1860267790499996e+00, 2.3449672408850000e+02 },
    { 5.7200000000000003e-08, 2.5848416370899998e+00, 3.1957326339430000e+02 },
    { 5.6419999999999998e-08, 1.3289820763200000e+00, 8.5319638175199998e+02 },
    { 5.6209999999999998e-08, 2.1105822548000000e-01, 5.2575881183150000e+02 },
    { 5.5379999999999997e-08, 3.8319040853900002e+00, 1.2518417474640000e+02 },
    { 5.5320000000000002e-08, 5.0312786735100001e+00, 1.6454105066160000e+02 },
    { 5.5140000000000003e-08, 5.1259120767999999e-01, 2.3242155054919999e+02 },
    { 5.4410000000000002e-08, 8.3308003167000000e-01, 3.1231936958100000e+01 },
    { 5.4329999999999997e-08, 3.9575551398100002e+00, 4.0843894361129998e+02 },
    { 5.4249999999999999e-08, 4.8761956768000001e+00, 7.6154196690999996e+01 },
    { 5.4249999999999999e-08, 2.9447994127900001e+00, 7.3409000443599993e+01 },
    { 5.4170000000000001e-08, 1.8050671509600000e+00, 1.6041635110000001e+01 },
    { 5.4009999999999999e-08, 3.7618322325700002e+00, 1.9269216761850001e+02 },
    { 5.3839999999999998e-08, 2.2417110061300001e+00, 3.1245971639350000e+02 },
    { 5.3809999999999997e-08, 4.0789208640899997e+00, 3.2261164478009999e+02 },
    { 5.3790000000000001e-08, 5.8718585994600003e+00, 1.6289651925890001e+02 },
    { 5.3680000000000002e-08, 8.9833267567999997e-01, 2.9370976829780000e+02 },
    { 5.3589999999999999e-08, 3.3498697829899999e+00, 3.7694637422229999e+02 },
    { 5.3400000000000002e-08, 4.1373323217000002e-01, 8.4215068148809996e+02 },
    { 5.3349999999999998e-08, 5.6389977118800001e+00, 2.3204904353370000e+02 },
    { 5.2579999999999998e-08, 2.8036298846499998e+00, 4.9178735901700001e+01 },
    { 5.2369999999999998e-08, 5.4386840592299999e+00, 2.5602862665600000e+01 },
    { 5.1760000000000002e-08, 8.5326249505999996e-01, 3.4621573499999997e+01 },
    { 5.1679999999999997e-08, 1.4057863269799999e+00, 1.4786627610520000e+02 },
    { 5.1510000000000003e-08, 6.0497714005700001e+00, 3.0141401612959999e+02 },
    { 5.1270000000000002e-08, 4.7049927138300003e+00, 2.1011770170029999e+02 },
    { 5.1249999999999999e-08, 5.1338841327400004e+00, 4.4068227252570000e+02 },
    { 5.0950000000000003e-08, 1.9201383737300000e+00, 1.0702492748170000e+02 },
    { 5.0290000000000002e-08, 5.5723389362200004e+00, 1.8918070548070000e+02 },
    { 4.9180000000000001e-08, 7.9029088220999999e-01, 8.2792358748649997e+02 },
    { 4.8009999999999998e-08, 3.5306969236699999e+00, 4.2266603761290003e+02 },
    { 4.7169999999999998e-08, 5.4487840057000003e+00, 9.5979227217800002e+01 },
    { 4.6660000000000002e-08, 1.4766850724500000e+00, 2.6992144674060000e+02 },
    { 4.6639999999999999e-08, 2.7183201574000000e+00, 1.1413847448250000e+02 },
    { 4.6579999999999998e-08, 2.6781368052299999e+00, 4.7069036920640002e+02 },
    { 4.6490000000000001e-08, 3.5873648654000001e+00, 2.1692243216040001e+02 },
    { 4.6460000000000000e-08, 6.1039952687300003e+00, 6.0447256366190004e+02 },
    { 4.6420000000000001e-08, 5.0538711064699999e+00, 7.4920983565610004e+02 },
    { 4.5699999999999999e-08, 3.9122505243200001e+00, 2.5060345944400002e+01 },
    { 4.5480000000000001e-08, 2.9148254960600002e+00, 9.9564603188579997e+02 },
    { 4.5480000000000001e-08, 8.2682823758000001e-01, 2.2345102382459999e+02 },
    { 4.5450000000000000e-08, 3.1133604475999999e-01, 5.2147681318300002e+01 },
    { 4.5069999999999999e-08, 3.1104187055299999e+00, 1.0005439283280000e+02 },
    { 4.4850000000000001e-08, 1.1188806655500001e+00, 3.0598797511699999e+01 },
    { 4.4820000000000000e-08, 3.9983200185699999e+00, 2.1197628650500000e+01 },
    { 4.4610000000000000e-08, 5.5173105286200004e+00, 3.0237722397610003e+02 },
    { 4.4470000000000000e-08, 3.4791196255700001e+00, 1.0186893394120000e+02 },
    { 4.4010000000000002e-08, 3.0097714991400002e+00, 4.8788640862459999e+02 },
    { 4.3989999999999999e-08, 2.4032563297100000e+00, 1.0335340664950000e+02 },
    { 4.3670000000000000e-08, 4.3310008629699999e+00, 5.9587454395279997e+02 },
    { 4.3439999999999997e-08, 2.4917558569699998e+00, 4.9651816653700003e+01 },
    { 4.3140000000000001e-08, 3.8133546124700000e+00, 6.1010163795439996e+02 },
    { 4.3100000000000002e-08, 2.9924011840600002e+00, 1.2247586176070000e+02 },
    { 4.2900000000000000e-08, 4.8537532098999998e+00, 6.6183578858199994e+01 },
    { 4.2879999999999998e-08, 3.2355339877299998e+00, 2.9823262239190001e+02 },
    { 4.2790000000000001e-08, 4.5645758862500001e+00, 1.7878939652259999e+02 },
    { 4.2729999999999999e-08, 1.6629755769500001e+00, 8.3110498122419995e+02 },
    { 4.2510000000000001e-08, 2.0715417590400000e+00, 4.8491746320800002e+02 },
    { 4.2510000000000001e-08, 2.8974257642799999e+00, 4.9033408917940000e+02 },
    { 4.2459999999999998e-08, 1.6786811942200000e+00, 5.2820649238630006e+02 },
    { 4.2429999999999997e-08, 4.4270540081299998e+00, 7.6343692965770003e+02 },
    { 4.2410000000000001e-08, 2.5741897283999999e+00, 3.3683873878169999e+02 },
    { 4.2370000000000002e-08, 5.4587726896200000e+00, 2.5739127376840003e+02 },
    { 4.2240000000000000e-08, 1.5348265494700000e+00, 5.0137978944330001e+02 },
    { 4.2219999999999997e-08, 3.6534087973000000e+00, 3.3928641933649999e+02 },
    { 4.2069999999999999e-08, 2.3893345502600001e+00, 3.5424722652100002e+01 },
    { 4.1960000000000000e-08, 2.1819676804500001e+00, 2.2189711514710001e+02 },
    { 4.1929999999999999e-08, 3.9448465073000000e-01, 3.8330916169770001e+02 },
    { 4.1719999999999999e-08, 6.3805328566999997e-01, 3.3217287233569999e+02 },
    { 4.1610000000000000e-08, 1.5109247735700000e+00, 4.3549661609200001e+01 },
    { 4.1540000000000000e-08, 6.0327286644200004e+00, 1.7017012495410000e+02 },
    { 4.1530000000000002e-08, 5.4166011587999998e+00, 7.5877106321170004e+02 },
    { 4.1420000000000003e-08, 2.1795676273100000e+00, 6.8865533109039995e+02 },
    { 4.0879999999999999e-08, 4.1495939150599996e+00, 2.7161836777000002e+02 },
    { 4.0870000000000001e-08, 3.9831621517800002e+00, 1.2648167191010000e+03 },
    { 4.0639999999999999e-08, 2.1788560463800000e+00, 4.8758044776399998e+01 },
    { 4.0569999999999999e-08, 1.9297900609700001e+00, 2.2908009811709999e+02 },
    { 4.0560000000000001e-08, 4.2657446152100000e+00, 5.8238116313410001e+02 },
    { 4.0410000000000003e-08, 1.4549613996399999e+00, 3.4225536475310003e+02 },
    { 4.0149999999999999e-08, 5.8518760460300001e+00, 4.4275170057060001e+02 },
    { 4.0070000000000001e-08, 1.2264589846600000e+00, 1.4214083359310001e+02 },
    { 4.0000000000000001e-08, 1.9901387413700000e+00, 1.8873055176757000e+03 },
    { 3.9909999999999998e-08, 5.9955001523600000e+00, 2.2679247625670001e+02 },
    { 3.9650000000000001e-08, 9.7771499557999997e-01, 2.5653994050649999e+02 },
    { 3.9400000000000002e-08, 6.0048804220300003e+00, 7.2876296653099996e+02 },
    { 3.9330000000000002e-08, 1.0712463624399999e+00, 1.8105576652360000e+02 },
    { 3.9209999999999998e-08, 4.1691722003700002e+00, 6.5716276170139997e+02 },
    { 3.9010000000000003e-08, 1.7766413397700000e+00, 1.2450285119020000e+02 },
    { 3.9010000000000003e-08, 7.2185997426000004e-01, 2.0470107572890001e+02 },
    { 3.8689999999999998e-08, 1.4016188548000000e-01, 1.9577298761969999e+02 },
    { 3.8479999999999998e-08, 3.2895770153999998e-01, 3.4639996633729999e+02 },
    { 3.8180000000000002e-08, 2.9415151060700002e+00, 3.2835259365719998e+02 },
    { 3.8059999999999998e-08, 1.2915462603400001e+00, 2.5345912050530001e+02 },
    { 3.7940000000000001e-08, 7.2421690887000001e-01, 5.5555446019110002e+02 },
    { 3.7879999999999999e-08, 2.2803870816999998e+00, 1.0538039607899999e+02 },
    { 3.7730000000000001e-08, 5.6550648880600001e+00, 3.5803636743220000e+02 },
    { 3.7590000000000001e-08, 1.1522543570699999e+00, 2.9318850343600002e+02 },
    { 3.7350000000000000e-08, 1.9369965198500001e+00, 5.9839494557221997e+03 },
    { 3.7340000000000002e-08, 1.7636103696500001e+00, 3.9096243484299997e+01 },
    { 3.7119999999999998e-08, 6.2418604700899998e+00, 9.0684978763799995e+02 },
    { 3.7079999999999999e-08, 5.1202320289800003e+00, 3.5499798604640000e+02 },
    { 3.6950000000000003e-08, 2.7027295802600002e+00, 3.2824071907259997e+02 },
    { 3.6900000000000000e-08, 2.9886359664600000e+00, 2.1951887056149999e+02 },
    { 3.6739999999999997e-08, 1.3791075233600001e+00, 5.4450875992720000e+02 },
    { 3.6690000000000000e-08, 5.3407678399099998e+00, 6.0150361824530000e+02 },
    { 3.6449999999999999e-08, 3.6713994215799999e+00, 3.1097524368519998e+02 },
    { 3.6419999999999998e-08, 5.2959149860999997e+00, 3.3280601178209997e+02 },
    { 3.6260000000000002e-08, 5.6977566654300000e+00, 3.5893013930950002e+02 },
    { 3.6150000000000003e-08, 1.6586836716500000e+00, 1.5667674413540001e+02 },
    { 3.5929999999999998e-08, 1.9299328967400000e+00, 6.4701083331480004e+02 },
    { 3.5920000000000000e-08, 5.1903449667599997e+00, 1.0623508756058000e+03 },
    { 3.5630000000000002e-08, 2.8125551580999999e+00, 8.2005928096030004e+02 },
    { 3.5579999999999998e-08, 1.0331165736400001e+00, 7.8159617692239999e+02 },
    { 3.5539999999999999e-08, 2.0461069581700002e+00, 1.3686602528450001e+03 },
    { 3.5500000000000000e-08, 1.6659982084600000e+00, 3.8012776796000003e+02 },
    { 3.5269999999999998e-08, 4.2427667590400002e+00, 2.9674814968359999e+02 },
    { 3.5259999999999999e-08, 5.3331092374500004e+00, 1.5214048437173001e+03 },
    { 3.5050000000000000e-08, 7.7576683039999997e-02, 5.1887048887399999e+01 },
    { 3.5019999999999999e-08, 3.3604066200900000e+00, 9.9141591519417998e+03 },
    { 3.4980000000000000e-08, 1.3870185349399999e+00, 6.3033605875839999e+02 },
    { 3.4720000000000003e-08, 3.0159988292500000e+00, 6.7442823708879996e+02 },
    { 3.4700000000000000e-08, 2.8133623261600000e+00, 1.0679799498983000e+03 },
    { 3.4620000000000002e-08, 4.2698039399200001e+00, 6.0920786607709999e+02 },
    { 3.4590000000000001e-08, 4.4613422329299999e+00, 5.6511568774670002e+02 },
    { 3.4510000000000003e-08, 4.7116548238100000e+00, 1.5071777497157000e+03 },
    { 3.4410000000000002e-08, 5.1775855080299999e+00, 6.8940609061580005e+02 },
    { 3.4359999999999998e-08, 5.8325268419400000e+00, 7.3587651353180001e+02 },
    { 3.4319999999999999e-08, 1.8061268511499999e+00, 1.8498791978669999e+02 },
    { 3.4289999999999998e-08, 6.2062282923199996e+00, 3.4470304530790003e+02 },
    { 3.4070000000000000e-08, 8.0345297226000001e-01, 7.4527768239299996e+02 },
    { 3.4039999999999999e-08, 1.0743720967499999e+00, 1.1042078217025000e+03 },
    { 3.3979999999999997e-08, 1.3929364369999999e+00, 5.4785021235930003e+02 },
    { 3.3950000000000003e-08, 1.7696562770800000e+00, 5.1153171782990000e+02 },
    { 3.3920000000000002e-08, 3.2924679840900000e+00, 2.9111907539110001e+02 },
    { 3.3909999999999997e-08, 3.6127912229899999e+00, 5.3197858695499997e+02 },
    { 3.3880000000000003e-08, 3.1017348801100000e+00, 1.4710022675381999e+03 },
    { 3.3740000000000003e-08, 1.0279709872700000e+00, 1.0656017053127000e+03 },
    { 3.3680000000000001e-08, 1.2792874639100000e+00, 1.0507839104801001e+03 },
    { 3.3680000000000001e-08, 6.1227420500000003e-03, 2.7555052103309998e+02 },
    { 3.3670000000000003e-08, 5.2218424031200001e+00, 4.7780391620720002e+02 },
    { 3.3659999999999998e-08, 5.9876268345999994e-01, 3.2658308281324998e+03 },
    { 3.3650000000000000e-08, 2.1946174330199999e+00, 2.7851946644970002e+02 },
    { 3.3589999999999998e-08, 1.5138663749200001e+00, 6.9970103135429997e+02 },
    { 3.3589999999999998e-08, 1.7261842202400000e+00, 3.7004527554260000e+02 },
    { 3.3589999999999998e-08, 3.3795583596299998e+00, 2.2879821382680001e+02 },
    { 3.3460000000000003e-08, 6.0110452830299996e+00, 6.7214061522839995e+02 },
    { 3.3449999999999998e-08, 3.8619851219200001e+00, 1.0021837280099400e+04 },
    { 3.3379999999999998e-08, 7.4390697506000003e-01, 8.9971900415400000e+01 },
    { 3.3290000000000002e-08, 5.0825960527199996e+00, 2.6013121543006899e+04 },
    { 3.3279999999999998e-08, 2.3891331189699998e+00, 3.2350541665740002e+02 },
    { 3.3050000000000001e-08, 3.5095888543800000e+00, 4.7365931462300000e+02 },
    { 3.3050000000000001e-08, 5.9936537021399996e+00, 5.3213864564940002e+02 },
    { 3.2920000000000000e-08, 1.8393053813900000e+00, 3.2983706636549999e+02 },
    { 3.2910000000000002e-08, 5.8291595736300001e+00, 2.4241100297711000e+03 },
    { 3.2789999999999998e-08, 2.7890072794899998e+00, 4.8077286162380000e+02 },
    { 3.2759999999999997e-08, 6.1688443381900004e+00, 4.3675011926259998e+02 },
    { 3.2649999999999998e-08, 4.8463727025500001e+00, 3.4200882374700001e+01 },
    { 3.2549999999999997e-08, 1.5538497598800001e+00, 1.1387784205160000e+02 },
    { 3.2520000000000003e-08, 4.4215469640700000e+00, 4.1800017116689997e+02 },
    { 3.2469999999999999e-08, 4.0873257406299999e+00, 2.5642806592189999e+02 },
    { 3.2450000000000003e-08, 2.6493584387100002e+00, 3.0689420236899998e+01 },
    { 3.2220000000000000e-08, 5.3628528769999999e+00, 4.2096911658350001e+02 },
    { 3.2150000000000000e-08, 2.4084597913599999e+00, 2.1840690486870000e+02 },
    { 3.2100000000000003e-08, 2.6920190572900000e+00, 9.1375088631769995e+02 },
    { 3.1989999999999997e-08, 3.9201382815299999e+00, 9.5628915597059995e+02 },
    { 3.1930000000000002e-08, 2.7386267189100000e+00, 3.7471114198859999e+02 },
    { 3.1769999999999999e-08, 4.9086987602600001e+00, 1.9428851491140000e+02 },
    { 3.1750000000000003e-08, 3.9731644169000000e+00, 1.5128068240082000e+03 },
    { 3.1590000000000000e-08, 2.8303325806499999e+00, 1.3364169239306000e+03 },
    { 3.1529999999999998e-08, 4.3446165895100002e+00, 6.9152524274800001e+01 },
    { 3.1230000000000002e-08, 1.1988670258300000e+00, 3.8427236954419999e+02 },
    { 3.1100000000000001e-08, 1.5688105357000000e+00, 6.2640390549530002e+02 },
    { 3.1090000000000002e-08, 4.0624781840999997e+00, 2.4161027108930000e+02 },
    { 3.0939999999999998e-08, 3.2237847486400000e+00, 9.6962247809489998e+02 },
    { 3.0869999999999998e-08, 4.4214789223100004e+00, 5.2523754696970002e+02 },
    { 3.0850000000000002e-08, 3.0094442817100000e+00, 6.2885158605009997e+02 },
    { 3.0780000000000002e-08, 5.2431831050000000e-01, 4.3281796599950002e+02 },
    { 3.0610000000000001e-08, 3.5586776068299999e+00, 1.7377423205411001e+03 },
    { 3.0570000000000002e-08, 6.2150532346700000e+00, 1.7519694145427000e+03 },
    { 3.0250000000000003e-08, 6.6197534140000003e-01, 4.8322054217860000e+02 },
    { 3.0239999999999998e-08, 5.6531866786900000e+00, 1.8125239191083999e+03 },
    { 3.0209999999999997e-08, 2.7398499837500001e+00, 8.3355266177900000e+02 },
    { 3.0199999999999999e-08, 5.1676581656999998e-01, 4.6972716135989998e+02 },
    { 3.0150000000000002e-08, 2.3233606032599998e+00, 1.0537528558967001e+03 },
    { 3.0139999999999997e-08, 2.6571629344000001e+00, 4.2118156490460001e+02 },
    { 2.9810000000000000e-08, 5.4552359896600002e+00, 5.6118353448360006e+02 },
    { 2.9760000000000000e-08, 3.4102688024200001e+00, 8.0410672859800002e+01 },
    { 2.9630000000000001e-08, 4.4605293530000001e-01, 5.4153981451059997e+02 },
    { 2.9620000000000000e-08, 2.8465669425800000e+00, 3.7056654040439997e+02 },
    { 2.9589999999999999e-08, 1.5625500489999999e-01, 1.5499118388799999e+01 },
    { 2.9499999999999999e-08, 3.7286839664000002e+00, 4.0695447090300001e+02 },
    { 2.9490000000000001e-08, 3.4094124403600001e+00, 2.5583736509089999e+02 },
    { 2.9469999999999999e-08, 3.1893394616899999e+00, 3.7619561469690001e+02 },
    { 2.9360000000000000e-08, 1.3069856658600001e+00, 3.4936891175390002e+02 },
    { 2.9210000000000001e-08, 1.4551503337600000e+00, 6.2082942514241004e+03 },
    { 2.9040000000000001e-08, 2.4433165487899999e+00, 9.0280716956099994e+01 },
    { 2.8990000000000000e-08, 9.6731641272000002e-01, 6.2025356634100001e+02 },
    { 2.8979999999999999e-08, 4.6460371975300001e+00, 1.0080515235820000e+02 },
    { 2.8900000000000001e-08, 1.1788720256200000e+00, 5.9491133610630004e+02 },
    { 2.8900000000000001e-08, 1.7439272717800001e+00, 3.6137781986430002e+02 },
    { 2.8900000000000001e-08, 2.2453756595300001e+00, 6.7836039035190004e+02 },
    { 2.8800000000000000e-08, 2.1150915288499998e+00, 3.6997583957339998e+02 },
    { 2.8780000000000000e-08, 4.4639367035399999e+00, 6.0587310542895002e+03 },
    { 2.8679999999999999e-08, 1.1181045700400001e+00, 1.0064509366380000e+02 },
    { 2.8629999999999999e-08, 1.3663925932300001e+00, 6.3744960575920004e+02 },
    { 2.8550000000000001e-08, 4.1469027531500000e+00, 4.8918103470799998e+01 },
    { 2.8450000000000000e-08, 1.7156034536300000e+00, 2.6036021918500001e+02 },
    { 2.8120000000000000e-08, 1.6414222431400001e+00, 2.8266406803389998e+02 },
    { 2.8040000000000002e-08, 4.4967384090000001e-01, 2.1116503133776000e+03 },
    { 2.7960000000000000e-08, 3.1551018581300001e+00, 1.0601353552540000e+02 },
    { 2.7789999999999999e-08, 5.8764961721600004e+00, 5.4695644048199995e+02 },
    { 2.7430000000000001e-08, 6.0500039642500001e+00, 1.2125202148330000e+02 },
    { 2.7290000000000002e-08, 1.3250630812299999e+00, 5.2012973753899996e+02 },
    { 2.7269999999999999e-08, 1.8530789124800000e+00, 1.0650110044817000e+03 },
    { 2.7269999999999999e-08, 6.1790660274100002e+00, 5.9194239068970001e+02 },
    { 2.7240000000000001e-08, 2.7837551260700000e+00, 2.8021638747909998e+02 },
    { 2.7220000000000002e-08, 2.2981889679599998e+00, 9.4753107756710006e+02 },
    { 2.7210000000000000e-08, 8.4302891091999999e-01, 4.6913646052889999e+02 },
    { 2.7170000000000001e-08, 6.0237049381299999e+00, 4.7940026350009998e+02 },
    { 2.7030000000000001e-08, 5.4952434625100004e+00, 1.0400779795530001e+02 },
    { 2.6880000000000000e-08, 8.3078536878999998e-01, 3.9590877063910000e+02 },
    { 2.6820000000000001e-08, 1.4703561821700000e+00, 7.7130123618389996e+02 },
    { 2.6820000000000001e-08, 5.8853242202600002e+00, 9.9889407505091003e+03 },
    { 2.6750000000000001e-08, 1.1774486719699999e+00, 7.3194436026870005e+02 },
    { 2.6729999999999998e-08, 4.5264351842100004e+00, 1.8802630117250001e+02 },
    { 2.6600000000000000e-08, 1.7157400926899999e+00, 1.5971512552120001e+02 },
    { 2.6560000000000001e-08, 1.8836581906600001e+00, 6.2322251175760005e+02 },
    { 2.6470000000000001e-08, 1.7180859558299999e+00, 3.6908206769610001e+02 },
    { 2.6289999999999999e-08, 2.5828867118100001e+00, 6.3203297978779995e+02 },
    { 2.6250000000000000e-08, 4.8091236033099998e+00, 5.6363121503840000e+02 },
    { 2.6230000000000001e-08, 2.8559904534900000e+00, 1.0138503947643700e+04 },
    { 2.6140000000000001e-08, 1.5155390553600001e+00, 7.1074673161819999e+02 },
    { 2.6099999999999999e-08, 2.5647544547200001e+00, 3.1162676309979001e+03 },
    { 2.6079999999999999e-08, 1.9257993274600000e+00, 5.3117543780289998e+02 },
    { 2.5930000000000001e-08, 4.7272621736099998e+00, 7.6588461021249998e+02 },
    { 2.5869999999999999e-08, 3.0066719595400002e+00, 2.7406604832480002e+02 },
    { 2.5860000000000001e-08, 3.7918573631700001e+00, 6.2343496007869999e+02 },
    { 2.5550000000000000e-08, 1.5047415083800000e+00, 6.6820846196529999e+02 },
    { 2.5500000000000000e-08, 1.8510088333200001e+00, 3.3514181775230003e+02 },
    { 2.5480000000000001e-08, 3.4420125581500001e+00, 5.9571448525840003e+02 },
    { 2.5300000000000002e-08, 3.7728557995299998e+00, 2.5357099508990001e+02 },
    { 2.5279999999999999e-08, 2.6712825509300000e+00, 6.9821655864599995e+02 },
    { 2.5279999999999999e-08, 1.9932380349300001e+00, 4.5045594840240000e+02 },
    { 2.5220000000000000e-08, 2.9770873528199999e+00, 1.1536231475990000e+02 },
    { 2.5160000000000002e-08, 3.3943142264400001e+00, 1.1978994270599001e+03 },
    { 2.5100000000000000e-08, 1.1511941312700000e+00, 1.2584539316256000e+03 },
    { 2.5089999999999998e-08, 1.3531047650300001e+00, 4.3240845068500001e+01 },
    { 2.5080000000000000e-08, 4.6458483566800002e+00, 5.8319272331999997e+01 },
    { 2.4970000000000001e-08, 4.1630752678200000e+00, 4.0580716192600001e+01 },
    { 2.4970000000000001e-08, 5.1632716818000002e-01, 1.0898248094200000e+02 },
    { 2.4909999999999999e-08, 6.3438778394999995e-01, 7.0756533788050001e+02 },
    { 2.4850000000000001e-08, 5.7182686925399997e+00, 4.6112914165080002e+02 },
    { 2.4439999999999999e-08, 1.0636685947700000e+00, 8.9484087952760001e+02 },
    { 2.4290000000000001e-08, 5.7237012416899997e+00, 3.9021026037740000e+02 },
    { 2.4150000000000001e-08, 4.1649797223000001e-01, 8.4363515419639998e+02 },
    { 2.4080000000000001e-08, 4.7310204863999999e-01, 1.2284982245716001e+03 },
    { 2.4000000000000000e-08, 2.4296367335200002e+00, 4.1237109687440000e+02 },
    { 2.3779999999999999e-08, 1.2444873968700001e+00, 2.2338158785540000e+02 },
    { 2.3540000000000001e-08, 5.4393803376700003e+00, 5.4547196777370004e+02 },
    { 2.3530000000000000e-08, 3.8852602506000000e+00, 7.4569150246199996e+01 },
    { 2.3530000000000000e-08, 3.9357348389900002e+00, 7.4994046888400007e+01 },
    { 2.3339999999999999e-08, 2.4779158407999999e-01, 6.1632141307790005e+02 },
    { 2.3310000000000002e-08, 4.2283330920499997e+00, 9.8066817835879999e+02 },
    { 2.3300000000000000e-08, 1.2003106682300000e+00, 5.3362311835770004e+02 },
    { 2.3149999999999999e-08, 5.8912464181199997e+00, 2.4697871295089999e+02 },
    { 2.3040000000000000e-08, 9.2872058513000000e-01, 7.2483081326790000e+02 },
    { 2.2670000000000001e-08, 3.0068162215199998e+00, 4.6447059408289999e+02 },
    { 2.2519999999999999e-08, 5.7877591071000001e-01, 1.3741292684431000e+03 },
    { 2.2500000000000000e-08, 5.5340095120999999e+00, 1.8079513409270001e+02 },
    { 2.2300000000000001e-08, 4.4902891335000001e+00, 4.1703696332039999e+02 },
    { 2.2270000000000000e-08, 3.7936255507599999e+00, 6.1445583531207003e+03 },
    { 2.2230000000000001e-08, 3.3085814506000000e-01, 5.3583969916800001e+01 },
};

static const VSOP87Term VSOP87A_URANUS_Y2[255] = {
    { 3.4812647000000000e-04, 3.1415926535900001e+00, 0.0000000000000000e+00 },
    { 1.6589194000000000e-04, 2.2955674062000000e+00, 7.4781598567299994e+01 },
    { 1.0905147000000000e-04, 1.4573796366800000e+00, 1.4956319713459999e+02 },
    { 7.9642979999999995e-05, 9.7230247086999999e-01, 6.3735898303399999e+01 },
    { 7.4846330000000004e-05, 2.2796807691800001e+00, 1.1045700263900001e+01 },
    { 6.7132550000000005e-05, 3.7414888118900000e+00, 8.5827298831199997e+01 },
    { 5.3071000000000001e-05, 3.2051922187800002e+00, 7.0849445304200003e+01 },
    { 5.2651700000000003e-05, 5.4901216905000005e-01, 7.3297125859000005e+01 },
    { 5.0185950000000000e-05, 1.4751852730299999e+00, 7.8713751830400000e+01 },
    { 4.8648220000000001e-05, 4.1136742682299996e+00, 7.6266071275599998e+01 },
    { 2.9958530000000000e-05, 2.4943219354899999e+00, 1.3851749687070000e+02 },
    { 2.5190209999999999e-05, 1.7889682434500001e+00, 7.1600204829600003e+01 },
    { 2.4183710000000001e-05, 2.8867500648800002e+00, 7.7962992305000000e+01 },
    { 2.1858560000000002e-05, 5.5886261497699996e+00, 3.9321532631000000e+00 },
    { 1.0444589999999999e-05, 4.2797223936000002e+00, 2.2434479570190001e+02 },
    { 1.0355780000000000e-05, 5.4475244827499996e+00, 1.4563104387150000e+02 },
    { 6.5918000000000001e-06, 5.7304829671200004e+00, 1.8159247264699999e+01 },
    { 5.0326400000000002e-06, 6.5556547194000003e-01, 1.5104766984290001e+02 },
    { 4.9644500000000004e-06, 3.7129453741999998e+00, 2.2091400527800001e+01 },
    { 4.8421599999999997e-06, 3.4519080477999999e+00, 3.1813937377000001e+00 },
    { 4.7787499999999998e-06, 1.6205930710500001e+00, 7.2333918012500007e+01 },
    { 4.0300299999999999e-06, 6.2742913477700002e+00, 1.6060889739850001e+02 },
    { 3.8475400000000003e-06, 3.1615920808900002e+00, 7.7229279122099996e+01 },
    { 3.8140299999999998e-06, 3.6283047938399999e+00, 7.7750543983900002e+01 },
    { 3.3623300000000000e-06, 4.7628290354400002e+00, 6.5220371011699996e+01 },
    { 3.3305900000000001e-06, 9.7920205194999999e-01, 7.1812653150700001e+01 },
    { 3.0129099999999998e-06, 2.0539719662000000e-01, 8.4342826122899993e+01 },
    { 2.9240400000000001e-06, 3.5836773569600000e+00, 5.6622351302600002e+01 },
    { 2.8961999999999999e-06, 5.2447522487500002e+00, 1.3140394986990000e+02 },
    { 2.8411400000000000e-06, 4.2313583306300000e+00, 1.4807872442630000e+02 },
    { 2.4999000000000000e-06, 1.3771562975000000e+00, 1.2747179660680000e+02 },
    { 2.2083300000000001e-06, 1.8993670036200001e+00, 2.2041264243879999e+02 },
    { 2.1329400000000002e-06, 4.2767033737999999e-01, 5.2690198039499997e+01 },
    { 2.1133900000000001e-06, 4.0113014772100000e+00, 1.5349535039770001e+02 },
    { 2.0603600000000000e-06, 1.3279665960500000e+00, 9.2940845831999994e+01 },
    { 2.0487599999999999e-06, 2.6779923990799999e+00, 8.7311771539500000e+01 },
    { 1.9886500000000000e-06, 1.1497855311000000e+00, 1.2530172972200001e+01 },
    { 1.5153200000000000e-06, 6.0907822994299998e+00, 2.0618554843720000e+02 },
    { 1.4272300000000001e-06, 1.0949652735000000e-01, 9.5612275556000004e+00 },
    { 1.3316399999999999e-06, 5.8084069943700003e+00, 2.1329909543799999e+02 },
    { 1.1788499999999999e-06, 2.5666841023700000e+00, 5.2257741809380002e+02 },
    { 1.1405200000000001e-06, 4.9234949619500004e+00, 5.3680451209540001e+02 },
    { 1.0632200000000000e-06, 4.3230363612699998e+00, 1.4638180339690001e+02 },
    { 1.0599000000000000e-06, 6.1109601762699999e+00, 7.1135470007999997e+00 },
    { 1.0044500000000000e-06, 2.0493523616600000e+00, 6.2251425595100002e+01 },
    { 9.8996000000000001e-07, 4.5350083876000002e-01, 8.0198224538700003e+01 },
    { 9.7067999999999995e-07, 5.5811400828500002e+00, 9.6872999095099999e+01 },
    { 9.0126999999999997e-07, 4.0528411721399999e+00, 2.9689454166000000e+00 },
    { 8.8291999999999997e-07, 1.1951545086399999e+00, 6.7668051566499997e+01 },
    { 8.6384000000000004e-07, 6.5961400870999998e-01, 2.9912639426919998e+02 },
    { 7.7418000000000001e-07, 3.4260217827199999e+00, 8.1895145568100006e+01 },
    { 7.4677999999999997e-07, 3.7751793307499999e+00, 5.2969096509459996e+02 },
    { 6.7304000000000000e-07, 1.9874265959599999e+00, 1.4977853527000001e+01 },
    { 6.6308999999999999e-07, 9.2926527070999998e-01, 4.2659819087599999e+02 },
    { 6.2450999999999997e-07, 3.5044973382600002e+00, 2.4476805547999998e+00 },
    { 6.1645000000000004e-07, 9.9125497203000001e-01, 1.5274459087229999e+02 },
    { 5.0857000000000001e-07, 2.5794767127300000e+00, 7.9235016692200006e+01 },
    { 4.9174999999999999e-07, 3.3370757270100002e+00, 2.2582926841020000e+02 },
    { 4.8895999999999995e-07, 4.6110857972100003e+00, 1.4244965013379999e+02 },
    { 4.7122000000000001e-07, 6.2448045710300004e+00, 1.9513984817330001e+02 },
    { 4.4835000000000001e-07, 2.1559086565099999e+00, 2.0936694217490000e+02 },
    { 4.4787000000000000e-07, 5.3057389654299998e+00, 1.4844727083000000e+00 },
    { 4.3658000000000001e-07, 2.0896385545100000e+00, 7.0328180442399997e+01 },
    { 4.1618000000000001e-07, 6.1435638369900003e+00, 1.5253214255120000e+02 },
    { 4.1218999999999999e-07, 4.0423281627499996e+00, 1.2035824960600000e+02 },
    { 3.9806000000000001e-07, 1.6466302833099999e+00, 2.9204947528600002e+01 },
    { 3.8365999999999998e-07, 3.5457230768399999e+00, 1.0593819301891999e+03 },
    { 3.6426000000000001e-07, 4.5351103222500004e+00, 5.4166259714000002e+00 },
    { 3.5324000000000001e-07, 3.9458203744499998e+00, 1.3703302416240001e+02 },
    { 3.5147000000000001e-07, 4.1920320297200000e+00, 6.9364972595899999e+01 },
    { 3.4455000000000000e-07, 2.9401253465499999e+00, 4.5576651038700000e+01 },
    { 3.1548000000000002e-07, 8.0148827340000006e-02, 2.0225339517410001e+02 },
    { 2.7599999999999998e-07, 5.3627797166400004e+00, 1.5201087768939999e+02 },
    { 2.5904999999999998e-07, 5.5175007514700001e+00, 2.2286032299359999e+02 },
    { 2.5726999999999997e-07, 3.3853028947200001e+00, 4.4534181248999998e+00 },
    { 2.5241000000000002e-07, 2.5052349712600002e+00, 2.3539049596580000e+02 },
    { 2.5105000000000001e-07, 6.1371252285600004e+00, 2.0606927819500001e+01 },
    { 2.5083999999999999e-07, 2.5967210554900002e+00, 2.3575873236100001e+01 },
    { 2.3160999999999999e-07, 4.2317469136000003e+00, 3.9617508346100003e+01 },
    { 2.1960000000000001e-07, 5.5015514828700001e+00, 3.3137100791700000e+01 },
    { 2.0191000000000001e-07, 6.0248233528500004e+00, 7.0115732121299999e+01 },
    { 1.9159000000000001e-07, 1.5180124054999999e-01, 1.4659425171800001e+02 },
    { 1.9020000000000000e-07, 1.8024827164799999e+00, 7.9447465013300004e+01 },
    { 1.8566999999999999e-07, 1.9411779959300000e+00, 1.5912442469019999e+02 },
    { 1.8561000000000001e-07, 4.2657488579500003e+00, 8.9759452094300002e+01 },
    { 1.7966000000000001e-07, 7.3075314120000001e-01, 7.3084677537900006e+01 },
    { 1.7894000000000001e-07, 2.9347205258000000e+00, 1.9120769491019999e+02 },
    { 1.7800999999999999e-07, 8.1410786051999995e-01, 7.6478519596699996e+01 },
    { 1.7688000000000001e-07, 5.8719730160000001e+00, 1.4227094001599999e+01 },
    { 1.6887000000000000e-07, 3.7282538528700000e+00, 5.4174670747800000e+01 },
    { 1.6486000000000001e-07, 4.3563630493999996e+00, 2.6598929347749998e+02 },
    { 1.6355000000000001e-07, 5.4926279809300000e+00, 4.1644497775600001e+01 },
    { 1.5318000000000000e-07, 3.4171416887000000e-01, 1.4414657116320001e+02 },
    { 1.5137000000000001e-07, 1.5355255464499999e+00, 3.3365734504400001e+02 },
    { 1.4945999999999999e-07, 5.2711471741500002e+00, 1.6209337010679999e+02 },
    { 1.4921000000000000e-07, 3.8986531487799998e+00, 1.4711551657979999e+02 },
    { 1.4742000000000001e-07, 2.4012355053000001e-01, 1.1642609634290000e+02 },
    { 1.4322000000000001e-07, 4.5067433763500002e+00, 1.9643719973000000e+01 },
    { 1.4035000000000001e-07, 3.2851267502899999e+00, 1.6674774556399999e+01 },
    { 1.3901000000000000e-07, 1.2394311155500000e+00, 4.3371173787679999e+02 },
    { 1.3463000000000001e-07, 3.8703940526400000e+00, 3.4788443904560000e+02 },
    { 1.2956000000000000e-07, 4.8384921564300001e+00, 1.2895626931510000e+02 },
    { 1.2785000000000000e-07, 5.2177992543800000e+00, 4.1948464387519999e+02 },
    { 1.1729000000000000e-07, 4.2839638232999999e-01, 1.0994568878849999e+02 },
    { 1.1487000000000000e-07, 1.3579638667700000e+00, 8.8796244247800004e+01 },
    { 1.1066000000000000e-07, 5.6098844523900002e+00, 2.5272794265500000e+01 },
    { 1.0722000000000000e-07, 1.3442342002100001e+00, 2.7703499374140000e+02 },
    { 1.0530000000000000e-07, 2.8164173134700001e+00, 3.4077089204480001e+02 },
    { 1.0143000000000000e-07, 5.9767960533100002e+00, 2.1181462272970001e+02 },
    { 1.0052000000000000e-07, 4.8509879871799999e+00, 1.4000196957899999e+02 },
    { 9.9429999999999998e-08, 2.7344941636700000e+00, 1.2598732389849999e+02 },
    { 9.9019999999999996e-08, 1.3107284184900001e+00, 5.1205725331200000e+01 },
    { 9.3690000000000001e-08, 6.6909026270000005e-02, 6.3989728631399998e+02 },
    { 9.2620000000000005e-08, 2.5380942211000002e-01, 2.2827694896500000e+02 },
    { 9.2609999999999993e-08, 3.8230007650700002e+00, 1.6772244439930000e+02 },
    { 8.9799999999999997e-08, 7.2406733830000000e-02, 2.5887574647669999e+02 },
    { 8.9570000000000001e-08, 3.9590256106199999e+00, 6.6704843720000000e+01 },
    { 8.5850000000000004e-08, 1.4391544003100001e+00, 2.7310284047829998e+02 },
    { 8.3999999999999998e-08, 1.6964470857600000e+00, 2.8808069400530002e+02 },
    { 8.1129999999999999e-08, 3.4965986736599999e+00, 4.1101981054399999e+01 },
    { 7.8950000000000002e-08, 1.4414878135800000e+00, 5.8106824010899999e+01 },
    { 7.5320000000000002e-08, 3.2709182628100000e+00, 8.9008692568900003e+01 },
    { 7.2180000000000002e-08, 1.2930624424399999e+00, 9.5388526386799995e+01 },
    { 6.7659999999999998e-08, 6.2497689044899998e+00, 1.5126011816400000e+02 },
    { 6.5060000000000002e-08, 3.7649051073400002e+00, 3.0534616939270001e+02 },
    { 6.3559999999999995e-08, 2.9503228432999999e+00, 1.5497982310600000e+02 },
    { 6.3430000000000006e-08, 2.6888254413000001e+00, 2.2752618943959999e+02 },
    { 6.3300000000000004e-08, 2.0888451980300000e+00, 2.1648048917570000e+02 },
    { 6.2660000000000007e-08, 1.2280372878700001e+00, 1.4510977900969999e+02 },
    { 6.2019999999999996e-08, 2.3823978888499999e+00, 2.4379022388199999e+01 },
    { 6.1550000000000005e-08, 5.1137904292299998e+00, 1.3533610313299999e+02 },
    { 5.9750000000000002e-08, 5.4822197938699997e+00, 1.0664954771900000e+03 },
    { 5.9750000000000002e-08, 5.1856904497200000e+00, 1.2429040286910001e+02 },
    { 5.9580000000000001e-08, 5.3787148480800004e+00, 4.6470422915999997e+01 },
    { 5.9380000000000000e-08, 4.0246238530999996e+00, 1.1020632121940000e+02 },
    { 5.8420000000000003e-08, 5.5024185444999996e-01, 3.9356875915200000e+01 },
    { 5.7930000000000003e-08, 5.6997692960400004e+00, 1.0309277421860000e+02 },
    { 5.7520000000000002e-08, 1.0386075040600000e+00, 6.0554504565700000e+01 },
    { 5.6960000000000002e-08, 3.2170537219100002e+00, 4.7217484191469998e+02 },
    { 5.4249999999999999e-08, 2.9572117284599999e+00, 9.1456373123700004e+01 },
    { 5.3640000000000003e-08, 9.6162321097000003e-01, 7.5744806413800006e+01 },
    { 5.3640000000000003e-08, 5.7618657143999996e-01, 7.3818390720799997e+01 },
    { 5.2490000000000002e-08, 3.5416244017400000e+00, 6.0766952886799999e+01 },
    { 5.1569999999999998e-08, 5.7528358362900001e+00, 3.8330916169770001e+02 },
    { 5.1429999999999998e-08, 1.9965470039500000e+00, 1.0522683831884001e+03 },
    { 5.0960000000000001e-08, 2.7168888776800002e+00, 5.1546387109299997e+02 },
    { 4.9950000000000001e-08, 5.5875238809700001e+00, 4.8640193591629998e+02 },
    { 4.8210000000000000e-08, 3.3857544522000000e+00, 3.7390799283650000e+02 },
    { 4.8149999999999998e-08, 5.9166972492900003e+00, 4.0450679034820001e+02 },
    { 4.7619999999999999e-08, 4.6319801916000003e+00, 3.2972519178089999e+02 },
    { 4.7479999999999999e-08, 4.3711622574600000e+00, 9.8357471803400003e+01 },
    { 4.6690000000000003e-08, 3.6738841577999999e+00, 4.9508804301799998e+01 },
    { 4.5989999999999997e-08, 2.0379910139200001e+00, 4.5097721326419997e+02 },
    { 4.5639999999999997e-08, 2.3865859755400001e+00, 2.2345102382459999e+02 },
    { 4.5219999999999997e-08, 2.9643593402400001e+00, 2.0373786788240000e+02 },
    { 4.4040000000000003e-08, 1.9082196048300000e+00, 1.7165459766239999e+02 },
    { 4.2869999999999999e-08, 1.1991815650500000e+00, 1.0846121608020000e+02 },
    { 4.0620000000000002e-08, 1.4357545553400000e+00, 1.1291463420510000e+02 },
    { 4.0000000000000001e-08, 4.7973471561699998e+00, 1.8409414790939999e+02 },
    { 3.9610000000000002e-08, 3.0426839767999998e+00, 3.6648562929500002e+01 },
    { 3.9209999999999998e-08, 1.0078180928700000e+00, 8.9484087952760001e+02 },
    { 3.8509999999999999e-08, 1.2357652581000000e+00, 8.2858353414600003e+01 },
    { 3.8019999999999999e-08, 5.6169185065200002e+00, 1.5578297225809999e+02 },
    { 3.7930000000000003e-08, 1.3879291233400000e+00, 2.9430046912879999e+02 },
    { 3.7870000000000001e-08, 1.3407716284800000e+00, 3.9739324334740002e+02 },
    { 3.7440000000000003e-08, 4.8711700919499998e+00, 4.3289029178299998e+01 },
    { 3.7100000000000001e-08, 1.6060396513000000e-01, 9.4425318540299997e+01 },
    { 3.6729999999999999e-08, 3.5644964846199998e+00, 6.9152524274800001e+01 },
    { 3.6729999999999999e-08, 1.1149059513799999e+00, 8.0410672859800002e+01 },
    { 3.6640000000000003e-08, 4.5544959559500002e+00, 6.1069233878540001e+02 },
    { 3.6570000000000003e-08, 3.2211946682899999e+00, 8.4926422848890002e+02 },
    { 3.6319999999999997e-08, 3.8576698275500001e+00, 5.7516123179899999e+01 },
    { 3.6169999999999999e-08, 3.3997848330800000e+00, 1.0398654609590000e+02 },
    { 3.5999999999999998e-08, 1.9065704569000000e-01, 6.1448276442999997e+01 },
    { 3.5450000000000003e-08, 4.5204228212900004e+00, 9.0195442652839995e+02 },
    { 3.5399999999999999e-08, 2.1209740628499998e+00, 9.5628915597059995e+02 },
    { 3.5350000000000002e-08, 1.8615490149800000e+00, 1.5142912967165000e+03 },
    { 3.5310000000000003e-08, 3.1761076237700001e+00, 4.1162033734900001e+02 },
    { 3.5040000000000001e-08, 7.7982010393000001e-01, 2.0470107572890001e+02 },
    { 3.4849999999999998e-08, 1.3519529418200000e+00, 4.4631134681819998e+02 },
    { 3.4539999999999997e-08, 1.0947311566400000e+00, 6.2491943278700001e+02 },
    { 3.4499999999999998e-08, 5.5616110981200002e+00, 3.3679617512900002e+01 },
    { 3.4450000000000001e-08, 4.8713639959999999e-02, 7.5314198891920000e+02 },
    { 3.4340000000000002e-08, 2.8858051263600002e+00, 2.3056457082540001e+02 },
    { 3.4049999999999997e-08, 3.8785724816900000e+00, 2.9519424100610001e+02 },
    { 3.3519999999999998e-08, 1.6045038494999999e+00, 6.0676018552230005e+02 },
    { 3.3360000000000002e-08, 4.3039010995000000e+00, 5.0402576179100002e+01 },
    { 3.3269999999999999e-08, 6.9765513922000000e-01, 3.8161224066829999e+02 },
    { 3.3230000000000000e-08, 4.8060015675000001e+00, 5.4391805909619995e+02 },
    { 3.3209999999999998e-08, 3.7044995036200001e+00, 4.6053844081979997e+02 },
    { 3.3200000000000000e-08, 1.3009577497600000e+00, 7.3587651353180001e+02 },
    { 3.2759999999999997e-08, 8.8487694830000005e-02, 3.2243328914400003e+01 },
    { 3.2450000000000003e-08, 2.8979659972600000e+00, 4.2808266358430001e+02 },
    { 3.2080000000000000e-08, 4.5657803388999998e+00, 2.3767811782620001e+02 },
    { 3.2080000000000000e-08, 3.7771366669600002e+00, 3.5499798604640000e+02 },
    { 3.1960000000000003e-08, 9.6757140567000000e-01, 3.3683873878169999e+02 },
    { 3.1790000000000002e-08, 4.7075466426399997e+00, 1.0578974574808999e+03 },
    { 3.1760000000000001e-08, 4.6465937268699999e+00, 4.7928838891549998e+02 },
    { 3.1699999999999999e-08, 1.6695199825300000e+00, 7.1074673161819999e+02 },
    { 3.1699999999999999e-08, 2.0144900388300000e+00, 1.3288842257819999e+02 },
    { 3.1639999999999997e-08, 1.4484865252100001e+00, 1.8196374661092000e+03 },
    { 3.1590000000000000e-08, 1.2978997943899999e+00, 3.8012776796000003e+02 },
    { 3.1459999999999998e-08, 4.4341107262200001e+00, 1.1695882514085999e+03 },
    { 3.1440000000000002e-08, 1.2156884430000001e-02, 1.9907200143640000e+02 },
    { 3.1210000000000000e-08, 4.4324934945800001e+00, 5.5137878594299998e+01 },
    { 3.1189999999999997e-08, 3.9901388194599998e+00, 1.5890728952838001e+03 },
    { 3.0720000000000000e-08, 2.8842194801999999e-01, 2.1723124870109999e+02 },
    { 3.0589999999999998e-08, 3.1741984715400000e+00, 1.0931254934210000e+02 },
    { 3.0549999999999999e-08, 6.1490028942499997e+00, 1.0627416795630000e+02 },
    { 3.0440000000000000e-08, 3.8574280761100002e+00, 3.9828701522469999e+02 },
    { 3.0409999999999999e-08, 5.2174888273000004e+00, 5.3117543780289998e+02 },
    { 3.0370000000000000e-08, 1.1316740553600000e+00, 3.2350541665740002e+02 },
    { 3.0059999999999999e-08, 4.9841852950599996e+00, 2.9111907539110001e+02 },
    { 2.9830000000000003e-08, 2.4843645915199999e+00, 4.2266603761290003e+02 },
    { 2.9819999999999998e-08, 5.2756804599100002e+00, 8.2717282796109998e+02 },
    { 2.9569999999999999e-08, 1.8158556587100001e+00, 2.0767002114549999e+02 },
    { 2.9480000000000000e-08, 1.0201764433600000e+00, 1.8016199464630000e+02 },
    { 2.9399999999999999e-08, 3.9079105450600000e+00, 2.8414854074220000e+02 },
    { 2.9290000000000000e-08, 3.8082104287599998e+00, 3.0683064210100002e+02 },
    { 2.9230000000000001e-08, 5.3358092472600003e+00, 1.9832124191099999e+02 },
    { 2.8380000000000000e-08, 1.5453414558000000e-01, 9.4206206196899996e+02 },
    { 2.8360000000000001e-08, 4.4887473291099997e+00, 8.8114920691600005e+01 },
    { 2.8150000000000001e-08, 2.7774327079000001e+00, 2.5494359321360000e+02 },
    { 2.7820000000000000e-08, 6.0203845394100002e+00, 4.2977958461370002e+02 },
    { 2.7640000000000001e-08, 1.0281515998099999e+00, 2.7555052103309998e+02 },
    { 2.6980000000000001e-08, 4.8673207029899999e+00, 3.7694637422229999e+02 },
    { 2.6580000000000000e-08, 2.5344925620800001e+00, 3.4321857259960001e+02 },
    { 2.6560000000000001e-08, 1.2291800770500001e+00, 4.1555249061209997e+02 },
    { 2.6560000000000001e-08, 2.2661131446999998e+00, 5.0456118318099999e+02 },
    { 2.6550000000000000e-08, 1.5101104492199999e+00, 2.9764192156090002e+02 },
    { 2.6540000000000001e-08, 1.7490588646999999e-01, 9.0588657979150003e+02 },
    { 2.6530000000000000e-08, 6.1908394951099996e+00, 4.7610699517780000e+02 },
    { 2.6490000000000001e-08, 1.9906595463999999e-01, 3.8724131496080003e+02 },
    { 2.6190000000000002e-08, 5.0212336905199999e+00, 2.2137585028530000e+02 },
    { 2.6130000000000000e-08, 1.9932984873099999e+00, 1.1553611574070001e+03 },
    { 2.6020000000000001e-08, 5.2738047849600003e+00, 1.4489733068859999e+02 },
    { 2.6020000000000001e-08, 5.6887829582200000e+00, 4.6658664459999999e+00 },
    { 2.5889999999999999e-08, 3.3528699132000002e+00, 6.8154178408960001e+02 },
    { 2.5860000000000001e-08, 6.2517592141099998e+00, 2.0076892246579999e+02 },
    { 2.5749999999999999e-08, 1.7088263369400001e+00, 4.6112914165080002e+02 },
    { 2.5749999999999999e-08, 5.0093001851399999e+00, 9.8778172535960005e+02 },
    { 2.5430000000000000e-08, 5.0076813842999996e+00, 3.5893013930950002e+02 },
    { 2.5119999999999999e-08, 2.7201164209400002e+00, 2.1892816973050000e+02 },
    { 2.4490000000000000e-08, 8.4593325495000005e-01, 9.5979227217800002e+01 },
    { 2.4389999999999999e-08, 1.1925710918400001e+00, 2.5602862665600000e+01 },
    { 2.4280000000000000e-08, 5.6377733426100001e+00, 7.1467888488129995e+02 },
    { 2.4280000000000000e-08, 3.1959581671400001e+00, 9.9171387862270001e+02 },
    { 2.4100000000000001e-08, 4.3940321609100002e+00, 9.6962247809489998e+02 },
    { 2.4000000000000000e-08, 7.3715855446000000e-01, 1.2828329540138000e+03 },
    { 2.3870000000000001e-08, 3.4717506009000001e+00, 3.4618751801619999e+02 },
    { 2.3770000000000000e-08, 8.3202602713999996e-01, 2.8021638747909998e+02 },
    { 2.3400000000000001e-08, 5.2712400626000004e+00, 9.0981873305459999e+02 },
    { 2.3300000000000000e-08, 2.0697553825899999e+00, 2.2731374111849999e+02 },
    { 2.2970000000000000e-08, 2.3769720964799999e+00, 9.8311585891360005e+02 },
    { 2.1920000000000000e-08, 3.1372096052699998e+00, 2.6747376618580000e+02 },
};

static const VSOP87Term VSOP87A_URANUS_Y3[80] = {
    { 1.2113800000000001e-05, 0.0000000000000000e+00, 0.0000000000000000e+00 },
    { 6.6853899999999997e-06, 1.3571957577800000e+00, 7.0849445304200003e+01 },
    { 6.3423200000000000e-06, 3.3194983371400002e+00, 7.8713751830400000e+01 },
    { 5.8028200000000003e-06, 5.5024989315999999e+00, 1.4956319713459999e+02 },
    { 5.5468799999999997e-06, 3.1123072138199999e+00, 6.3735898303399999e+01 },
    { 4.5552199999999997e-06, 1.6005796278400000e+00, 8.5827298831199997e+01 },
    { 4.4688700000000001e-06, 3.4373018998500000e+00, 7.1600204829600003e+01 },
    { 4.2830899999999998e-06, 2.5233953969499998e+00, 7.3297125859000005e+01 },
    { 4.2245900000000003e-06, 1.2420608888900000e+00, 7.7962992305000000e+01 },
    { 4.1092399999999998e-06, 2.1590408383100002e+00, 7.6266071275599998e+01 },
    { 3.5424000000000002e-06, 6.7890104591000000e-01, 1.1045700263900001e+01 },
    { 2.4668599999999998e-06, 9.4534563236000002e-01, 3.9321532631000000e+00 },
    { 2.1387800000000000e-06, 5.6591529203600004e+00, 7.4781598567299994e+01 },
    { 1.3236400000000000e-06, 4.1502651278800000e+00, 1.3851749687070000e+02 },
    { 1.2039400000000001e-06, 3.7466563770999999e+00, 1.4563104387150000e+02 },
    { 1.1945000000000000e-06, 3.9058553764099999e+00, 1.8159247264699999e+01 },
    { 8.8186999999999999e-07, 1.9041110542399999e+00, 3.1813937377000001e+00 },
    { 5.3379000000000002e-07, 5.3942727641200001e+00, 5.6622351302600002e+01 },
    { 5.2631999999999996e-07, 1.7549819280700001e+00, 2.2434479570190001e+02 },
    { 5.0576000000000005e-07, 7.8027105095000004e-01, 1.3140394986990000e+02 },
    { 4.6166000000000002e-07, 1.3535070137700000e+00, 2.2091400527800001e+01 },
    { 3.4956000000000001e-07, 4.5810044501800000e+00, 1.5104766984290001e+02 },
    { 3.4550999999999998e-07, 5.7233374610599999e+00, 9.2940845831999994e+01 },
    { 2.9345000000000002e-07, 2.0241030705999999e-01, 2.2041264243879999e+02 },
    { 2.8364000000000002e-07, 4.1398417606400004e+00, 1.6060889739850001e+02 },
    { 2.6417999999999997e-07, 5.8586604358100001e+00, 1.5349535039770001e+02 },
    { 2.4289999999999998e-07, 3.4738045111500000e+00, 1.2747179660680000e+02 },
    { 2.0837999999999999e-07, 1.4965781254099999e+00, 2.0618554843720000e+02 },
    { 2.0751999999999999e-07, 4.9251712200400002e+00, 8.4342826122899993e+01 },
    { 2.0447999999999999e-07, 2.2938864196800002e+00, 5.2690198039499997e+01 },
    { 2.0164000000000000e-07, 6.2245749077600001e+00, 6.5220371011699996e+01 },
    { 1.8708000000000000e-07, 1.8772746709699999e+00, 8.1895145568100006e+01 },
    { 1.7870000000000001e-07, 2.7621188347899999e+00, 6.7668051566499997e+01 },
    { 1.7277000000000000e-07, 5.9578728856399996e+00, 1.4638180339690001e+02 },
    { 1.5808000000000000e-07, 4.2841831667400001e+00, 5.2257741809380002e+02 },
    { 1.5540000000000000e-07, 4.6398262033999998e-01, 8.7311771539500000e+01 },
    { 1.5362000000000000e-07, 3.1985661083200001e+00, 5.3680451209540001e+02 },
    { 1.5215000000000000e-07, 4.6592326741300001e+00, 1.4807872442630000e+02 },
    { 8.7740000000000003e-08, 4.2966278106100004e+00, 6.2251425595100002e+01 },
    { 8.6640000000000000e-08, 3.1561401230600001e+00, 9.6872999095099999e+01 },
    { 7.5479999999999998e-08, 5.7902748159200002e+00, 1.5274459087229999e+02 },
    { 7.2740000000000002e-08, 4.2348579343999998e-01, 2.0936694217490000e+02 },
    { 6.8919999999999998e-08, 5.1352928413500001e+00, 6.9152524274800001e+01 },
    { 6.8919999999999998e-08, 5.8272949018300002e+00, 8.0410672859800002e+01 },
    { 6.7160000000000001e-08, 6.0513526950400003e+00, 1.2035824960600000e+02 },
    { 6.6360000000000007e-08, 1.7346513419600000e+00, 1.9513984817330001e+02 },
    { 6.5130000000000002e-08, 5.1622628285700003e+00, 4.5576651038700000e+01 },
    { 6.1029999999999998e-08, 1.4509613981900000e+00, 2.0225339517410001e+02 },
    { 5.5540000000000000e-08, 1.2709439361499999e+00, 7.7229279122099996e+01 },
    { 5.4140000000000000e-08, 5.6296103988699997e+00, 6.9364972595899999e+01 },
    { 5.3190000000000002e-08, 2.4108290967000001e-01, 7.2333918012500007e+01 },
    { 5.0230000000000000e-08, 3.1886761727000001e-01, 2.9823262239190001e+02 },
    { 4.5559999999999999e-08, 5.8116780878299998e+00, 2.9204947528600002e+01 },
    { 4.4199999999999999e-08, 2.3177508901800001e+00, 8.0198224538700003e+01 },
    { 4.0959999999999998e-08, 1.7464279137700001e+00, 5.9803745040300001e+01 },
    { 4.0560000000000001e-08, 5.4180538588499996e+00, 3.1245971639350000e+02 },
    { 3.7720000000000003e-08, 4.5736408307999998e-01, 7.1812653150700001e+01 },
    { 3.7720000000000003e-08, 1.0804456993300000e+00, 7.7750543983900002e+01 },
    { 3.7359999999999998e-08, 1.9488998459400000e+00, 2.5887574647669999e+02 },
    { 3.5910000000000002e-08, 1.5531675180300000e+00, 1.9907200143640000e+02 },
    { 3.5640000000000000e-08, 1.4032434833000000e+00, 7.0115732121299999e+01 },
    { 3.5640000000000000e-08, 1.3456629911000001e-01, 7.9447465013300004e+01 },
    { 3.5220000000000000e-08, 2.2720627073099999e+00, 6.3596513305090002e+02 },
    { 3.4709999999999998e-08, 3.8389480794300002e+00, 4.8640193591629998e+02 },
    { 3.4270000000000002e-08, 3.2333050791900000e+00, 6.2173803904930003e+02 },
    { 3.4090000000000003e-08, 5.2995496312399997e+00, 4.2659819087599999e+02 },
    { 3.2450000000000003e-08, 2.4835081264299999e+00, 6.1158611066269998e+02 },
    { 3.1340000000000001e-08, 1.6461922297000001e-01, 4.0843894361129998e+02 },
    { 3.1060000000000002e-08, 3.9529482439699999e+00, 4.6202291352809999e+02 },
    { 3.0580000000000000e-08, 4.9886138255299999e+00, 4.7217484191469998e+02 },
    { 2.9980000000000001e-08, 2.3930946585499999e+00, 7.3084677537900006e+01 },
    { 2.9980000000000001e-08, 5.4279004310400003e+00, 7.6478519596699996e+01 },
    { 2.8380000000000000e-08, 4.2008984634599997e+00, 4.2266603761290003e+02 },
    { 2.8069999999999999e-08, 4.7052339807200001e+00, 6.4701083331480004e+02 },
    { 2.7949999999999999e-08, 5.9811556135300004e+00, 2.7310284047829998e+02 },
    { 2.6720000000000000e-08, 4.3585842642700001e+00, 3.7619561469690001e+02 },
    { 2.6400000000000001e-08, 6.4993830114999995e-01, 2.8808069400530002e+02 },
    { 2.5069999999999999e-08, 4.3132240368600003e+00, 1.3458534360760001e+02 },
    { 2.3409999999999999e-08, 5.5763051076799997e+00, 4.3371173787679999e+02 },
    { 2.2180000000000001e-08, 4.5744608006899998e+00, 3.9117346822389999e+02 },
};

static const VSOP87Term VSOP87A_URANUS_Y4[12] = {
    { 4.9429999999999997e-07, 0.0000000000000000e+00, 0.0000000000000000e+00 },
    { 3.4198000000000000e-07, 5.1394029225100004e+00, 6.3735898303399999e+01 },
    { 2.7043999999999998e-07, 5.8286556954400002e+00, 8.5827298831199997e+01 },
    { 2.6702000000000002e-07, 3.7741410502599999e+00, 1.4956319713459999e+02 },
    { 1.6280999999999999e-07, 2.0566973768199999e+00, 1.8159247264699999e+01 },
    { 7.9899999999999994e-08, 8.8303095823000000e-01, 5.6622351302600002e+01 },
    { 6.6740000000000001e-08, 2.4765369251800000e+00, 1.3140394986990000e+02 },
    { 5.2280000000000002e-08, 3.8400241934300001e+00, 9.2940845831999994e+01 },
    { 2.4830000000000001e-08, 2.1418772398099999e+00, 2.8414854074220000e+02 },
    { 2.3510000000000000e-08, 4.8134208760000003e+00, 2.2041264243879999e+02 },
    { 2.2939999999999999e-08, 4.6070078769000000e+00, 3.6590067395839998e+02 },
    { 2.2259999999999999e-08, 3.2191255161600001e+00, 2.0618554843720000e+02 },
};

static const VSOP87Term VSOP87A_URANUS_Y5[1] = {
    { 0.0, 0.0, 0.0 }
};

static const VSOP87Term VSOP87A_URANUS_Z0[235] = {
    { 2.5878127697999997e-01, 2.6186127257799998e+00, 7.4781598567299994e+01 },
    { 1.7743187779999999e-02, 3.1415926535900001e+00, 0.0000000000000000e+00 },
    { 5.9931613099999999e-03, 5.0811950058499997e+00, 1.4956319713459999e+02 },
    { 1.9088168500000001e-03, 5.7869575951999996e-01, 7.3297125859000005e+01 },
    { 1.9028189000000000e-03, 1.6164384119299999e+00, 7.6266071275599998e+01 },
    { 8.4626761000000000e-04, 2.2603015016599999e+00, 1.4844727083000000e+00 },
    { 3.0734257000000000e-04, 2.3571721555000000e-01, 6.3735898303399999e+01 },
    { 2.0842052000000000e-04, 1.2605420809100001e+00, 2.2434479570190001e+02 },
    { 1.9734272999999999e-04, 6.0431467768799996e+00, 1.4807872442630000e+02 },
    { 1.4582863999999999e-04, 6.1485203721200001e+00, 7.1812653150700001e+01 },
    { 1.2537529999999999e-04, 5.1716905146599998e+00, 1.1045700263900001e+01 },
    { 1.1261541000000001e-04, 3.5597376968600001e+00, 5.2969096509459996e+02 },
    { 1.0407529000000000e-04, 3.6532041703800000e+00, 2.1329909543799999e+02 },
    { 8.8556690000000005e-05, 4.0377450573899996e+00, 1.5104766984290001e+02 },
    { 8.2394599999999999e-05, 3.4225652714999999e-01, 7.7750543983900002e+01 },
    { 7.9501690000000004e-05, 7.2564903050999996e-01, 8.5827298831199997e+01 },
    { 6.8674689999999994e-05, 8.1417174224000000e-01, 2.9689454166000000e+00 },
    { 5.6487199999999997e-05, 3.4532471954299999e+00, 1.3851749687070000e+02 },
    { 4.5819380000000002e-05, 1.6966868234400001e+00, 3.8133035637799999e+01 },
    { 2.9640699999999999e-05, 6.1433880223899999e+00, 3.5164090221199999e+01 },
    { 2.5783989999999998e-05, 5.1969644738999996e+00, 1.1143016149680000e+02 },
    { 2.3303040000000000e-05, 5.7264022614999996e+00, 7.0849445304200003e+01 },
    { 2.0370110000000001e-05, 9.5353587036999998e-01, 7.0328180442399997e+01 },
    { 1.9852149999999999e-05, 7.6408839811999996e-01, 3.9617508346100003e+01 },
    { 1.8841040000000001e-05, 2.6119247264799998e+00, 7.8713751830400000e+01 },
    { 1.7431540000000000e-05, 1.2058628178899999e+00, 1.4659425171800001e+02 },
    { 1.5089240000000001e-05, 2.2619544855300000e+00, 2.2286032299359999e+02 },
    { 1.0827360000000000e-05, 2.6287287405700002e+00, 1.0846121608020000e+02 },
    { 1.0519680000000001e-05, 7.5560753839999995e-01, 1.0994568878849999e+02 },
    { 8.6014200000000001e-06, 3.7228557297500000e+00, 2.9912639426919998e+02 },
    { 8.1618200000000001e-06, 3.9530377946000002e+00, 5.2690198039499997e+01 },
    { 7.1242100000000002e-06, 1.0834394187800001e+00, 3.9321532631000000e+00 },
    { 6.6124899999999999e-06, 3.8345916093899999e+00, 3.6648562929500002e+01 },
    { 6.2432699999999996e-06, 5.5599806336000004e+00, 2.0225339517410001e+02 },
    { 6.1865699999999998e-06, 5.7703324107600000e+00, 4.4534181248999998e+00 },
    { 5.6195899999999997e-06, 1.8760120432800000e+00, 3.5181659230870002e+02 },
    { 5.2637799999999998e-06, 3.8817828008399999e+00, 1.1291463420510000e+02 },
    { 4.6535899999999999e-06, 3.2401505863100000e+00, 1.2747179660680000e+02 },
    { 4.6412999999999999e-06, 2.0598822419000001e-01, 2.2582926841020000e+02 },
    { 4.5813300000000001e-06, 5.4911322148900004e+00, 7.9235016692200006e+01 },
    { 4.4811000000000002e-06, 2.7054774028600002e+00, 1.4510977900969999e+02 },
    { 4.3738600000000003e-06, 1.2489271892100000e+00, 3.3679617512900002e+01 },
    { 4.2472099999999996e-06, 2.7394521810199999e+00, 1.5253214255120000e+02 },
    { 4.2156199999999996e-06, 3.6672134976500002e+00, 1.8472728735580000e+02 },
    { 3.9361700000000001e-06, 1.2501849238600000e+00, 6.2251425595100002e+01 },
    { 3.7574700000000000e-06, 3.2045964680100001e+00, 1.6060889739850001e+02 },
    { 3.5045000000000001e-06, 2.5778826143600000e+00, 8.4342826122899993e+01 },
    { 3.4683299999999999e-06, 2.9126139462000000e+00, 4.2659819087599999e+02 },
    { 3.2540900000000000e-06, 2.4821485784699999e+00, 9.5612275556000004e+00 },
    { 3.1315200000000001e-06, 1.3211754313099999e+00, 1.4563104387150000e+02 },
    { 2.8445600000000000e-06, 1.7035683583700001e+00, 2.2091400527800001e+01 },
    { 2.7441900000000001e-06, 3.9064662044100000e+00, 1.0593819301891999e+03 },
    { 2.6551700000000000e-06, 2.6717268440100002e+00, 7.4669723982700006e+01 },
    { 2.6521000000000000e-06, 5.7072128726500004e+00, 7.4893473151899997e+01 },
    { 2.6072700000000000e-06, 2.6432051819000002e-01, 2.7703499374140000e+02 },
    { 2.6066600000000001e-06, 5.7869988607499998e+00, 4.1101981054399999e+01 },
    { 2.5894799999999998e-06, 4.1292158234600000e+00, 1.2530172972200001e+01 },
    { 2.3772100000000001e-06, 8.0288483705000002e-01, 6.5220371011699996e+01 },
    { 2.2399800000000000e-06, 3.9125016540700002e+00, 2.2137585028530000e+02 },
    { 2.2062600000000001e-06, 3.9352911055800002e+00, 1.0697674337190000e+02 },
    { 2.1676300000000002e-06, 2.1435501657299998e+00, 6.8843707734099993e+01 },
    { 2.0126800000000001e-06, 5.9490639872299997e+00, 8.7311771539500000e+01 },
    { 1.6650999999999999e-06, 3.3720886309200000e+00, 7.2333918012500007e+01 },
    { 1.6547199999999999e-06, 3.5410884822600002e+00, 4.9033408917940000e+02 },
    { 1.5599800000000000e-06, 5.1928481259500003e+00, 7.7229279122099996e+01 },
    { 1.5175200000000000e-06, 3.2533598189399999e+00, 7.1600204829600003e+01 },
    { 1.4579499999999999e-06, 5.1214735284900001e+00, 7.7962992305000000e+01 },
    { 1.4494200000000000e-06, 3.2437405429299999e+00, 5.6622351302600002e+01 },
    { 1.2361999999999999e-06, 1.9601919402300001e+00, 2.8808069400530002e+02 },
    { 1.2339700000000000e-06, 2.0610025683300002e+00, 1.8621176006409999e+02 },
    { 1.0982400000000000e-06, 3.7438524749900002e+00, 6.7668051566499997e+01 },
    { 1.0754999999999999e-06, 1.0802577797899999e+00, 3.4077089204480001e+02 },
    { 1.0013600000000000e-06, 2.8230110990599999e+00, 1.1439910691340000e+02 },
    { 9.5939000000000002e-07, 4.7208511975200000e+00, 2.9764192156090002e+02 },
    { 9.5640000000000007e-07, 3.5390618635400002e+00, 2.2041264243879999e+02 },
    { 9.3119999999999997e-07, 2.9198951446799999e+00, 1.8175834193919999e+02 },
    { 8.7545999999999998e-07, 5.0653159497600004e+00, 1.8324281464750001e+02 },
    { 8.5881999999999996e-07, 5.1026884129700001e+00, 1.5349535039770001e+02 },
    { 8.1506999999999995e-07, 4.0389144013299996e+00, 1.4362530630140000e+02 },
    { 8.0745999999999995e-07, 6.4185046063999995e-01, 2.0618554843720000e+02 },
    { 7.3015999999999997e-07, 1.7551425593100001e+00, 4.1555249061209997e+02 },
    { 7.2498999999999998e-07, 1.2623193949200000e+00, 7.5302863429100000e+01 },
    { 7.2251000000000003e-07, 8.3002325084999995e-01, 7.4260333705500003e+01 },
    { 7.0289999999999997e-07, 4.2524338797499999e+00, 1.3703302416240001e+02 },
    { 6.6851999999999995e-07, 5.3763825077599998e+00, 2.6598929347749998e+02 },
    { 6.4038999999999998e-07, 2.0952781203100002e+00, 7.3818390720799997e+01 },
    { 6.2160999999999997e-07, 1.2434513600000001e-02, 7.5744806413800006e+01 },
    { 6.2038999999999996e-07, 4.4833446121499998e+00, 6.6704843720000000e+01 },
    { 6.1440999999999995e-07, 1.9311221788399999e+00, 1.8159247264699999e+01 },
    { 5.9320999999999995e-07, 5.9787435734000005e-01, 1.4000196957899999e+02 },
    { 5.7080999999999996e-07, 4.5296910504900003e+00, 5.9378908332000000e+00 },
    { 5.6535000000000002e-07, 5.2328898846099996e+00, 6.2885158605009997e+02 },
    { 5.4171000000000000e-07, 2.5694617228999999e+00, 3.2195144804599998e+01 },
    { 5.1592999999999999e-07, 2.1614625456300001e+00, 5.4174670747800000e+01 },
    { 5.0182000000000003e-07, 3.9519315542100002e+00, 8.2858353414600003e+01 },
    { 4.8938000000000001e-07, 3.5089315189599999e+00, 7.1135470007999997e+00 },
    { 4.6516999999999998e-07, 3.0368318376199999e+00, 5.9803745040300001e+01 },
    { 4.6178000000000000e-07, 4.9809404706999999e-01, 1.3140394986990000e+02 },
    { 4.3400000000000000e-07, 4.9921659313099997e+00, 2.5802441321480001e+02 },
    { 4.2339999999999999e-07, 3.0490222040399999e+00, 2.1840690486870000e+02 },
    { 4.0711000000000000e-07, 3.5979234125600001e+00, 5.2257741809380002e+02 },
    { 3.9572000000000000e-07, 5.3588738546499997e+00, 2.1989137757699999e+02 },
    { 3.9089999999999999e-07, 6.1792633292000003e+00, 3.7390799283650000e+02 },
    { 3.6837000000000001e-07, 3.4876038271600001e+00, 9.6872999095099999e+01 },
    { 3.6403000000000001e-07, 1.9885043751400000e+00, 8.1895145568100006e+01 },
    { 3.4919999999999998e-07, 3.2367925293200002e+00, 3.8012776796000003e+02 },
    { 3.4900000000000001e-07, 1.5976919995400001e+00, 8.0198224538700003e+01 },
    { 3.4349000000000001e-07, 5.2115250480200004e+00, 1.0549227066360000e+02 },
    { 3.2230999999999998e-07, 3.6106547644600000e+00, 2.0606927819500001e+01 },
    { 3.2126000000000001e-07, 3.2729856530600001e+00, 1.9120769491019999e+02 },
    { 3.0914999999999999e-07, 4.6043337626899996e+00, 4.2586453762700003e+01 },
    { 2.9887999999999999e-07, 1.8097385916299999e+00, 3.1813937377000001e+00 },
    { 2.9731000000000001e-07, 4.3750999048899999e+00, 8.0719489400499995e+01 },
    { 2.9315000000000000e-07, 5.0362985250700003e+00, 5.1205725331200000e+01 },
    { 2.9189000000000000e-07, 2.7378667209000001e+00, 4.7928838891549998e+02 },
    { 2.8855999999999998e-07, 1.3465325940499999e+00, 7.4520966136400006e+01 },
    { 2.8696000000000002e-07, 7.4986873597000003e-01, 7.5042230998199997e+01 },
    { 2.8644000000000002e-07, 5.7551016065100002e+00, 2.5950888592310002e+02 },
    { 2.8208000000000000e-07, 4.6664232313999996e+00, 1.5912442469019999e+02 },
    { 2.7066000000000002e-07, 3.3693693820699999e+00, 5.5406998748280000e+02 },
    { 2.6684999999999998e-07, 2.9866776245800000e+00, 1.4977853527000001e+01 },
    { 2.6128000000000002e-07, 1.3440940380999999e-01, 8.3503713448730002e+02 },
    { 2.6095999999999998e-07, 5.4869124026999998e+00, 7.4829782677099999e+01 },
    { 2.6095999999999998e-07, 2.8930515791900002e+00, 7.4733414457500004e+01 },
    { 2.5879000000000000e-07, 3.0577656698800002e+00, 8.0767548473000002e+00 },
    { 2.5722000000000002e-07, 4.2614221176799996e+00, 1.8027386923090000e+02 },
    { 2.5720000000000000e-07, 2.4233262633199999e+00, 3.0534616939270001e+02 },
    { 2.5646000000000001e-07, 2.6374705710400002e+00, 3.0061086697750000e+02 },
    { 2.5366000000000001e-07, 2.6521632572999998e-01, 5.3680451209540001e+02 },
    { 2.5190999999999998e-07, 6.1617071489199997e+00, 9.9160620955499994e+01 },
    { 2.5184000000000001e-07, 1.5321996903599999e+00, 1.5401661525950001e+02 },
    { 2.4247999999999998e-07, 2.8711666320300000e+00, 7.4621539872900001e+01 },
    { 2.4190000000000002e-07, 5.5092297927500002e+00, 7.4941657261700001e+01 },
    { 2.4161000000000000e-07, 5.1949972281600001e+00, 2.2731374111849999e+02 },
    { 2.3545000000000001e-07, 3.1620415574099998e+00, 6.7359235025800004e+01 },
    { 2.2294999999999999e-07, 3.7533774434399998e+00, 3.9356875915200000e+01 },
    { 2.2219999999999999e-07, 3.7226291335899999e+00, 9.2940845831999994e+01 },
    { 2.1752000000000000e-07, 5.2614000812299997e+00, 2.9467297614429998e+02 },
    { 2.1467999999999999e-07, 5.5919282671200001e+00, 2.4476805547999998e+00 },
    { 2.1014000000000000e-07, 6.6302313851000005e-01, 7.6736908292080000e+02 },
    { 2.0972000000000000e-07, 3.1791689642400001e+00, 2.5505546779820000e+02 },
    { 2.0577000000000000e-07, 9.8634295840999997e-01, 1.1642609634290000e+02 },
    { 1.9430000000000001e-07, 5.6606771722799998e+00, 2.3539049596580000e+02 },
    { 1.8776999999999999e-07, 6.0348363638000002e-01, 2.1181462272970001e+02 },
    { 1.7310999999999999e-07, 8.0301716068999995e-01, 1.8769623277240001e+02 },
    { 1.7210000000000001e-07, 8.7217558691999997e-01, 4.0450679034820001e+02 },
    { 1.6675000000000001e-07, 4.4525261537400000e+00, 1.2598732389849999e+02 },
    { 1.6392000000000000e-07, 7.3044725202000005e-01, 1.4244965013379999e+02 },
    { 1.6318000000000001e-07, 3.1360230024900000e+00, 2.8311175651300001e+01 },
    { 1.5780000000000000e-07, 5.9509139302399996e+00, 2.9615744885260000e+02 },
    { 1.5522999999999999e-07, 1.7613816243000000e+00, 4.1644497775600001e+01 },
    { 1.5442999999999999e-07, 2.7617327363499999e+00, 1.4014645680499999e+01 },
    { 1.5178000000000000e-07, 2.7928766518099999e+00, 5.0402576179100002e+01 },
    { 1.4962000000000001e-07, 4.2127674390900003e+00, 4.5490936652729999e+02 },
    { 1.4941000000000001e-07, 1.8697683131300000e+00, 2.0373786788240000e+02 },
    { 1.4560000000000001e-07, 3.1573630494999999e-01, 2.0076892246579999e+02 },
    { 1.3992000000000000e-07, 2.8961235638099998e+00, 6.0766952886799999e+01 },
    { 1.3756000000000000e-07, 4.4413004405800001e+00, 2.1692243216040001e+02 },
    { 1.3750000000000001e-07, 6.2184493728400003e+00, 5.4166259714000002e+00 },
    { 1.3680999999999999e-07, 2.0663672669399999e+00, 6.3989728631399998e+02 },
    { 1.3507999999999999e-07, 1.1147459770499999e+00, 9.8460033162189995e+02 },
    { 1.3092000000000001e-07, 5.1647978239499999e+00, 7.4030839041899995e+01 },
    { 1.3033999999999999e-07, 5.5043059925000004e-01, 2.3575873236100001e+01 },
    { 1.3024000000000001e-07, 1.5416724797900001e+00, 1.1588357962169999e+02 },
    { 1.3012000000000000e-07, 6.0010863354500001e+00, 2.5653994050649999e+02 },
    { 1.2912000000000001e-07, 2.6554417568200002e+00, 7.5532358092699994e+01 },
    { 1.2269000000000000e-07, 2.1188070280200000e+00, 1.6209337010679999e+02 },
    { 1.1955999999999999e-07, 5.0878632534400001e+00, 8.9759452094300002e+01 },
    { 1.1862000000000000e-07, 5.5018524328099998e+00, 3.3132153907380001e+02 },
    { 1.1850000000000000e-07, 4.8355199916400000e+00, 1.4945132254999999e+02 },
    { 1.1814000000000000e-07, 5.3101026752099996e+00, 1.4214083359310001e+02 },
    { 1.1801999999999999e-07, 1.5876027842300000e+00, 1.4967507171919999e+02 },
    { 1.1677000000000001e-07, 2.4093159077699999e+00, 8.1001373690799994e+01 },
    { 1.1409000000000000e-07, 3.3965082618100002e+00, 2.9170403072770000e+02 },
    { 1.1138000000000000e-07, 2.9193597524600001e+00, 6.8189316428300003e+01 },
    { 1.0932000000000000e-07, 1.7212866351000000e-01, 2.3056457082540001e+02 },
    { 1.0875000000000000e-07, 1.4632073583999999e-01, 1.2895626931510000e+02 },
    { 1.0662000000000000e-07, 5.5279708992499996e+00, 3.5424722652100002e+01 },
    { 1.0635999999999999e-07, 3.4135481155799998e+00, 2.8659622129700000e+02 },
    { 9.9470000000000003e-08, 4.0036716951600004e+00, 7.6025553592000006e+02 },
    { 9.9470000000000003e-08, 5.6059063584900004e+00, 6.8561823443799994e+01 },
    { 9.9260000000000003e-08, 4.6553035231099997e+00, 8.8796244247800004e+01 },
    { 9.8780000000000002e-08, 5.7611696852399996e+00, 1.1020632121940000e+02 },
    { 9.5830000000000005e-08, 4.2674378861999998e+00, 1.5890728952838001e+03 },
    { 9.3340000000000001e-08, 4.5928779674999998e-01, 1.2035824960600000e+02 },
    { 9.1619999999999996e-08, 3.7139661418299998e+00, 3.6286229257259998e+02 },
    { 9.1399999999999998e-08, 4.9282651374300004e+00, 8.1373880706300000e+01 },
    { 9.0410000000000000e-08, 5.7130047621899998e+00, 4.1948464387519999e+02 },
    { 9.0100000000000006e-08, 5.1578638509100001e+00, 6.9258748435350003e+02 },
    { 8.9970000000000004e-08, 2.3796567754300000e+00, 1.9513984817330001e+02 },
    { 8.5689999999999995e-08, 5.3660841822899998e+00, 2.8571808082200000e+01 },
    { 8.4870000000000005e-08, 1.7292401792000001e+00, 2.4379022388199999e+01 },
    { 8.3340000000000004e-08, 2.3758565080999999e-01, 5.6511568774670002e+02 },
    { 8.2430000000000004e-08, 4.7470876574699998e+00, 3.1639186965660002e+02 },
    { 8.0939999999999996e-08, 4.3008018898399998e+00, 6.1780588578619995e+02 },
    { 7.9049999999999996e-08, 4.2587031675000002e+00, 2.6099335863139999e+02 },
    { 7.6840000000000005e-08, 1.6703068133600001e+00, 1.1413847448250000e+02 },
    { 7.6199999999999994e-08, 7.6643908895000001e-01, 1.5201087768939999e+02 },
    { 7.5940000000000004e-08, 6.1024651235800000e+00, 1.0309277421860000e+02 },
    { 7.5919999999999994e-08, 5.0522526589999998e-02, 6.9364972595899999e+01 },
    { 7.4709999999999999e-08, 5.6709190296600003e+00, 3.6797010200329998e+02 },
    { 7.4610000000000004e-08, 2.3203173268200001e+00, 9.0588657979150003e+02 },
    { 7.4229999999999997e-08, 1.0794595769000000e-01, 2.9318850343600002e+02 },
    { 7.3659999999999999e-08, 6.1556592005199997e+00, 1.3554855145409999e+02 },
    { 7.2520000000000004e-08, 6.0577373650300004e+00, 6.2197751234999998e+00 },
    { 6.8579999999999996e-08, 5.6900283748199998e+00, 1.4148644228730001e+02 },
    { 6.8519999999999994e-08, 5.4220318587699996e+00, 1.4638180339690001e+02 },
    { 6.8050000000000004e-08, 3.7937941779200002e+00, 2.1478356814630001e+02 },
    { 6.5320000000000005e-08, 3.6853841522200002e+00, 2.7851946644970002e+02 },
    { 6.3139999999999995e-08, 1.8733033209000000e-01, 1.0213285546211000e+04 },
    { 6.2890000000000003e-08, 3.4725226396600002e+00, 7.3409000443599993e+01 },
    { 6.2400000000000003e-08, 3.8441676897099999e+00, 3.0710672096300001e+01 },
    { 6.1760000000000005e-08, 4.0992940380400000e+00, 3.2835259365719998e+02 },
    { 6.0969999999999996e-08, 5.1086323555900002e+00, 3.2972519178089999e+02 },
    { 6.0279999999999994e-08, 5.2402525925100001e+00, 9.5388526386799995e+01 },
    { 5.9999999999999995e-08, 5.6119539011899997e+00, 1.5763995198190000e+02 },
    { 5.9880000000000004e-08, 1.1156507911700000e+00, 3.7242352012819998e+02 },
    { 5.8260000000000000e-08, 5.2312532006600003e+00, 2.8414854074220000e+02 },
    { 5.8070000000000003e-08, 2.2266954408099999e+00, 3.5893013930950002e+02 },
    { 5.7560000000000001e-08, 6.0729000612999995e-01, 4.5576651038700000e+01 },
    { 5.7259999999999998e-08, 7.9025128961000002e-01, 4.8322054217860000e+02 },
    { 5.4249999999999999e-08, 1.8177305932500001e+00, 6.4701083331480004e+02 },
    { 5.3370000000000001e-08, 2.5569714669299999e+00, 5.4302428721889999e+02 },
    { 5.3309999999999999e-08, 5.7774785878000001e-01, 4.6202291352809999e+02 },
    { 5.3220000000000003e-08, 5.5393077779900004e+00, 4.7791579079180002e+02 },
    { 5.2990000000000000e-08, 2.8596890733000002e-01, 9.6320784650000002e-01 },
    { 5.2140000000000002e-08, 3.2350613101099999e+00, 7.4223635415000002e+00 },
    { 5.0250000000000003e-08, 2.8271057439899998e+00, 1.5274459087229999e+02 },
    { 4.9600000000000001e-08, 3.8636649026500001e+00, 4.6209790485100001e+01 },
    { 4.7110000000000003e-08, 1.0400853409499999e+00, 1.3324847706675000e+03 },
    { 4.6399999999999999e-08, 1.3772081333299999e+00, 1.2125202148330000e+02 },
    { 4.5599999999999998e-08, 5.1852574686399997e+00, 1.2151649024472999e+03 },
    { 4.5279999999999999e-08, 5.3810018323200000e+00, 9.8311585891360005e+02 },
    { 4.4139999999999997e-08, 5.2223271622700000e+00, 9.0981873305459999e+02 },
    { 4.3649999999999997e-08, 2.1506297171100002e+00, 6.1158611066269998e+02 },
};

static const VSOP87Term VSOP87A_URANUS_Z1[98] = {
    { 6.5591662599999998e-03, 1.2719476600000000e-02, 7.4781598567299994e+01 },
    { 4.9648951000000005e-04, 0.0000000000000000e+00, 0.0000000000000000e+00 },
    { 2.3874178000000000e-04, 2.7387049121999998e+00, 1.4956319713459999e+02 },
    { 7.5521769999999997e-05, 5.4930420770000001e+00, 7.6266071275599998e+01 },
    { 5.9413039999999998e-05, 3.6125407330399999e+00, 7.3297125859000005e+01 },
    { 2.8684289999999998e-05, 4.1795415787800003e+00, 6.3735898303399999e+01 },
    { 2.0874550000000002e-05, 5.9785862581700000e+00, 1.4844727083000000e+00 },
    { 1.8276969999999999e-05, 2.7181081333499999e+00, 1.1045700263900001e+01 },
    { 1.3050630000000000e-05, 4.5233700219499999e+00, 8.5827298831199997e+01 },
    { 1.1582500000000000e-05, 5.3191350411200000e+00, 2.2434479570190001e+02 },
    { 7.3411199999999996e-06, 3.8133172821999999e+00, 7.0849445304200003e+01 },
    { 6.9030400000000001e-06, 1.0863199360000001e-02, 1.3851749687070000e+02 },
    { 5.2593400000000000e-06, 4.2279014891599997e+00, 7.1812653150700001e+01 },
    { 4.9709100000000000e-06, 4.7051866731100001e+00, 7.8713751830400000e+01 },
    { 4.8913299999999996e-06, 1.7919001378899999e+00, 1.5104766984290001e+02 },
    { 4.2952799999999996e-06, 4.4075834336800002e+00, 7.7750543983900002e+01 },
    { 4.2941400000000004e-06, 4.8768114352599996e+00, 2.1329909543799999e+02 },
    { 3.9002499999999996e-06, 3.5945830381600001e+00, 1.4807872442630000e+02 },
    { 3.8644200000000003e-06, 1.5270984372900001e+00, 5.2969096509459996e+02 },
    { 2.2511500000000000e-06, 5.0281464758199998e+00, 2.9689454166000000e+00 },
    { 1.9325199999999998e-06, 2.7488258737800000e+00, 3.9321532631000000e+00 },
    { 1.0698100000000000e-06, 1.1063174412700001e+00, 5.2690198039499997e+01 },
    { 1.0322700000000001e-06, 5.0694360625800003e+00, 7.1600204829600003e+01 },
    { 1.0278500000000000e-06, 3.3150276072099998e+00, 7.7962992305000000e+01 },
    { 9.7903000000000005e-07, 5.8566405112400002e+00, 1.4563104387150000e+02 },
    { 7.7286000000000000e-07, 3.7280788440200001e+00, 3.5164090221199999e+01 },
    { 7.6598999999999999e-07, 1.3068371025200001e+00, 6.5220371011699996e+01 },
    { 6.7594000000000000e-07, 7.7601310739999996e-01, 1.6060889739850001e+02 },
    { 6.3560999999999996e-07, 5.0101668439199996e+00, 2.2091400527800001e+01 },
    { 6.1714999999999999e-07, 1.5549709738799999e+00, 2.9912639426919998e+02 },
    { 5.8625000000000000e-07, 5.1757690359300002e+00, 3.9617508346100003e+01 },
    { 5.5754000000000000e-07, 6.2620325200600000e+00, 2.2286032299359999e+02 },
    { 5.3114000000000003e-07, 2.8640562195500001e+00, 2.0225339517410001e+02 },
    { 4.9819999999999995e-07, 5.1483501247700003e+00, 5.6622351302600002e+01 },
    { 4.9390000000000005e-07, 9.6663399742000000e-01, 1.2747179660680000e+02 },
    { 4.6591999999999998e-07, 3.8562695813999998e-01, 8.4342826122899993e+01 },
    { 4.2230000000000002e-07, 1.6706809200199999e+00, 1.2530172972200001e+01 },
    { 4.0266999999999999e-07, 3.9282094222000002e-01, 9.5612275556000004e+00 },
    { 3.9706999999999999e-07, 3.4658959983900002e+00, 3.5181659230870002e+02 },
    { 3.5031000000000002e-07, 1.6059494160000000e-02, 1.8159247264699999e+01 },
    { 3.4980000000000002e-07, 3.4821429948099998e+00, 8.7311771539500000e+01 },
    { 3.3047000000000002e-07, 4.3065182579999997e+00, 2.2582926841020000e+02 },
    { 3.2809000000000002e-07, 2.3140739538799999e+00, 2.0618554843720000e+02 },
    { 3.2542000000000001e-07, 1.8477780721299999e+00, 2.2041264243879999e+02 },
    { 3.2123999999999998e-07, 1.8403518431999999e+00, 7.2333918012500007e+01 },
    { 3.1136000000000000e-07, 5.1664956467999996e+00, 6.2251425595100002e+01 },
    { 3.0675000000000002e-07, 3.2220704387999999e+00, 7.9235016692200006e+01 },
    { 3.0161000000000001e-07, 2.8353141525600001e+00, 2.7703499374140000e+02 },
    { 2.9270999999999998e-07, 3.9080672361000000e+00, 4.4534181248999998e+00 },
    { 2.9256999999999998e-07, 6.2309320704000004e-01, 1.5253214255120000e+02 },
    { 2.9181999999999998e-07, 3.1881057654000000e+00, 3.6648562929500002e+01 },
    { 2.8570000000000002e-07, 5.6348689848299998e+00, 6.7668051566499997e+01 },
    { 2.7379999999999998e-07, 2.3224788432999999e-01, 3.8133035637799999e+01 },
    { 2.6455999999999998e-07, 4.1401739438000001e-01, 4.2659819087599999e+02 },
    { 2.1586000000000000e-07, 5.4298324173000001e-01, 7.7229279122099996e+01 },
    { 2.0989000000000001e-07, 1.2750164230500001e+00, 7.3818390720799997e+01 },
    { 1.9927000000000001e-07, 8.1036596458999999e-01, 7.5744806413800006e+01 },
    { 1.9908999999999999e-07, 1.1299264852000000e-01, 3.1813937377000001e+00 },
    { 1.9408000000000000e-07, 5.4423897855699996e+00, 7.0328180442399997e+01 },
    { 1.8887000000000000e-07, 2.6068003446699999e+00, 1.0593819301891999e+03 },
    { 1.8561999999999999e-07, 8.5762153591000001e-01, 1.5349535039770001e+02 },
    { 1.8351999999999999e-07, 3.6643092073200001e+00, 7.4893473151899997e+01 },
    { 1.7431999999999999e-07, 3.5513400104800001e+00, 1.1143016149680000e+02 },
    { 1.6838000000000000e-07, 2.4085094099300002e+00, 1.3140394986990000e+02 },
    { 1.6213000000000001e-07, 4.5951764013099998e+00, 1.4659425171800001e+02 },
    { 1.5050000000000000e-07, 5.0059176616000001e-01, 1.8472728735580000e+02 },
    { 1.2767999999999999e-07, 4.4144035900400000e+00, 2.8808069400530002e+02 },
    { 1.2658000000000001e-07, 4.9648357144100004e+00, 7.4669723982700006e+01 },
    { 1.1825000000000000e-07, 5.1362311120799999e+00, 4.9033408917940000e+02 },
    { 1.1567000000000000e-07, 6.1973366609199996e+00, 8.1895145568100006e+01 },
    { 1.1019999999999999e-07, 4.0119996007100003e+00, 4.1101981054399999e+01 },
    { 1.1009000000000000e-07, 1.8542291904899999e+00, 9.2940845831999994e+01 },
    { 1.0653000000000000e-07, 1.4891828351800001e+00, 7.1135470007999997e+00 },
    { 1.0481000000000000e-07, 4.8405338319300002e+00, 5.3680451209540001e+02 },
    { 1.0155000000000001e-07, 5.3802288608200000e+00, 5.2257741809380002e+02 },
    { 9.7010000000000006e-08, 2.4977374112600002e+00, 2.6598929347749998e+02 },
    { 9.6470000000000003e-08, 6.6790581741999999e-01, 1.3703302416240001e+02 },
    { 9.5990000000000001e-08, 6.1604618418800001e+00, 1.9120769491019999e+02 },
    { 9.2560000000000003e-08, 2.0810158582299998e+00, 1.1291463420510000e+02 },
    { 8.9960000000000006e-08, 4.4928083852000000e-01, 9.6872999095099999e+01 },
    { 8.0179999999999994e-08, 3.3996477996599999e+00, 8.0198224538700003e+01 },
    { 7.6129999999999994e-08, 4.0630562491599997e+00, 1.0994568878849999e+02 },
    { 7.4410000000000003e-08, 4.2270752963299998e+00, 4.1555249061209997e+02 },
    { 7.4229999999999997e-08, 4.5419903716599999e+00, 3.4077089204480001e+02 },
    { 6.6850000000000006e-08, 8.5215242388000001e-01, 1.0846121608020000e+02 },
    { 6.6679999999999999e-08, 6.8169549964999998e-01, 6.6704843720000000e+01 },
    { 5.8390000000000002e-08, 2.9395919427999999e+00, 1.4244965013379999e+02 },
    { 5.5789999999999999e-08, 4.8897964186499996e+00, 1.9513984817330001e+02 },
    { 5.5190000000000000e-08, 2.0351331063600000e+00, 2.9764192156090002e+02 },
    { 5.3640000000000003e-08, 4.3609199455300001e+00, 3.3679617512900002e+01 },
    { 5.2560000000000002e-08, 5.0564889123899999e+00, 1.4977853527000001e+01 },
    { 5.2040000000000001e-08, 4.7156330352899998e+00, 4.1644497775600001e+01 },
    { 5.0230000000000000e-08, 1.2647102202500000e+00, 7.0363318461740005e+02 },
    { 4.9780000000000000e-08, 4.5168703792999998e-01, 3.3137100791700000e+01 },
    { 4.9010000000000000e-08, 5.9194496069000002e-01, 2.4476805547999998e+00 },
    { 4.6810000000000000e-08, 3.7945277437299998e+00, 6.1780588578619995e+02 },
    { 4.6760000000000003e-08, 1.5055022297000000e+00, 8.3503713448730002e+02 },
    { 4.4680000000000000e-08, 4.4171542653499998e+00, 1.1642609634290000e+02 },
};

static const VSOP87Term VSOP87A_URANUS_Z2[33] = {
    { 1.4697857999999999e-04, 1.7514916500300000e+00, 7.4781598567299994e+01 },
    { 1.6000440000000000e-05, 3.1415926535900001e+00, 0.0000000000000000e+00 },
    { 2.5713900000000002e-06, 5.9176689529499997e+00, 7.3297125859000005e+01 },
    { 2.4741300000000001e-06, 5.6719795690300003e+00, 1.4956319713459999e+02 },
    { 1.2184000000000001e-06, 7.5865025350000004e-01, 1.1045700263900001e+01 },
    { 1.1362900000000000e-06, 2.2836542855799999e+00, 8.5827298831199997e+01 },
    { 9.4450000000000002e-07, 2.2123556011600001e+00, 7.6266071275599998e+01 },
    { 8.3529999999999997e-07, 1.8092012802899999e+00, 7.0849445304200003e+01 },
    { 6.5242999999999997e-07, 3.6612733106000001e-01, 7.8713751830400000e+01 },
    { 4.7174000000000001e-07, 1.4171199561600001e+00, 1.4844727083000000e+00 },
    { 4.3727000000000000e-07, 1.8301398887200000e+00, 1.3851749687070000e+02 },
    { 3.4299999999999999e-07, 1.5402839123500001e+00, 7.7962992305000000e+01 },
    { 3.3192999999999998e-07, 5.7224102908999996e-01, 7.1600204829600003e+01 },
    { 3.3009000000000001e-07, 5.7522872646299996e+00, 6.3735898303399999e+01 },
    { 3.1720000000000001e-07, 3.1398307809999998e-02, 2.1329909543799999e+02 },
    { 2.7912999999999999e-07, 4.4546913818500000e+00, 3.9321532631000000e+00 },
    { 2.6348999999999999e-07, 3.0472023762000000e+00, 5.2969096509459996e+02 },
    { 2.2889999999999999e-07, 2.7230195995100002e+00, 2.2434479570190001e+02 },
    { 1.3762000000000001e-07, 3.7538925848300000e+00, 1.4563104387150000e+02 },
    { 1.1941999999999999e-07, 7.1017756424000000e-01, 7.1812653150700001e+01 },
    { 1.0375000000000000e-07, 5.4343851476599996e+00, 1.5104766984290001e+02 },
    { 8.5819999999999997e-08, 4.1570377070399998e+00, 1.8159247264699999e+01 },
    { 7.8629999999999997e-08, 4.2500331341899997e+00, 1.4807872442630000e+02 },
    { 7.3000000000000005e-08, 2.0448645913900001e+00, 7.7750543983900002e+01 },
    { 7.1099999999999995e-08, 3.8052362515699998e+00, 4.5490936652729999e+02 },
    { 6.6909999999999995e-08, 4.4514691257700001e+00, 1.6060889739850001e+02 },
    { 6.4500000000000002e-08, 2.1357394882400000e+00, 2.2091400527800001e+01 },
    { 6.2050000000000003e-08, 4.9486758600000000e-02, 7.2333918012500007e+01 },
    { 6.2050000000000003e-08, 2.2661429564600000e+00, 5.6622351302600002e+01 },
    { 6.0940000000000002e-08, 4.9664046760099998e+00, 3.1813937377000001e+00 },
    { 5.7609999999999998e-08, 3.3686947883900000e+00, 5.2690198039499997e+01 },
    { 5.2140000000000002e-08, 5.5490671421900002e+00, 1.2747179660680000e+02 },
    { 4.7909999999999997e-08, 2.2533283202000001e-01, 2.2041264243879999e+02 },
};

static const VSOP87Term VSOP87A_URANUS_Z3[12] = {
    { 4.0696099999999998e-06, 3.1631403445999999e+00, 7.4781598567299994e+01 },
    { 3.2961000000000000e-07, 3.1415926535900001e+00, 0.0000000000000000e+00 },
    { 1.1311000000000000e-07, 4.4438075842200000e+00, 1.4956319713459999e+02 },
    { 9.2439999999999999e-08, 5.9427236998500002e+00, 7.0849445304200003e+01 },
    { 8.5889999999999997e-08, 1.8810390990400001e+00, 7.8713751830400000e+01 },
    { 8.3610000000000006e-08, 1.7270503955600001e+00, 6.3735898303399999e+01 },
    { 8.3120000000000006e-08, 1.1037178033999999e+00, 7.3297125859000005e+01 },
    { 7.9049999999999996e-08, 4.3984657946000000e-01, 7.6266071275599998e+01 },
    { 6.8929999999999996e-08, 6.1207928834300001e+00, 8.5827298831199997e+01 },
    { 5.7970000000000002e-08, 1.8665091414099999e+00, 7.1600204829600003e+01 },
    { 5.5040000000000002e-08, 5.9548514043100003e+00, 7.7962992305000000e+01 },
    { 4.8690000000000001e-08, 4.9959394502299999e+00, 1.1045700263900001e+01 },
};

static const VSOP87Term VSOP87A_URANUS_Z4[1] = {
    { 0.0, 0.0, 0.0 }
};

static const VSOP87Term VSOP87A_URANUS_Z5[1] = {
    { 0.0, 0.0, 0.0 }
};

static const VSOP87Block VSOP87A_URANUS_X[6] = {
    { VSOP87A_URANUS_X0, 1464 },
    { VSOP87A_URANUS_X1, 649 },
    { VSOP87A_URANUS_X2, 249 },
    { VSOP87A_URANUS_X3, 84 },
    { VSOP87A_URANUS_X4, 12 },
    { VSOP87A_URANUS_X5, 1 },
};

static const VSOP87Block VSOP87A_URANUS_Y[6] = {
    { VSOP87A_URANUS_Y0, 1447 },
    { VSOP87A_URANUS_Y1, 659 },
    { VSOP87A_URANUS_Y2, 255 },
    { VSOP87A_URANUS_Y3, 80 },
    { VSOP87A_URANUS_Y4, 12 },
    { VSOP87A_URANUS_Y5, 1 },
};

static const VSOP87Block VSOP87A_URANUS_Z[6] = {
    { VSOP87A_URANUS_Z0, 235 },
    { VSOP87A_URANUS_Z1, 98 },
    { VSOP87A_URANUS_Z2, 33 },
    { VSOP87A_URANUS_Z3, 12 },
    { VSOP87A_URANUS_Z4, 1 },
    { VSOP87A_URANUS_Z5, 1 },
};

static const int32 VSOP87A_URANUS_TRUNC_COUNT[11][3][6] = {
    {
        { 1464, 649, 249, 84, 12, 0 },
        { 1447, 659, 255, 80, 12, 0 },
        { 235, 98, 33, 12, 0, 0 },
    },
    {
        { 1464, 649, 249, 84, 12, 0 },
        { 1447, 659, 255, 80, 12, 0 },
        { 235, 98, 33, 12, 0, 0 },
    },
    {
        { 918, 412, 150, 50, 8, 0 },
        { 917, 416, 146, 52, 8, 0 },
        { 228, 93, 32, 11, 0, 0 },
    },
    {
        { 712, 308, 110, 38, 5, 0 },
        { 715, 310, 110, 38, 5, 0 },
        { 179, 75, 21, 3, 0, 0 },
    },
    {
        { 401, 161, 60, 19, 1, 0 },
        { 393, 156, 57, 20, 0, 0 },
        { 95, 33, 9, 1, 0, 0 },
    },
    {
        { 295, 128, 45, 16, 0, 0 },
        { 290, 121, 45, 16, 0, 0 },
        { 73, 24, 6, 1, 0, 0 },
    },
    {
        { 172, 64, 18, 5, 0, 0 },
        { 174, 65, 18, 5, 0, 0 },
        { 37, 13, 2, 0, 0, 0 },
    },
    {
        { 140, 51, 16, 1, 0, 0 },
        { 135, 52, 16, 1, 0, 0 },
        { 29, 10, 2, 0, 0, 0 },
    },
    {
        { 70, 21, 8, 0, 0, 0 },
        { 69, 21, 9, 0, 0, 0 },
        { 18, 5, 1, 0, 0, 0 },
    },
    {
        { 60, 16, 2, 0, 0, 0 },
        { 58, 16, 3, 0, 0, 0 },
        { 13, 3, 1, 0, 0, 0 },
    },
    {
        { 26, 5, 0, 0, 0, 0 },
        { 26, 5, 0, 0, 0, 0 },
        { 5, 1, 0, 0, 0, 0 },
    },
};

