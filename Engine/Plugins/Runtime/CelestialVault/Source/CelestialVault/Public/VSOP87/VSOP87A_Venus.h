// Copyright Epic Games, Inc. All Rights Reserved.

// Auto-generated VSOP87A header for VENUS (rectangular XYZ, J2000)
// Source: IMCCE/BDL VSOP87A.venus file
// Generated by vsop87a_header_gen_imcce.py

#pragma once

#include "VSOP87A_Common.h"

static const VSOP87Term VSOP87A_VENUS_X0[548] = {
    { 7.2211281391000004e-01, 3.1757583636100000e+00, 1.0213285546211000e+04 },
    { 4.8644801800000002e-03, 0.0000000000000000e+00, 0.0000000000000000e+00 },
    { 2.4450047399999998e-03, 4.0556661386100004e+00, 2.0426571092422000e+04 },
    { 2.8002810000000000e-05, 3.3147492492000002e-01, 2.3528661537717999e+03 },
    { 1.9496689999999998e-05, 4.2319601680099996e+00, 1.5773435424478000e+03 },
    { 1.2417170000000000e-05, 4.9357378705799997e+00, 3.0639856638632999e+04 },
    { 1.1622580000000000e-05, 2.8795824618900001e+00, 1.8073704938650200e+04 },
    { 1.0466900000000000e-05, 1.7543492041300000e+00, 6.2830758499914000e+03 },
    { 7.6429299999999997e-06, 5.9379588767000002e-01, 5.2969096509459996e+02 },
    { 6.6946100000000001e-06, 1.4572122884200001e+00, 1.4143495242430599e+04 },
    { 6.5719499999999998e-06, 5.0086450258000004e-01, 8.6359420037632008e+03 },
    { 5.5907399999999999e-06, 1.1655478330100000e+00, 2.2003914634869801e+04 },
    { 5.4677799999999999e-06, 2.7149088412800002e+00, 1.1790629088658799e+04 },
    { 4.7644499999999998e-06, 5.8430978284000004e+00, 1.0186987226411200e+04 },
    { 4.7446600000000002e-06, 3.6499116350400000e+00, 1.0239583866010800e+04 },
    { 4.0898799999999999e-06, 3.9272543199299998e+00, 7.7552261132399997e+02 },
    { 2.9774200000000002e-06, 5.6565581116599999e+00, 5.5075532386674004e+03 },
    { 2.8705899999999998e-06, 2.7957895695800001e+00, 9.6835945811164001e+03 },
    { 2.7102200000000000e-06, 2.6237778031999999e+00, 1.9896880127327400e+04 },
    { 2.6882200000000001e-06, 4.2000307859000002e-01, 1.0742976511305600e+04 },
    { 2.5448000000000002e-06, 5.0996141324100002e+00, 9.1539036160218002e+03 },
    { 2.4110299999999999e-06, 5.8062762709799998e+00, 1.0988808157535001e+04 },
    { 2.1414899999999999e-06, 7.4884072597999995e-01, 1.0021837280099400e+04 },
    { 2.0930300000000001e-06, 2.4712991943499998e+00, 1.0404733812322600e+04 },
    { 1.7504000000000000e-06, 3.5316397756000000e+00, 9.4377629348869996e+03 },
    { 1.7498500000000001e-06, 6.1270491139100001e+00, 1.9144826611159999e+02 },
    { 1.5437900000000001e-06, 5.3560770439000001e+00, 4.7057323075435997e+03 },
    { 1.4745499999999999e-06, 5.5578002264100004e+00, 1.9651048481098002e+04 },
    { 1.2299000000000001e-06, 1.5892543937400001e+00, 1.0593819301891999e+03 },
    { 8.2633000000000001e-07, 4.1403837209800001e+00, 1.5720838784878400e+04 },
    { 6.9503999999999996e-07, 9.2841539819999999e-02, 9.1039069941176003e+03 },
    { 6.2363000000000003e-07, 2.5916761092899998e+00, 2.5934124331089399e+04 },
    { 6.0671999999999997e-07, 3.7838056833599998e+00, 4.5519534970588002e+03 },
    { 5.8863000000000005e-07, 3.0582541673499999e+00, 2.0618019358533598e+04 },
    { 5.7904999999999999e-07, 3.0911217779800002e+00, 1.1322664098304400e+04 },
    { 5.7243999999999996e-07, 6.4178511675000005e-01, 3.1546870848956000e+03 },
    { 4.9846000000000003e-07, 4.7455900463600003e+00, 8.6242126509272002e+03 },
    { 4.7628999999999999e-07, 8.6806456391999998e-01, 2.1329909543799999e+02 },
    { 4.6372000000000003e-07, 5.2050501625000001e-01, 1.0192510150718599e+04 },
    { 4.6246999999999998e-07, 2.6894090098299999e+00, 1.0234060941703399e+04 },
    { 4.5105000000000001e-07, 2.0506227679400002e+00, 2.9580474708443799e+04 },
    { 4.4351999999999999e-07, 1.6290535538900000e+00, 1.9367189162232800e+04 },
    { 4.0142000000000000e-07, 3.2113728266499999e+00, 9.4114646150871995e+03 },
    { 3.9831000000000002e-07, 1.6391090652800000e+00, 8.0182093112380005e+02 },
    { 3.6963000000000001e-07, 4.3947179660900000e+00, 7.0585984613153996e+03 },
    { 3.5988000000000002e-07, 3.9166138867400000e+00, 9.9999864507729999e+03 },
    { 3.5335000000000002e-07, 5.5825767941600004e+00, 1.0426584641649000e+04 },
    { 3.1395999999999999e-07, 5.7056030642100000e+00, 1.5874617595363199e+04 },
    { 2.9387000000000002e-07, 6.2001200331000001e+00, 1.1015106477334801e+04 },
    { 2.7029999999999998e-07, 1.8225560898399999e+00, 1.1093785520934000e+03 },
    { 2.4531999999999998e-07, 5.4700974886699996e+00, 7.8604193924392002e+03 },
    { 2.3556000000000000e-07, 4.9621972694199998e+00, 2.1228392023545799e+04 },
    { 2.1701999999999999e-07, 2.5122930720999999e+00, 1.3367972631106601e+04 },
    { 2.1313999999999999e-07, 2.9496204210700001e+00, 3.1283887650958000e+03 },
    { 2.0347999999999999e-07, 5.0298205800999995e-01, 1.0206171999210201e+04 },
    { 2.0289000000000001e-07, 2.7024166150900002e+00, 1.0220399093211799e+04 },
    { 2.0165000000000001e-07, 3.0341831384800000e+00, 3.9302096962196001e+03 },
    { 1.8918000000000000e-07, 5.3270783233600003e+00, 2.6298319799800002e+01 },
    { 1.8806000000000000e-07, 2.0300009280900002e+00, 7.1135470007999997e+00 },
    { 1.7623000000000001e-07, 1.3461569203299999e+00, 1.8837498197138189e+04 },
    { 1.7193000000000000e-07, 2.1214926889999999e-01, 3.5320606928113998e+03 },
    { 1.6091999999999999e-07, 2.3571288411900002e+00, 2.0213271996984000e+04 },
    { 1.5237000000000001e-07, 4.3845159303500001e+00, 9.8303890139878004e+03 },
    { 1.4921000000000000e-07, 4.0722927719399999e+00, 2.9864334027309000e+04 },
    { 1.4674999999999999e-07, 5.1215944203500001e+00, 1.0596182078434200e+04 },
    { 1.4632999999999999e-07, 2.1030662024300000e+00, 7.0848967811151997e+03 },
    { 1.4040000000000001e-07, 3.8638164040200000e+00, 2.3581258177317599e+04 },
    { 1.2814999999999999e-07, 3.8147867136400002e+00, 1.7298182327326202e+04 },
    { 1.2814000000000001e-07, 2.0499647671100001e+00, 1.5890728952838001e+03 },
    { 1.2650000000000000e-07, 4.5862972498500003e+00, 9.7866873553349997e+03 },
    { 1.1612000000000000e-07, 6.1959644042299997e+00, 3.3406124266997999e+03 },
    { 1.1103000000000000e-07, 3.7701702625900002e+00, 2.8286990484861199e+04 },
    { 1.0763000000000000e-07, 1.2330242612400000e+00, 9.9294262273458007e+03 },
    { 1.0377999999999999e-07, 1.9776866878299999e+00, 1.0497144865076199e+04 },
    { 9.8819999999999994e-08, 5.0546866527100001e+00, 2.1535949644515400e+04 },
    { 9.3909999999999999e-08, 6.2349717206999999e-01, 1.1272667476400200e+04 },
    { 8.9340000000000005e-08, 1.6784994345299999e+00, 2.9050783743349199e+04 },
    { 8.7919999999999996e-08, 4.6879154922999999e-01, 1.0207762621903599e+04 },
    { 8.7859999999999994e-08, 2.7418960990799999e+00, 1.0218808470518399e+04 },
    { 7.9129999999999995e-08, 3.4675286385700002e+00, 1.2566151699982800e+04 },
    { 7.4980000000000000e-08, 5.8162125698900002e+00, 4.0853142184844000e+04 },
    { 7.4229999999999997e-08, 4.1614758247300001e+00, 1.0175257873575199e+04 },
    { 7.3860000000000001e-08, 5.3324173147099998e+00, 1.0251313218846801e+04 },
    { 7.3609999999999995e-08, 8.0024676602000000e-01, 1.6496361396202399e+04 },
    { 7.2419999999999996e-08, 1.8019941143000001e-01, 2.5158601719765400e+04 },
    { 6.6559999999999995e-08, 4.3804635222600004e+00, 8.0945216858326003e+03 },
    { 6.5309999999999994e-08, 1.1704061140599999e+00, 2.0419457545421192e+04 },
    { 5.9820000000000002e-08, 2.1650759865200002e+00, 4.2659819087599999e+02 },
    { 5.9580000000000001e-08, 3.3507108266000002e-01, 6.6812248533995999e+03 },
    { 5.8710000000000001e-08, 2.0282679581899998e+00, 3.2217200181080800e+04 },
    { 5.8640000000000001e-08, 2.0073358491000001e-01, 5.6613320491521999e+03 },
    { 5.7170000000000002e-08, 2.3077563061099999e+00, 2.4356780788641601e+04 },
    { 5.6370000000000001e-08, 2.3802539133599998e+00, 3.8289653222319998e+02 },
    { 5.4949999999999999e-08, 3.2934784330500002e+00, 1.7085958665722199e+04 },
    { 5.4359999999999998e-08, 5.4185971439999996e+00, 3.3794543723528601e+04 },
    { 5.3319999999999997e-08, 4.4405858688300004e+00, 2.6087903141574199e+04 },
    { 5.0810000000000003e-08, 2.7757465792399998e+00, 2.0452869412221800e+04 },
    { 4.9399999999999999e-08, 3.6588350104499998e+00, 3.1491641605882000e+03 },
    { 4.6889999999999998e-08, 2.7530100708999999e-01, 1.8849227549974199e+04 },
    { 4.5699999999999999e-08, 2.6325780367200000e+00, 1.4765239043269799e+04 },
    { 4.3270000000000003e-08, 3.4042959882699999e+00, 2.3958631785233400e+04 },
    { 3.9430000000000003e-08, 3.0891245316700000e+00, 4.7320306273433998e+03 },
    { 3.8850000000000000e-08, 1.6151322832199999e+00, 2.9088811415985001e+04 },
    { 3.8659999999999997e-08, 2.8620801272300000e+00, 1.3745346239022399e+04 },
    { 3.7720000000000003e-08, 1.0168912468700000e+00, 1.8307807232043600e+04 },
    { 3.6349999999999998e-08, 3.5356507493000000e-01, 1.5510452226479999e+03 },
    { 3.2210000000000002e-08, 4.3638422741999999e-01, 2.0400272772622200e+04 },
    { 2.9490000000000001e-08, 2.8236312975900000e+00, 1.0198033075026000e+04 },
    { 2.9440000000000001e-08, 3.8707317908000000e-01, 1.0228538017396000e+04 },
    { 2.7470000000000000e-08, 1.7815023702499999e+00, 5.4812549188676003e+03 },
    { 2.7420000000000000e-08, 5.3028014266600003e+00, 2.7511467873537200e+04 },
    { 2.6490000000000001e-08, 3.6961004486000002e+00, 9.5805018068978006e+03 },
    { 2.5489999999999999e-08, 2.8598070062100001e+00, 6.8726731195111997e+03 },
    { 2.4929999999999999e-08, 5.9544015105600003e+00, 1.7277406931833801e+04 },
    { 2.4710000000000001e-08, 5.7472442242000001e+00, 1.0846069285524200e+04 },
    { 2.4310000000000001e-08, 3.8558540160799999e+00, 3.0110165673538399e+04 },
    { 2.4240000000000001e-08, 9.6902079851000000e-01, 2.0956262057516600e+04 },
    { 2.3720000000000000e-08, 5.3579188154599997e+00, 9.5733882598969994e+03 },
    { 2.2939999999999999e-08, 3.8956627824600001e+00, 3.4425749449653999e+03 },
    { 2.2780000000000000e-08, 4.3918994269000002e+00, 1.0110192771992401e+04 },
    { 2.2780000000000000e-08, 5.1019866204700000e+00, 1.0316378320429600e+04 },
    { 2.2460000000000001e-08, 3.0397981504199998e+00, 3.3019021112204602e+04 },
    { 2.2119999999999999e-08, 5.1657277552399998e+00, 1.4945316173554400e+04 },
    { 2.2049999999999999e-08, 3.6099239603300002e+00, 5.3680451209540001e+02 },
    { 2.1880000000000001e-08, 5.9820892805200003e+00, 9.9674538999815995e+03 },
    { 2.1179999999999998e-08, 3.5115994191700000e+00, 1.0459117192440401e+04 },
    { 2.1039999999999998e-08, 7.1768958426999996e-01, 5.2257741809380002e+02 },
    { 2.0570000000000002e-08, 5.6837576345999996e-01, 3.7724753419748202e+04 },
    { 2.0500000000000002e-08, 4.2574851306000001e+00, 9.6907081281171995e+03 },
    { 2.0210000000000000e-08, 1.0660278636400000e+00, 1.9999972901545989e+04 },
    { 2.0109999999999999e-08, 5.6822837266000004e-01, 2.1202093703745999e+04 },
    { 1.9869999999999999e-08, 1.9715036385100000e+00, 1.0309277421860000e+02 },
    { 1.9620000000000000e-08, 5.2348018662999998e+00, 1.0735862964304801e+04 },
    { 1.8730000000000000e-08, 1.5026181383899999e+00, 3.0213258447757002e+04 },
    { 1.8320000000000001e-08, 5.5622426207700002e+00, 2.0809467624645200e+04 },
    { 1.8270000000000001e-08, 3.0685108348600001e+00, 7.0641213856227996e+03 },
    { 1.8119999999999999e-08, 4.0045543686200000e+00, 2.2779437246193800e+04 },
    { 1.7529999999999999e-08, 3.9882451629999999e-01, 1.3553897972910800e+04 },
    { 1.6590000000000002e-08, 1.5730969234000001e+00, 2.0235122826310399e+04 },
    { 1.5920000000000000e-08, 2.4895121235300000e+00, 1.0288067144778301e+04 },
    { 1.5920000000000000e-08, 7.2118861666000000e-01, 1.0138503947643700e+04 },
    { 1.5600000000000001e-08, 3.6404331782999999e-01, 9.8968936765544004e+03 },
    { 1.5600000000000001e-08, 2.8466574223599999e+00, 1.0529677415867600e+04 },
    { 1.5530000000000001e-08, 4.8845999558599997e+00, 9.1610171630225996e+03 },
    { 1.5410000000000000e-08, 4.0222002279600000e+00, 6.4897765872880000e+03 },
    { 1.4929999999999999e-08, 1.4065363532499999e+00, 2.2805735565993600e+04 },
    { 1.4839999999999999e-08, 2.7408879719099999e+00, 3.7235089589230001e+03 },
    { 1.4660000000000000e-08, 2.0423932063199999e+00, 9.1467900690210008e+03 },
    { 1.4020000000000001e-08, 5.0043012706600001e+00, 1.0639883737087001e+04 },
    { 1.3599999999999999e-08, 5.7999889968399998e+00, 1.0232955307110789e+04 },
    { 1.3599999999999999e-08, 3.6938970505299999e+00, 1.0193615785311211e+04 },
    { 1.3420000000000000e-08, 4.4662507267100002e+00, 3.6949230808424203e+04 },
    { 1.2709999999999999e-08, 5.7093451126600003e+00, 1.3936794505133999e+04 },
    { 1.2590000000000001e-08, 4.2356000005499999e+00, 2.3791644735716000e+03 },
    { 1.2380000000000001e-08, 3.7910250199800002e+00, 2.0775395492400001e+01 },
    { 1.2260000000000000e-08, 5.5076884772600003e+00, 7.4781598567299994e+01 },
    { 1.1900000000000001e-08, 4.5280315224400001e+00, 3.9264069289560197e+04 },
    { 1.1790000000000000e-08, 6.0053897534100003e+00, 1.4919017853754600e+04 },
    { 1.1580000000000000e-08, 3.8590245772300000e+00, 1.0632770190086199e+04 },
    { 1.1390000000000000e-08, 2.4322939978200000e+00, 2.1187638603783998e+03 },
    { 1.1290000000000001e-08, 5.7765823627100001e+00, 9.7938009023358009e+03 },
    { 1.1260000000000000e-08, 1.8348427473400000e+00, 2.4583164622940001e+02 },
    { 1.0130000000000000e-08, 2.1088579262599998e+00, 1.0419471094648199e+04 },
    { 1.0099999999999999e-08, 1.0956533286200001e+00, 1.0007099997773799e+04 },
    { 9.9599999999999995e-09, 4.5384942621000004e+00, 1.3341674311306801e+04 },
    { 9.6799999999999997e-09, 1.5966205938600000e+00, 6.3989728631399998e+02 },
    { 9.0599999999999997e-09, 2.9625818950999999e-01, 3.1441677569756801e+04 },
    { 8.6499999999999997e-09, 1.9960933523800000e+00, 4.1654963115967803e+04 },
    { 8.4599999999999993e-09, 6.3826672604000001e-01, 1.8875525869773999e+04 },
    { 8.4499999999999996e-09, 6.0817687756199996e+00, 9.6764810341156008e+03 },
    { 8.2299999999999999e-09, 5.8897483458600002e+00, 4.0879440504643797e+04 },
    { 7.9500000000000001e-09, 3.4243229189600002e+00, 1.0750090058306399e+04 },
    { 7.8800000000000001e-09, 1.8032696972000001e-01, 7.9945284420241996e+03 },
    { 7.8700000000000003e-09, 5.8900246316500002e+00, 1.9889766580326592e+04 },
    { 7.8600000000000006e-09, 2.0832323818399998e+00, 1.1764330768858999e+04 },
    { 7.6999999999999995e-09, 6.8487491198999995e-01, 7.8341210726394002e+03 },
    { 7.6999999999999995e-09, 2.8848488561300001e+00, 2.6735945262213201e+04 },
    { 7.6000000000000002e-09, 4.0181078140800004e+00, 7.5648307207380003e+03 },
    { 7.4400000000000000e-09, 2.5152663290200001e+00, 1.9903993674328191e+04 },
    { 7.1500000000000003e-09, 5.2419484662600002e+00, 1.6983996147456601e+04 },
    { 6.9800000000000003e-09, 3.4035851078700001e+00, 3.6147409877300401e+04 },
    { 6.8900000000000000e-09, 4.6051805980699996e+00, 1.0664954771900000e+03 },
    { 6.7599999999999998e-09, 5.2022255242900002e+00, 2.0529663866640589e+04 },
    { 6.6100000000000001e-09, 5.5277912730000003e-02, 1.1624747044077999e+03 },
    { 6.2400000000000000e-09, 2.8476778965900000e+00, 3.9793760254654801e+04 },
    { 6.2099999999999999e-09, 1.1705502882200001e+00, 5.2175806283148399e+04 },
    { 6.0500000000000004e-09, 4.6344575392599996e+00, 1.0059506735726200e+04 },
    { 5.9300000000000000e-09, 4.8595056430800003e+00, 1.0367064356695801e+04 },
    { 5.9200000000000002e-09, 6.5691971752000000e-01, 1.7778116266948990e+04 },
    { 5.6900000000000000e-09, 1.1163292037699999e+00, 1.9786673806107989e+04 },
    { 5.5200000000000000e-09, 2.1494173809800001e+00, 2.8521092778254599e+04 },
    { 5.4800000000000001e-09, 1.6057055372700000e+00, 2.7197281693667599e+04 },
    { 5.4800000000000001e-09, 5.6890492432400004e+00, 8.5211198767086007e+03 },
    { 5.4400000000000002e-09, 3.1609249325800000e+00, 1.1020632121940000e+02 },
    { 5.1499999999999998e-09, 2.9222914485099998e+00, 1.2432042650397800e+04 },
    { 5.1400000000000000e-09, 5.0689452507099997e+00, 8.6622403235630009e+03 },
    { 5.1099999999999999e-09, 1.0294488740800001e+00, 4.4809650200863398e+04 },
    { 4.8900000000000003e-09, 4.2610262972499999e+00, 3.0666154958432799e+04 },
    { 4.6900000000000001e-09, 5.4883930577699997e+00, 8.6313261979279996e+03 },
    { 4.6600000000000000e-09, 5.2326130162400002e+00, 6.7707106012455997e+03 },
    { 4.6600000000000000e-09, 5.3054807259699999e+00, 3.8133035637799999e+01 },
    { 4.5999999999999998e-09, 6.9107495801999996e-01, 1.1802358441494800e+04 },
    { 4.5399999999999996e-09, 7.9241873074000002e-01, 1.0042612675591799e+04 },
    { 4.4400000000000004e-09, 2.4182820690300000e+00, 1.0383958416830201e+04 },
    { 4.4400000000000004e-09, 1.7341049466600000e+00, 2.2187571041868000e+03 },
    { 4.3599999999999998e-09, 1.8569348568499999e+00, 9.6389407478762005e+03 },
    { 4.3100000000000002e-09, 1.7299498947300000e+00, 8.6170991039264009e+03 },
    { 4.2700000000000004e-09, 2.6696545409499999e+00, 1.3655860491176400e+04 },
    { 4.2199999999999999e-09, 1.3999148798000001e+00, 1.0787630344545800e+04 },
    { 4.1599999999999997e-09, 1.7260569113099999e+00, 1.0225014899047001e+04 },
    { 4.1599999999999997e-09, 1.4846438288799999e+00, 1.0201556193375000e+04 },
    { 4.0400000000000001e-09, 3.9859308097900001e+00, 2.0351789493854700e+04 },
    { 4.0000000000000002e-09, 9.8930698980999998e-01, 1.0522683831884001e+03 },
    { 3.9899999999999997e-09, 4.2491284265499996e+00, 1.0211801073502700e+04 },
    { 3.9899999999999997e-09, 5.2447576208199997e+00, 1.0214770018919300e+04 },
    { 3.9799999999999999e-09, 3.6483672922300001e+00, 9.8359119382951994e+03 },
    { 3.9000000000000002e-09, 3.4241594637600001e+00, 4.5585172812187397e+04 },
    { 3.8799999999999998e-09, 2.2816806595300001e+00, 2.9999959352319001e+04 },
    { 3.8499999999999997e-09, 1.3686788655300000e+00, 2.0180739446192601e+04 },
    { 3.8099999999999999e-09, 2.2534351228999999e-01, 1.9317192540328600e+04 },
    { 3.7900000000000004e-09, 5.8455188772600000e+00, 1.0590659154126801e+04 },
    { 3.7700000000000000e-09, 5.6732548344199998e+00, 1.4956319713459999e+02 },
    { 3.6600000000000002e-09, 3.2103953505999998e+00, 2.8385931886520001e+02 },
    { 3.6500000000000000e-09, 1.4246784510199999e+00, 1.0348910871221000e+04 },
    { 3.6500000000000000e-09, 1.7860222891699999e+00, 1.0077660221201000e+04 },
    { 3.6399999999999998e-09, 4.2257382592799999e+00, 3.8734378324465601e+04 },
    { 3.6300000000000001e-09, 4.2651447418900004e+00, 3.0831304904744600e+04 },
    { 3.5899999999999998e-09, 9.1781389599999999e-02, 2.7490692478044799e+04 },
    { 3.5400000000000002e-09, 2.8430643721100002e+00, 1.0323491867430401e+04 },
    { 3.5400000000000002e-09, 3.6763636808000000e-01, 1.0103079224991599e+04 },
    { 3.4299999999999999e-09, 3.9243856620000002e+00, 2.0618554843720000e+02 },
    { 3.4299999999999999e-09, 4.6429605611099998e+00, 8.1062510386685999e+03 },
    { 3.3600000000000000e-09, 7.4367409336000001e-01, 2.2041264243879999e+02 },
    { 3.2700000000000001e-09, 6.1177415424400001e+00, 9.3600891644589992e+03 },
    { 3.2400000000000000e-09, 3.9878291188300001e+00, 3.1749235190726398e+04 },
    { 3.2200000000000000e-09, 2.4557648958899998e+00, 4.8739859897082999e+04 },
    { 3.1500000000000001e-09, 1.8704955440000001e-02, 2.0639870187860000e+04 },
    { 3.1399999999999999e-09, 1.4008059966299999e+00, 2.0405795696929599e+04 },
    { 3.0899999999999999e-09, 5.6705243050099998e+00, 6.3093741697912001e+03 },
    { 3.0800000000000001e-09, 4.6119280399000001e-01, 3.7410567239878597e+04 },
    { 3.0600000000000002e-09, 1.1258727245100000e+00, 2.9577158944766002e+03 },
    { 3.0000000000000000e-09, 5.6808805937500004e+00, 2.4150080051345001e+04 },
    { 2.9499999999999999e-09, 5.7514537470100002e+00, 3.4596364654652403e+04 },
    { 2.8800000000000000e-09, 1.8008346621900000e+00, 3.5371887265976402e+04 },
    { 2.8299999999999999e-09, 5.5947506165499998e+00, 2.6709646942413401e+04 },
    { 2.8299999999999999e-09, 5.5346924550000001e+00, 3.9039113764198000e+03 },
    { 2.8100000000000000e-09, 7.8332222977999999e-01, 9.3997352622512008e+03 },
    { 2.7200000000000001e-09, 4.8509227403699997e+00, 1.2320320053753399e+04 },
    { 2.7099999999999999e-09, 3.8564866896400001e+00, 9.7453205558566005e+03 },
    { 2.6700000000000001e-09, 4.3011842327299998e+00, 3.7737360791579999e+02 },
    { 2.6400000000000000e-09, 5.1612935887800004e+00, 7.6266071275599998e+01 },
    { 2.6299999999999998e-09, 3.1613027587600002e+00, 2.0433684639422801e+04 },
    { 2.6099999999999999e-09, 5.5410777795000001e+00, 6.3278373931320004e+02 },
    { 2.5800000000000002e-09, 5.2299993427300002e+00, 2.9573361161443001e+04 },
    { 2.5800000000000002e-09, 2.5233948838400000e+00, 9.8643611257630000e+03 },
    { 2.5500000000000001e-09, 4.0346464175300003e+00, 9.9928729037722005e+03 },
    { 2.5500000000000001e-09, 5.4592396298399999e+00, 1.0433698188649800e+04 },
    { 2.5500000000000001e-09, 2.4274581767000001e+00, 1.1026835830170799e+04 },
    { 2.5399999999999999e-09, 5.6374522364799997e+00, 1.0681250536565400e+04 },
    { 2.5399999999999999e-09, 3.1593108474700000e+00, 1.9264096388014201e+04 },
    { 2.4699999999999999e-09, 1.9080870776900001e+00, 2.9587588255444600e+04 },
    { 2.4600000000000002e-09, 6.8740438300999995e-01, 1.0562209966659000e+04 },
    { 2.4000000000000000e-09, 1.3086913881200000e+00, 1.6522659716002199e+04 },
    { 2.3400000000000002e-09, 3.8201524473899999e+00, 1.9624750161298201e+04 },
    { 2.3300000000000000e-09, 4.3507853346500003e+00, 9.2641099372412009e+03 },
    { 2.2999999999999999e-09, 4.9140406856099998e+00, 5.7434479833479998e+02 },
    { 2.2800000000000000e-09, 4.4581403195200000e+00, 9.4917560896980001e+02 },
    { 2.2400000000000001e-09, 4.7824423312900004e+00, 1.9360075615231988e+04 },
    { 2.2100000000000000e-09, 5.6282855969000001e-01, 3.8027672635800002e+01 },
    { 2.1400000000000001e-09, 5.9816494895000005e-01, 1.2592450019782600e+04 },
    { 2.1200000000000001e-09, 3.1604535307899999e+00, 1.0206656987310989e+04 },
    { 2.1200000000000001e-09, 5.0247209399999999e-02, 1.0219914105111009e+04 },
    { 2.1100000000000000e-09, 1.8276464349999999e-01, 1.0249313412888399e+04 },
    { 2.1100000000000000e-09, 3.0279360966900000e+00, 1.0177257679533601e+04 },
    { 2.1100000000000000e-09, 5.1152933498299999e+00, 1.1162461155180799e+04 },
    { 2.1100000000000000e-09, 6.1625477108600002e+00, 1.1375760250618800e+04 },
    { 2.1000000000000002e-09, 5.1515697589900000e+00, 2.2645328196608789e+04 },
    { 2.0599999999999999e-09, 3.3746911985199999e+00, 3.0426557543194998e+04 },
    { 2.0500000000000002e-09, 3.9057504556299998e+00, 5.2670069593302600e+04 },
    { 2.0200000000000001e-09, 8.2731831246999998e-01, 1.6894510399610601e+04 },
    { 1.9800000000000002e-09, 3.2740227562599999e+00, 2.7991401813159999e+04 },
    { 1.9700000000000000e-09, 9.6052683602000000e-01, 1.6921656695024001e+03 },
    { 1.9599999999999998e-09, 4.6937286380099996e+00, 9.7795738083342003e+03 },
    { 1.9599999999999998e-09, 4.8749936967699998e+00, 4.9515382508406998e+04 },
    { 1.9599999999999998e-09, 3.4437065026200000e+00, 8.4608283475120004e+02 },
    { 1.9599999999999998e-09, 2.2733342268999999e+00, 1.2934842235826000e+03 },
    { 1.9399999999999999e-09, 9.0820651456000001e-01, 3.8526574350871997e+04 },
    { 1.9399999999999999e-09, 2.2671178523900002e+00, 8.8375117463652005e+03 },
    { 1.9300000000000002e-09, 2.3964231522800001e+00, 1.7468855197945399e+04 },
    { 1.8899999999999999e-09, 6.6769513205999997e-01, 9.2569963902403997e+03 },
    { 1.8600000000000000e-09, 4.5370457421200001e+00, 1.0251418581848800e+04 },
    { 1.8600000000000000e-09, 4.9568403052500001e+00, 1.0175152510573200e+04 },
    { 1.8600000000000000e-09, 4.8758676127199996e+00, 1.7271884007526402e+04 },
    { 1.7800000000000001e-09, 5.0910605504999995e-01, 1.0949162059742799e+04 },
    { 1.7800000000000001e-09, 2.7015946851399999e+00, 9.4774090326791993e+03 },
    { 1.7700000000000001e-09, 1.2290786955000001e-01, 2.0316364771202589e+04 },
    { 1.7400000000000000e-09, 4.7853563393999998e+00, 4.0077619573520002e+04 },
    { 1.7300000000000000e-09, 1.3327233480600000e+00, 8.7344189721465991e+03 },
    { 1.7200000000000001e-09, 1.8914957545200000e+00, 1.4227094001599999e+01 },
    { 1.6900000000000000e-09, 5.7191820167599996e+00, 1.1265553929399401e+04 },
    { 1.6800000000000000e-09, 5.3358429469099997e+00, 7.9914289116139998e+03 },
    { 1.6600000000000001e-09, 3.7516214047099998e+00, 1.0263042571682799e+04 },
    { 1.6600000000000001e-09, 5.7422646426600004e+00, 1.0163528520739201e+04 },
    { 1.6200000000000000e-09, 5.5919923319300002e+00, 7.2555696517344004e+03 },
    { 1.6200000000000000e-09, 1.6937460480600000e+00, 1.0063722349076401e+04 },
    { 1.5600000000000000e-09, 4.1908112878699999e+00, 2.0388438056784202e+04 },
    { 1.5500000000000000e-09, 8.5940335797000000e-01, 8.2377930003549991e+03 },
    { 1.5400000000000001e-09, 7.2358681866999996e-01, 1.0199058452209400e+04 },
    { 1.5400000000000001e-09, 2.4871139215200002e+00, 1.0227512640212601e+04 },
    { 1.5199999999999999e-09, 4.6073037654300002e+00, 1.8830384650137388e+04 },
    { 1.5100000000000000e-09, 8.4241735171999998e-01, 1.0245818097002400e+04 },
    { 1.5100000000000000e-09, 2.3682833884700001e+00, 1.0180752995419600e+04 },
    { 1.4900000000000000e-09, 2.2885893414499998e+00, 1.9374302709233601e+04 },
    { 1.4900000000000000e-09, 2.1856463164100002e+00, 1.0135611775783000e+04 },
    { 1.4900000000000000e-09, 1.0250544237799999e+00, 1.0290959316639000e+04 },
    { 1.4400000000000000e-09, 3.9838244358799999e+00, 2.4383079108441401e+04 },
    { 1.4400000000000000e-09, 3.8269494216700002e+00, 9.9858094134219991e+03 },
    { 1.4400000000000000e-09, 5.6669366257000000e+00, 1.0440761678999999e+04 },
    { 1.4300000000000000e-09, 1.4033287678199999e+00, 2.0447346487914401e+04 },
    { 1.4300000000000000e-09, 1.5604854250700000e+00, 1.8844611744138991e+04 },
    { 1.3999999999999999e-09, 3.3029139369999999e-01, 1.0191434716884600e+04 },
    { 1.3999999999999999e-09, 2.8804093464900000e+00, 1.0235136375537400e+04 },
    { 1.3800000000000000e-09, 3.8495137130000001e-01, 1.0217217699474100e+04 },
    { 1.3800000000000000e-09, 2.8257493688899999e+00, 1.0209353392947900e+04 },
    { 1.3600000000000001e-09, 6.0774237037000001e+00, 5.9992165311261997e+03 },
    { 1.3200000000000000e-09, 5.3326011521299996e+00, 5.6600279289522201e+04 },
    { 1.3100000000000000e-09, 5.3953011121800003e+00, 8.1016352328333996e+03 },
    { 1.3100000000000000e-09, 3.2186997586300001e+00, 4.1760413424489998e+03 },
    { 1.3000000000000001e-09, 5.5717194149499996e+00, 1.0236060747661801e+04 },
    { 1.3000000000000001e-09, 3.9221666324200002e+00, 1.0190510344760200e+04 },
    { 1.2700000000000000e-09, 2.3349199926100002e+00, 4.2456784047091598e+04 },
    { 1.2600000000000000e-09, 3.8207381134500000e+00, 2.0277007895287399e+04 },
    { 1.2500000000000000e-09, 2.3513004712500001e+00, 1.2188778092066999e+04 },
    { 1.2199999999999999e-09, 5.4324679971899998e+00, 2.6367254726370002e+03 },
    { 1.1700000000000001e-09, 9.3777977091999998e-01, 1.5819593482830001e+03 },
    { 1.1400000000000000e-09, 4.1618405638600002e+00, 3.2858613742819791e+04 },
    { 1.1300000000000000e-09, 4.2932342565299999e+00, 3.9321532631000000e+00 },
    { 1.1200000000000001e-09, 2.4705681440800000e+00, 2.0206158449983192e+04 },
    { 1.1100000000000001e-09, 2.9335676907599999e+00, 4.3071899289030800e+04 },
    { 1.0700000000000000e-09, 4.6545926544700000e+00, 2.0690068349066000e+03 },
    { 1.0600000000000001e-09, 1.5752366213100000e+00, 1.9573374710669988e+04 },
    { 1.0500000000000001e-09, 3.9651207622900002e+00, 1.9793787353108801e+04 },
    { 1.0300000000000000e-09, 5.1524033600300001e+00, 1.0244199671846000e+04 },
    { 1.0300000000000000e-09, 4.3414826873400001e+00, 1.0182371420576001e+04 },
    { 1.0100000000000000e-09, 5.2643771325099999e+00, 2.0043674560198800e+04 },
    { 1.0100000000000000e-09, 3.2420649050000000e-02, 5.3445592204626599e+04 },
    { 1.0000000000000001e-09, 5.0710688320899999e+00, 1.5961864422845999e+03 },
    { 1.0000000000000001e-09, 4.4404700059900000e+00, 3.8500276031072201e+04 },
    { 1.0000000000000001e-09, 2.6403193335099999e+00, 2.8947690969130599e+04 },
    { 9.9000000000000010e-10, 4.3274578100200003e+00, 2.4978524589480799e+04 },
    { 9.9000000000000010e-10, 2.8024642508399999e+00, 2.6484548254729998e+03 },
    { 9.7999999999999992e-10, 3.1253186223600000e+00, 1.1059368380962200e+04 },
    { 9.7999999999999992e-10, 5.3719150669599998e+00, 2.0220385543984801e+04 },
    { 9.6999999999999996e-10, 9.2060560080999998e-01, 8.5319638175199998e+02 },
    { 9.5999999999999999e-10, 1.6215499658200001e+00, 3.0206144900756190e+04 },
    { 9.5000000000000003e-10, 4.1881163172899996e+00, 7.8263709424722590e+04 },
    { 9.5000000000000003e-10, 3.9071190621400000e+00, 2.3331963928720002e+03 },
    { 9.5000000000000003e-10, 2.0308612478499999e+00, 1.0202239845947101e+04 },
    { 9.5000000000000003e-10, 1.1798394923400000e+00, 1.0224331246474900e+04 },
    { 9.5000000000000003e-10, 3.3513651593799998e+00, 2.9477381934225199e+04 },
    { 9.5000000000000003e-10, 4.5177232011699999e+00, 3.0220371994757799e+04 },
    { 9.4000000000000006e-10, 5.4415764578900001e+00, 2.8313288804660999e+04 },
    { 9.2999999999999999e-10, 2.3284844018000000e-01, 8.2047280070519992e+03 },
    { 9.2999999999999999e-10, 4.3560253676300000e+00, 5.5229243074000003e+00 },
    { 8.9999999999999999e-10, 2.4062195093200001e+00, 2.9057897290349989e+04 },
    { 8.9999999999999999e-10, 5.1877072834700000e+00, 2.0049197484506200e+04 },
    { 8.9000000000000003e-10, 3.1311205562200000e+00, 2.9786660256881001e+04 },
    { 8.7999999999999996e-10, 4.3329298810600001e+00, 2.0350305021146400e+04 },
    { 8.6999999999999999e-10, 1.3604597067399999e+00, 8.0874081388318000e+03 },
    { 8.6999999999999999e-10, 4.9743123299600001e+00, 2.9043670196348390e+04 },
    { 8.6999999999999999e-10, 3.7842865253200002e+00, 4.6386993743311199e+04 },
    { 8.6999999999999999e-10, 5.4790859276299999e+00, 2.9264082838787199e+04 },
    { 8.6000000000000003e-10, 3.6474923196400000e+00, 7.0351397556434003e+03 },
    { 8.6000000000000003e-10, 2.2975684169499999e+00, 1.8734405422919601e+04 },
    { 8.4999999999999996e-10, 5.0372759162599996e+00, 1.9477395483452190e+04 },
    { 8.4999999999999996e-10, 5.2733545303399998e+00, 3.9149572250346000e+03 },
    { 8.4999999999999996e-10, 4.7649339020999998e-01, 6.0530488985741802e+04 },
    { 8.3999999999999999e-10, 1.2279961453699999e+00, 1.0401210693973600e+04 },
    { 8.3999999999999999e-10, 1.9827045948199999e+00, 1.0025360398448400e+04 },
    { 8.3000000000000003e-10, 5.7933840358900000e+00, 9.7216222537522008e+03 },
    { 8.3000000000000003e-10, 3.7005020114799998e+00, 1.0704948838669799e+04 },
    { 8.1999999999999996e-10, 6.1739044567799999e+00, 4.4007829269739603e+04 },
    { 8.0000000000000003e-10, 2.1179407336499998e+00, 1.4128242771245599e+04 },
    { 7.8999999999999996e-10, 2.6591907435400000e+00, 1.0018314161750401e+04 },
    { 7.8999999999999996e-10, 5.5150999665000000e-01, 1.0408256930671600e+04 },
    { 7.8999999999999996e-10, 1.9779576011500000e+00, 1.0137019474935400e+04 },
    { 7.7000000000000003e-10, 3.5858690573700001e+00, 1.0853182832525001e+04 },
    { 7.5999999999999996e-10, 4.8746610262200001e+00, 1.0243106984359800e+04 },
    { 7.5999999999999996e-10, 4.6192250211500001e+00, 1.0183464108062200e+04 },
    { 7.5999999999999996e-10, 6.0847953857099997e+00, 9.5628915597059995e+02 },
    { 7.5999999999999996e-10, 1.0520167879800000e+00, 2.1485953022611189e+04 },
    { 7.5999999999999996e-10, 3.9398713914100001e+00, 2.9683567482662391e+04 },
    { 7.4000000000000003e-10, 1.9242073921500000e+00, 8.8273902698748007e+03 },
    { 7.4000000000000003e-10, 5.4846688221299997e+00, 2.1720055316004600e+04 },
    { 7.2999999999999996e-10, 1.6828093282700001e+00, 2.1000915890756802e+04 },
    { 7.2999999999999996e-10, 3.2333160897000002e-01, 1.7248425301854400e+04 },
    { 7.2999999999999996e-10, 3.8506789340999998e+00, 5.1449324263278198e+04 },
    { 7.2000000000000000e-10, 3.2461955812499999e+00, 3.9302096962196003e+04 },
    { 7.2000000000000000e-10, 4.4714092098700000e+00, 1.0060810855926179e+05 },
    { 7.1000000000000003e-10, 3.9587109858099998e+00, 3.4171917331444398e+04 },
    { 7.1000000000000003e-10, 6.1440894376899999e+00, 1.5738204240988000e+03 },
    { 7.1000000000000003e-10, 5.4619848913400002e+00, 1.5808666607968000e+03 },
    { 6.9999999999999996e-10, 1.4915524420099999e+00, 6.2983283211764001e+03 },
    { 6.9999999999999996e-10, 5.1536077794199997e+00, 5.0317203439530800e+04 },
    { 6.9000000000000000e-10, 2.2968155594800002e+00, 3.4570066334852600e+04 },
    { 6.9000000000000000e-10, 9.0460126436999999e-01, 4.1044590450955598e+04 },
    { 6.9000000000000000e-10, 2.2542705403499999e+00, 9.6444636721835996e+03 },
    { 6.9000000000000000e-10, 9.5643019983999999e-01, 1.0782107420238401e+04 },
    { 6.6999999999999996e-10, 4.8620096429999998e-01, 3.9565080160194002e+03 },
    { 6.6000000000000000e-10, 3.6757937988500000e+00, 1.8100003258450000e+04 },
    { 6.6000000000000000e-10, 6.0516671026499997e+00, 1.9580488257670801e+04 },
    { 6.5000000000000003e-10, 1.6794653311200001e+00, 5.3285184835241802e+04 },
    { 6.3999999999999996e-10, 1.6106648163299999e+00, 1.8314920779044402e+04 },
    { 6.3999999999999996e-10, 4.2991944505199999e+00, 1.8300693685042788e+04 },
    { 6.3999999999999996e-10, 5.5715575450000001e+00, 3.0007072899319799e+04 },
    { 6.2000000000000003e-10, 1.2864938937199999e+00, 1.1599180822547200e+04 },
    { 6.0999999999999996e-10, 5.9866472365699996e+00, 1.0139988420352000e+04 },
    { 6.0999999999999996e-10, 3.5072388108000001e+00, 1.0286582672070001e+04 },
    { 6.0999999999999996e-10, 5.7175309157999998e-01, 6.5669351688566003e+03 },
    { 6.0000000000000000e-10, 1.3487100373700001e+00, 2.0421048168114601e+04 },
    { 6.0000000000000000e-10, 2.0060900283300001e+00, 1.8204714457824990e+04 },
    { 6.0000000000000000e-10, 2.7694433670200000e+00, 3.6843780499902000e+03 },
    { 6.0000000000000000e-10, 3.4170162662400001e+00, 1.4427354561295801e+04 },
    { 5.9000000000000003e-10, 1.2604080946500000e+00, 1.0254836337195800e+04 },
    { 5.9000000000000003e-10, 1.9502926455400000e+00, 1.0171734755226200e+04 },
    { 5.7999999999999996e-10, 3.0821204721500002e+00, 7.5765600735739999e+03 },
    { 5.7999999999999996e-10, 3.0573413615299998e+00, 7.5449506839496389e+04 },
    { 5.7999999999999996e-10, 1.3118736784200000e+00, 2.0412343998420391e+04 },
    { 5.7999999999999996e-10, 6.1929597261599998e+00, 1.8612028613130001e+03 },
    { 5.7000000000000000e-10, 3.6791513751600000e+00, 1.1279781023400999e+04 },
    { 5.7000000000000000e-10, 4.0626318326400002e+00, 2.0464598765057799e+04 },
    { 5.7000000000000000e-10, 1.8976131153100000e+00, 8.3078207812706005e+03 },
    { 5.6000000000000003e-10, 3.5383357837000000e+00, 3.4363365597556003e+04 },
    { 5.6000000000000003e-10, 1.0786693861600001e+00, 9.9365397743466001e+03 },
    { 5.6000000000000003e-10, 2.1320313540300000e+00, 1.0490031318075400e+04 },
    { 5.6000000000000003e-10, 1.9038225757200000e+00, 6.4460698681961403e+04 },
    { 5.4999999999999996e-10, 5.7562332374400000e+00, 1.4788665740644001e+03 },
    { 5.4000000000000000e-10, 6.8665384143999997e-01, 4.8947663870676603e+04 },
    { 5.4000000000000000e-10, 4.4966482702499997e+00, 9.9223126803449995e+03 },
    { 5.4000000000000000e-10, 4.9972377771199996e+00, 1.0504258412077001e+04 },
    { 5.4000000000000000e-10, 1.4763290860300000e+00, 5.7375801900846200e+04 },
    { 5.4000000000000000e-10, 4.5475827511900002e+00, 6.6038042224409190e+04 },
    { 5.1999999999999996e-10, 2.0926346921199999e+00, 4.0426543993968000e+04 },
    { 5.1999999999999996e-10, 6.1290282760000002e-02, 8.7273054251457997e+03 },
    { 5.1999999999999996e-10, 2.9272266267100000e+00, 9.0394823013050787e+04 },
    { 5.1999999999999996e-10, 5.2096374719499998e+00, 3.8631285550246997e+04 },
    { 5.1999999999999996e-10, 1.9963448515200000e+00, 1.4844727083000000e+00 },
    { 5.1000000000000000e-10, 5.4311718376200000e+00, 9.5662747128962001e+03 },
    { 5.1000000000000000e-10, 5.8931382945299999e+00, 1.0453831825548140e+05 },
    { 5.1000000000000000e-10, 6.2759379855199997e+00, 1.6250529749973000e+04 },
    { 5.0000000000000003e-10, 4.6970703527400000e+00, 8.4180271024899994e+03 },
    { 5.0000000000000003e-10, 1.3399009908199999e+00, 5.9160949416887597e+04 },
    { 5.0000000000000003e-10, 3.0228423431500002e+00, 3.6301188687785187e+04 },
    { 4.8999999999999996e-10, 2.3810457386900001e+00, 4.7623852786089599e+04 },
    { 4.8999999999999996e-10, 1.6562793345700000e+00, 9.7943611473103992e+03 },
    { 4.8999999999999996e-10, 1.5544214056200001e+00, 1.0632209945111599e+04 },
    { 4.8999999999999996e-10, 1.9633602230000000e+00, 4.9166329245880002e+02 },
    { 4.8999999999999996e-10, 6.8713949783999995e-01, 3.2243498500880600e+04 },
    { 4.8000000000000000e-10, 1.2273524092799999e+00, 3.8727264777464799e+04 },
    { 4.8000000000000000e-10, 3.3519657096000000e+00, 9.2463146687753997e+03 },
    { 4.8000000000000000e-10, 2.0539136991800002e+00, 4.3371173787679999e+02 },
    { 4.7000000000000003e-10, 4.8185314722100001e+00, 3.8741491871466402e+04 },
    { 4.7000000000000003e-10, 1.5091728048999999e+00, 2.0142711773556799e+04 },
    { 4.7000000000000003e-10, 4.8973236516099998e+00, 1.8792514776260001e+02 },
    { 4.6000000000000001e-10, 5.3858459418000004e+00, 2.3493430354227999e+03 },
    { 4.6000000000000001e-10, 4.7037395168199998e+00, 2.3563892721207999e+03 },
    { 4.5000000000000000e-10, 4.2152652708199998e+00, 1.9497138446060001e+02 },
    { 4.5000000000000000e-10, 5.5208608697399999e+00, 1.0263282168115200e+04 },
    { 4.5000000000000000e-10, 3.9730251776299998e+00, 1.0163288924306800e+04 },
    { 4.3999999999999998e-10, 5.8404095199499997e+00, 3.8204687359370997e+04 },
    { 4.3999999999999998e-10, 2.0480418955799999e+00, 3.0632743091632190e+04 },
    { 4.3999999999999998e-10, 3.1961837646400002e+00, 2.7461710848065399e+04 },
    { 4.3999999999999998e-10, 4.2756771890000003e-02, 9.3672027114598004e+03 },
    { 4.3999999999999998e-10, 4.4686416275900003e+00, 7.9379716535715997e+04 },
    { 4.3999999999999998e-10, 1.1709146692600001e+00, 1.0168144326574400e+04 },
    { 4.3999999999999998e-10, 2.0397860709300000e+00, 1.0258426765847600e+04 },
    { 4.3999999999999998e-10, 4.3064871368600004e+00, 9.6978216751180007e+03 },
    { 4.3999999999999998e-10, 5.1873989105099998e+00, 1.0728749417304000e+04 },
    { 4.3000000000000001e-10, 6.1419201996500004e+00, 1.1180256423646601e+04 },
    { 4.3000000000000001e-10, 5.1221024682199996e+00, 2.2030212954669601e+04 },
    { 4.3000000000000001e-10, 4.9233864042700004e+00, 1.0362848743345599e+04 },
    { 4.2000000000000000e-10, 2.8174793197699999e+00, 2.5132303399965600e+04 },
    { 4.2000000000000000e-10, 5.0830524655099998e+00, 7.7802372993268189e+04 },
    { 4.2000000000000000e-10, 2.0257599682600000e+00, 3.6173708197100197e+04 },
    { 4.2000000000000000e-10, 2.4875569655900001e+00, 1.4169793562230399e+04 },
    { 4.0999999999999998e-10, 1.2857964444000000e-01, 1.2850011018847999e+04 },
    { 4.0000000000000001e-10, 1.1648847308000001e+00, 3.1022753170856198e+04 },
    { 4.0000000000000001e-10, 2.3497278107000000e-01, 3.8519945791972001e+04 },
    { 4.0000000000000001e-10, 2.2341672511800001e+00, 2.8418000004035999e+04 },
    { 3.9000000000000000e-10, 3.4442720971699998e+00, 2.8631299099473988e+04 },
    { 3.9000000000000000e-10, 1.0248238021200000e+00, 1.2655674786264001e+03 },
    { 3.9000000000000000e-10, 4.7445767706000003e-01, 5.4981384956089598e+04 },
    { 3.9000000000000000e-10, 2.7301380877299999e+00, 3.1639186965660002e+02 },
    { 3.9000000000000000e-10, 5.9700069258099999e+00, 6.9968251920628798e+04 },
    { 3.9000000000000000e-10, 5.1893661535200000e+00, 3.0563590567357402e+04 },
    { 3.9000000000000000e-10, 1.3325533373700000e+00, 5.7914125575325997e+03 },
    { 3.7999999999999998e-10, 1.2971605865400000e+00, 4.7938038965959197e+04 },
    { 3.7999999999999998e-10, 4.1358961767800002e+00, 4.7917263570466799e+04 },
    { 3.7999999999999998e-10, 2.8978893307700000e+00, 1.1567427666460119e+05 },
    { 3.7000000000000001e-10, 4.4869147444799999e+00, 2.9160990064568599e+04 },
    { 3.7000000000000001e-10, 4.4129818443400000e+00, 6.2567775301915999e+03 },
    { 3.7000000000000001e-10, 3.2522001410199999e+00, 3.5191810135691798e+04 },
    { 3.7000000000000001e-10, 1.4200128600000000e-02, 4.7964337285759000e+04 },
    { 3.7000000000000001e-10, 3.3314166471200002e+00, 6.8390908378180990e+04 },
    { 3.7000000000000001e-10, 2.6388959380000001e+00, 1.3859635923565400e+04 },
    { 3.6000000000000000e-10, 2.9702838869099999e+00, 8.6739696763989996e+03 },
    { 3.6000000000000000e-10, 2.2366926294599998e+00, 6.9941953600828987e+04 },
    { 3.6000000000000000e-10, 1.2286762382000000e+00, 2.0430503245685089e+04 },
    { 3.4999999999999998e-10, 4.2388712097600001e+00, 5.5798458358398399e+04 },
    { 3.4999999999999998e-10, 2.7531661812800001e+00, 1.2546481939083000e+04 },
    { 3.4999999999999998e-10, 1.5403060399499999e+00, 8.0482466528909383e+04 },
    { 3.4999999999999998e-10, 4.7921493261899997e+00, 1.0011032151036899e+04 },
    { 3.4999999999999998e-10, 4.7017367211799996e+00, 1.0415538941385101e+04 },
    { 3.4000000000000001e-10, 5.4164430761500002e+00, 4.8733231338182988e+04 },
    { 3.4000000000000001e-10, 4.9363314387599999e+00, 7.4617379465193999e+03 },
    { 3.4000000000000001e-10, 4.1644198083799999e+00, 5.8946516884393990e+04 },
    { 3.4000000000000001e-10, 4.2308812002499998e+00, 1.5393158698120001e+03 },
    { 3.4000000000000001e-10, 3.4797068324599998e+00, 3.7703978024255797e+04 },
    { 3.4000000000000001e-10, 3.6787141453499999e+00, 4.0213244898530000e+04 },
    { 3.3000000000000000e-10, 5.3801525556399996e+00, 4.9477354835771192e+04 },
    { 3.3000000000000000e-10, 3.7488973079000001e+00, 1.0117306318993200e+04 },
    { 3.3000000000000000e-10, 5.7449887394700001e+00, 1.0309264773428800e+04 },
    { 3.3000000000000000e-10, 5.8811523604599998e+00, 8.3309926231935591e+04 },
    { 3.1999999999999998e-10, 4.8207306667500003e+00, 1.0090903762133139e+05 },
    { 3.1999999999999998e-10, 3.7643894104700002e+00, 2.1977616315070001e+04 },
    { 3.1999999999999998e-10, 2.2257491587000000e-01, 8.1732582689487797e+04 },
    { 3.1000000000000002e-10, 3.4478832364000001e+00, 3.0103052126537590e+04 },
    { 3.1000000000000002e-10, 1.4672443212599999e+00, 5.0290905119730996e+04 },
    { 3.1000000000000002e-10, 2.4901156691900002e+00, 1.1045700263900001e+01 },
    { 3.1000000000000002e-10, 3.6535827941500001e+00, 6.3498470381452789e+04 },
    { 3.1000000000000002e-10, 5.4665144456599997e+00, 3.9786646707653999e+04 },
    { 3.1000000000000002e-10, 4.7690983589500000e+00, 1.0657036967048359e+05 },
    { 3.1000000000000002e-10, 9.5396015997000005e-01, 2.7682140744156401e+04 },
    { 3.0000000000000000e-10, 2.5268537296400000e+00, 4.1948464387519999e+02 },
    { 3.0000000000000000e-10, 2.2687369463000001e-01, 5.1640772529389797e+04 },
    { 3.0000000000000000e-10, 3.6218054090799998e+00, 2.0432094016729399e+04 },
    { 2.8999999999999998e-10, 2.4041707519000000e-01, 1.1752601416023001e+04 },
    { 2.8999999999999998e-10, 5.5168801790200002e+00, 9.8009144493366002e+03 },
    { 2.8999999999999998e-10, 3.9770058683500000e+00, 1.0625656643085400e+04 },
    { 2.8999999999999998e-10, 4.4092578462700001e+00, 9.8897801295535992e+03 },
    { 2.8999999999999998e-10, 1.7761414185000000e+00, 2.2434479570190001e+02 },
    { 2.8000000000000002e-10, 2.7335698936000002e-01, 1.0098846257525791e+04 },
    { 2.8000000000000002e-10, 2.9373437508300002e+00, 1.0327724834896209e+04 },
    { 2.8000000000000002e-10, 5.4351097787300002e+00, 1.1791945390395999e+03 },
    { 2.8000000000000002e-10, 3.6595061856100002e+00, 7.3872163297048595e+04 },
    { 2.8000000000000002e-10, 1.9495081135200001e+00, 2.8528206325255400e+04 },
    { 2.8000000000000002e-10, 4.1915165036000002e-01, 1.5694540465078600e+04 },
    { 2.8000000000000002e-10, 1.9241987375200000e+00, 7.8867177122390003e+03 },
};

static const VSOP87Term VSOP87A_VENUS_X1[338] = {
    { 3.3862635999999999e-04, 3.1415926535900001e+00, 0.0000000000000000e+00 },
    { 1.7234992000000001e-04, 9.2721124604000005e-01, 2.0426571092422000e+04 },
    { 6.5104160000000003e-05, 2.1928988973300001e+00, 1.0213285546211000e+04 },
    { 1.7515300000000001e-06, 1.8066237585600000e+00, 3.0639856638632999e+04 },
    { 6.2875000000000002e-07, 2.0913937673300000e+00, 1.0186987226411200e+04 },
    { 6.2672999999999995e-07, 1.1193177082900001e+00, 1.0239583866010800e+04 },
    { 6.0307999999999997e-07, 4.9281850424300000e+00, 8.6359420037632008e+03 },
    { 4.9287999999999995e-07, 4.5767239987200004e+00, 1.1790629088658799e+04 },
    { 3.6777999999999998e-07, 2.0685124225100000e+00, 7.7552261132399997e+02 },
    { 3.0106000000000002e-07, 5.3272033526699998e+00, 1.0021837280099400e+04 },
    { 2.9274000000000001e-07, 4.1654064425500001e+00, 1.0404733812322600e+04 },
    { 1.9966999999999999e-07, 1.1660477496299999e+00, 9.4377629348869996e+03 },
    { 1.7823000000000000e-07, 2.0783654260000000e+00, 1.0988808157535001e+04 },
    { 1.4023000000000001e-07, 3.4982789695599998e+00, 4.7057323075435997e+03 },
    { 1.3944000000000001e-07, 3.6908030279999999e+00, 9.6835945811164001e+03 },
    { 1.3514000000000000e-07, 5.7732651315599997e+00, 1.0742976511305600e+04 },
    { 1.3304999999999999e-07, 1.1525217942299999e+00, 1.9651048481098002e+04 },
    { 1.3302000000000001e-07, 1.3468448210899999e+00, 1.9144826611159999e+02 },
    { 7.5180000000000002e-08, 5.5735525629999998e+00, 1.0593819301891999e+03 },
    { 7.3140000000000005e-08, 6.0071503046099997e+00, 1.5720838784878400e+04 },
    { 6.8779999999999998e-08, 4.8364378989399999e+00, 9.1039069941176003e+03 },
    { 5.9660000000000006e-08, 2.1033885124300000e+00, 1.0206171999210201e+04 },
    { 5.9660000000000006e-08, 1.1073122277600000e+00, 1.0220399093211799e+04 },
    { 5.8929999999999999e-08, 4.6507978841500002e+00, 1.1322664098304400e+04 },
    { 5.1459999999999999e-08, 2.4979707826599999e+00, 3.1546870848956000e+03 },
    { 4.6490000000000001e-08, 2.8657078250199999e+00, 9.8303890139878004e+03 },
    { 4.5699999999999999e-08, 4.5912859140200002e+00, 2.0618019358533598e+04 },
    { 4.4570000000000001e-08, 3.5149067415000002e-01, 1.0596182078434200e+04 },
    { 4.2529999999999998e-08, 6.0444188804200003e+00, 9.9999864507729999e+03 },
    { 4.1570000000000001e-08, 3.4398047361200002e+00, 1.0426584641649000e+04 },
    { 3.8309999999999997e-08, 3.2591122928999999e-01, 7.1135470007999997e+00 },
    { 3.2199999999999997e-08, 2.3159655765500000e+00, 7.8604193924392002e+03 },
    { 2.7899999999999998e-08, 3.9114050305800001e+00, 1.9367189162232800e+04 },
    { 2.7420000000000000e-08, 4.9711261952900001e+00, 8.0182093112380005e+02 },
    { 2.7010000000000002e-08, 6.5622385382000004e-01, 7.0585984613153996e+03 },
    { 2.5390000000000001e-08, 5.9695364385899996e+00, 9.4114646150871995e+03 },
    { 2.4260000000000000e-08, 5.6370711142000003e+00, 8.6242126509272002e+03 },
    { 2.2280000000000002e-08, 3.8047641546300000e+00, 1.1015106477334801e+04 },
    { 2.0070000000000000e-08, 2.5271755770500000e+00, 2.6298319799800002e+01 },
    { 1.8200000000000001e-08, 2.5330941388900001e+00, 1.3367972631106601e+04 },
    { 1.7400000000000000e-08, 2.2636685861800001e+00, 4.5519534970588002e+03 },
    { 1.7330000000000000e-08, 3.3588228519599999e+00, 1.1093785520934000e+03 },
    { 1.6950000000000000e-08, 4.8638459305700001e+00, 3.9302096962196001e+03 },
    { 1.5860000000000001e-08, 2.6930592477799999e+00, 4.0853142184844000e+04 },
    { 1.4909999999999999e-08, 6.3663352936999995e-01, 1.2566151699982800e+04 },
    { 1.4800000000000000e-08, 5.6817519591899996e+00, 3.1283887650958000e+03 },
    { 1.4240000000000000e-08, 3.8487776451700002e+00, 7.0848967811151997e+03 },
    { 1.4000000000000000e-08, 6.2473953905700004e+00, 1.1272667476400200e+04 },
    { 1.3520000000000000e-08, 2.8639405232800001e+00, 2.0419457545421192e+04 },
    { 1.3310000000000000e-08, 5.6774081336400002e+00, 2.3581258177317599e+04 },
    { 1.2830000000000000e-08, 3.8848918469700000e+00, 3.8289653222319998e+02 },
    { 1.2820000000000000e-08, 4.6631011705300001e+00, 1.0218808470518399e+04 },
    { 1.2820000000000000e-08, 4.8307848768400001e+00, 1.0207762621903599e+04 },
    { 1.2170000000000001e-08, 3.1391484806899999e+00, 9.1539036160218002e+03 },
    { 1.2040000000000001e-08, 4.9631104558399999e+00, 1.6496361396202399e+04 },
    { 1.1650000000000000e-08, 1.8691214852800000e+00, 1.5773435424478000e+03 },
    { 1.1100000000000000e-08, 2.8360276400000001e-01, 1.5890728952838001e+03 },
    { 1.0840000000000000e-08, 4.3552438467099996e+00, 1.0198033075026000e+04 },
    { 1.0840000000000000e-08, 5.1386422006599997e+00, 1.0228538017396000e+04 },
    { 9.3499999999999994e-09, 2.6915551010800001e+00, 2.3528661537717999e+03 },
    { 9.2699999999999996e-09, 9.5159471681999996e-01, 1.5874617595363199e+04 },
    { 8.4399999999999998e-09, 1.9152554091800000e+00, 2.1228392023545799e+04 },
    { 8.0000000000000005e-09, 5.1114715847500003e+00, 4.2659819087599999e+02 },
    { 7.4499999999999997e-09, 2.1524421011600001e+00, 3.1491641605882000e+03 },
    { 7.2699999999999999e-09, 7.1047310572000000e-01, 2.8286990484861199e+04 },
    { 6.7299999999999997e-09, 5.2760117941600004e+00, 8.0945216858326003e+03 },
    { 6.5899999999999998e-09, 3.8507482364999999e+00, 1.7298182327326202e+04 },
    { 6.4700000000000002e-09, 1.9140441826100001e+00, 5.3680451209540001e+02 },
    { 6.3499999999999998e-09, 2.4814486022000001e-01, 1.8849227549974199e+04 },
    { 6.2900000000000004e-09, 4.0979495307600002e+00, 6.2830758499914000e+03 },
    { 6.2499999999999997e-09, 3.1853172660600002e+00, 2.0400272772622200e+04 },
    { 6.1700000000000000e-09, 2.4025722311600002e+00, 5.2257741809380002e+02 },
    { 6.1300000000000001e-09, 2.2192928894000000e-01, 2.1535949644515400e+04 },
    { 5.6900000000000000e-09, 2.2708999131300001e+00, 5.2969096509459996e+02 },
    { 5.5800000000000002e-09, 2.1721227961900000e+00, 5.6613320491521999e+03 },
    { 5.0400000000000000e-09, 5.0752256281999997e-01, 2.0452869412221800e+04 },
    { 4.7099999999999997e-09, 5.0884995911899997e+00, 3.2217200181080800e+04 },
    { 4.4900000000000000e-09, 3.2305066937800002e+00, 9.1610171630225996e+03 },
    { 4.4200000000000000e-09, 2.4796076772300002e+00, 9.6907081281171995e+03 },
    { 4.3999999999999997e-09, 7.3691128332000000e-01, 1.0735862964304801e+04 },
    { 4.3299999999999997e-09, 3.7432653616099998e+00, 9.1467900690210008e+03 },
    { 4.3200000000000000e-09, 8.3861429466000004e-01, 2.0809467624645200e+04 },
    { 4.2999999999999996e-09, 2.9446782965799998e+00, 1.8837498197138189e+04 },
    { 4.2100000000000001e-09, 2.5769584345099998e+00, 2.9050783743349199e+04 },
    { 4.1700000000000003e-09, 5.9556796989800000e+00, 2.2003914634869801e+04 },
    { 4.1599999999999997e-09, 4.9171186264499998e+00, 6.6812248533995999e+03 },
    { 4.0599999999999996e-09, 5.4719930917599999e+00, 2.4356780788641601e+04 },
    { 3.7200000000000000e-09, 5.0265145827000002e-01, 1.0251313218846801e+04 },
    { 3.7200000000000000e-09, 2.7080492819200002e+00, 1.0175257873575199e+04 },
    { 3.6800000000000001e-09, 5.0948592293999995e-01, 9.7866873553349997e+03 },
    { 3.6399999999999998e-09, 2.7158001851900000e+00, 1.0639883737087001e+04 },
    { 3.5100000000000001e-09, 2.5078250968200000e+00, 6.4897765872880000e+03 },
    { 3.1899999999999999e-09, 1.0579914112500000e+00, 1.7277406931833801e+04 },
    { 3.0500000000000000e-09, 1.0770560849999999e+00, 1.4765239043269799e+04 },
    { 3.0300000000000001e-09, 7.6647154702999998e-01, 2.7511467873537200e+04 },
    { 3.0300000000000001e-09, 1.6166252665000000e-01, 1.4945316173554400e+04 },
    { 2.9899999999999998e-09, 1.1704932721000001e+00, 9.5733882598969994e+03 },
    { 2.9699999999999999e-09, 1.0683180442400000e+00, 1.0232955307110789e+04 },
    { 2.9699999999999999e-09, 2.1423826959499999e+00, 1.0193615785311211e+04 },
    { 2.8900000000000002e-09, 5.7861248873999997e+00, 2.0235122826310399e+04 },
    { 2.8499999999999999e-09, 5.5267872107700002e+00, 2.0956262057516600e+04 },
    { 2.8299999999999999e-09, 1.4337559903099999e+00, 9.6764810341156008e+03 },
    { 2.8200000000000002e-09, 2.3546908312000001e+00, 3.4425749449653999e+03 },
    { 2.8100000000000000e-09, 6.2402387847199998e+00, 4.7320306273433998e+03 },
    { 2.7999999999999998e-09, 4.5490934315000002e+00, 1.3745346239022399e+04 },
    { 2.7999999999999998e-09, 5.0897518216200002e+00, 1.4919017853754600e+04 },
    { 2.7299999999999999e-09, 4.3879861602799997e+00, 1.9999972901545989e+04 },
    { 2.6500000000000002e-09, 5.6341198639299996e+00, 2.2779437246193800e+04 },
    { 2.6200000000000001e-09, 3.2242429884499999e+00, 2.1202093703745999e+04 },
    { 2.6200000000000001e-09, 1.8049296758400000e+00, 1.0750090058306399e+04 },
    { 2.5599999999999998e-09, 4.4150265270900002e+00, 1.0234060941703399e+04 },
    { 2.5599999999999998e-09, 5.0788595202800000e+00, 1.0192510150718599e+04 },
    { 2.4800000000000001e-09, 8.0860047226999998e-01, 1.3936794505133999e+04 },
    { 2.3600000000000001e-09, 3.2977919848299999e+00, 1.5510452226479999e+03 },
    { 2.3499999999999999e-09, 2.8169764861900002e+00, 1.0664954771900000e+03 },
    { 2.3199999999999998e-09, 1.2992866549800000e+00, 1.9889766580326592e+04 },
    { 2.3100000000000001e-09, 4.2458406844800001e+00, 3.7235089589230001e+03 },
    { 2.2900000000000002e-09, 2.0039308202699999e+00, 1.8307807232043600e+04 },
    { 2.1799999999999999e-09, 8.1835340957000002e-01, 1.9903993674328191e+04 },
    { 2.0799999999999998e-09, 5.6184278852100000e+00, 1.0007099997773799e+04 },
    { 2.0799999999999998e-09, 3.8754581621600002e+00, 1.0419471094648199e+04 },
    { 1.9900000000000000e-09, 3.3116594459999998e-01, 9.6389407478762005e+03 },
    { 1.8400000000000001e-09, 1.6610829139500001e+00, 9.9674538999815995e+03 },
    { 1.8300000000000001e-09, 2.8636158226399999e+00, 1.0787630344545800e+04 },
    { 1.8199999999999999e-09, 4.1968378234600001e+00, 6.8726731195111997e+03 },
    { 1.7900000000000000e-09, 1.3299783559000000e-01, 3.0110165673538399e+04 },
    { 1.7800000000000001e-09, 1.5494621354500000e+00, 1.0459117192440401e+04 },
    { 1.7700000000000001e-09, 4.6080362001199999e+00, 5.4812549188676003e+03 },
    { 1.6300000000000000e-09, 4.9597414696900000e+00, 1.0225014899047001e+04 },
    { 1.6300000000000000e-09, 4.5341445776800002e+00, 1.0201556193375000e+04 },
    { 1.6300000000000000e-09, 3.3366623463799998e+00, 8.6170991039264009e+03 },
    { 1.6200000000000000e-09, 1.4106605049600001e+00, 1.8875525869773999e+04 },
    { 1.5700000000000000e-09, 4.8614767647699999e+00, 5.5075532386674004e+03 },
    { 1.5700000000000000e-09, 5.1556849942699996e+00, 1.3553897972910800e+04 },
    { 1.5600000000000000e-09, 6.0669926686100002e+00, 1.0316378320429600e+04 },
    { 1.5600000000000000e-09, 3.4268933787600000e+00, 1.0110192771992401e+04 },
    { 1.5400000000000001e-09, 9.8203597063000003e-01, 2.1187638603783998e+03 },
    { 1.5300000000000001e-09, 4.5699128543500001e+00, 7.0641213856227996e+03 },
    { 1.5199999999999999e-09, 5.1909046347899999e+00, 6.3989728631399998e+02 },
    { 1.5100000000000000e-09, 2.5150612975600000e+00, 2.9864334027309000e+04 },
    { 1.4300000000000000e-09, 4.8908417397299999e+00, 7.9945284420241996e+03 },
    { 1.3999999999999999e-09, 3.7416827801400001e+00, 8.6313261979279996e+03 },
    { 1.3300000000000000e-09, 4.6656904335100000e+00, 1.2432042650397800e+04 },
    { 1.3100000000000000e-09, 2.2581059229200000e+00, 3.1441677569756801e+04 },
    { 1.2900000000000001e-09, 4.9046321628599996e+00, 7.5648307207380003e+03 },
    { 1.2799999999999999e-09, 5.4698276149099998e+00, 1.0632770190086199e+04 },
    { 1.2799999999999999e-09, 4.8332496526600002e+00, 1.4143495242430599e+04 },
    { 1.2600000000000000e-09, 4.4556756987900004e+00, 9.7938009023358009e+03 },
    { 1.2199999999999999e-09, 3.5296378321100002e+00, 2.2805735565993600e+04 },
    { 1.1900000000000000e-09, 2.1881767253900000e+00, 2.8521092778254599e+04 },
    { 1.0999999999999999e-09, 3.0263518590999999e-01, 1.1802358441494800e+04 },
    { 1.0800000000000000e-09, 5.1340066033799996e+00, 2.9580474708443799e+04 },
    { 1.0500000000000001e-09, 6.9367524807000003e-01, 1.6983996147456601e+04 },
    { 1.0399999999999999e-09, 6.1753154710600002e+00, 3.6147409877300401e+04 },
    { 1.0100000000000000e-09, 3.1454138158400000e+00, 1.0522683831884001e+03 },
    { 9.9000000000000010e-10, 2.3076164884099999e+00, 2.0775395492400001e+01 },
    { 9.5999999999999999e-10, 5.3311857668100000e+00, 2.2041264243879999e+02 },
    { 9.5000000000000003e-10, 3.6678280120400002e+00, 1.0433698188649800e+04 },
    { 9.5000000000000003e-10, 5.8260580353200000e+00, 9.9928729037722005e+03 },
    { 9.4000000000000006e-10, 1.4797876382999999e+00, 2.0433684639422801e+04 },
    { 9.2999999999999999e-10, 1.4711850987000000e-01, 5.7434479833479998e+02 },
    { 9.2999999999999999e-10, 5.8927007006299998e+00, 2.9577158944766002e+03 },
    { 9.2000000000000003e-10, 1.9269380019900000e+00, 9.3600891644589992e+03 },
    { 9.2000000000000003e-10, 5.7359631717199999e+00, 2.0618554843720000e+02 },
    { 9.0999999999999996e-10, 4.5051499731000000e-01, 1.9360075615231988e+04 },
    { 8.7999999999999996e-10, 1.1595689446999999e+00, 2.3791644735716000e+03 },
    { 8.3999999999999999e-10, 3.0152805043900002e+00, 2.5158601719765400e+04 },
    { 8.0999999999999999e-10, 1.7273168667000000e-01, 1.9896880127327400e+04 },
    { 8.0999999999999999e-10, 4.7458557533199999e+00, 1.1764330768858999e+04 },
    { 7.7000000000000003e-10, 4.5052062476900003e+00, 1.0177257679533601e+04 },
    { 7.7000000000000003e-10, 4.9886797996799999e+00, 1.0249313412888399e+04 },
    { 7.7000000000000003e-10, 6.5927841966000000e-01, 2.9573361161443001e+04 },
    { 7.7000000000000003e-10, 7.1010883429000005e-01, 2.0213271996984000e+04 },
    { 7.7000000000000003e-10, 3.3140239583800000e+00, 7.8341210726394002e+03 },
    { 7.5999999999999996e-10, 2.9423884277900001e+00, 2.1329909543799999e+02 },
    { 7.2999999999999996e-10, 2.1800067624000000e-01, 2.9587588255444600e+04 },
    { 7.1000000000000003e-10, 1.1726641838600000e+00, 2.4150080051345001e+04 },
    { 7.1000000000000003e-10, 2.6608127921000002e-01, 2.9043670196348390e+04 },
    { 6.9000000000000000e-10, 4.8696406486599999e+00, 2.6735945262213201e+04 },
    { 6.8000000000000003e-10, 3.9604612329500002e+00, 1.0383958416830201e+04 },
    { 6.8000000000000003e-10, 5.5334248144200000e+00, 1.0042612675591799e+04 },
    { 6.6999999999999996e-10, 1.1591888344300001e+00, 4.0077619573520002e+04 },
    { 6.6999999999999996e-10, 1.5709948867700001e+00, 1.7778116266948990e+04 },
    { 6.6000000000000000e-10, 3.8783954724699998e+00, 3.5320606928113998e+03 },
    { 6.5000000000000003e-10, 5.4649727394600003e+00, 3.9302096962196003e+04 },
    { 6.0999999999999996e-10, 6.6096812913000003e-01, 2.9057897290349989e+04 },
    { 6.0000000000000000e-10, 6.2142404987699997e+00, 3.0831304904744600e+04 },
    { 6.0000000000000000e-10, 1.0100099885000000e-01, 9.7795738083342003e+03 },
    { 5.9000000000000003e-10, 3.6096031783600000e+00, 3.5371887265976402e+04 },
    { 5.7000000000000000e-10, 1.4041525923400000e+00, 1.1020632121940000e+02 },
    { 5.6000000000000003e-10, 3.8730989122300001e+00, 1.7468855197945399e+04 },
    { 5.4999999999999996e-10, 1.1043942341999999e+00, 6.7707106012455997e+03 },
    { 5.4999999999999996e-10, 1.5588897913399999e+00, 2.7490692478044799e+04 },
    { 5.4999999999999996e-10, 3.2804571510599998e+00, 2.2187571041868000e+03 },
    { 5.4000000000000000e-10, 4.9010954987900002e+00, 1.9317192540328600e+04 },
    { 5.4000000000000000e-10, 9.3200345594000000e-01, 2.5934124331089399e+04 },
    { 5.1999999999999996e-10, 6.1784659161000000e-01, 2.6709646942413401e+04 },
    { 5.1000000000000000e-10, 2.5051303262700002e+00, 4.4007829269739603e+04 },
    { 5.1000000000000000e-10, 1.2947662642700000e+00, 1.3341674311306801e+04 },
    { 5.1000000000000000e-10, 1.1282593391500000e+00, 1.3655860491176400e+04 },
    { 5.0000000000000003e-10, 1.7553992282699999e+00, 1.0103079224991599e+04 },
    { 5.0000000000000003e-10, 1.4553015119199999e+00, 1.0323491867430401e+04 },
    { 5.0000000000000003e-10, 6.2098983805900003e+00, 3.9793760254654801e+04 },
    { 5.0000000000000003e-10, 3.8843006381200000e+00, 1.6522659716002199e+04 },
    { 4.8000000000000000e-10, 3.0397844569000001e+00, 2.7197281693667599e+04 },
    { 4.7000000000000003e-10, 3.6242325269300002e+00, 2.0639870187860000e+04 },
    { 4.7000000000000003e-10, 2.6182972030400000e+00, 5.7837138332300601e+04 },
    { 4.7000000000000003e-10, 6.2238579752699996e+00, 8.6622403235630009e+03 },
    { 4.6000000000000001e-10, 4.3171894049999997e+00, 1.9786673806107989e+04 },
    { 4.6000000000000001e-10, 4.3631203221600003e+00, 2.9088811415985001e+04 },
    { 4.5000000000000000e-10, 4.3758312158999999e+00, 2.9999959352319001e+04 },
    { 4.5000000000000000e-10, 6.4539022706000004e-01, 4.3232306658415597e+04 },
    { 4.5000000000000000e-10, 2.2159759700000000e-02, 3.0666154958432799e+04 },
    { 4.3000000000000001e-10, 5.1387209201399999e+00, 2.9786660256881001e+04 },
    { 4.2000000000000000e-10, 5.0159500441000004e-01, 3.9149572250346000e+03 },
    { 4.2000000000000000e-10, 4.7775701634700001e+00, 3.8204687359370997e+04 },
    { 4.2000000000000000e-10, 3.7006972782999997e-01, 5.5798458358398399e+04 },
    { 4.0999999999999998e-10, 7.2651386380000002e-02, 1.8830384650137388e+04 },
    { 4.0000000000000001e-10, 5.8748072734500001e+00, 1.8844611744138991e+04 },
    { 4.0000000000000001e-10, 3.8866695740099999e+00, 4.7938038965959197e+04 },
    { 3.9000000000000000e-10, 5.7784508928599996e+00, 2.0447346487914401e+04 },
    { 3.7999999999999998e-10, 3.4663201189500001e+00, 2.1000915890756802e+04 },
    { 3.7999999999999998e-10, 5.7347992417800002e+00, 3.3019021112204602e+04 },
    { 3.7999999999999998e-10, 7.8775617354000005e-01, 7.2555696517344004e+03 },
    { 3.6000000000000000e-10, 5.9530109633499997e+00, 1.0077660221201000e+04 },
    { 3.6000000000000000e-10, 3.5408750840200001e+00, 1.0348910871221000e+04 },
    { 3.6000000000000000e-10, 1.0124988420600001e+00, 3.3794543723528601e+04 },
    { 3.4999999999999998e-10, 5.3506995984600003e+00, 9.9858094134219991e+03 },
    { 3.4999999999999998e-10, 4.1431864489099999e+00, 1.0440761678999999e+04 },
    { 3.4000000000000001e-10, 8.2715360609999997e-02, 1.9374302709233601e+04 },
    { 3.4000000000000001e-10, 5.2719125534300000e+00, 5.4247413135750401e+04 },
    { 3.3000000000000000e-10, 1.4434008360899999e+00, 4.8417972905581992e+04 },
    { 3.3000000000000000e-10, 1.1358399272199999e+00, 1.2592450019782600e+04 },
    { 3.3000000000000000e-10, 1.0934894461300000e+00, 2.7461710848065399e+04 },
    { 3.3000000000000000e-10, 1.7908853807400000e+00, 5.9728668054618000e+04 },
    { 3.1999999999999998e-10, 5.4994074748199999e+00, 1.6894510399610601e+04 },
    { 3.1999999999999998e-10, 1.9856397894600000e+00, 4.7162516354635198e+04 },
    { 3.1999999999999998e-10, 2.1208048543800002e+00, 1.0191434716884600e+04 },
    { 3.1999999999999998e-10, 1.0898958858200001e+00, 1.0235136375537400e+04 },
    { 3.1999999999999998e-10, 4.1658259180000003e+00, 2.0206158449983192e+04 },
    { 3.1999999999999998e-10, 5.3725583633299996e+00, 9.8359119382951994e+03 },
    { 3.1999999999999998e-10, 4.1213276840399997e+00, 1.0590659154126801e+04 },
    { 3.1999999999999998e-10, 1.0666963752900001e+00, 1.1265553929399401e+04 },
    { 3.1999999999999998e-10, 5.2851992190099999e+00, 5.1868248662178798e+04 },
    { 3.1000000000000002e-10, 3.5164888024700001e+00, 2.0043674560198800e+04 },
    { 3.1000000000000002e-10, 2.8864625082600002e+00, 8.0874081388318000e+03 },
    { 3.0000000000000000e-10, 1.9666196175999999e+00, 1.0853182832525001e+04 },
    { 3.0000000000000000e-10, 7.9959461256999997e-01, 3.6949230808424203e+04 },
    { 3.0000000000000000e-10, 4.3568964667600003e+00, 1.7248425301854400e+04 },
    { 2.8999999999999998e-10, 5.2116298220399999e+00, 1.0290959316639000e+04 },
    { 2.8999999999999998e-10, 4.2822562253300003e+00, 1.0135611775783000e+04 },
    { 2.8999999999999998e-10, 1.4736848293900000e+00, 3.4596364654652403e+04 },
    { 2.8999999999999998e-10, 3.3118038423799998e+00, 3.0206144900756190e+04 },
    { 2.8999999999999998e-10, 3.7022905926599998e+00, 2.0220385543984801e+04 },
    { 2.8999999999999998e-10, 2.9359509778500001e+00, 2.6087903141574199e+04 },
    { 2.8000000000000002e-10, 5.7406973614699996e+00, 1.0435161256629678e+05 },
    { 2.8000000000000002e-10, 4.6703206601699998e+00, 5.5022935747074400e+04 },
    { 2.8000000000000002e-10, 5.4837105378999995e-01, 9.5805018068978006e+03 },
    { 2.7000000000000000e-10, 6.2529340451400000e+00, 6.2983283211764001e+03 },
    { 2.7000000000000000e-10, 2.4007071910300000e+00, 3.7724753419748202e+04 },
    { 2.7000000000000000e-10, 2.8275833435400002e+00, 3.0220371994757799e+04 },
    { 2.7000000000000000e-10, 3.9321520493600000e+00, 2.4341528317456599e+04 },
    { 2.5999999999999998e-10, 4.7760467657900003e+00, 3.4363365597556003e+04 },
    { 2.5999999999999998e-10, 2.7974971333199998e+00, 3.9271182836560998e+04 },
    { 2.5999999999999998e-10, 3.2799788905099998e+00, 3.9256955742559388e+04 },
    { 2.5999999999999998e-10, 3.6829112091299998e+00, 8.1016352328333996e+03 },
    { 2.5999999999999998e-10, 3.2132556457599999e+00, 6.3658877750837601e+04 },
    { 2.5000000000000002e-10, 5.6128817699900004e+00, 4.6386993743311199e+04 },
    { 2.4000000000000000e-10, 5.8553915120899998e+00, 8.7344189721465991e+03 },
    { 2.4000000000000000e-10, 1.2135194796900000e+00, 3.6173708197100197e+04 },
    { 2.4000000000000000e-10, 4.4228298639799997e+00, 3.1022753170856198e+04 },
    { 2.4000000000000000e-10, 2.1610209684199999e+00, 4.0879440504643797e+04 },
    { 2.4000000000000000e-10, 5.6697771725000001e-01, 1.0288067144778301e+04 },
    { 2.4000000000000000e-10, 2.6437230229400002e+00, 1.0138503947643700e+04 },
    { 2.3000000000000001e-10, 2.6922480403999999e-01, 2.4383079108441401e+04 },
    { 2.3000000000000001e-10, 3.4099370222199998e+00, 5.1092726050854799e+04 },
    { 2.3000000000000001e-10, 2.7114844979599999e+00, 1.0367064356695801e+04 },
    { 2.3000000000000001e-10, 4.9921624223000000e-01, 1.0059506735726200e+04 },
    { 2.3000000000000001e-10, 3.9390595084400002e+00, 4.9515382508406998e+04 },
    { 2.3000000000000001e-10, 2.4779091617400000e+00, 6.3093741697912001e+03 },
    { 2.3000000000000001e-10, 5.2435902720999996e+00, 4.5585172812187397e+04 },
    { 2.3000000000000001e-10, 2.6752629429199999e+00, 1.0263042571682799e+04 },
    { 2.3000000000000001e-10, 5.3543779726999996e-01, 1.0163528520739201e+04 },
    { 2.3000000000000001e-10, 5.8684168609500000e+00, 5.2175806283148399e+04 },
    { 2.1999999999999999e-10, 2.9285154353800000e+00, 2.4978524589480799e+04 },
    { 2.1999999999999999e-10, 6.1619120316999998e-01, 2.2645328196608789e+04 },
    { 2.1999999999999999e-10, 1.7542013229300000e+00, 3.0426557543194998e+04 },
    { 2.1999999999999999e-10, 2.5603544858700000e+00, 6.6813564835733196e+04 },
    { 2.1999999999999999e-10, 9.3565021176999996e-01, 9.5662747128962001e+03 },
    { 2.1000000000000000e-10, 2.0577529014499998e+00, 1.0949162059742799e+04 },
    { 2.1000000000000000e-10, 1.1529478387400001e+00, 9.4774090326791993e+03 },
    { 2.1000000000000000e-10, 6.0886684131699997e+00, 5.8953145443294001e+04 },
    { 2.1000000000000000e-10, 5.9088719526800002e+00, 1.8300693685042788e+04 },
    { 2.0000000000000001e-10, 2.2125861093400001e+00, 1.1279781023400999e+04 },
    { 2.0000000000000001e-10, 3.6881650530900001e+00, 1.8073704938650200e+04 },
    { 2.0000000000000001e-10, 2.9091026930999999e+00, 3.8526574350871997e+04 },
    { 2.0000000000000001e-10, 9.2201968087999997e-01, 6.2081534208389799e+04 },
    { 2.0000000000000001e-10, 4.2394124059099996e+00, 4.2456784047091598e+04 },
    { 2.0000000000000001e-10, 2.7081909108000000e+00, 4.9477354835771192e+04 },
    { 2.0000000000000001e-10, 1.5385848438400000e+00, 4.3071899289030800e+04 },
    { 1.8999999999999999e-10, 1.8088301102600000e+00, 2.0316364771202589e+04 },
    { 1.8999999999999999e-10, 3.5328805655400002e+00, 4.4809650200863398e+04 },
    { 1.8999999999999999e-10, 6.0160133849399999e+00, 2.8513979231253790e+04 },
    { 1.8999999999999999e-10, 8.4224065579999993e-02, 4.7623852786089599e+04 },
    { 1.8999999999999999e-10, 4.6038019653799997e+00, 6.7589087447057187e+04 },
    { 1.8000000000000000e-10, 5.4043707877799996e+00, 9.0508108418032007e+03 },
    { 1.8000000000000000e-10, 4.3435416852500000e+00, 2.7682140744156401e+04 },
    { 1.8000000000000000e-10, 7.5283830224000003e-01, 5.0317203439530800e+04 },
    { 1.8000000000000000e-10, 2.8982862169999998e-02, 4.2430485727291802e+04 },
    { 1.7000000000000001e-10, 4.4694174924299999e+00, 1.0497144865076199e+04 },
    { 1.7000000000000001e-10, 5.0244685549400003e+00, 9.9294262273458007e+03 },
    { 1.7000000000000001e-10, 1.6359043769299999e+00, 2.8313288804660999e+04 },
    { 1.7000000000000001e-10, 7.7261247444000003e-01, 1.9624750161298201e+04 },
    { 1.7000000000000001e-10, 6.1324505854600000e+00, 1.8314920779044402e+04 },
    { 1.7000000000000001e-10, 3.9813642931300000e+00, 7.0743774531952789e+04 },
    { 1.5999999999999999e-10, 5.4351992486400000e+00, 9.7453205558566005e+03 },
    { 1.5999999999999999e-10, 4.0586867987300002e+00, 1.0681250536565400e+04 },
    { 1.5999999999999999e-10, 4.4084149901799998e+00, 4.1654963115967803e+04 },
    { 1.5000000000000000e-10, 5.4028715954299997e+00, 1.0199058452209400e+04 },
    { 1.5000000000000000e-10, 4.0910144519399996e+00, 1.0227512640212601e+04 },
    { 1.5000000000000000e-10, 4.9128650992300003e+00, 4.8739859897082999e+04 },
    { 1.5000000000000000e-10, 4.8349577774599997e+00, 3.8734378324465601e+04 },
    { 1.5000000000000000e-10, 1.2236231163100000e+00, 6.2883355139513602e+04 },
    { 1.5000000000000000e-10, 2.7040699416699998e+00, 5.3285184835241802e+04 },
    { 1.5000000000000000e-10, 2.3456890527600001e+00, 6.6011743904609393e+04 },
    { 1.5000000000000000e-10, 2.8372924695299999e+00, 3.8727264777464799e+04 },
    { 1.5000000000000000e-10, 1.5187335904000001e+00, 4.6360695423511403e+04 },
    { 1.5000000000000000e-10, 6.2236195717799996e+00, 2.1485953022611189e+04 },
    { 1.5000000000000000e-10, 3.1875379723599999e+00, 2.7991401813159999e+04 },
    { 1.4000000000000001e-10, 3.7440982130000000e-01, 3.9999945803092000e+04 },
    { 1.4000000000000001e-10, 1.5273435800599999e+00, 1.1026835830170799e+04 },
    { 1.4000000000000001e-10, 1.6833571601299999e+00, 9.3997352622512008e+03 },
    { 1.4000000000000001e-10, 4.0438387909999998e-02, 2.8528206325255400e+04 },
    { 1.4000000000000001e-10, 1.3951511224099999e+00, 3.7895426290367403e+04 },
    { 1.4000000000000001e-10, 1.4488965264500000e+00, 3.7410567239878597e+04 },
    { 1.4000000000000001e-10, 2.7797460690500002e+00, 1.4128242771245599e+04 },
    { 1.4000000000000001e-10, 2.9421744308500002e+00, 5.0290905119730996e+04 },
    { 1.2999999999999999e-10, 6.1413279803499998e+00, 5.2670069593302600e+04 },
};

static const VSOP87Term VSOP87A_VENUS_X2[99] = {
    { 7.0479400000000002e-06, 5.0987439991599999e+00, 2.0426571092422000e+04 },
    { 6.4900999999999999e-06, 3.1415926535900001e+00, 0.0000000000000000e+00 },
    { 6.2447700000000004e-06, 3.8683677675700001e+00, 1.0213285546211000e+04 },
    { 1.1644000000000000e-07, 5.5098281498999997e+00, 3.0639856638632999e+04 },
    { 8.6200000000000004e-08, 2.6998872301999999e+00, 1.0186987226411200e+04 },
    { 8.5899999999999995e-08, 5.0870967404000000e-01, 1.0239583866010800e+04 },
    { 3.8150000000000001e-08, 7.7119888833000005e-01, 9.4377629348869996e+03 },
    { 3.4079999999999998e-08, 2.4310704663400000e+00, 1.0988808157535001e+04 },
    { 2.8060000000000001e-08, 3.2277097089900000e+00, 8.6359420037632008e+03 },
    { 2.2790000000000001e-08, 6.2704720958399998e+00, 1.1790629088658799e+04 },
    { 2.1999999999999998e-08, 3.7477935666400000e+00, 1.0021837280099400e+04 },
    { 2.1419999999999999e-08, 5.7452978513700002e+00, 1.0404733812322600e+04 },
    { 1.6969999999999999e-08, 3.7726443109000002e-01, 7.7552261132399997e+02 },
    { 7.1600000000000001e-09, 1.3543261313499999e+00, 9.8303890139878004e+03 },
    { 6.8200000000000000e-09, 1.8563766173700000e+00, 1.0596182078434200e+04 },
    { 6.4499999999999999e-09, 1.8015857903500001e+00, 4.7057323075435997e+03 },
    { 5.9399999999999998e-09, 2.8161415723599998e+00, 1.9651048481098002e+04 },
    { 5.6999999999999998e-09, 5.0571164919299996e+00, 8.0182093112380005e+02 },
    { 5.3199999999999998e-09, 2.9326046328800000e+00, 1.9144826611159999e+02 },
    { 5.2700000000000002e-09, 4.5172374732699998e+00, 1.0742976511305600e+04 },
    { 5.2700000000000002e-09, 4.9638614713200004e+00, 9.6835945811164001e+03 },
    { 5.2599999999999996e-09, 1.0116096883400001e+00, 7.0585984613153996e+03 },
    { 3.6199999999999999e-09, 5.9178668543899997e+00, 9.4114646150871995e+03 },
    { 3.5899999999999998e-09, 2.1907563003399999e+00, 1.3367972631106601e+04 },
    { 3.2900000000000000e-09, 1.4154862806599999e+00, 1.5720838784878400e+04 },
    { 3.0199999999999999e-09, 5.8356427963800002e+00, 3.1283887650958000e+03 },
    { 3.0100000000000002e-09, 2.2701186670500002e+00, 2.6298319799800002e+01 },
    { 2.9800000000000000e-09, 3.5435307363100002e+00, 1.1015106477334801e+04 },
    { 2.8600000000000000e-09, 3.2550323908499998e+00, 9.1039069941176003e+03 },
    { 2.5000000000000001e-09, 6.2390330841299999e+00, 1.1322664098304400e+04 },
    { 2.3899999999999998e-09, 4.1953879933099998e+00, 3.1546870848956000e+03 },
    { 1.9200000000000000e-09, 6.1939538125300002e+00, 2.0618019358533598e+04 },
    { 1.9099999999999998e-09, 3.6120065498599998e+00, 1.0593819301891999e+03 },
    { 1.9000000000000001e-09, 1.7832950167099999e+00, 1.0426584641649000e+04 },
    { 1.9000000000000001e-09, 1.4274057234799999e+00, 9.9999864507729999e+03 },
    { 1.8700000000000000e-09, 3.8219508218999998e-01, 1.5773435424478000e+03 },
    { 1.7900000000000000e-09, 1.9998953182799999e+00, 2.1228392023545799e+04 },
    { 1.6100000000000000e-09, 4.5100527689599996e+00, 7.8604193924392002e+03 },
    { 1.5799999999999999e-09, 5.4329445069900002e+00, 3.8289653222319998e+02 },
    { 1.5600000000000000e-09, 6.2008095572600004e+00, 4.0853142184844000e+04 },
    { 1.4599999999999999e-09, 4.4142561395000000e+00, 2.3528661537717999e+03 },
    { 1.3399999999999999e-09, 3.6431216678700000e+00, 1.7298182327326202e+04 },
    { 1.3200000000000000e-09, 1.6898152725000001e-01, 3.9302096962196001e+03 },
    { 1.2500000000000000e-09, 2.6419710262899998e+00, 1.8849227549974199e+04 },
    { 1.0999999999999999e-09, 5.9245910440000005e-01, 8.6242126509272002e+03 },
    { 1.0700000000000000e-09, 5.8296687925599997e+00, 2.0452869412221800e+04 },
    { 7.4000000000000003e-10, 3.1617847054100001e+00, 1.6496361396202399e+04 },
    { 7.2000000000000000e-10, 1.1077815012700001e+00, 2.3581258177317599e+04 },
    { 6.9999999999999996e-10, 5.6329906178300000e+00, 7.0848967811151997e+03 },
    { 6.8000000000000003e-10, 5.8459820296200000e+00, 1.9367189162232800e+04 },
    { 5.7000000000000000e-10, 4.9623746029899998e+00, 1.1093785520934000e+03 },
    { 5.4999999999999996e-10, 2.2978754206999999e-01, 4.7320306273433998e+03 },
    { 5.4000000000000000e-10, 2.3646623360399999e+00, 2.0809467624645200e+04 },
    { 5.3000000000000003e-10, 2.3660583791700001e+00, 2.2003914634869801e+04 },
    { 4.8000000000000000e-10, 4.9356234938199997e+00, 1.8073704938650200e+04 },
    { 4.3999999999999998e-10, 5.1053397762900001e+00, 9.6389407478762005e+03 },
    { 4.3999999999999998e-10, 4.3885462710700001e+00, 1.0787630344545800e+04 },
    { 4.3000000000000001e-10, 3.7870637437100001e+00, 2.9050783743349199e+04 },
    { 4.3000000000000001e-10, 1.5185271000000000e-01, 8.0945216858326003e+03 },
    { 4.3000000000000001e-10, 4.9884925662499997e+00, 1.2566151699982800e+04 },
    { 4.0000000000000001e-10, 5.3207471148399996e+00, 6.2830758499914000e+03 },
    { 3.7000000000000001e-10, 5.1104531080899998e+00, 9.1539036160218002e+03 },
    { 3.3000000000000000e-10, 1.0417489836800000e+00, 1.4919017853754600e+04 },
    { 3.3000000000000000e-10, 9.9530602493999998e-01, 6.4897765872880000e+03 },
    { 3.1000000000000002e-10, 8.2947045576000000e-01, 5.5075532386674004e+03 },
    { 3.0000000000000000e-10, 3.3747169711899998e+00, 2.4356780788641601e+04 },
    { 3.0000000000000000e-10, 3.8800529857799999e+00, 2.0400272772622200e+04 },
    { 2.8999999999999998e-10, 4.8703261837899996e+00, 2.8286990484861199e+04 },
    { 2.8000000000000002e-10, 3.8539031767199998e+00, 2.0235122826310399e+04 },
    { 2.7000000000000000e-10, 8.1625295039000001e-01, 1.0639883737087001e+04 },
    { 2.7000000000000000e-10, 2.3944477898000001e+00, 9.7866873553349997e+03 },
    { 2.7000000000000000e-10, 5.0085931658199998e+00, 1.4143495242430599e+04 },
    { 2.5999999999999998e-10, 6.3216688286000000e-01, 3.1491641605882000e+03 },
    { 2.5999999999999998e-10, 4.6698874540000002e-01, 1.8875525869773999e+04 },
    { 2.5999999999999998e-10, 2.4700170788700002e+00, 2.7511467873537200e+04 },
    { 2.4000000000000000e-10, 5.3070456919700000e+00, 1.1272667476400200e+04 },
    { 2.4000000000000000e-10, 1.8604821968400000e+00, 2.1535949644515400e+04 },
    { 2.3000000000000001e-10, 2.2162272374300001e+00, 1.3936794505133999e+04 },
    { 2.1999999999999999e-10, 1.2662468600000000e-03, 1.8307807232043600e+04 },
    { 1.8000000000000000e-10, 3.3608988955600001e+00, 2.8521092778254599e+04 },
    { 1.5999999999999999e-10, 2.1997260118200002e+00, 1.4945316173554400e+04 },
    { 1.5999999999999999e-10, 2.9203000238199999e+00, 3.2217200181080800e+04 },
    { 1.5000000000000000e-10, 2.2353566884100000e+00, 8.6622403235630009e+03 },
    { 1.5000000000000000e-10, 3.6044385172500002e+00, 2.9580474708443799e+04 },
    { 1.4000000000000001e-10, 3.1972971168700002e+00, 2.1202093703745999e+04 },
    { 1.4000000000000001e-10, 2.5806866766100001e+00, 1.7277406931833801e+04 },
    { 1.4000000000000001e-10, 5.2646303274199999e+00, 1.1764330768858999e+04 },
    { 1.2999999999999999e-10, 1.9348822032199999e+00, 3.4570066334852600e+04 },
    { 1.2999999999999999e-10, 6.8541371816999996e-01, 2.2779437246193800e+04 },
    { 1.2999999999999999e-10, 3.9924891314100002e+00, 2.5934124331089399e+04 },
    { 1.2000000000000000e-10, 3.8974296176999998e+00, 3.1441677569756801e+04 },
    { 1.2000000000000000e-10, 3.5056624922499999e+00, 2.5158601719765400e+04 },
    { 8.9999999999999999e-11, 3.3594760027600001e+00, 3.8500276031072201e+04 },
    { 8.9999999999999999e-11, 4.9608263316299999e+00, 2.9088811415985001e+04 },
    { 7.0000000000000004e-11, 1.9486834287899999e+00, 4.3232306658415597e+04 },
    { 7.0000000000000004e-11, 1.8795639281200001e+00, 2.2805735565993600e+04 },
    { 7.0000000000000004e-11, 6.0140159619500002e+00, 2.9999959352319001e+04 },
    { 7.0000000000000004e-11, 5.9448964612799999e+00, 9.5733882598969994e+03 },
    { 7.0000000000000004e-11, 3.4050134557699998e+00, 3.0110165673538399e+04 },
};

static const VSOP87Term VSOP87A_VENUS_X3[5] = {
    { 7.3605999999999999e-07, 0.0000000000000000e+00, 0.0000000000000000e+00 },
    { 2.4718000000000002e-07, 3.2534054880499998e+00, 2.0426571092422000e+04 },
    { 1.9385999999999999e-07, 5.2303824320199999e+00, 1.0213285546211000e+04 },
    { 6.2300000000000002e-09, 3.2470980689500002e+00, 3.0639856638632999e+04 },
    { 8.9999999999999999e-11, 3.7236792311000002e+00, 4.0853142184844000e+04 },
};

static const VSOP87Term VSOP87A_VENUS_X4[4] = {
    { 9.4600000000000000e-09, 3.9076344759999998e-01, 1.0213285546211000e+04 },
    { 5.4599999999999998e-09, 3.1415926535900001e+00, 0.0000000000000000e+00 },
    { 2.8900000000000002e-09, 9.5993858755000006e-01, 2.0426571092422000e+04 },
    { 3.0000000000000000e-10, 9.8741714871999997e-01, 3.0639856638632999e+04 },
};

static const VSOP87Term VSOP87A_VENUS_X5[3] = {
    { 6.3000000000000000e-10, 3.1415926535900001e+00, 0.0000000000000000e+00 },
    { 2.8999999999999998e-10, 1.9473045775700000e+00, 1.0213285546211000e+04 },
    { 2.1999999999999999e-10, 3.4341584206999998e-01, 2.0426571092422000e+04 },
};

static const VSOP87Term VSOP87A_VENUS_Y0[565] = {
    { 7.2324820730999995e-01, 1.6057380835599999e+00, 1.0213285546211000e+04 },
    { 5.4950627299999999e-03, 3.1415926535900001e+00, 0.0000000000000000e+00 },
    { 2.4488479000000000e-03, 2.4856495400399998e+00, 2.0426571092422000e+04 },
    { 2.7898070000000001e-05, 5.0421452360599996e+00, 2.3528661537717999e+03 },
    { 1.9338680000000000e-05, 5.8059799026099999e+00, 1.5773435424478000e+03 },
    { 1.2436580000000001e-05, 3.3657369734399998e+00, 3.0639856638632999e+04 },
    { 1.1644800000000001e-05, 1.3097062027699999e+00, 1.8073704938650200e+04 },
    { 1.0418720000000000e-05, 1.8129136925000000e-01, 6.2830758499914000e+03 },
    { 7.7054900000000003e-06, 5.3036668000200002e+00, 5.2969096509459996e+02 },
    { 6.7052699999999996e-06, 6.1703243037600002e+00, 1.4143495242430599e+04 },
    { 6.5767500000000002e-06, 5.2136042704900003e+00, 8.6359420037632008e+03 },
    { 5.5963199999999998e-06, 5.8784244580799996e+00, 2.2003914634869801e+04 },
    { 5.4238100000000003e-06, 1.1504007819299999e+00, 1.1790629088658799e+04 },
    { 4.7718200000000003e-06, 4.2730938785700001e+00, 1.0186987226411200e+04 },
    { 4.7569000000000001e-06, 2.0802666077900001e+00, 1.0239583866010800e+04 },
    { 4.0705200000000000e-06, 2.3541192310699999e+00, 7.7552261132399997e+02 },
    { 3.6777800000000000e-06, 2.1762393962500002e+00, 9.4377629348869996e+03 },
    { 3.0221900000000002e-06, 9.4310085463000004e-01, 5.5075532386674004e+03 },
    { 2.7564599999999999e-06, 1.2396834852100000e+00, 9.6835945811164001e+03 },
    { 2.7418100000000000e-06, 4.2777141449000000e-01, 9.1539036160218002e+03 },
    { 2.7142700000000000e-06, 1.0537672065999999e+00, 1.9896880127327400e+04 },
    { 2.6889799999999999e-06, 5.1321865367299999e+00, 1.0742976511305600e+04 },
    { 2.4159099999999999e-06, 4.2365728945700001e+00, 1.0988808157535001e+04 },
    { 2.1446499999999999e-06, 5.4620211653600004e+00, 1.0021837280099400e+04 },
    { 2.0745600000000002e-06, 8.8354754906999999e-01, 1.0404733812322600e+04 },
    { 1.7599300000000001e-06, 1.4072111935900000e+00, 1.9144826611159999e+02 },
    { 1.5407999999999999e-06, 3.7843289345300000e+00, 4.7057323075435997e+03 },
    { 1.4661799999999999e-06, 3.9884886923099998e+00, 1.9651048481098002e+04 },
    { 1.2312000000000000e-06, 1.7105844240000001e-02, 1.0593819301891999e+03 },
    { 8.0946000000000000e-07, 2.5756527443500001e+00, 1.5720838784878400e+04 },
    { 6.9607000000000001e-07, 4.8057821315900000e+00, 9.1039069941176003e+03 },
    { 6.9363000000000000e-07, 3.6996417036299998e+00, 1.3367972631106601e+04 },
    { 6.2765000000000004e-07, 1.5696035711300000e+00, 1.1322664098304400e+04 },
    { 6.2213999999999998e-07, 1.0197206796900000e+00, 2.5934124331089399e+04 },
    { 6.0880000000000000e-07, 2.2148123669099999e+00, 4.5519534970588002e+03 },
    { 5.8886000000000005e-07, 1.4862420551500000e+00, 2.0618019358533598e+04 },
    { 5.7304999999999996e-07, 2.2137943526299999e+00, 3.1546870848956000e+03 },
    { 5.0885999999999997e-07, 4.5776609300000000e-02, 8.6242126509272002e+03 },
    { 4.7803000000000000e-07, 5.5704318154500001e+00, 2.1329909543799999e+02 },
    { 4.6449000000000000e-07, 5.2332880160800004e+00, 1.0192510150718599e+04 },
    { 4.6324000000000001e-07, 1.1190067333900000e+00, 1.0234060941703399e+04 },
    { 4.5634999999999998e-07, 6.4229163389999996e-02, 1.9367189162232800e+04 },
    { 4.5195000000000002e-07, 4.8101910279999999e-01, 2.9580474708443799e+04 },
    { 4.3008000000000000e-07, 4.7740648411500004e+00, 1.1015106477334801e+04 },
    { 4.0190999999999998e-07, 1.6411730535100000e+00, 9.4114646150871995e+03 },
    { 3.9695000000000002e-07, 3.2121942357000002e+00, 8.0182093112380005e+02 },
    { 3.6983000000000002e-07, 2.8242591620200002e+00, 7.0585984613153996e+03 },
    { 3.6570000000000000e-07, 2.3454244582800001e+00, 9.9999864507729999e+03 },
    { 3.5385000000000001e-07, 4.0109867743900001e+00, 1.0426584641649000e+04 },
    { 3.2033000000000001e-07, 4.1438437164700002e+00, 1.5874617595363199e+04 },
    { 3.1203000000000002e-07, 5.1415324252900003e+00, 1.7298182327326202e+04 },
    { 2.8622999999999999e-07, 3.3561217063700002e+00, 1.1093785520934000e+03 },
    { 2.6683000000000001e-07, 4.0550501062100004e+00, 7.8604193924392002e+03 },
    { 2.1268000000000000e-07, 1.3779999999699999e+00, 3.1283887650958000e+03 },
    { 2.0281000000000000e-07, 1.1313907244100001e+00, 1.0220399093211799e+04 },
    { 2.0202999999999999e-07, 5.2188195503400001e+00, 1.0206171999210201e+04 },
    { 1.9675000000000001e-07, 1.3496658384900000e+00, 3.9302096962196001e+03 },
    { 1.9544999999999999e-07, 4.8811065403999998e-01, 7.1135470007999997e+00 },
    { 1.8883999999999999e-07, 6.1668932815999999e-01, 2.6298319799800002e+01 },
    { 1.7690999999999999e-07, 3.6080098799800000e+00, 7.0848967811151997e+03 },
    { 1.7676999999999999e-07, 1.7473941091100000e+00, 3.5320606928113998e+03 },
    { 1.6089999999999999e-07, 7.8654424918999999e-01, 2.0213271996984000e+04 },
    { 1.5237000000000001e-07, 2.8137193337099999e+00, 9.8303890139878004e+03 },
    { 1.4763000000000001e-07, 2.4967614561899998e+00, 2.9864334027309000e+04 },
    { 1.4712000000000001e-07, 2.2913368499500000e+00, 2.3581258177317599e+04 },
    { 1.4371000000000001e-07, 3.5259833807000001e+00, 1.0596182078434200e+04 },
    { 1.3327000000000000e-07, 5.5345319938799999e+00, 5.6613320491521999e+03 },
    { 1.2842000000000001e-07, 4.7911260034999997e-01, 1.5890728952838001e+03 },
    { 1.1695000000000000e-07, 4.6367226652799998e+00, 3.3406124266997999e+03 },
    { 1.1121000000000000e-07, 2.2021649038900000e+00, 2.8286990484861199e+04 },
    { 1.0763000000000000e-07, 5.9454130375099998e+00, 9.9294262273458007e+03 },
    { 1.0377999999999999e-07, 4.0689057274000001e-01, 1.0497144865076199e+04 },
    { 9.9130000000000002e-08, 6.1499710182299996e+00, 9.7866873553349997e+03 },
    { 9.3940000000000006e-08, 5.3353349155199998e+00, 1.1272667476400200e+04 },
    { 8.7919999999999996e-08, 5.1811805252900003e+00, 1.0207762621903599e+04 },
    { 8.7859999999999994e-08, 1.1710997765900000e+00, 1.0218808470518399e+04 },
    { 8.7610000000000002e-08, 1.1029927382000000e-01, 2.9050783743349199e+04 },
    { 8.1479999999999999e-08, 3.1277414380500002e+00, 1.8837498197138189e+04 },
    { 7.9780000000000004e-08, 1.8912784808100001e+00, 1.2566151699982800e+04 },
    { 7.4980000000000000e-08, 4.2453350798100002e+00, 4.0853142184844000e+04 },
    { 7.4449999999999995e-08, 3.2864286073400000e+00, 2.1535949644515400e+04 },
    { 7.4229999999999997e-08, 2.5906794696700000e+00, 1.0175257873575199e+04 },
    { 7.3889999999999995e-08, 5.5125509577100003e+00, 1.6496361396202399e+04 },
    { 7.3860000000000001e-08, 3.7616210163299999e+00, 1.0251313218846801e+04 },
    { 6.7280000000000004e-08, 5.9568407492000004e+00, 8.0945216858326003e+03 },
    { 6.5309999999999994e-08, 5.8827950944499996e+00, 2.0419457545421192e+04 },
    { 5.9649999999999995e-08, 5.9433125558000000e-01, 4.2659819087599999e+02 },
    { 5.9629999999999999e-08, 5.0493894264400003e+00, 6.6812248533995999e+03 },
    { 5.8820000000000000e-08, 4.5870719779000002e-01, 3.2217200181080800e+04 },
    { 5.7280000000000001e-08, 7.4065551441999999e-01, 2.4356780788641601e+04 },
    { 5.6400000000000002e-08, 3.9501841556300001e+00, 3.8289653222319998e+02 },
    { 5.5070000000000003e-08, 1.8040686781199999e+00, 2.5158601719765400e+04 },
    { 5.4949999999999999e-08, 1.7207478840299999e+00, 1.7085958665722199e+04 },
    { 5.4109999999999999e-08, 2.8802465429700002e+00, 2.6087903141574199e+04 },
    { 5.3260000000000002e-08, 3.8323756674400000e+00, 3.3794543723528601e+04 },
    { 5.0890000000000001e-08, 1.0337236314999999e-01, 1.4945316173554400e+04 },
    { 4.9439999999999998e-08, 2.0895695782699999e+00, 3.1491641605882000e+03 },
    { 4.7069999999999997e-08, 4.9865091332800002e+00, 1.8849227549974199e+04 },
    { 4.6590000000000002e-08, 1.2240445893700000e+00, 2.0452869412221800e+04 },
    { 4.5930000000000002e-08, 1.0635835575899999e+00, 1.4765239043269799e+04 },
    { 4.3299999999999997e-08, 1.8356372110900001e+00, 2.3958631785233400e+04 },
    { 3.9979999999999998e-08, 4.6564827194899996e+00, 4.7320306273433998e+03 },
    { 3.9780000000000003e-08, 1.3252841256300001e+00, 1.3745346239022399e+04 },
    { 3.4770000000000000e-08, 5.0546852663599999e+00, 1.5510452226479999e+03 },
    { 3.4219999999999998e-08, 3.2026017277299998e+00, 2.9088811415985001e+04 },
    { 3.3169999999999999e-08, 1.4927589801800001e+00, 1.8875525869773999e+04 },
    { 3.2210000000000002e-08, 5.1487732014700001e+00, 2.0400272772622200e+04 },
    { 3.1669999999999998e-08, 3.7283394293300001e+00, 2.7511467873537200e+04 },
    { 2.9490000000000001e-08, 1.2528349599899999e+00, 1.0198033075026000e+04 },
    { 2.9440000000000001e-08, 5.0994621702899998e+00, 1.0228538017396000e+04 },
    { 2.8880000000000001e-08, 2.6588765482899999e+00, 1.8307807232043600e+04 },
    { 2.7430000000000001e-08, 2.1013076684000001e-01, 5.4812549188676003e+03 },
    { 2.6009999999999999e-08, 2.2243168636299999e+00, 9.5805018068978006e+03 },
    { 2.4710000000000001e-08, 4.1766328550100003e+00, 1.0846069285524200e+04 },
    { 2.4310000000000001e-08, 2.2852510533300001e+00, 3.0110165673538399e+04 },
    { 2.4310000000000001e-08, 2.1051627740300001e+00, 2.1228392023545799e+04 },
    { 2.4240000000000001e-08, 5.6814095480900004e+00, 2.0956262057516600e+04 },
    { 2.3759999999999999e-08, 5.9939003106700000e+00, 8.6622403235630009e+03 },
    { 2.3079999999999999e-08, 1.1939979472000000e+00, 6.8726731195111997e+03 },
    { 2.3059999999999999e-08, 2.3272914866300001e+00, 3.4425749449653999e+03 },
    { 2.2780000000000000e-08, 2.8211031001000002e+00, 1.0110192771992401e+04 },
    { 2.2780000000000000e-08, 3.5311902936799999e+00, 1.0316378320429600e+04 },
    { 2.2600000000000001e-08, 5.9802721739999998e-01, 9.5733882598969994e+03 },
    { 2.2069999999999998e-08, 2.0300052943500000e+00, 5.3680451209540001e+02 },
    { 2.1880000000000001e-08, 4.4112927699600002e+00, 9.9674538999815995e+03 },
    { 2.1279999999999999e-08, 3.5707150013400000e+00, 1.0309277421860000e+02 },
    { 2.1179999999999998e-08, 1.9408032819800001e+00, 1.0459117192440401e+04 },
    { 2.0999999999999999e-08, 5.8190532922899996e+00, 2.8521092778254599e+04 },
    { 2.0929999999999999e-08, 2.6926416845599999e+00, 9.6907081281171995e+03 },
    { 2.0809999999999999e-08, 5.3887335735799997e+00, 5.2257741809380002e+02 },
    { 2.0739999999999999e-08, 4.6213763340499998e+00, 3.3019021112204602e+04 },
    { 2.0440000000000000e-08, 5.7528965524100002e+00, 1.9999972901545989e+04 },
    { 2.0100000000000001e-08, 5.2797413212700004e+00, 2.1202093703745999e+04 },
    { 1.9880000000000000e-08, 4.0861464786399999e+00, 2.0809467624645200e+04 },
    { 1.9840000000000001e-08, 4.1115516107000003e+00, 1.7277406931833801e+04 },
    { 1.9760000000000000e-08, 5.2515566190299996e+00, 3.7724753419748202e+04 },
    { 1.9620000000000000e-08, 3.6641664936699998e+00, 1.0735862964304801e+04 },
    { 1.8730000000000000e-08, 6.2148691077700002e+00, 3.0213258447757002e+04 },
    { 1.8040000000000001e-08, 2.4320609997000000e+00, 2.2779437246193800e+04 },
    { 1.7660000000000001e-08, 4.7016089611499998e+00, 7.0641213856227996e+03 },
    { 1.7529999999999999e-08, 5.1095108027800000e+00, 1.3553897972910800e+04 },
    { 1.6590000000000002e-08, 2.3005584299999999e-03, 2.0235122826310399e+04 },
    { 1.5920000000000000e-08, 9.1871579673000003e-01, 1.0288067144778301e+04 },
    { 1.5920000000000000e-08, 5.4335775970500002e+00, 1.0138503947643700e+04 },
    { 1.5670000000000001e-08, 3.6646068927600002e+00, 9.1467900690210008e+03 },
    { 1.5600000000000001e-08, 5.0764322982200003e+00, 9.8968936765544004e+03 },
    { 1.5600000000000001e-08, 1.2758610955600000e+00, 1.0529677415867600e+04 },
    { 1.5419999999999998e-08, 2.4518100718500002e+00, 6.4897765872880000e+03 },
    { 1.4829999999999999e-08, 4.3135075348800003e+00, 3.7235089589230001e+03 },
    { 1.4800000000000000e-08, 4.2017256541599997e+00, 1.6983996147456601e+04 },
    { 1.4720000000000000e-08, 5.5808934154100003e+00, 6.7707106012455997e+03 },
    { 1.4349999999999999e-08, 3.9193883020999998e-01, 9.1610171630225996e+03 },
    { 1.4020000000000001e-08, 3.4328115011300002e+00, 1.0639883737087001e+04 },
    { 1.3599999999999999e-08, 4.2291926700399998e+00, 1.0232955307110789e+04 },
    { 1.3599999999999999e-08, 2.1231007237399999e+00, 1.0193615785311211e+04 },
    { 1.2690000000000000e-08, 6.0384289948700003e+00, 3.6949230808424203e+04 },
    { 1.2569999999999999e-08, 3.9359817272500002e+00, 9.7938009023358009e+03 },
    { 1.2540000000000000e-08, 5.8079587948100002e+00, 2.3791644735716000e+03 },
    { 1.2369999999999999e-08, 1.1151640632700000e+00, 1.2592450019782600e+04 },
    { 1.2320000000000000e-08, 5.3638907799300002e+00, 2.0775395492400001e+01 },
    { 1.2159999999999999e-08, 3.9252531823200001e+00, 7.4781598567299994e+01 },
    { 1.1920000000000000e-08, 2.9560157423000000e+00, 3.9264069289560197e+04 },
    { 1.1870000000000000e-08, 5.0435039497399998e+00, 3.1441677569756801e+04 },
    { 1.1780000000000000e-08, 4.4524704501799999e+00, 1.4919017853754600e+04 },
    { 1.1580000000000000e-08, 2.2882282504300000e+00, 1.0632770190086199e+04 },
    { 1.1390000000000000e-08, 8.6087945459000004e-01, 2.1187638603783998e+03 },
    { 1.1199999999999999e-08, 2.6190073863000002e-01, 2.4583164622940001e+02 },
    { 1.0099999999999999e-08, 5.8080423090100002e+00, 1.0007099997773799e+04 },
    { 1.0080000000000000e-08, 5.5046820997000001e-01, 1.0419471094648199e+04 },
    { 9.9499999999999998e-09, 2.9674694255600000e+00, 1.3341674311306801e+04 },
    { 9.6699999999999999e-09, 2.5812261659999999e-02, 6.3989728631399998e+02 },
    { 8.4499999999999996e-09, 3.1638968566000001e+00, 2.2805735565993600e+04 },
    { 8.0800000000000002e-09, 3.7657594142000000e-01, 4.1654963115967803e+04 },
    { 7.9799999999999993e-09, 3.5117512169200000e+00, 1.3936794505133999e+04 },
    { 7.9500000000000001e-09, 1.8535265921699999e+00, 1.0750090058306399e+04 },
    { 7.8899999999999998e-09, 4.8937701935000000e+00, 7.9945284420241996e+03 },
    { 7.8700000000000003e-09, 4.3192287273099996e+00, 1.9889766580326592e+04 },
    { 7.8600000000000006e-09, 5.1228624161000003e-01, 1.1764330768858999e+04 },
    { 7.8399999999999994e-09, 4.4866617405999998e+00, 9.6764810341156008e+03 },
    { 7.8299999999999996e-09, 1.1753791497499999e+00, 4.0879440504643797e+04 },
    { 7.6899999999999997e-09, 5.3970900696699999e+00, 7.8341210726394002e+03 },
    { 7.6100000000000000e-09, 5.5900441644900001e+00, 7.5648307207380003e+03 },
    { 7.4400000000000000e-09, 9.4447044888999998e-01, 1.9903993674328191e+04 },
    { 6.9500000000000002e-09, 1.8200133448300000e+00, 3.6147409877300401e+04 },
    { 6.8900000000000000e-09, 3.0328815472100001e+00, 1.0664954771900000e+03 },
    { 6.8599999999999999e-09, 1.5414380724700001e+00, 1.2432042650397800e+04 },
    { 6.7599999999999998e-09, 3.6314291974900001e+00, 2.0529663866640589e+04 },
    { 6.6500000000000000e-09, 4.7649062986499997e+00, 1.1624747044077999e+03 },
    { 6.4700000000000002e-09, 4.4959679391399998e+00, 2.6735945262213201e+04 },
    { 6.3199999999999997e-09, 5.8958914619499998e+00, 5.2175806283148399e+04 },
    { 6.2400000000000000e-09, 1.2767742847300001e+00, 3.9793760254654801e+04 },
    { 6.0500000000000004e-09, 3.0636611027599998e+00, 1.0059506735726200e+04 },
    { 5.9300000000000000e-09, 3.2887094282399998e+00, 1.0367064356695801e+04 },
    { 5.8200000000000002e-09, 1.0110665708700000e+00, 8.5211198767086007e+03 },
    { 5.2000000000000002e-09, 1.4488150746899999e+00, 1.1020632121940000e+02 },
    { 5.0300000000000002e-09, 2.2697808557700001e+00, 1.7778116266948990e+04 },
    { 4.9799999999999998e-09, 2.6130132474500001e+00, 1.6522659716002199e+04 },
    { 4.9000000000000000e-09, 2.5960140034100001e+00, 4.4809650200863398e+04 },
    { 4.7799999999999996e-09, 1.9721186884000000e-01, 3.5371887265976402e+04 },
    { 4.7200000000000002e-09, 3.2662262297600000e+00, 2.2187571041868000e+03 },
    { 4.6800000000000004e-09, 3.7298821053100002e+00, 3.8133035637799999e+01 },
    { 4.6399999999999997e-09, 8.4118413790000002e-01, 8.6313261979279996e+03 },
    { 4.5999999999999998e-09, 5.4029716828299996e+00, 1.1802358441494800e+04 },
    { 4.5399999999999996e-09, 5.5048075853500000e+00, 1.0042612675591799e+04 },
    { 4.4400000000000004e-09, 8.4748587084000004e-01, 1.0383958416830201e+04 },
    { 4.3899999999999999e-09, 4.5701093700200000e+00, 2.4150080051345001e+04 },
    { 4.3899999999999999e-09, 5.3380755895099998e+00, 2.7991401813159999e+04 },
    { 4.3599999999999998e-09, 2.8613813561000001e-01, 9.6389407478762005e+03 },
    { 4.3100000000000002e-09, 3.3024406104900002e+00, 8.6170991039264009e+03 },
    { 4.2899999999999999e-09, 1.1016970940499999e+00, 1.3655860491176400e+04 },
    { 4.2000000000000004e-09, 1.7727512570299999e+00, 9.0508108418032007e+03 },
    { 4.1599999999999997e-09, 1.5526058451999999e-01, 1.0225014899047001e+04 },
    { 4.1599999999999997e-09, 6.1970328092600004e+00, 1.0201556193375000e+04 },
    { 4.0400000000000001e-09, 5.9801568429899996e+00, 3.0666154958432799e+04 },
    { 4.0400000000000001e-09, 2.4151344829900001e+00, 2.0351789493854700e+04 },
    { 3.9899999999999997e-09, 2.6783320997500000e+00, 1.0211801073502700e+04 },
    { 3.9899999999999997e-09, 3.6739612940300002e+00, 1.0214770018919300e+04 },
    { 3.9799999999999999e-09, 5.6936966140900003e+00, 1.0522683831884001e+03 },
    { 3.9799999999999999e-09, 2.0775706910000000e+00, 9.8359119382951994e+03 },
    { 3.9199999999999997e-09, 4.0532910074900004e+00, 6.3278373931320004e+02 },
    { 3.9199999999999997e-09, 7.1173908582000001e-01, 2.9999959352319001e+04 },
    { 3.9099999999999999e-09, 6.0160975858599999e+00, 1.0787630344545800e+04 },
    { 3.8499999999999997e-09, 6.0810682531999998e+00, 2.0180739446192601e+04 },
    { 3.7900000000000004e-09, 4.9418402605800003e+00, 1.9317192540328600e+04 },
    { 3.7900000000000004e-09, 4.2747228387599998e+00, 1.0590659154126801e+04 },
    { 3.7700000000000000e-09, 4.1022448573599997e+00, 1.4956319713459999e+02 },
    { 3.6500000000000000e-09, 6.1370674314100002e+00, 1.0348910871221000e+04 },
    { 3.6500000000000000e-09, 2.1522596237000000e-01, 1.0077660221201000e+04 },
    { 3.5499999999999999e-09, 2.6857312723000000e+00, 3.0831304904744600e+04 },
    { 3.5400000000000002e-09, 1.2722680453099999e+00, 1.0323491867430401e+04 },
    { 3.5400000000000002e-09, 5.0800253484700004e+00, 1.0103079224991599e+04 },
    { 3.4299999999999999e-09, 3.0721630454300000e+00, 8.1062510386685999e+03 },
    { 3.3600000000000000e-09, 5.4489846078699999e+00, 2.2041264243879999e+02 },
    { 3.3099999999999999e-09, 4.6933116598400000e+00, 2.7490692478044799e+04 },
    { 3.2900000000000000e-09, 1.7636584283500001e+00, 4.5585172812187397e+04 },
    { 3.2799999999999998e-09, 1.0143802747599999e+00, 1.9786673806107989e+04 },
    { 3.2700000000000001e-09, 1.4063004971299999e+00, 9.3600891644589992e+03 },
    { 3.1800000000000002e-09, 9.4946325025000000e-01, 6.3093741697912001e+03 },
    { 3.1500000000000001e-09, 4.7310939358199997e+00, 2.0639870187860000e+04 },
    { 3.1399999999999999e-09, 6.1131949770200000e+00, 2.0405795696929599e+04 },
    { 3.0899999999999999e-09, 4.0156867936499996e+00, 4.8739859897082999e+04 },
    { 3.0600000000000002e-09, 5.8387052389200003e+00, 2.9577158944766002e+03 },
    { 2.9400000000000002e-09, 3.4094498327200000e+00, 1.9360075615231988e+04 },
    { 2.9199999999999998e-09, 3.6581460866500000e+00, 2.7197281693667599e+04 },
    { 2.8299999999999999e-09, 3.9631637380500000e+00, 3.9039113764198000e+03 },
    { 2.8100000000000000e-09, 5.4957106665300000e+00, 9.3997352622512008e+03 },
    { 2.7999999999999998e-09, 2.6057698704500001e+00, 3.8734378324465601e+04 },
    { 2.7499999999999998e-09, 1.0397628458599999e+00, 3.4596364654652403e+04 },
    { 2.7299999999999999e-09, 2.8139860818000000e+00, 2.0618554843720000e+02 },
    { 2.7200000000000001e-09, 3.2801279152500000e+00, 1.2320320053753399e+04 },
    { 2.7099999999999999e-09, 2.2856900168699998e+00, 9.7453205558566005e+03 },
    { 2.6900000000000000e-09, 4.0159938991299997e+00, 2.6709646942413401e+04 },
    { 2.6700000000000001e-09, 2.3039473695999999e+00, 3.1749235190726398e+04 },
    { 2.6599999999999999e-09, 2.7282919188000001e+00, 3.7737360791579999e+02 },
    { 2.6400000000000000e-09, 3.5898223447800000e+00, 7.6266071275599998e+01 },
    { 2.6299999999999998e-09, 1.5905064319600000e+00, 2.0433684639422801e+04 },
    { 2.5899999999999999e-09, 3.7391507722899999e+00, 2.9573361161443001e+04 },
    { 2.5800000000000002e-09, 9.5259827691999999e-01, 9.8643611257630000e+03 },
    { 2.5700000000000000e-09, 3.2089486145900001e+00, 1.0063722349076401e+04 },
    { 2.5500000000000001e-09, 2.4638500907299998e+00, 9.9928729037722005e+03 },
    { 2.5500000000000001e-09, 3.8884433030499999e+00, 1.0433698188649800e+04 },
    { 2.5500000000000001e-09, 8.5666245088000004e-01, 1.1026835830170799e+04 },
    { 2.5399999999999999e-09, 4.0666562780600000e+00, 1.0681250536565400e+04 },
    { 2.5399999999999999e-09, 2.1751790602700001e+00, 3.7410567239878597e+04 },
    { 2.5399999999999999e-09, 1.5885154809900000e+00, 1.9264096388014201e+04 },
    { 2.4699999999999999e-09, 3.3731775966999999e-01, 2.9587588255444600e+04 },
    { 2.4600000000000002e-09, 5.3997936581100001e+00, 1.0562209966659000e+04 },
    { 2.3400000000000002e-09, 2.2493561196499998e+00, 1.9624750161298201e+04 },
    { 2.3300000000000000e-09, 1.5922959731199999e+00, 3.9302096962196003e+04 },
    { 2.2900000000000002e-09, 2.0430029595999999e-01, 5.7434479833479998e+02 },
    { 2.2900000000000002e-09, 2.8844255402800001e+00, 9.4917560896980001e+02 },
    { 2.2900000000000002e-09, 2.8521629763299998e+00, 9.2641099372412009e+03 },
    { 2.2200000000000002e-09, 2.8178928585000000e+00, 8.7344189721465991e+03 },
    { 2.1999999999999998e-09, 2.1324274255500000e+00, 3.8027672635800002e+01 },
    { 2.1200000000000001e-09, 1.5896572039900001e+00, 1.0206656987310989e+04 },
    { 2.1200000000000001e-09, 4.7626361897900003e+00, 1.0219914105111009e+04 },
    { 2.1100000000000000e-09, 4.8951536238899997e+00, 1.0249313412888399e+04 },
    { 2.1100000000000000e-09, 1.4571397698899999e+00, 1.0177257679533601e+04 },
    { 2.1100000000000000e-09, 3.5444970230299999e+00, 1.1162461155180799e+04 },
    { 2.1100000000000000e-09, 4.5917513840600002e+00, 1.1375760250618800e+04 },
    { 2.0599999999999999e-09, 1.8039653947800001e+00, 3.0426557543194998e+04 },
    { 2.0200000000000001e-09, 5.5377150860700004e+00, 1.6894510399610601e+04 },
    { 1.9700000000000000e-09, 3.2027146807800002e+00, 4.0077619573520002e+04 },
    { 1.9700000000000000e-09, 5.6712497732299996e+00, 1.6921656695024001e+03 },
    { 1.9700000000000000e-09, 1.8699440035299999e+00, 8.4608283475120004e+02 },
    { 1.9599999999999998e-09, 3.8451760631899998e+00, 1.2934842235826000e+03 },
    { 1.9500000000000001e-09, 3.8405186353800000e+00, 8.8375117463652005e+03 },
    { 1.9099999999999998e-09, 5.4346070503400004e+00, 5.2670069593302600e+04 },
    { 1.8600000000000000e-09, 2.9662494153200001e+00, 1.0251418581848800e+04 },
    { 1.8600000000000000e-09, 3.3860439784600000e+00, 1.0175152510573200e+04 },
    { 1.8600000000000000e-09, 3.2930225127799999e+00, 1.7271884007526402e+04 },
    { 1.8300000000000001e-09, 2.4736384967100000e+00, 3.8526574350871997e+04 },
    { 1.8199999999999999e-09, 2.2707302681299999e+00, 9.2569963902403997e+03 },
    { 1.7800000000000001e-09, 5.2214950354300003e+00, 1.0949162059742799e+04 },
    { 1.7800000000000001e-09, 1.1307983583500001e+00, 9.4774090326791993e+03 },
    { 1.7700000000000001e-09, 4.8352968499299998e+00, 2.0316364771202589e+04 },
    { 1.7599999999999999e-09, 3.3520501485999998e-01, 1.4227094001599999e+01 },
    { 1.6900000000000000e-09, 4.1483856899699996e+00, 1.1265553929399401e+04 },
    { 1.6800000000000000e-09, 6.2501827358999995e-01, 7.9914289116139998e+03 },
    { 1.6600000000000001e-09, 2.1808250779099998e+00, 1.0263042571682799e+04 },
    { 1.6600000000000001e-09, 4.1714683158700003e+00, 1.0163528520739201e+04 },
    { 1.5600000000000000e-09, 6.2609772133900004e+00, 9.7795738083342003e+03 },
    { 1.5600000000000000e-09, 2.6200149610799999e+00, 2.0388438056784202e+04 },
    { 1.5500000000000000e-09, 5.5717912358300001e+00, 8.2377930003549991e+03 },
    { 1.5400000000000001e-09, 5.4359757990600004e+00, 1.0199058452209400e+04 },
    { 1.5400000000000001e-09, 9.1631759471999996e-01, 1.0227512640212601e+04 },
    { 1.5199999999999999e-09, 4.1936467028399997e+00, 2.8385931886520001e+02 },
    { 1.5199999999999999e-09, 4.8105416395700002e+00, 9.3672027114598004e+03 },
    { 1.5100000000000000e-09, 5.5548063321000001e+00, 1.0245818097002400e+04 },
    { 1.5100000000000000e-09, 7.9748706168000005e-01, 1.0180752995419600e+04 },
    { 1.4900000000000000e-09, 6.1484998961000004e-01, 1.0135611775783000e+04 },
    { 1.4900000000000000e-09, 5.7374434041700004e+00, 1.0290959316639000e+04 },
    { 1.4800000000000001e-09, 7.7105069393000003e-01, 1.9374302709233601e+04 },
    { 1.4400000000000000e-09, 2.2561530948800002e+00, 9.9858094134219991e+03 },
    { 1.4400000000000000e-09, 4.0961402989000000e+00, 1.0440761678999999e+04 },
    { 1.4300000000000000e-09, 6.1157177482099998e+00, 2.0447346487914401e+04 },
    { 1.3999999999999999e-09, 5.0426803740799997e+00, 1.0191434716884600e+04 },
    { 1.3999999999999999e-09, 1.3096130197000000e+00, 1.0235136375537400e+04 },
    { 1.3800000000000000e-09, 5.0973403516899998e+00, 1.0217217699474100e+04 },
    { 1.3800000000000000e-09, 1.2549530420899999e+00, 1.0209353392947900e+04 },
    { 1.3500000000000001e-09, 4.5055323246999999e+00, 5.9992165311261997e+03 },
    { 1.3399999999999999e-09, 3.0973816898900002e+00, 4.9515382508406998e+04 },
    { 1.3100000000000000e-09, 6.8489685701000003e-01, 8.1016352328333996e+03 },
    { 1.3000000000000001e-09, 4.0009230881500004e+00, 1.0236060747661801e+04 },
    { 1.3000000000000001e-09, 2.3513703056300002e+00, 1.0190510344760200e+04 },
    { 1.3000000000000001e-09, 2.0238133384400001e+00, 3.8204687359370997e+04 },
    { 1.3000000000000001e-09, 1.6461126682900000e+00, 4.1760413424489998e+03 },
    { 1.2600000000000000e-09, 2.2499417866500000e+00, 2.0277007895287399e+04 },
    { 1.2500000000000000e-09, 7.8050551091999998e-01, 1.2188778092066999e+04 },
    { 1.2300000000000001e-09, 2.9902757604499999e+00, 4.3232306658415597e+04 },
    { 1.2199999999999999e-09, 2.3410923235099999e+00, 1.9793787353108801e+04 },
    { 1.2199999999999999e-09, 5.6016938115899997e+00, 2.4383079108441401e+04 },
    { 1.2199999999999999e-09, 5.7162307533000001e-01, 5.6600279289522201e+04 },
    { 1.2199999999999999e-09, 3.8608713526099998e+00, 2.6367254726370002e+03 },
    { 1.2100000000000000e-09, 3.8953674773300002e+00, 4.2456784047091598e+04 },
    { 1.1900000000000000e-09, 1.1970682755499999e+00, 7.2555696517344004e+03 },
    { 1.1700000000000001e-09, 5.6482118394000000e+00, 1.5819593482830001e+03 },
    { 1.1200000000000001e-09, 8.9977181729000000e-01, 2.0206158449983192e+04 },
    { 1.0900000000000000e-09, 4.5901878612000004e+00, 4.3071899289030800e+04 },
    { 1.0700000000000000e-09, 3.0827657551400001e+00, 2.0690068349066000e+03 },
    { 1.0600000000000001e-09, 6.1960248836799998e+00, 1.7468855197945399e+04 },
    { 1.0300000000000000e-09, 2.8078047956900001e+00, 2.4978524589480799e+04 },
    { 1.0300000000000000e-09, 3.5816070332300001e+00, 1.0244199671846000e+04 },
    { 1.0300000000000000e-09, 2.7706863605500001e+00, 1.0182371420576001e+04 },
    { 1.0200000000000000e-09, 2.5240557652500000e+00, 4.1948464387519999e+02 },
    { 1.0200000000000000e-09, 5.8806900142299998e+00, 3.9321532631000000e+00 },
    { 1.0100000000000000e-09, 3.6935808057199999e+00, 2.0043674560198800e+04 },
    { 1.0100000000000000e-09, 2.8642252933000001e+00, 3.8500276031072201e+04 },
    { 1.0000000000000001e-09, 3.4994255690300000e+00, 1.5961864422845999e+03 },
    { 1.0000000000000001e-09, 1.0674711847100000e+00, 2.8947690969130599e+04 },
    { 9.9000000000000010e-10, 2.6724175591599999e+00, 7.8263709424722590e+04 },
    { 9.9000000000000010e-10, 1.2312063469300001e+00, 2.6484548254729998e+03 },
    { 9.7999999999999992e-10, 1.5545222955700000e+00, 1.1059368380962200e+04 },
    { 9.7999999999999992e-10, 3.8011187401700002e+00, 2.0220385543984801e+04 },
    { 9.6999999999999996e-10, 5.6329001699300001e+00, 8.5319638175199998e+02 },
    { 9.6999999999999996e-10, 5.8828638026700002e+00, 3.2858613742819791e+04 },
    { 9.6999999999999996e-10, 2.3487446678600001e+00, 2.3331963928720002e+03 },
    { 9.5999999999999999e-10, 5.0742223390000001e-02, 3.0206144900756190e+04 },
    { 9.5000000000000003e-10, 4.6006492106000002e-01, 1.0202239845947101e+04 },
    { 9.5000000000000003e-10, 5.8922284727200003e+00, 1.0224331246474900e+04 },
    { 9.5000000000000003e-10, 1.7805574169500000e+00, 2.9477381934225199e+04 },
    { 9.5000000000000003e-10, 2.9469154587399999e+00, 3.0220371994757799e+04 },
    { 9.2999999999999999e-10, 1.8049398362000000e+00, 8.2047280070519992e+03 },
    { 9.0999999999999996e-10, 2.4575413854999999e+00, 1.8844611744138991e+04 },
    { 9.0999999999999996e-10, 3.8414288392899998e+00, 1.4169793562230399e+04 },
    { 8.9999999999999999e-10, 3.6169109566700000e+00, 2.0049197484506200e+04 },
    { 8.7999999999999996e-10, 2.7621335542600001e+00, 2.0350305021146400e+04 },
    { 8.6999999999999999e-10, 2.9324141628199998e+00, 8.0874081388318000e+03 },
    { 8.6999999999999999e-10, 3.9082781851999999e+00, 2.9264082838787199e+04 },
    { 8.6000000000000003e-10, 5.2192309163399999e+00, 7.0351397556434003e+03 },
    { 8.6000000000000003e-10, 8.9639333662999998e-01, 2.9057897290349989e+04 },
    { 8.4999999999999996e-10, 3.4664795894700000e+00, 1.9477395483452190e+04 },
    { 8.4999999999999996e-10, 5.6779936902000006e-01, 3.9149572250346000e+03 },
    { 8.3999999999999999e-10, 5.0394550320800002e+00, 2.7461710848065399e+04 },
    { 8.3999999999999999e-10, 5.9403851257599998e+00, 1.0401210693973600e+04 },
    { 8.3999999999999999e-10, 4.1190826802000002e-01, 1.0025360398448400e+04 },
    { 8.3999999999999999e-10, 4.1831737617400000e+00, 1.4788665740644001e+03 },
    { 8.3000000000000003e-10, 4.2225877090899999e+00, 9.7216222537522008e+03 },
    { 8.3000000000000003e-10, 2.1297056846900002e+00, 1.0704948838669799e+04 },
    { 8.3000000000000003e-10, 5.3415146183899997e+00, 4.6386993743311199e+04 },
    { 8.1999999999999996e-10, 5.9594832456200004e+00, 5.5229243074000003e+00 },
    { 7.8999999999999996e-10, 1.0883944167499999e+00, 1.0018314161750401e+04 },
    { 7.8999999999999996e-10, 5.2638989770300002e+00, 1.0408256930671600e+04 },
    { 7.8999999999999996e-10, 7.8455831235999995e-01, 2.8313288804660999e+04 },
    { 7.8999999999999996e-10, 1.9917043366500000e+00, 6.0530488985741802e+04 },
    { 7.8999999999999996e-10, 3.5501612714799999e+00, 1.0137019474935400e+04 },
    { 7.7999999999999999e-10, 5.3799845504599997e+00, 7.8800891533389904e+03 },
    { 7.7999999999999999e-10, 3.3727210767500000e+00, 2.9043670196348390e+04 },
    { 7.7000000000000003e-10, 2.0150727305700000e+00, 1.0853182832525001e+04 },
    { 7.5999999999999996e-10, 3.3038646994300001e+00, 1.0243106984359800e+04 },
    { 7.5999999999999996e-10, 3.0484286943500001e+00, 1.0183464108062200e+04 },
    { 7.5999999999999996e-10, 4.5395025924699999e+00, 4.4007829269739603e+04 },
    { 7.5999999999999996e-10, 4.5127640838699996e+00, 9.5628915597059995e+02 },
    { 7.5999999999999996e-10, 5.7644053200599998e+00, 2.1485953022611189e+04 },
    { 7.5999999999999996e-10, 2.3690636489800001e+00, 2.9683567482662391e+04 },
    { 7.5000000000000000e-10, 4.4027902661000001e-01, 2.1000915890756802e+04 },
    { 7.5000000000000000e-10, 6.4877739612999996e-01, 1.8734405422919601e+04 },
    { 7.4000000000000003e-10, 3.5341011835000002e-01, 8.8273902698748007e+03 },
    { 7.4000000000000003e-10, 3.3511273120200000e+00, 5.3285184835241802e+04 },
    { 7.4000000000000003e-10, 3.9136973991100001e+00, 2.1720055316004600e+04 },
    { 7.2999999999999996e-10, 2.2798711916799999e+00, 5.1449324263278198e+04 },
    { 7.2000000000000000e-10, 1.9002954875000000e+00, 1.7248425301854400e+04 },
    { 7.2000000000000000e-10, 2.3886704867200002e+00, 3.4171917331444398e+04 },
    { 7.2000000000000000e-10, 1.3848852336000000e-01, 1.8830384650137388e+04 },
    { 7.2000000000000000e-10, 2.5696452809900001e+00, 1.9573374710669988e+04 },
    { 7.2000000000000000e-10, 2.9006014674400000e+00, 1.0060810855926179e+05 },
    { 7.1000000000000003e-10, 2.5222384707100001e+00, 2.2645328196608789e+04 },
    { 7.1000000000000003e-10, 6.6382087848000004e-01, 2.8418000004035999e+04 },
    { 6.9999999999999996e-10, 6.2039395656099998e+00, 6.2983283211764001e+03 },
    { 6.9999999999999996e-10, 1.4347819725800000e+00, 1.5738204240988000e+03 },
    { 6.9999999999999996e-10, 7.5267806535000004e-01, 1.5808666607968000e+03 },
    { 6.9000000000000000e-10, 5.6169790333099998e+00, 4.1044590450955598e+04 },
    { 6.9000000000000000e-10, 6.8347421356000004e-01, 9.6444636721835996e+03 },
    { 6.9000000000000000e-10, 5.6688191802199999e+00, 1.0782107420238401e+04 },
    { 6.9000000000000000e-10, 4.2936836187500003e+00, 1.8093374699549990e+04 },
    { 6.6999999999999996e-10, 2.0586207724099999e+00, 3.9565080160194002e+03 },
    { 6.6000000000000000e-10, 3.3410993124399999e+00, 7.8604193924391991e+04 },
    { 6.6000000000000000e-10, 4.4808707758499997e+00, 1.9580488257670801e+04 },
    { 6.5000000000000003e-10, 2.1422959020999999e+00, 3.2243498500880600e+04 },
    { 6.3999999999999996e-10, 5.8774739291699998e+00, 1.8300693685042788e+04 },
    { 6.3999999999999996e-10, 4.0007498025699997e+00, 3.0007072899319799e+04 },
    { 6.3000000000000000e-10, 3.1864645061200001e+00, 1.8314920779044402e+04 },
    { 6.2000000000000003e-10, 5.9988840094900002e+00, 1.1599180822547200e+04 },
    { 6.2000000000000003e-10, 1.8989519658500000e+00, 3.1022753170856198e+04 },
    { 6.0999999999999996e-10, 1.5584625629800000e+00, 2.7682140744156401e+04 },
    { 6.0999999999999996e-10, 5.2709693280999996e+00, 1.8100003258450000e+04 },
    { 6.0999999999999996e-10, 4.4158509097699996e+00, 1.0139988420352000e+04 },
    { 6.0999999999999996e-10, 1.9364424840100001e+00, 1.0286582672070001e+04 },
    { 6.0999999999999996e-10, 5.2841397876400000e+00, 6.5669351688566003e+03 },
    { 6.0000000000000000e-10, 6.0610990177500002e+00, 2.0421048168114601e+04 },
    { 6.0000000000000000e-10, 6.1840161278199997e+00, 8.6464613316831194e+04 },
    { 6.0000000000000000e-10, 4.3422620990400000e+00, 3.6843780499902000e+03 },
    { 6.0000000000000000e-10, 1.8462226813800000e+00, 1.4427354561295801e+04 },
    { 5.9000000000000003e-10, 5.9727970750299999e+00, 1.0254836337195800e+04 },
    { 5.9000000000000003e-10, 3.7949631875000001e-01, 1.0171734755226200e+04 },
    { 5.9000000000000003e-10, 4.7393122017999998e+00, 4.7162516354635198e+04 },
    { 5.9000000000000003e-10, 3.5933017098300000e+00, 1.8204714457824990e+04 },
    { 5.7999999999999996e-10, 1.5113224691400000e+00, 7.5765600735739999e+03 },
    { 5.7999999999999996e-10, 1.4865333809800001e+00, 7.5449506839496389e+04 },
    { 5.7999999999999996e-10, 6.0242626588099997e+00, 2.0412343998420391e+04 },
    { 5.7999999999999996e-10, 1.4816501352900000e+00, 1.8612028613130001e+03 },
    { 5.7000000000000000e-10, 2.1083550483700000e+00, 1.1279781023400999e+04 },
    { 5.7000000000000000e-10, 2.4918355058500001e+00, 2.0464598765057799e+04 },
    { 5.7000000000000000e-10, 3.4696428699599999e+00, 8.3078207812706005e+03 },
    { 5.6000000000000003e-10, 4.3988677570499997e+00, 5.3445592204626599e+04 },
    { 5.6000000000000003e-10, 5.7910583665399997e+00, 9.9365397743466001e+03 },
    { 5.6000000000000003e-10, 5.6123502723999996e-01, 1.0490031318075400e+04 },
    { 5.4999999999999996e-10, 5.4141698621999996e+00, 4.8947663870676603e+04 },
    { 5.4000000000000000e-10, 2.9258519434500001e+00, 9.9223126803449995e+03 },
    { 5.4000000000000000e-10, 3.4264414503300000e+00, 1.0504258412077001e+04 },
    { 5.4000000000000000e-10, 3.7891023259900001e+00, 4.7623852786089599e+04 },
    { 5.1999999999999996e-10, 5.2182858466000004e-01, 4.0426543993968000e+04 },
    { 5.1999999999999996e-10, 1.6325383075500000e+00, 8.7273054251457997e+03 },
    { 5.1999999999999996e-10, 3.6388297295199998e+00, 3.8631285550246997e+04 },
    { 5.1999999999999996e-10, 4.2186741523999999e-01, 1.4844727083000000e+00 },
    { 5.1000000000000000e-10, 7.1928664610000004e-01, 9.5662747128962001e+03 },
    { 5.1000000000000000e-10, 3.4116420710200002e+00, 6.4460698681961403e+04 },
    { 5.1000000000000000e-10, 4.3223305521000004e+00, 1.0453831825548140e+05 },
    { 5.1000000000000000e-10, 4.7051448285899999e+00, 1.6250529749973000e+04 },
    { 5.0000000000000003e-10, 6.2709293562699999e+00, 8.4180271024899994e+03 },
    { 5.0000000000000003e-10, 6.0522790190600002e+00, 5.9160949416887597e+04 },
    { 5.0000000000000003e-10, 1.8378940827100001e+00, 5.4247413135750401e+04 },
    { 4.8999999999999996e-10, 8.5483007769999994e-02, 9.7943611473103992e+03 },
    { 4.8999999999999996e-10, 6.2668103860100004e+00, 1.0632209945111599e+04 },
    { 4.8999999999999996e-10, 3.9141584771999999e-01, 4.9166329245880002e+02 },
    { 4.8000000000000000e-10, 5.9397299740299996e+00, 3.8727264777464799e+04 },
    { 4.8000000000000000e-10, 1.7811687050600000e+00, 9.2463146687753997e+03 },
    { 4.8000000000000000e-10, 4.8345704135000001e-01, 4.3371173787679999e+02 },
    { 4.7000000000000003e-10, 3.2477237297800001e+00, 3.8741491871466402e+04 },
    { 4.7000000000000003e-10, 6.2215617852899996e+00, 2.0142711773556799e+04 },
    { 4.7000000000000003e-10, 1.7782919736000000e-01, 1.8792514776260001e+02 },
    { 4.7000000000000003e-10, 2.5304473913900001e+00, 8.8041956859279002e+04 },
    { 4.6000000000000001e-10, 5.7789574760800004e+00, 1.9497138446060001e+02 },
    { 4.6000000000000001e-10, 1.4273479096799999e+00, 1.1567427666460119e+05 },
    { 4.5000000000000000e-10, 3.9500645429499999e+00, 1.0263282168115200e+04 },
    { 4.5000000000000000e-10, 2.4022288508299998e+00, 1.0163288924306800e+04 },
    { 4.5000000000000000e-10, 3.8112862575699999e+00, 2.3493430354227999e+03 },
    { 4.5000000000000000e-10, 3.1291828266500001e+00, 2.3563892721207999e+03 },
    { 4.5000000000000000e-10, 6.1517422927699998e+00, 6.6038042224409190e+04 },
    { 4.3999999999999998e-10, 4.7724556879000002e-01, 3.0632743091632190e+04 },
    { 4.3999999999999998e-10, 2.8978336618400000e+00, 7.9379716535715997e+04 },
    { 4.3999999999999998e-10, 5.8833036496400002e+00, 1.0168144326574400e+04 },
    { 4.3999999999999998e-10, 4.6898974414000000e-01, 1.0258426765847600e+04 },
    { 4.3999999999999998e-10, 5.6832443467000004e-01, 5.0317203439530800e+04 },
    { 4.3999999999999998e-10, 2.7356908100699999e+00, 9.6978216751180007e+03 },
    { 4.3999999999999998e-10, 3.6166025837100002e+00, 1.0728749417304000e+04 },
    { 4.3999999999999998e-10, 1.2098953194199999e+00, 3.6301188687785187e+04 },
    { 4.3999999999999998e-10, 1.3235235561900001e+00, 9.0394823013050787e+04 },
    { 4.3000000000000001e-10, 4.5711246419100000e+00, 1.1180256423646601e+04 },
    { 4.3000000000000001e-10, 4.0869970248999998e-01, 2.2030212954669601e+04 },
    { 4.3000000000000001e-10, 3.3525900774699999e+00, 1.0362848743345599e+04 },
    { 4.3000000000000001e-10, 3.2461436757200000e+00, 5.8177622831970002e+04 },
    { 4.2000000000000000e-10, 3.5122447230799998e+00, 7.7802372993268189e+04 },
    { 4.0999999999999998e-10, 4.8409709979700004e+00, 1.2850011018847999e+04 },
    { 4.0999999999999998e-10, 1.2352258648100001e+00, 2.5132303399965600e+04 },
    { 4.0999999999999998e-10, 1.1612398134799999e+00, 2.4341528317456599e+04 },
    { 4.0999999999999998e-10, 9.9171071955000001e-01, 4.7358590975392799e+04 },
    { 4.0000000000000001e-10, 5.3323134572099997e+00, 2.8513979231253790e+04 },
    { 4.0000000000000001e-10, 2.8435226839199998e+00, 8.0181537466839785e+04 },
    { 3.9000000000000000e-10, 1.8734643547400001e+00, 2.8631299099473988e+04 },
    { 3.9000000000000000e-10, 5.7323707897500000e+00, 1.2655674786264001e+03 },
    { 3.9000000000000000e-10, 5.1868352418099999e+00, 5.4981384956089598e+04 },
    { 3.9000000000000000e-10, 1.6299023969499999e+00, 3.7674996394276401e+04 },
    { 3.9000000000000000e-10, 1.1492745381700000e+00, 3.1639186965660002e+02 },
    { 3.9000000000000000e-10, 5.9594223400499997e+00, 4.7888281940487403e+04 },
    { 3.9000000000000000e-10, 5.5164068642400004e+00, 1.4128242771245599e+04 },
    { 3.9000000000000000e-10, 5.8188104435100003e+00, 7.3898461616848392e+04 },
    { 3.9000000000000000e-10, 2.9036116961200000e+00, 5.7914125575325997e+03 },
    { 3.9000000000000000e-10, 3.6185584110900000e+00, 3.0563590567357402e+04 },
    { 3.7999999999999998e-10, 2.5650884343499998e+00, 4.7917263570466799e+04 },
    { 3.7999999999999998e-10, 1.1884565783500001e+00, 7.4821134479757180e+04 },
    { 3.7999999999999998e-10, 1.5769957637700001e+00, 4.7964337285759000e+04 },
    { 3.7000000000000001e-10, 2.9161639379500000e+00, 2.9160990064568599e+04 },
    { 3.7000000000000001e-10, 2.8421838022100001e+00, 6.2567775301915999e+03 },
    { 3.7000000000000001e-10, 1.6813923985899999e+00, 3.5191810135691798e+04 },
    { 3.7000000000000001e-10, 5.8040858661100003e+00, 5.8946516884393990e+04 },
    { 3.7000000000000001e-10, 4.7498394355900002e+00, 8.2534403620611585e+04 },
    { 3.7000000000000001e-10, 1.0681034196500001e+00, 1.3859635923565400e+04 },
    { 3.7000000000000001e-10, 2.0694230854200000e+00, 3.7703978024255797e+04 },
    { 3.6000000000000000e-10, 1.3994865178500000e+00, 8.6739696763989996e+03 },
    { 3.6000000000000000e-10, 6.6587329374000004e-01, 6.9941953600828987e+04 },
    { 3.6000000000000000e-10, 3.4012564127200000e+00, 1.0090903762133139e+05 },
    { 3.6000000000000000e-10, 5.9410652185900004e+00, 2.0430503245685089e+04 },
    { 3.4999999999999998e-10, 1.1823724355100000e+00, 1.2546481939083000e+04 },
    { 3.4999999999999998e-10, 7.6656495981999995e-01, 4.8733231338182988e+04 },
    { 3.4999999999999998e-10, 3.2213529993900001e+00, 1.0011032151036899e+04 },
    { 3.4999999999999998e-10, 3.1309403943900000e+00, 1.0415538941385101e+04 },
    { 3.4999999999999998e-10, 6.2079391843499998e+00, 5.1092726050854799e+04 },
    { 3.4000000000000001e-10, 5.9640761141100000e+00, 4.7938038965959197e+04 },
    { 3.4000000000000001e-10, 4.5764932299399996e+00, 3.1214201436967800e+04 },
    { 3.4000000000000001e-10, 2.2515043919000000e-01, 7.4617379465193999e+03 },
    { 3.4000000000000001e-10, 2.6580628036300000e+00, 1.5393158698120001e+03 },
    { 3.4000000000000001e-10, 2.1076157379899998e+00, 4.0213244898530000e+04 },
    { 3.4000000000000001e-10, 1.3350118013600001e+00, 4.2430485727291802e+04 },
    { 3.3000000000000000e-10, 3.8092249168799999e+00, 4.9477354835771192e+04 },
    { 3.3000000000000000e-10, 1.2882984222600000e+00, 3.4363365597556003e+04 },
    { 3.3000000000000000e-10, 2.7226006900600002e+00, 5.5798458358398399e+04 },
    { 3.3000000000000000e-10, 3.4069223377000002e+00, 5.4394563387335191e+04 },
    { 3.3000000000000000e-10, 2.1781009811100001e+00, 1.0117306318993200e+04 },
    { 3.3000000000000000e-10, 4.1741924126700001e+00, 1.0309264773428800e+04 },
    { 3.3000000000000000e-10, 1.3897101146899999e+00, 5.5022935747074400e+04 },
    { 3.3000000000000000e-10, 4.3103444076499997e+00, 8.3309926231935591e+04 },
    { 3.1999999999999998e-10, 9.8619416723999997e-01, 3.4570066334852600e+04 },
    { 3.1999999999999998e-10, 1.8143898635800000e+00, 2.9786660256881001e+04 },
    { 3.1999999999999998e-10, 2.1935930836700002e+00, 2.1977616315070001e+04 },
    { 3.1999999999999998e-10, 4.9349524806199998e+00, 8.1732582689487797e+04 },
    { 3.1000000000000002e-10, 1.8770754939700001e+00, 3.0103052126537590e+04 },
    { 3.1000000000000002e-10, 3.8957069553500001e+00, 3.9786646707653999e+04 },
    { 3.1000000000000002e-10, 1.8419634498699999e+00, 2.5960422650889199e+04 },
    { 3.0000000000000000e-10, 9.1907247414000004e-01, 1.1045700263900001e+01 },
    { 3.0000000000000000e-10, 2.0510090822899998e+00, 2.0432094016729399e+04 },
    { 3.0000000000000000e-10, 4.2467961933799998e+00, 6.2883355139513602e+04 },
    { 2.8999999999999998e-10, 4.9528073306799998e+00, 1.1752601416023001e+04 },
    { 2.8999999999999998e-10, 3.9460838522300001e+00, 9.8009144493366002e+03 },
    { 2.8999999999999998e-10, 2.4062095415500000e+00, 1.0625656643085400e+04 },
    { 2.8999999999999998e-10, 5.9829715721200003e+00, 9.8897801295535992e+03 },
    { 2.8999999999999998e-10, 1.6711557706400000e+00, 8.5034420025968182e+04 },
    { 2.8999999999999998e-10, 2.0497537275000000e-01, 2.2434479570190001e+02 },
    { 2.8999999999999998e-10, 5.8482379414999999e+00, 1.1001294461544899e+05 },
    { 2.8000000000000002e-10, 4.9857459697500000e+00, 1.0098846257525791e+04 },
    { 2.8000000000000002e-10, 1.3665474240300000e+00, 1.0327724834896209e+04 },
    { 2.8000000000000002e-10, 7.2550860398000006e-01, 1.1791945390395999e+03 },
    { 2.8000000000000002e-10, 2.0886876069200002e+00, 7.3872163297048595e+04 },
    { 2.8000000000000002e-10, 5.1315292151099996e+00, 1.5694540465078600e+04 },
    { 2.8000000000000002e-10, 3.4962537675399998e+00, 7.8867177122390003e+03 },
};

static const VSOP87Term VSOP87A_VENUS_Y1[325] = {
    { 3.9231429999999999e-04, 0.0000000000000000e+00, 0.0000000000000000e+00 },
    { 1.7282326000000000e-04, 5.6382473590000002e+00, 2.0426571092422000e+04 },
    { 5.9680750000000003e-05, 3.6085494408600001e+00, 1.0213285546211000e+04 },
    { 1.7552900000000001e-06, 2.3554665358999999e-01, 3.0639856638632999e+04 },
    { 6.3046000000000000e-07, 5.2083190822000003e-01, 1.0186987226411200e+04 },
    { 6.2776999999999996e-07, 5.8313103699399997e+00, 1.0239583866010800e+04 },
    { 6.0345999999999998e-07, 3.3575256380799998e+00, 8.6359420037632008e+03 },
    { 4.9332999999999998e-07, 3.0015917740800000e+00, 1.1790629088658799e+04 },
    { 3.6922999999999998e-07, 5.0255601130000005e-01, 7.7552261132399997e+02 },
    { 3.0123000000000000e-07, 3.7564310223300001e+00, 1.0021837280099400e+04 },
    { 2.9363000000000002e-07, 2.5948116671800001e+00, 1.0404733812322600e+04 },
    { 1.9871000000000001e-07, 5.8559190331900002e+00, 9.4377629348869996e+03 },
    { 1.7907000000000000e-07, 5.0709742814000003e-01, 1.0988808157535001e+04 },
    { 1.4387000000000000e-07, 2.1485293765900000e+00, 9.6835945811164001e+03 },
    { 1.4058999999999999e-07, 1.9299930165900001e+00, 4.7057323075435997e+03 },
    { 1.3526999999999999e-07, 4.2018554143999998e+00, 1.0742976511305600e+04 },
    { 1.3498000000000001e-07, 2.9008441261800000e+00, 1.9144826611159999e+02 },
    { 1.3447000000000001e-07, 5.8603173910599997e+00, 1.9651048481098002e+04 },
    { 7.5870000000000004e-08, 4.0080723121200004e+00, 1.0593819301891999e+03 },
    { 7.4540000000000004e-08, 4.4230126350800001e+00, 1.5720838784878400e+04 },
    { 6.8770000000000000e-08, 3.2654884771599999e+00, 9.1039069941176003e+03 },
    { 6.0399999999999998e-08, 3.0932881727399999e+00, 1.1322664098304400e+04 },
    { 5.9660000000000006e-08, 5.3259218562999999e-01, 1.0206171999210201e+04 },
    { 5.9660000000000006e-08, 5.8197012081499997e+00, 1.0220399093211799e+04 },
    { 5.1760000000000002e-08, 4.0641017912499997e+00, 3.1546870848956000e+03 },
    { 4.6490000000000001e-08, 1.2949112290300000e+00, 9.8303890139878004e+03 },
    { 4.5639999999999997e-08, 3.0189192771700002e+00, 2.0618019358533598e+04 },
    { 4.4050000000000001e-08, 5.0508138031899996e+00, 1.0596182078434200e+04 },
    { 4.2400000000000002e-08, 4.4931679543699996e+00, 9.9999864507729999e+03 },
    { 4.1570000000000001e-08, 1.8690319111800000e+00, 1.0426584641649000e+04 },
    { 4.0490000000000001e-08, 5.1003193351700000e+00, 7.1135470007999997e+00 },
    { 3.5899999999999997e-08, 8.9608249763000003e-01, 7.8604193924392002e+03 },
    { 2.7260000000000001e-08, 2.3398141883500001e+00, 1.9367189162232800e+04 },
    { 2.7070000000000000e-08, 2.4920171339000000e-01, 8.0182093112380005e+02 },
    { 2.6960000000000001e-08, 5.3694111165000002e+00, 7.0585984613153996e+03 },
    { 2.6250000000000000e-08, 9.5628165682999999e-01, 8.6242126509272002e+03 },
    { 2.5460000000000001e-08, 1.7028041388099999e+00, 1.1015106477334801e+04 },
    { 2.5390000000000001e-08, 4.3987395920300001e+00, 9.4114646150871995e+03 },
    { 1.9919999999999999e-08, 4.0961069867699997e+00, 2.6298319799800002e+01 },
    { 1.9329999999999998e-08, 3.2823056919600000e+00, 3.9302096962196001e+03 },
    { 1.9119999999999998e-08, 4.8676262793399996e+00, 1.1093785520934000e+03 },
    { 1.8329999999999999e-08, 9.7295205972999999e-01, 1.3367972631106601e+04 },
    { 1.7579999999999999e-08, 6.9532491253999995e-01, 4.5519534970588002e+03 },
    { 1.6940000000000001e-08, 7.6426907625999996e-01, 1.5773435424478000e+03 },
    { 1.5860000000000001e-08, 1.1222629175600001e+00, 4.0853142184844000e+04 },
    { 1.5150000000000000e-08, 5.3556898267599999e+00, 1.2566151699982800e+04 },
    { 1.4839999999999999e-08, 5.5499990898800000e+00, 7.0848967811151997e+03 },
    { 1.4690000000000000e-08, 4.1146057874700004e+00, 3.1283887650958000e+03 },
    { 1.4000000000000000e-08, 4.6764236294600003e+00, 1.1272667476400200e+04 },
    { 1.3520000000000000e-08, 1.2931441964899999e+00, 2.0419457545421192e+04 },
    { 1.3070000000000001e-08, 1.8250047573199999e+00, 9.1539036160218002e+03 },
    { 1.2840000000000000e-08, 5.4540848765099996e+00, 3.8289653222319998e+02 },
    { 1.2820000000000000e-08, 3.0923048437300000e+00, 1.0218808470518399e+04 },
    { 1.2820000000000000e-08, 3.2599885500500001e+00, 1.0207762621903599e+04 },
    { 1.2410000000000000e-08, 4.1414382268300001e+00, 2.3581258177317599e+04 },
    { 1.1920000000000000e-08, 3.3857704927499999e+00, 1.6496361396202399e+04 },
    { 1.1910000000000000e-08, 3.6951528209500002e+00, 2.3528661537717999e+03 },
    { 1.1129999999999999e-08, 4.9952075510600000e+00, 1.5890728952838001e+03 },
    { 1.0840000000000000e-08, 2.7844475199100001e+00, 1.0198033075026000e+04 },
    { 1.0840000000000000e-08, 3.5678458738700001e+00, 1.0228538017396000e+04 },
    { 9.1700000000000004e-09, 5.6730003448500002e+00, 1.5874617595363199e+04 },
    { 8.2000000000000006e-09, 3.6295378776999998e-01, 2.1228392023545799e+04 },
    { 7.9500000000000001e-09, 3.5416000521000002e+00, 4.2659819087599999e+02 },
    { 7.6000000000000002e-09, 5.3663818911399996e+00, 2.8286990484861199e+04 },
    { 7.4700000000000001e-09, 5.8426038651000001e-01, 3.1491641605882000e+03 },
    { 6.7400000000000003e-09, 5.6550938813999996e-01, 8.0945216858326003e+03 },
    { 6.5700000000000003e-09, 2.2098560242100000e+00, 1.7298182327326202e+04 },
    { 6.5000000000000003e-09, 3.4096101675000001e-01, 5.3680451209540001e+02 },
    { 6.3700000000000001e-09, 4.9664714048400000e+00, 1.8849227549974199e+04 },
    { 6.2499999999999997e-09, 1.6145209392600000e+00, 2.0400272772622200e+04 },
    { 6.1700000000000000e-09, 4.9360319629999996e+00, 2.1535949644515400e+04 },
    { 6.1300000000000001e-09, 5.9191874324000004e-01, 5.6613320491521999e+03 },
    { 6.1300000000000001e-09, 8.1482028073000001e-01, 5.2257741809380002e+02 },
    { 5.1199999999999997e-09, 4.2477125252199999e+00, 5.2969096509459996e+02 },
    { 5.1000000000000002e-09, 5.3098193341500002e+00, 2.0452869412221800e+04 },
    { 4.7200000000000002e-09, 5.3502748127500004e+00, 9.1467900690210008e+03 },
    { 4.5699999999999997e-09, 1.0144799808700000e+00, 2.9050783743349199e+04 },
    { 4.3999999999999997e-09, 5.4493002637099996e+00, 1.0735862964304801e+04 },
    { 4.3899999999999999e-09, 8.9714640250000000e-01, 9.6907081281171995e+03 },
    { 4.3200000000000000e-09, 3.6013555045999999e+00, 3.2217200181080800e+04 },
    { 4.2500000000000000e-09, 5.5122426343499997e+00, 2.0809467624645200e+04 },
    { 4.2299999999999997e-09, 1.3963446708700000e+00, 1.8837498197138189e+04 },
    { 4.2199999999999999e-09, 4.9439443189199999e+00, 9.1610171630225996e+03 },
    { 4.1800000000000000e-09, 3.3519063741599999e+00, 6.6812248533995999e+03 },
    { 4.1400000000000002e-09, 3.8959827158000002e+00, 2.4356780788641601e+04 },
    { 4.1199999999999998e-09, 4.3548409111200002e+00, 2.2003914634869801e+04 },
    { 3.9400000000000000e-09, 2.9095484140600001e+00, 5.5075532386674004e+03 },
    { 3.8899999999999996e-09, 4.7644657707200002e+00, 6.2830758499914000e+03 },
    { 3.7200000000000000e-09, 5.2150404386500000e+00, 1.0251313218846801e+04 },
    { 3.7200000000000000e-09, 1.1372529551299999e+00, 1.0175257873575199e+04 },
    { 3.6399999999999998e-09, 1.1450038584000000e+00, 1.0639883737087001e+04 },
    { 3.6100000000000001e-09, 2.3186987347399999e+00, 1.4945316173554400e+04 },
    { 3.6000000000000000e-09, 5.1923821213699997e+00, 9.7866873553349997e+03 },
    { 3.5100000000000001e-09, 9.3761476307000002e-01, 6.4897765872880000e+03 },
    { 3.1899999999999999e-09, 5.7703831155899996e+00, 1.7277406931833801e+04 },
    { 3.0100000000000002e-09, 5.7805075112199997e+00, 1.4765239043269799e+04 },
    { 2.9699999999999999e-09, 5.7807070246199999e+00, 1.0232955307110789e+04 },
    { 2.9699999999999999e-09, 5.7158636916000005e-01, 1.0193615785311211e+04 },
    { 2.9100000000000001e-09, 3.3368351792199999e+00, 1.8073704938650200e+04 },
    { 2.8999999999999999e-09, 3.0129653999600001e+00, 1.3745346239022399e+04 },
    { 2.8900000000000002e-09, 4.2153285337800002e+00, 2.0235122826310399e+04 },
    { 2.8699999999999998e-09, 7.9492340372000003e-01, 3.4425749449653999e+03 },
    { 2.8499999999999999e-09, 3.9559905536900000e+00, 2.0956262057516600e+04 },
    { 2.8400000000000001e-09, 1.5245189391500000e+00, 4.7320306273433998e+03 },
    { 2.8100000000000000e-09, 3.5303280579999998e+00, 1.4919017853754600e+04 },
    { 2.7000000000000002e-09, 2.8343870673099998e+00, 1.9999972901545989e+04 },
    { 2.6900000000000000e-09, 6.1292478486100004e+00, 9.6764810341156008e+03 },
    { 2.6200000000000001e-09, 1.6534472307700001e+00, 2.1202093703745999e+04 },
    { 2.6200000000000001e-09, 2.3413334904999999e-01, 1.0750090058306399e+04 },
    { 2.6099999999999999e-09, 4.0570970708700003e+00, 2.2779437246193800e+04 },
    { 2.5599999999999998e-09, 2.8442302002900002e+00, 1.0234060941703399e+04 },
    { 2.5599999999999998e-09, 3.5080631934900000e+00, 1.0192510150718599e+04 },
    { 2.3800000000000001e-09, 2.7226243886099999e+00, 9.5733882598969994e+03 },
    { 2.3499999999999999e-09, 1.2446508764899999e+00, 1.0664954771900000e+03 },
    { 2.3400000000000002e-09, 5.5914394641799996e+00, 2.7511467873537200e+04 },
    { 2.3400000000000002e-09, 1.7331037994700000e+00, 1.5510452226479999e+03 },
    { 2.3300000000000000e-09, 3.8529450385000001e+00, 1.8875525869773999e+04 },
    { 2.3199999999999998e-09, 5.8129283669299996e+00, 3.7235089589230001e+03 },
    { 2.3199999999999998e-09, 6.0116760563699998e+00, 1.9889766580326592e+04 },
    { 2.2499999999999999e-09, 2.5361150770099998e+00, 1.4143495242430599e+04 },
    { 2.1799999999999999e-09, 5.5307428364400000e+00, 1.9903993674328191e+04 },
    { 2.1100000000000000e-09, 2.9624849976299998e+00, 8.6622403235630009e+03 },
    { 2.0799999999999998e-09, 4.0476315584100000e+00, 1.0007099997773799e+04 },
    { 2.0799999999999998e-09, 2.3046618353700001e+00, 1.0419471094648199e+04 },
    { 2.0700000000000001e-09, 1.0510886336300000e+00, 2.9864334027309000e+04 },
    { 2.0400000000000000e-09, 5.2863353834300000e+00, 1.3936794505133999e+04 },
    { 2.0299999999999998e-09, 2.8854815023899998e+00, 6.8726731195111997e+03 },
    { 1.9900000000000000e-09, 5.0435545304299998e+00, 9.6389407478762005e+03 },
    { 1.8800000000000001e-09, 1.3241853749900001e+00, 1.0787630344545800e+04 },
    { 1.8400000000000001e-09, 9.0286402700000004e-02, 9.9674538999815995e+03 },
    { 1.7900000000000000e-09, 4.8453269462000002e+00, 3.0110165673538399e+04 },
    { 1.7800000000000001e-09, 6.2618513062299996e+00, 1.0459117192440401e+04 },
    { 1.7700000000000001e-09, 7.8176584306999997e-01, 6.7707106012455997e+03 },
    { 1.7599999999999999e-09, 3.0396824227799999e+00, 5.4812549188676003e+03 },
    { 1.6999999999999999e-09, 1.1735945305600000e+00, 2.1329909543799999e+02 },
    { 1.6300000000000000e-09, 3.3889451428899999e+00, 1.0225014899047001e+04 },
    { 1.6300000000000000e-09, 2.9633482508900002e+00, 1.0201556193375000e+04 },
    { 1.6300000000000000e-09, 4.9088162677999998e+00, 8.6170991039264009e+03 },
    { 1.6000000000000001e-09, 5.6714468620999998e+00, 1.6983996147456601e+04 },
    { 1.5700000000000000e-09, 2.1085567416200002e+00, 9.7938009023358009e+03 },
    { 1.5700000000000000e-09, 3.5848886674800000e+00, 1.3553897972910800e+04 },
    { 1.5600000000000000e-09, 4.4961963418200002e+00, 1.0316378320429600e+04 },
    { 1.5600000000000000e-09, 1.8560970519600000e+00, 1.0110192771992401e+04 },
    { 1.5400000000000001e-09, 5.6929716405899997e+00, 2.1187638603783998e+03 },
    { 1.5100000000000000e-09, 3.6202402454799998e+00, 6.3989728631399998e+02 },
    { 1.4500000000000000e-09, 6.1953899848100003e+00, 7.0641213856227996e+03 },
    { 1.4300000000000000e-09, 3.3200442135300001e+00, 7.9945284420241996e+03 },
    { 1.3999999999999999e-09, 5.3163425714999999e+00, 8.6313261979279996e+03 },
    { 1.2900000000000001e-09, 1.9360167860999999e-01, 7.5648307207380003e+03 },
    { 1.2799999999999999e-09, 3.8990312881200002e+00, 1.0632770190086199e+04 },
    { 1.2199999999999999e-09, 6.6618818020000004e-01, 2.8521092778254599e+04 },
    { 1.1700000000000001e-09, 3.0967504710200000e+00, 1.2432042650397800e+04 },
    { 1.1400000000000000e-09, 4.3361730919700001e+00, 1.2592450019782600e+04 },
    { 1.1200000000000001e-09, 5.9664455787500001e+00, 2.5934124331089399e+04 },
    { 1.0999999999999999e-09, 5.0150241662899999e+00, 1.1802358441494800e+04 },
    { 1.0800000000000000e-09, 3.5632037962299998e+00, 2.9580474708443799e+04 },
    { 1.0600000000000001e-09, 3.3272191215700002e+00, 1.8307807232043600e+04 },
    { 1.0399999999999999e-09, 4.6045054815500004e+00, 3.6147409877300401e+04 },
    { 1.0000000000000001e-09, 1.5696643035500000e+00, 1.0522683831884001e+03 },
    { 9.7999999999999992e-10, 3.8640764947799999e+00, 2.0775395492400001e+01 },
    { 9.6999999999999996e-10, 4.7082353478799996e+00, 2.2805735565993600e+04 },
    { 9.5999999999999999e-10, 3.7541137618199998e+00, 2.2041264243879999e+02 },
    { 9.5000000000000003e-10, 2.0970316852500002e+00, 1.0433698188649800e+04 },
    { 9.5000000000000003e-10, 4.2552617085300000e+00, 9.9928729037722005e+03 },
    { 9.4000000000000006e-10, 6.1921766186899996e+00, 2.0433684639422801e+04 },
    { 9.2999999999999999e-10, 1.7201632145700001e+00, 5.7434479833479998e+02 },
    { 9.2999999999999999e-10, 4.3229262402800002e+00, 2.9577158944766002e+03 },
    { 9.2000000000000003e-10, 3.4983025879500000e+00, 9.3600891644589992e+03 },
    { 8.6999999999999999e-10, 2.7206548507899999e+00, 2.3791644735716000e+03 },
    { 8.3999999999999999e-10, 4.2926622785199999e+00, 2.0618554843720000e+02 },
    { 8.0999999999999999e-10, 3.1750611636000001e+00, 1.1764330768858999e+04 },
    { 8.0999999999999999e-10, 5.0972371469600004e+00, 1.9360075615231988e+04 },
    { 8.0000000000000003e-10, 4.5200369879600002e+00, 1.9896880127327400e+04 },
    { 7.7000000000000003e-10, 2.9344099208899999e+00, 1.0177257679533601e+04 },
    { 7.7000000000000003e-10, 3.4178834728899998e+00, 1.0249313412888399e+04 },
    { 7.7000000000000003e-10, 5.3717050143399998e+00, 2.9573361161443001e+04 },
    { 7.7000000000000003e-10, 5.4224978146699998e+00, 2.0213271996984000e+04 },
    { 7.7000000000000003e-10, 1.7432264294699999e+00, 7.8341210726394002e+03 },
    { 7.2999999999999996e-10, 4.9304174385900001e+00, 2.9587588255444600e+04 },
    { 7.1000000000000003e-10, 5.8850465052000001e+00, 2.4150080051345001e+04 },
    { 7.1000000000000003e-10, 4.9784588439600004e+00, 2.9043670196348390e+04 },
    { 6.8000000000000003e-10, 2.3896649061600002e+00, 1.0383958416830201e+04 },
    { 6.8000000000000003e-10, 3.9626284876200000e+00, 1.0042612675591799e+04 },
    { 6.6999999999999996e-10, 8.7827389480999996e-01, 3.1441677569756801e+04 },
    { 6.6999999999999996e-10, 5.8715598735099999e+00, 4.0077619573520002e+04 },
    { 6.6999999999999996e-10, 3.1452801942900002e+00, 1.7778116266948990e+04 },
    { 6.2000000000000003e-10, 2.0387922560000001e-02, 2.6735945262213201e+04 },
    { 6.0999999999999996e-10, 5.3733456938800002e+00, 2.9057897290349989e+04 },
    { 6.0000000000000000e-10, 4.6434441719799997e+00, 3.0831304904744600e+04 },
    { 5.9000000000000003e-10, 1.6720221479299999e+00, 9.7795738083342003e+03 },
    { 5.9000000000000003e-10, 4.8124105901299998e+00, 2.2187571041868000e+03 },
    { 5.7999999999999996e-10, 6.1120066862300000e+00, 1.1020632121940000e+02 },
    { 5.4000000000000000e-10, 3.3303002124200001e+00, 1.9317192540328600e+04 },
    { 5.1000000000000000e-10, 2.5420667941500001e+00, 2.9088811415985001e+04 },
    { 5.1000000000000000e-10, 6.1896995038300000e+00, 2.7490692478044799e+04 },
    { 5.1000000000000000e-10, 9.3431467650000000e-01, 4.4007829269739603e+04 },
    { 5.1000000000000000e-10, 6.0071609519800004e+00, 1.3341674311306801e+04 },
    { 5.1000000000000000e-10, 5.8406534817300004e+00, 1.3655860491176400e+04 },
    { 5.0000000000000003e-10, 1.8460290148000000e-01, 1.0103079224991599e+04 },
    { 5.0000000000000003e-10, 6.1676904923000002e+00, 1.0323491867430401e+04 },
    { 5.0000000000000003e-10, 4.6396791417200003e+00, 3.9793760254654801e+04 },
    { 4.8999999999999996e-10, 5.2869688832600001e+00, 2.6709646942413401e+04 },
    { 4.8999999999999996e-10, 1.8640060195599999e+00, 2.5158601719765400e+04 },
    { 4.7000000000000003e-10, 2.0534362001400002e+00, 2.0639870187860000e+04 },
    { 4.7000000000000003e-10, 4.1891049454599996e+00, 5.7837138332300601e+04 },
    { 4.6000000000000001e-10, 2.7463939977199998e+00, 1.9786673806107989e+04 },
    { 4.3999999999999998e-10, 2.8242708964799998e+00, 2.9999959352319001e+04 },
    { 4.3999999999999998e-10, 3.8398851483000001e+00, 3.3019021112204602e+04 },
    { 4.3999999999999998e-10, 5.3857968043800000e+00, 1.6522659716002199e+04 },
    { 4.3000000000000001e-10, 3.5679131777099999e+00, 2.9786660256881001e+04 },
    { 4.2000000000000000e-10, 3.2067555376099999e+00, 3.8204687359370997e+04 },
    { 4.2000000000000000e-10, 5.0824467968600002e+00, 5.5798458358398399e+04 },
    { 4.0999999999999998e-10, 2.0785966621999998e+00, 3.9149572250346000e+03 },
    { 4.0999999999999998e-10, 4.2483169510000002e-02, 2.7991401813159999e+04 },
    { 4.0000000000000001e-10, 2.3158534028300002e+00, 4.7938038965959197e+04 },
    { 3.9000000000000000e-10, 4.2076545660700004e+00, 2.0447346487914401e+04 },
    { 3.9000000000000000e-10, 1.6672265769800001e+00, 1.8830384650137388e+04 },
    { 3.9000000000000000e-10, 1.4409550931199999e+00, 3.0666154958432799e+04 },
    { 3.9000000000000000e-10, 1.1763040392699999e+00, 1.8844611744138991e+04 },
    { 3.7000000000000001e-10, 4.9025759844800003e+00, 1.9374302709233601e+04 },
    { 3.7000000000000001e-10, 5.1302167867700001e+00, 4.9515382508406998e+04 },
    { 3.6000000000000000e-10, 4.3822146365599997e+00, 1.0077660221201000e+04 },
    { 3.6000000000000000e-10, 1.9700787572200000e+00, 1.0348910871221000e+04 },
    { 3.4999999999999998e-10, 5.1940805268499997e+00, 3.6949230808424203e+04 },
    { 3.4999999999999998e-10, 3.7799032716599998e+00, 9.9858094134219991e+03 },
    { 3.4999999999999998e-10, 2.5723901221199998e+00, 1.0440761678999999e+04 },
    { 3.4000000000000001e-10, 5.5952854815999997e-01, 5.4247413135750401e+04 },
    { 3.3000000000000000e-10, 6.1557784008400001e+00, 4.8417972905581992e+04 },
    { 3.3000000000000000e-10, 2.5151190015500000e+00, 7.2555696517344004e+03 },
    { 3.3000000000000000e-10, 2.2007720628999999e-01, 5.9728668054618000e+04 },
    { 3.1999999999999998e-10, 1.5436387445899999e+00, 1.7468855197945399e+04 },
    { 3.1999999999999998e-10, 3.9286111480199999e+00, 1.6894510399610601e+04 },
    { 3.1999999999999998e-10, 5.5000852757999996e-01, 1.0191434716884600e+04 },
    { 3.1999999999999998e-10, 5.8022848661999999e+00, 1.0235136375537400e+04 },
    { 3.1999999999999998e-10, 2.5950295911999999e+00, 2.0206158449983192e+04 },
    { 3.1999999999999998e-10, 8.9449566882999998e-01, 6.6813564835733196e+04 },
    { 3.1999999999999998e-10, 3.8017620365400000e+00, 9.8359119382951994e+03 },
    { 3.1999999999999998e-10, 2.5505313572400001e+00, 1.0590659154126801e+04 },
    { 3.1999999999999998e-10, 5.7790853556700004e+00, 1.1265553929399401e+04 },
    { 3.1999999999999998e-10, 3.7143829552400001e+00, 5.1868248662178798e+04 },
    { 3.1000000000000002e-10, 1.6687783031900001e+00, 2.1000915890756802e+04 },
    { 3.1000000000000002e-10, 1.9456924756800000e+00, 2.0043674560198800e+04 },
    { 3.1000000000000002e-10, 4.4583722928100000e+00, 8.0874081388318000e+03 },
    { 3.0000000000000000e-10, 3.9582329080000000e-01, 1.0853182832525001e+04 },
    { 3.0000000000000000e-10, 5.9276977564099997e+00, 1.7248425301854400e+04 },
    { 2.8999999999999998e-10, 3.6408334952399999e+00, 1.0290959316639000e+04 },
    { 2.8999999999999998e-10, 2.7114598985399998e+00, 1.0135611775783000e+04 },
    { 2.8999999999999998e-10, 1.7409960999500000e+00, 3.0206144900756190e+04 },
    { 2.8999999999999998e-10, 2.1314942658599998e+00, 2.0220385543984801e+04 },
    { 2.8000000000000002e-10, 4.1698949084299999e+00, 1.0435161256629678e+05 },
    { 2.8000000000000002e-10, 2.7992982934999999e-01, 4.0879440504643797e+04 },
    { 2.8000000000000002e-10, 5.7515134228200004e+00, 6.2883355139513602e+04 },
    { 2.7000000000000000e-10, 4.6821358602699998e+00, 6.2983283211764001e+03 },
    { 2.7000000000000000e-10, 1.2567756011100000e+00, 3.0220371994757799e+04 },
    { 2.7000000000000000e-10, 2.1190619224599998e+00, 9.5805018068978006e+03 },
    { 2.5999999999999998e-10, 3.2052390233599999e+00, 3.4363365597556003e+04 },
    { 2.5999999999999998e-10, 1.2266825583800001e+00, 3.9271182836560998e+04 },
    { 2.5999999999999998e-10, 1.7091643291400000e+00, 3.9256955742559388e+04 },
    { 2.5999999999999998e-10, 5.2554304895400001e+00, 8.1016352328333996e+03 },
    { 2.5999999999999998e-10, 1.6424475211900000e+00, 6.3658877750837601e+04 },
    { 2.5000000000000002e-10, 2.5307777489499999e+00, 3.5320606928113998e+03 },
    { 2.5000000000000002e-10, 4.3575586657200001e+00, 5.2175806283148399e+04 },
    { 2.4000000000000000e-10, 1.1457015029399999e+00, 8.7344189721465991e+03 },
    { 2.4000000000000000e-10, 2.7843206875900002e+00, 3.6173708197100197e+04 },
    { 2.4000000000000000e-10, 2.8520221215500001e+00, 3.1022753170856198e+04 },
    { 2.4000000000000000e-10, 1.8304894873199999e+00, 2.4383079108441401e+04 },
    { 2.4000000000000000e-10, 1.9343658623400000e+00, 1.4128242771245599e+04 },
    { 2.4000000000000000e-10, 2.8777190544600000e+00, 3.4596364654652403e+04 },
    { 2.4000000000000000e-10, 5.0039481925100002e+00, 4.7623852786089599e+04 },
    { 2.4000000000000000e-10, 2.3157339587500001e+00, 7.0743774531952789e+04 },
    { 2.4000000000000000e-10, 5.2793666976300004e+00, 1.0288067144778301e+04 },
    { 2.4000000000000000e-10, 1.0729266961499999e+00, 1.0138503947643700e+04 },
    { 2.3000000000000001e-10, 1.1406881711700001e+00, 1.0367064356695801e+04 },
    { 2.3000000000000001e-10, 5.2116052226100003e+00, 1.0059506735726200e+04 },
    { 2.3000000000000001e-10, 1.6596374054300000e+00, 4.4809650200863398e+04 },
    { 2.3000000000000001e-10, 2.4867378234599999e+00, 2.7461710848065399e+04 },
    { 2.3000000000000001e-10, 4.0365904570200000e+00, 6.3093741697912001e+03 },
    { 2.3000000000000001e-10, 1.1044666161200001e+00, 1.0263042571682799e+04 },
    { 2.3000000000000001e-10, 5.2478267776600003e+00, 1.0163528520739201e+04 },
    { 2.1999999999999999e-10, 1.3577076929500000e+00, 2.4978524589480799e+04 },
    { 2.1999999999999999e-10, 2.6335074166000001e+00, 4.3071899289030800e+04 },
    { 2.1999999999999999e-10, 1.8339358049999999e-01, 3.0426557543194998e+04 },
    { 2.1999999999999999e-10, 2.5065377193299998e+00, 9.5662747128962001e+03 },
    { 2.1999999999999999e-10, 8.1492467589999995e-01, 4.6386993743311199e+04 },
    { 2.1000000000000000e-10, 4.8695657464999997e-01, 1.0949162059742799e+04 },
    { 2.1000000000000000e-10, 5.8653368191300004e+00, 9.4774090326791993e+03 },
    { 2.0000000000000001e-10, 6.4178978254999997e-01, 1.1279781023400999e+04 },
    { 2.0000000000000001e-10, 1.2051601114499999e+00, 1.8300693685042788e+04 },
    { 2.0000000000000001e-10, 5.6343972456299998e+00, 6.2081534208389799e+04 },
    { 2.0000000000000001e-10, 1.1373810040700001e+00, 4.9477354835771192e+04 },
    { 1.8999999999999999e-10, 2.3803378346000001e-01, 2.0316364771202589e+04 },
    { 1.8999999999999999e-10, 5.6803842615100004e+00, 5.5022935747074400e+04 },
    { 1.8999999999999999e-10, 3.0329942229600002e+00, 6.7589087447057187e+04 },
    { 1.8000000000000000e-10, 3.0472100400400000e+00, 4.8739859897082999e+04 },
    { 1.8000000000000000e-10, 6.9437828159000003e-01, 9.0508108418032007e+03 },
    { 1.8000000000000000e-10, 4.7413315566199996e+00, 4.2430485727291802e+04 },
    { 1.7000000000000001e-10, 3.1984593567399999e+00, 2.8313288804660999e+04 },
    { 1.7000000000000001e-10, 2.8986211656399998e+00, 1.0497144865076199e+04 },
    { 1.7000000000000001e-10, 3.4536722281399999e+00, 9.9294262273458007e+03 },
    { 1.7000000000000001e-10, 1.8055486596600001e+00, 2.4341528317456599e+04 },
    { 1.7000000000000001e-10, 3.4108276428800002e+00, 3.8734378324465601e+04 },
    { 1.7000000000000001e-10, 5.4850014548199999e+00, 1.9624750161298201e+04 },
    { 1.7000000000000001e-10, 1.4239753008800000e+00, 1.8314920779044402e+04 },
    { 1.5999999999999999e-10, 3.8644029218400000e+00, 9.7453205558566005e+03 },
    { 1.5999999999999999e-10, 2.4878904719400001e+00, 1.0681250536565400e+04 },
    { 1.5999999999999999e-10, 5.6638009678600003e+00, 4.2456784047091598e+04 },
    { 1.5999999999999999e-10, 2.2331698767199999e+00, 5.0317203439530800e+04 },
    { 1.5000000000000000e-10, 3.8320752686300001e+00, 1.0199058452209400e+04 },
    { 1.5000000000000000e-10, 2.5202181251500000e+00, 1.0227512640212601e+04 },
    { 1.5000000000000000e-10, 5.6470099338999997e+00, 2.7197281693667599e+04 },
    { 1.5000000000000000e-10, 1.1332621992500000e+00, 5.3285184835241802e+04 },
    { 1.5000000000000000e-10, 7.7488131033999996e-01, 6.6011743904609393e+04 },
    { 1.5000000000000000e-10, 1.2664847270999999e+00, 3.8727264777464799e+04 },
    { 1.5000000000000000e-10, 6.2310827846099999e+00, 4.6360695423511403e+04 },
    { 1.5000000000000000e-10, 4.2955682964499999e+00, 3.8526574350871997e+04 },
    { 1.5000000000000000e-10, 4.6528232449900004e+00, 2.1485953022611189e+04 },
    { 1.4000000000000001e-10, 5.9676716322800001e+00, 4.1654963115967803e+04 },
    { 1.4000000000000001e-10, 5.0867873860600001e+00, 3.9999945803092000e+04 },
    { 1.4000000000000001e-10, 6.2397325604400002e+00, 1.1026835830170799e+04 },
    { 1.4000000000000001e-10, 1.1256083334000000e-01, 9.3997352622512008e+03 },
    { 1.4000000000000001e-10, 6.1075286871700003e+00, 3.7895426290367403e+04 },
    { 1.4000000000000001e-10, 6.1612609396500000e+00, 3.7410567239878597e+04 },
    { 1.4000000000000001e-10, 1.3713439082200001e+00, 5.0290905119730996e+04 },
    { 1.2999999999999999e-10, 4.5705202379200003e+00, 5.2670069593302600e+04 },
    { 1.2999999999999999e-10, 4.5756824616499996e+00, 3.2243498500880600e+04 },
};

static const VSOP87Term VSOP87A_VENUS_Y2[99] = {
    { 2.0071549999999999e-05, 3.1415926535900001e+00, 0.0000000000000000e+00 },
    { 7.0205199999999997e-06, 3.5272496475300001e+00, 2.0426571092422000e+04 },
    { 2.6570900000000001e-06, 4.6809183698499996e+00, 1.0213285546211000e+04 },
    { 1.1646000000000000e-07, 3.9374476181900002e+00, 3.0639856638632999e+04 },
    { 8.6389999999999995e-08, 1.1313582362000001e+00, 1.0186987226411200e+04 },
    { 8.6039999999999995e-08, 5.2229715999700002e+00, 1.0239583866010800e+04 },
    { 3.8110000000000002e-08, 5.4879816094800002e+00, 9.4377629348869996e+03 },
    { 3.3689999999999999e-08, 8.6189885485999995e-01, 1.0988808157535001e+04 },
    { 2.8069999999999999e-08, 1.6567629238499999e+00, 8.6359420037632008e+03 },
    { 2.2889999999999999e-08, 4.6981487424699999e+00, 1.1790629088658799e+04 },
    { 2.1999999999999998e-08, 2.1769971046399998e+00, 1.0021837280099400e+04 },
    { 2.1480000000000001e-08, 4.1752176198899997e+00, 1.0404733812322600e+04 },
    { 1.6960000000000001e-08, 5.0831314495399997e+00, 7.7552261132399997e+02 },
    { 7.1600000000000001e-09, 6.0667148421199997e+00, 9.8303890139878004e+03 },
    { 6.8200000000000000e-09, 2.8558057356000000e-01, 1.0596182078434200e+04 },
    { 6.4400000000000001e-09, 2.2734493456999999e-01, 4.7057323075435997e+03 },
    { 5.9399999999999998e-09, 1.2386138002700000e+00, 1.9651048481098002e+04 },
    { 5.6599999999999999e-09, 3.5292137178999999e-01, 8.0182093112380005e+02 },
    { 5.5400000000000003e-09, 4.4445269897299999e+00, 1.9144826611159999e+02 },
    { 5.2800000000000000e-09, 3.4186165059900002e+00, 9.6835945811164001e+03 },
    { 5.2700000000000002e-09, 2.9464411464700002e+00, 1.0742976511305600e+04 },
    { 5.2499999999999999e-09, 5.7222759654299997e+00, 7.0585984613153996e+03 },
    { 3.6199999999999999e-09, 4.3470699966200002e+00, 9.4114646150871995e+03 },
    { 3.5899999999999998e-09, 6.1996179972999998e-01, 1.3367972631106601e+04 },
    { 3.5800000000000000e-09, 2.0295402091499999e+00, 1.1015106477334801e+04 },
    { 3.2799999999999998e-09, 6.1164609196499997e+00, 1.5720838784878400e+04 },
    { 3.0100000000000002e-09, 4.2596241425499999e+00, 3.1283887650958000e+03 },
    { 2.9899999999999998e-09, 3.8446220379799998e+00, 2.6298319799800002e+01 },
    { 2.8600000000000000e-09, 1.6842353556700000e+00, 9.1039069941176003e+03 },
    { 2.5000000000000001e-09, 4.6682375663700002e+00, 1.1322664098304400e+04 },
    { 2.3899999999999998e-09, 5.7730754983199999e+00, 3.1546870848956000e+03 },
    { 1.9200000000000000e-09, 4.6231572323999996e+00, 2.0618019358533598e+04 },
    { 1.9099999999999998e-09, 2.0361638644000002e+00, 1.0593819301891999e+03 },
    { 1.9000000000000001e-09, 2.1249868991000001e-01, 1.0426584641649000e+04 },
    { 1.9000000000000001e-09, 6.1397947038699998e+00, 9.9999864507729999e+03 },
    { 1.7900000000000000e-09, 4.2909787895000001e-01, 2.1228392023545799e+04 },
    { 1.6200000000000000e-09, 6.8333343155000004e-01, 3.8289653222319998e+02 },
    { 1.5600000000000000e-09, 4.6300132304600004e+00, 4.0853142184844000e+04 },
    { 1.4200000000000001e-09, 5.9760613406500003e+00, 1.5773435424478000e+03 },
    { 1.4100000000000001e-09, 2.9351368398800002e+00, 7.8604193924392002e+03 },
    { 1.3399999999999999e-09, 2.0723276343900001e+00, 1.7298182327326202e+04 },
    { 1.2500000000000000e-09, 1.0711737870900000e+00, 1.8849227549974199e+04 },
    { 1.1500000000000000e-09, 4.2771077064299998e+00, 2.0452869412221800e+04 },
    { 1.0999999999999999e-09, 2.1412832521100000e+00, 8.6242126509272002e+03 },
    { 1.0900000000000000e-09, 4.5595621181899997e+00, 2.3528661537717999e+03 },
    { 7.5999999999999996e-10, 8.9056324143999999e-01, 7.0848967811151997e+03 },
    { 7.4000000000000003e-10, 1.5909877330500000e+00, 1.6496361396202399e+04 },
    { 7.1000000000000003e-10, 5.8725569082600000e+00, 2.3581258177317599e+04 },
    { 6.3999999999999996e-10, 4.2301424994700003e+00, 1.9367189162232800e+04 },
    { 6.3000000000000000e-10, 1.9465269649000000e-01, 1.1093785520934000e+03 },
    { 6.3000000000000000e-10, 2.5825392909299998e+00, 1.8073704938650200e+04 },
    { 6.0999999999999996e-10, 2.4537436175099998e+00, 9.1539036160218002e+03 },
    { 5.4999999999999996e-10, 1.8111457033700000e+00, 4.7320306273433998e+03 },
    { 5.4000000000000000e-10, 7.9386600923999995e-01, 2.0809467624645200e+04 },
    { 4.8999999999999996e-10, 4.7743080600500001e+00, 3.9302096962196001e+03 },
    { 4.6000000000000001e-10, 9.7289096934999997e-01, 2.2003914634869801e+04 },
    { 4.3999999999999998e-10, 3.5345434495000001e+00, 9.6389407478762005e+03 },
    { 4.3999999999999998e-10, 2.8177499442800000e+00, 1.0787630344545800e+04 },
    { 4.3000000000000001e-10, 2.2162535005300001e+00, 2.9050783743349199e+04 },
    { 4.3000000000000001e-10, 1.7225363488500001e+00, 8.0945216858326003e+03 },
    { 4.3000000000000001e-10, 3.4177098169500000e+00, 1.2566151699982800e+04 },
    { 3.9000000000000000e-10, 5.3727814725000000e-01, 6.2830758499914000e+03 },
    { 3.7000000000000001e-10, 3.0475177003299998e+00, 8.6622403235630009e+03 },
    { 3.3000000000000000e-10, 5.7541289845800003e+00, 1.4919017853754600e+04 },
    { 3.3000000000000000e-10, 5.7076932193200003e+00, 6.4897765872880000e+03 },
    { 3.0000000000000000e-10, 1.8039220354500001e+00, 2.4356780788641601e+04 },
    { 3.0000000000000000e-10, 2.3092566589899999e+00, 2.0400272772622200e+04 },
    { 2.8999999999999998e-10, 3.2995338262099998e+00, 2.8286990484861199e+04 },
    { 2.8000000000000002e-10, 2.2831068499199998e+00, 2.0235122826310399e+04 },
    { 2.7000000000000000e-10, 5.5286419307700001e+00, 1.0639883737087001e+04 },
    { 2.7000000000000000e-10, 8.2365146301000003e-01, 9.7866873553349997e+03 },
    { 2.7000000000000000e-10, 2.9620716953999998e-01, 1.4143495242430599e+04 },
    { 2.5999999999999998e-10, 5.3445532616099998e+00, 3.1491641605882000e+03 },
    { 2.5999999999999998e-10, 2.0377895904000001e+00, 1.8875525869773999e+04 },
    { 2.5000000000000002e-10, 4.1879570611599997e+00, 5.5075532386674004e+03 },
    { 2.4000000000000000e-10, 3.7380986984700000e+00, 1.1272667476400200e+04 },
    { 2.4000000000000000e-10, 1.0526616276000000e+00, 2.7511467873537200e+04 },
    { 2.4000000000000000e-10, 2.8968474188999999e-01, 2.1535949644515400e+04 },
    { 2.3000000000000001e-10, 6.4543351156999995e-01, 1.3936794505133999e+04 },
    { 2.1999999999999999e-10, 1.5720671001700000e+00, 1.8307807232043600e+04 },
    { 1.8000000000000000e-10, 1.7900911531299999e+00, 2.8521092778254599e+04 },
    { 1.5999999999999999e-10, 1.3494976582800000e+00, 3.2217200181080800e+04 },
    { 1.5000000000000000e-10, 2.0336307748300002e+00, 2.9580474708443799e+04 },
    { 1.4000000000000001e-10, 1.6265011526499999e+00, 2.1202093703745999e+04 },
    { 1.4000000000000001e-10, 1.0098951356300001e+00, 1.7277406931833801e+04 },
    { 1.4000000000000001e-10, 3.6938340006299999e+00, 1.1764330768858999e+04 },
    { 1.2999999999999999e-10, 3.6409212654000001e-01, 3.4570066334852600e+04 },
    { 1.2999999999999999e-10, 5.3978026985499996e+00, 2.2779437246193800e+04 },
    { 1.2000000000000000e-10, 2.3266161346000001e+00, 3.1441677569756801e+04 },
    { 1.2000000000000000e-10, 1.9348547498199999e+00, 2.5158601719765400e+04 },
    { 1.0999999999999999e-10, 3.2049192237400002e+00, 2.5934124331089399e+04 },
    { 1.0000000000000000e-10, 3.9394646885200002e+00, 1.4945316173554400e+04 },
    { 8.9999999999999999e-11, 1.7886878292699999e+00, 3.8500276031072201e+04 },
    { 8.9999999999999999e-11, 3.3900185892099999e+00, 2.9088811415985001e+04 },
    { 7.0000000000000004e-11, 3.4503716705500000e+00, 2.2805735565993600e+04 },
    { 7.0000000000000004e-11, 3.7787568636000002e-01, 4.3232306658415597e+04 },
    { 7.0000000000000004e-11, 1.2325188965300000e+00, 9.5733882598969994e+03 },
    { 7.0000000000000004e-11, 4.4432082195199998e+00, 2.9999959352319001e+04 },
    { 7.0000000000000004e-11, 1.8342057133400000e+00, 3.0110165673538399e+04 },
};

static const VSOP87Term VSOP87A_VENUS_Y3[5] = {
    { 2.4859999999999999e-07, 1.6857282573400001e+00, 2.0426571092422000e+04 },
    { 2.2966000000000001e-07, 3.8709241285000001e-01, 1.0213285546211000e+04 },
    { 3.7429999999999998e-08, 3.1415926535900001e+00, 0.0000000000000000e+00 },
    { 6.2300000000000002e-09, 1.6763017421499999e+00, 3.0639856638632999e+04 },
    { 8.9999999999999999e-11, 2.1528829043100002e+00, 4.0853142184844000e+04 },
};

static const VSOP87Term VSOP87A_VENUS_Y4[4] = {
    { 8.0100000000000003e-09, 2.0307494204299998e+00, 1.0213285546211000e+04 },
    { 6.8999999999999997e-09, 0.0000000000000000e+00, 0.0000000000000000e+00 },
    { 2.8600000000000000e-09, 5.6488433794399997e+00, 2.0426571092422000e+04 },
    { 3.0000000000000000e-10, 5.6998061290999997e+00, 3.0639856638632999e+04 },
};

static const VSOP87Term VSOP87A_VENUS_Y5[3] = {
    { 2.8999999999999998e-10, 3.5142487377900000e+00, 1.0213285546211000e+04 },
    { 2.1999999999999999e-10, 5.0558048224599998e+00, 2.0426571092422000e+04 },
    { 1.8000000000000000e-10, 0.0000000000000000e+00, 0.0000000000000000e+00 },
};

static const VSOP87Term VSOP87A_VENUS_Z0[190] = {
    { 4.2829903019999997e-02, 2.6703856475999999e-01, 1.0213285546211000e+04 },
    { 3.5588343000000000e-04, 3.1415926535900001e+00, 0.0000000000000000e+00 },
    { 1.4501879000000000e-04, 1.1469691139000000e+00, 2.0426571092422000e+04 },
    { 1.4067499999999999e-06, 8.5984113219000002e-01, 1.5773435424478000e+03 },
    { 1.3492100000000000e-06, 3.7046578785299999e+00, 2.3528661537717999e+03 },
    { 9.5749999999999994e-07, 3.6696254707299998e+00, 9.4377629348869996e+03 },
    { 7.3654000000000000e-07, 2.0277843477999999e+00, 3.0639856638632999e+04 },
    { 7.2910000000000004e-07, 1.5575027824000001e+00, 9.1539036160218002e+03 },
    { 5.9714000000000001e-07, 6.2539037164900000e+00, 1.8073704938650200e+04 },
    { 4.5307000000000002e-07, 2.2807562026800001e+00, 5.5075532386674004e+03 },
    { 4.3023000000000001e-07, 3.9562023347199999e+00, 5.2969096509459996e+02 },
    { 3.7415000000000000e-07, 4.5390035993300000e+00, 2.2003914634869801e+04 },
    { 3.2943000000000001e-07, 7.2316171903000004e-01, 1.0239583866010800e+04 },
    { 3.2446999999999999e-07, 3.8619625232999999e+00, 8.6359420037632008e+03 },
    { 2.8155999999999999e-07, 2.9341572723899998e+00, 1.0186987226411200e+04 },
    { 2.7852999999999999e-07, 6.1279051175800001e+00, 1.1790629088658799e+04 },
    { 2.3013000000000001e-07, 9.8699428892999996e-01, 7.7552261132399997e+02 },
    { 1.9425000000000001e-07, 5.1275946274699997e+00, 6.2830758499914000e+03 },
    { 1.6289000000000000e-07, 2.9085268519500000e+00, 1.0988808157535001e+04 },
    { 1.5507000000000000e-07, 2.0040125945399998e+00, 1.3367972631106601e+04 },
    { 1.5382000000000001e-07, 5.9858870981300001e+00, 1.9896880127327400e+04 },
    { 1.3003999999999999e-07, 4.8319549246999998e+00, 1.4143495242430599e+04 },
    { 1.2412000000000000e-07, 4.1364403091700002e+00, 1.0021837280099400e+04 },
    { 1.0848000000000000e-07, 3.2941991025399999e+00, 1.1015106477334801e+04 },
    { 1.0730000000000000e-07, 5.6123540938199996e+00, 1.0404733812322600e+04 },
    { 1.0335999999999999e-07, 2.7516753154100000e+00, 1.9144826611159999e+02 },
    { 1.0158000000000000e-07, 5.7195418382699996e+00, 9.6835945811164001e+03 },
    { 9.7749999999999998e-08, 1.2141888120400000e+00, 8.6242126509272002e+03 },
    { 8.2090000000000003e-08, 3.5537102189000001e+00, 1.0742976511305600e+04 },
    { 8.0799999999999996e-08, 2.6983104909300000e+00, 1.9651048481098002e+04 },
    { 6.7220000000000002e-08, 2.4184848604500000e+00, 4.7057323075435997e+03 },
    { 6.4290000000000002e-08, 4.9148107365599998e+00, 1.0593819301891999e+03 },
    { 6.4160000000000000e-08, 4.6264840233999999e-01, 9.7866873553349997e+03 },
    { 5.9030000000000000e-08, 5.9621583140299999e+00, 2.5934124331089399e+04 },
    { 5.6470000000000002e-08, 3.5759471583600000e+00, 3.1546870848956000e+03 },
    { 4.2100000000000000e-08, 4.9923213292500002e+00, 7.0848967811151997e+03 },
    { 4.1350000000000003e-08, 3.4378283250299999e+00, 1.7298182327326202e+04 },
    { 4.0439999999999997e-08, 6.2465053772200001e+00, 1.1322664098304400e+04 },
    { 4.0329999999999998e-08, 3.4288782079400000e+00, 9.1039069941176003e+03 },
    { 3.7469999999999997e-08, 1.3031872201900001e+00, 1.5720838784878400e+04 },
    { 3.4879999999999999e-08, 1.4633368219000001e-01, 2.0618019358533598e+04 },
    { 3.4399999999999997e-08, 7.7140273510000001e-01, 4.5519534970588002e+03 },
    { 3.4170000000000001e-08, 3.9547498916400001e+00, 7.8604193924392002e+03 },
    { 2.8509999999999999e-08, 5.6859170615299996e+00, 1.0206171999210201e+04 },
    { 2.8320000000000001e-08, 4.2424123321999998e+00, 2.1329909543799999e+02 },
    { 2.7560000000000000e-08, 5.4323695766400002e+00, 2.9580474708443799e+04 },
    { 2.7500000000000001e-08, 3.8949709536100001e+00, 1.0192510150718599e+04 },
    { 2.7430000000000001e-08, 6.0638749480299996e+00, 1.0234060941703399e+04 },
    { 2.7339999999999999e-08, 5.6299647223299996e+00, 1.2566151699982800e+04 },
    { 2.6820000000000001e-08, 4.5801803349599997e+00, 8.0182093112380005e+02 },
    { 2.4030000000000001e-08, 4.9053321084499997e+00, 1.9367189162232800e+04 },
    { 2.1550000000000001e-08, 2.9731467355000002e-01, 9.4114646150871995e+03 },
    { 2.1030000000000000e-08, 1.1251477193599999e+00, 2.9864334027309000e+04 },
    { 1.7870000000000001e-08, 2.5250633874099999e+00, 1.5874617595363199e+04 },
    { 1.6320000000000000e-08, 4.6712339725599996e+00, 1.1093785520934000e+03 },
    { 1.6130000000000000e-08, 1.4752516580499999e+00, 7.0585984613153996e+03 },
    { 1.5980000000000001e-08, 2.2175738924999999e+00, 1.0426584641649000e+04 },
    { 1.4680000000000000e-08, 7.5541488791000000e-01, 9.9999864507729999e+03 },
    { 1.2100000000000001e-08, 3.1771764635399999e+00, 3.5320606928113998e+03 },
    { 1.1539999999999999e-08, 1.9881839617499999e+00, 2.6298319799800002e+01 },
    { 1.1320000000000000e-08, 5.3851760355199998e+00, 7.1135470007999997e+00 },
    { 1.1180000000000000e-08, 9.6571599573000000e-01, 2.3581258177317599e+04 },
    { 1.0300000000000000e-08, 8.4929643011000000e-01, 8.0945216858326003e+03 },
    { 1.0270000000000000e-08, 2.5159269179999999e+00, 3.3794543723528601e+04 },
    { 9.8700000000000000e-09, 4.7049594442700000e+00, 1.4945316173554400e+04 },
    { 9.6500000000000004e-09, 1.1619318090000001e-02, 3.1283887650958000e+03 },
    { 9.5300000000000000e-09, 5.7311846025099999e+00, 2.0213271996984000e+04 },
    { 8.8900000000000005e-09, 1.4821494279100000e+00, 9.8303890139878004e+03 },
    { 8.5999999999999993e-09, 5.7428195513600002e+00, 5.6613320491521999e+03 },
    { 8.4700000000000007e-09, 1.2488870695000001e+00, 9.5733882598969994e+03 },
    { 7.7799999999999992e-09, 1.1637851098300001e+00, 8.6622403235630009e+03 },
    { 7.6999999999999995e-09, 4.5419635347499998e+00, 2.1228392023545799e+04 },
    { 7.4000000000000001e-09, 1.9761428076500001e+00, 1.0596182078434200e+04 },
    { 6.6999999999999996e-09, 5.3686420458899997e+00, 1.5890728952838001e+03 },
    { 6.1499999999999996e-09, 1.9628978161899999e+00, 2.6087903141574199e+04 },
    { 6.0600000000000002e-09, 6.2207015960699996e+00, 2.5158601719765400e+04 },
    { 5.9200000000000002e-09, 3.3263251318600000e+00, 3.3406124266997999e+03 },
    { 5.9099999999999997e-09, 4.6237402215900003e+00, 9.9294262273458007e+03 },
    { 5.7500000000000002e-09, 2.2907387588499999e+00, 2.1535949644515400e+04 },
    { 5.6900000000000000e-09, 5.3727665533100000e+00, 1.0497144865076199e+04 },
    { 5.4700000000000003e-09, 5.0665396450599998e+00, 2.9050783743349199e+04 },
    { 5.4199999999999999e-09, 3.9417153955800002e+00, 3.7724753419748202e+04 },
    { 5.3700000000000003e-09, 6.0032753925500000e+00, 4.7320306273433998e+03 },
    { 5.3300000000000004e-09, 7.3765107493000004e-01, 2.8286990484861199e+04 },
    { 5.2100000000000000e-09, 3.8428634519900000e+00, 1.0207762621903599e+04 },
    { 5.2000000000000002e-09, 6.1159680020699998e+00, 1.0218808470518399e+04 },
    { 4.5900000000000001e-09, 1.6915306140099999e+00, 3.9302096962196001e+03 },
    { 4.5699999999999997e-09, 6.1697474237899996e+00, 1.0220399093211799e+04 },
    { 4.3999999999999997e-09, 1.2523624196800001e+00, 1.0175257873575199e+04 },
    { 4.3899999999999999e-09, 2.9023782848400002e+00, 4.0853142184844000e+04 },
    { 4.3699999999999996e-09, 2.4233039112000001e+00, 1.0251313218846801e+04 },
    { 4.1800000000000000e-09, 5.4811080324299999e+00, 3.2217200181080800e+04 },
    { 4.0700000000000002e-09, 6.1573126284599997e+00, 1.8875525869773999e+04 },
    { 4.0499999999999999e-09, 1.4292787587300000e+00, 9.1610171630225996e+03 },
    { 3.9600000000000004e-09, 4.5620923128199999e+00, 1.0063722349076401e+04 },
    { 3.9400000000000000e-09, 4.8165068449800001e+00, 9.1467900690210008e+03 },
    { 3.7900000000000004e-09, 4.5651154398199996e+00, 2.0419457545421192e+04 },
    { 3.6199999999999999e-09, 2.2595431209999999e-02, 1.4765239043269799e+04 },
    { 3.4900000000000001e-09, 5.7850752254900000e+00, 1.2592450019782600e+04 },
    { 3.4299999999999999e-09, 2.3651673423099999e+00, 2.7511467873537200e+04 },
    { 3.3400000000000000e-09, 5.2892008372900001e+00, 3.8289653222319998e+02 },
    { 3.2200000000000000e-09, 1.3684642006700001e+00, 2.9088811415985001e+04 },
    { 3.1399999999999999e-09, 5.0825905041999997e+00, 1.8837498197138189e+04 },
    { 3.1399999999999999e-09, 5.2322090526400000e+00, 4.2659819087599999e+02 },
    { 3.0600000000000002e-09, 4.7092243462000000e-01, 2.3958631785233400e+04 },
    { 3.0399999999999998e-09, 5.3318163205899998e+00, 4.1654963115967803e+04 },
    { 3.0100000000000002e-09, 4.2497181509899997e+00, 1.4919017853754600e+04 },
    { 2.9199999999999998e-09, 3.7661845555500002e+00, 6.6812248533995999e+03 },
    { 2.9199999999999998e-09, 7.4997511569999997e-01, 3.1491641605882000e+03 },
    { 2.8900000000000002e-09, 3.5984138473599998e+00, 1.1272667476400200e+04 },
    { 2.7799999999999999e-09, 1.0022232617000000e-01, 1.6496361396202399e+04 },
    { 2.6400000000000000e-09, 1.1534521450000000e-02, 2.0452869412221800e+04 },
    { 2.5399999999999999e-09, 4.3402657448999999e+00, 1.0137019474935400e+04 },
    { 2.5000000000000001e-09, 5.0539546912300004e+00, 2.4356780788641601e+04 },
    { 2.3600000000000001e-09, 1.0344219605100000e+00, 2.2779437246193800e+04 },
    { 2.3199999999999998e-09, 4.8064662618999998e-01, 1.7085958665722199e+04 },
    { 2.0500000000000002e-09, 6.1963940991799999e+00, 7.0641213856227996e+03 },
    { 2.0000000000000001e-09, 4.0651677950999998e+00, 1.8849227549974199e+04 },
    { 1.9700000000000000e-09, 3.7138037723999999e+00, 1.5510452226479999e+03 },
    { 1.8000000000000000e-09, 4.8168923158999999e-01, 4.5585172812187397e+04 },
    { 1.7700000000000001e-09, 3.5824802924400001e+00, 2.0400272772622200e+04 },
    { 1.6999999999999999e-09, 5.9652239511799996e+00, 1.0198033075026000e+04 },
    { 1.6999999999999999e-09, 3.5286658326699998e+00, 1.0228538017396000e+04 },
    { 1.6999999999999999e-09, 3.7228219048800000e+00, 3.1441677569756801e+04 },
    { 1.6600000000000001e-09, 3.5697687499999999e-02, 1.3745346239022399e+04 },
    { 1.4400000000000000e-09, 2.8128088569699998e+00, 1.7277406931833801e+04 },
    { 1.3800000000000000e-09, 1.1925397782500000e+00, 2.2805735565993600e+04 },
    { 1.3800000000000000e-09, 4.0140310769300003e+00, 2.1202093703745999e+04 },
    { 1.3700000000000000e-09, 2.8940795036600000e+00, 3.3019021112204602e+04 },
    { 1.3100000000000000e-09, 9.2317994328999997e-01, 3.4425749449653999e+03 },
    { 1.2700000000000000e-09, 5.1438930238599996e+00, 5.4812549188676003e+03 },
    { 1.2700000000000000e-09, 4.6833130674000001e-01, 5.3680451209540001e+02 },
    { 1.2600000000000000e-09, 4.2159418537400004e+00, 2.0956262057516600e+04 },
    { 1.2600000000000000e-09, 2.8404966269299998e+00, 9.9674538999815995e+03 },
    { 1.2199999999999999e-09, 3.7000676558000001e-01, 1.0459117192440401e+04 },
    { 1.2100000000000000e-09, 3.8592822378600000e+00, 5.2257741809380002e+02 },
    { 1.1800000000000001e-09, 1.7170655041600000e+00, 1.3341674311306801e+04 },
    { 1.1400000000000000e-09, 5.1130962920999998e+00, 1.0309277421860000e+02 },
    { 1.0999999999999999e-09, 7.3944778373999998e-01, 3.0110165673538399e+04 },
    { 1.0999999999999999e-09, 1.4469991800299999e+00, 4.3232306658415597e+04 },
    { 1.0700000000000000e-09, 4.1904539431499996e+00, 1.9999972901545989e+04 },
    { 1.0600000000000001e-09, 2.4206499671800001e+00, 2.0809467624645200e+04 },
    { 1.0399999999999999e-09, 2.4734735867400000e+00, 2.6709646942413401e+04 },
    { 1.0399999999999999e-09, 5.0272474040199997e+00, 3.5371887265976402e+04 },
    { 9.9000000000000010e-10, 5.6114980311600000e+00, 1.0529677415867600e+04 },
    { 9.9000000000000010e-10, 1.9332963773200000e+00, 4.9515382508406998e+04 },
    { 9.7999999999999992e-10, 4.6680755280300001e+00, 2.0235122826310399e+04 },
    { 9.5999999999999999e-10, 2.9068239930400002e+00, 9.7938009023358009e+03 },
    { 9.5000000000000003e-10, 5.2543513547699998e+00, 3.0213258447757002e+04 },
    { 9.5000000000000003e-10, 3.1131780158900000e+00, 9.8968936765544004e+03 },
    { 9.2999999999999999e-10, 4.2135614445999998e+00, 3.6301188687785187e+04 },
    { 9.2000000000000003e-10, 5.6311047771200000e+00, 1.0288067144778301e+04 },
    { 9.2000000000000003e-10, 3.8627812702500002e+00, 1.0138503947643700e+04 },
    { 8.9999999999999999e-10, 4.3432419307299996e+00, 3.6949230808424203e+04 },
    { 8.9999999999999999e-10, 7.3196634230000002e-02, 6.8726731195111997e+03 },
    { 8.9000000000000003e-10, 2.3371421606100000e+00, 1.0316378320429600e+04 },
    { 8.9000000000000003e-10, 8.8060757437000003e-01, 6.4897765872880000e+03 },
    { 8.7999999999999996e-10, 1.5901718304400001e+00, 1.0110192771992401e+04 },
    { 8.6000000000000003e-10, 8.5755223960000004e-01, 2.3791644735716000e+03 },
    { 8.6000000000000003e-10, 8.7055324194000006e-01, 1.6522659716002199e+04 },
    { 8.6000000000000003e-10, 1.0247617538400000e+00, 6.5236221293285402e+04 },
    { 8.4999999999999996e-10, 5.8824806255000004e+00, 3.7235089589230001e+03 },
    { 8.3999999999999999e-10, 4.9374537391000004e+00, 5.2175806283148399e+04 },
    { 8.3000000000000003e-10, 2.0754377559399999e+00, 3.1749235190726398e+04 },
    { 8.0000000000000003e-10, 2.8722060204900002e+00, 4.7162516354635198e+04 },
    { 7.8999999999999996e-10, 1.6544614503199999e+00, 3.9264069289560197e+04 },
    { 7.7999999999999999e-10, 2.6583963432500002e+00, 1.0232955307110789e+04 },
    { 7.7999999999999999e-10, 5.5230439694000000e-01, 1.0193615785311211e+04 },
    { 7.2999999999999996e-10, 2.5677524590699998e+00, 1.3936794505133999e+04 },
    { 7.1000000000000003e-10, 6.4943236639000002e-01, 2.0775395492400001e+01 },
    { 7.1000000000000003e-10, 7.0714638996000001e-01, 3.6147409877300401e+04 },
    { 7.1000000000000003e-10, 2.5287976000099999e+00, 2.6735945262213201e+04 },
    { 7.1000000000000003e-10, 2.3660958236699998e+00, 7.4781598567299994e+01 },
    { 6.9000000000000000e-10, 1.5291477441200001e+00, 9.6907081281171995e+03 },
    { 6.9000000000000000e-10, 4.2270579845000000e-01, 9.5805018068978006e+03 },
    { 6.8000000000000003e-10, 2.2850186557800001e+00, 8.5211198767086007e+03 },
    { 6.6000000000000000e-10, 4.1224431205699998e+00, 2.8521092778254599e+04 },
    { 6.5000000000000003e-10, 2.5331265668100000e+00, 1.0735862964304801e+04 },
    { 6.5000000000000003e-10, 2.1242358662699998e+00, 2.8313288804660999e+04 },
    { 6.3000000000000000e-10, 5.8007667807800001e+00, 2.1187638603783998e+03 },
    { 6.3000000000000000e-10, 3.3569000698400000e+00, 5.3445592204626599e+04 },
    { 6.2000000000000003e-10, 2.2858791530899998e+00, 3.9039113764198000e+03 },
    { 6.2000000000000003e-10, 2.4623906921000001e+00, 1.0846069285524200e+04 },
    { 6.0999999999999996e-10, 5.4794329765800001e+00, 1.1764330768858999e+04 },
    { 6.0999999999999996e-10, 3.7733323044299998e+00, 5.1868248662178798e+04 },
    { 5.7999999999999996e-10, 5.2504505798499999e+00, 1.0419471094648199e+04 },
    { 5.7999999999999996e-10, 4.2372459822100002e+00, 1.0007099997773799e+04 },
    { 5.6000000000000003e-10, 4.7382132474500001e+00, 6.3989728631399998e+02 },
    { 5.3000000000000003e-10, 2.1458352329600001e+00, 8.6313261979279996e+03 },
    { 5.3000000000000003e-10, 3.0237563285800002e+00, 1.9317192540328600e+04 },
};

static const VSOP87Term VSOP87A_VENUS_Z1[108] = {
    { 2.0809640200000002e-03, 1.8896727874200001e+00, 1.0213285546211000e+04 },
    { 1.3641440000000001e-05, 0.0000000000000000e+00, 0.0000000000000000e+00 },
    { 1.2649890000000000e-05, 3.7103750132100002e+00, 2.0426571092422000e+04 },
    { 1.1138999999999999e-07, 4.8543787820500004e+00, 3.0639856638632999e+04 },
    { 9.5570000000000001e-08, 2.2789109335500002e+00, 9.4377629348869996e+03 },
    { 7.8219999999999995e-08, 5.5431541953799996e+00, 1.5773435424478000e+03 },
    { 5.7600000000000000e-08, 2.7988422334000002e-01, 9.1539036160218002e+03 },
    { 5.3900000000000000e-08, 5.3720358885300001e+00, 2.3528661537717999e+03 },
    { 4.6810000000000000e-08, 5.2342757208900004e+00, 1.0186987226411200e+04 },
    { 3.6809999999999997e-08, 4.1740692135400002e+00, 1.0239583866010800e+04 },
    { 3.5889999999999999e-08, 1.5938377930600001e+00, 1.1790629088658799e+04 },
    { 3.2969999999999997e-08, 7.3050419137000000e-01, 5.5075532386674004e+03 },
    { 2.5570000000000000e-08, 1.6190535782700000e+00, 1.8073704938650200e+04 },
    { 2.3949999999999999e-08, 1.3453102875800000e+00, 1.0404733812322600e+04 },
    { 2.0790000000000000e-08, 5.5990044054099997e+00, 5.2969096509459996e+02 },
    { 1.9569999999999999e-08, 6.1458611892099997e+00, 2.2003914634869801e+04 },
    { 1.8930000000000001e-08, 5.0455637432299998e+00, 1.0988808157535001e+04 },
    { 1.7490000000000000e-08, 1.7287648355100000e+00, 8.6359420037632008e+03 },
    { 1.3610000000000001e-08, 4.0192228966999999e-01, 1.3367972631106601e+04 },
    { 1.1590000000000000e-08, 2.3199441291600000e+00, 1.0021837280099400e+04 },
    { 1.0600000000000001e-08, 4.4054999807400002e+00, 1.9651048481098002e+04 },
    { 1.0260000000000001e-08, 5.0544960385000000e+00, 7.7552261132399997e+02 },
    { 8.9399999999999993e-09, 6.1876960824699996e+00, 7.8604193924392002e+03 },
    { 7.4099999999999998e-09, 3.1529323923799999e+00, 6.2830758499914000e+03 },
    { 6.8599999999999999e-09, 1.3769280748499999e+00, 1.9896880127327400e+04 },
    { 5.5899999999999999e-09, 6.3256033185000005e-01, 8.6242126509272002e+03 },
    { 5.3100000000000001e-09, 1.7761082615099999e+00, 1.1322664098304400e+04 },
    { 4.3999999999999997e-09, 1.7161789846500000e+00, 2.0618019358533598e+04 },
    { 4.2599999999999998e-09, 2.8305635937800000e+00, 1.4143495242430599e+04 },
    { 4.1499999999999999e-09, 3.0015376356400001e+00, 1.5720838784878400e+04 },
    { 3.9000000000000002e-09, 5.1064639131899998e+00, 9.7866873553349997e+03 },
    { 3.8199999999999996e-09, 2.9076532254999998e-01, 4.7057323075435997e+03 },
    { 3.7200000000000000e-09, 1.8054199335000001e+00, 1.7298182327326202e+04 },
    { 3.6899999999999999e-09, 1.2593166363999999e+00, 2.5934124331089399e+04 },
    { 3.2500000000000002e-09, 2.0788633678999999e-01, 1.1015106477334801e+04 },
    { 3.1599999999999998e-09, 2.2956582071799998e+00, 1.0742976511305600e+04 },
    { 3.0500000000000000e-09, 3.7351434600800002e+00, 1.0596182078434200e+04 },
    { 2.9600000000000001e-09, 6.8834496753999996e-01, 1.9367189162232800e+04 },
    { 2.7799999999999999e-09, 4.2243789319199996e+00, 1.9144826611159999e+02 },
    { 2.6799999999999998e-09, 1.8173158012799999e+00, 1.0593819301891999e+03 },
    { 2.5300000000000002e-09, 2.2788746878400001e+00, 9.9999864507729999e+03 },
    { 2.3300000000000000e-09, 2.1446630563100002e+00, 8.0182093112380005e+02 },
    { 2.3100000000000001e-09, 6.2397863221199996e+00, 9.8303890139878004e+03 },
    { 2.2999999999999999e-09, 4.9292449163000001e+00, 3.9302096962196001e+03 },
    { 2.0700000000000001e-09, 1.9071696139200001e+00, 9.1039069941176003e+03 },
    { 1.9900000000000000e-09, 4.7936489347900002e+00, 1.0206171999210201e+04 },
    { 1.9800000000000002e-09, 2.6103639887200001e+00, 9.4114646150871995e+03 },
    { 1.9300000000000002e-09, 2.6858441073399999e+00, 2.3581258177317599e+04 },
    { 1.8000000000000000e-09, 3.6516043908900002e+00, 2.1228392023545799e+04 },
    { 1.7300000000000000e-09, 3.7502676498900001e+00, 7.0585984613153996e+03 },
    { 1.6900000000000000e-09, 3.6539769891699998e+00, 7.1135470007999997e+00 },
    { 1.6600000000000001e-09, 3.2248110746299998e+00, 1.2566151699982800e+04 },
    { 1.5300000000000001e-09, 5.9045093130700002e+00, 3.1546870848956000e+03 },
    { 1.4800000000000001e-09, 1.3905090593900000e+00, 1.0234060941703399e+04 },
    { 1.4800000000000001e-09, 5.7827362391900001e+00, 2.6298319799800002e+01 },
    { 1.4700000000000001e-09, 4.4041746908099997e+00, 1.0220399093211799e+04 },
    { 1.4700000000000001e-09, 4.3857594938500002e+00, 1.5874617595363199e+04 },
    { 1.3900000000000000e-09, 2.6910938405599998e+00, 2.9864334027309000e+04 },
    { 1.3800000000000000e-09, 6.1384351561799999e+00, 9.1610171630225996e+03 },
    { 1.3700000000000000e-09, 5.8551335903100004e+00, 2.1329909543799999e+02 },
    { 1.3200000000000000e-09, 6.5236823857000004e-01, 2.9580474708443799e+04 },
    { 1.2700000000000000e-09, 6.1179470611699998e+00, 1.0426584641649000e+04 },
    { 1.2000000000000000e-09, 5.5697906720099999e+00, 1.0192510150718599e+04 },
    { 1.0999999999999999e-09, 1.5417876594100000e+00, 9.6835945811164001e+03 },
    { 1.0300000000000000e-09, 2.3797661557000001e+00, 3.1283887650958000e+03 },
    { 9.9000000000000010e-10, 1.1805378819500001e+00, 7.0848967811151997e+03 },
    { 9.6999999999999996e-10, 5.8335065932200001e+00, 4.0853142184844000e+04 },
    { 9.6999999999999996e-10, 4.6465990473199996e+00, 2.5158601719765400e+04 },
    { 9.2000000000000003e-10, 1.7084455774000001e-01, 9.1467900690210008e+03 },
    { 8.6000000000000003e-10, 5.4826692802499997e+00, 3.7724753419748202e+04 },
    { 7.8999999999999996e-10, 6.0720662685800004e+00, 2.0419457545421192e+04 },
    { 7.7999999999999999e-10, 5.1688891428400003e+00, 8.6622403235630009e+03 },
    { 7.7000000000000003e-10, 1.6020610226000001e+00, 1.0207762621903599e+04 },
    { 7.7000000000000003e-10, 1.6085757523399999e+00, 1.0218808470518399e+04 },
    { 7.5000000000000000e-10, 4.0169882762900002e+00, 3.3794543723528601e+04 },
    { 7.4000000000000003e-10, 7.4329919337999995e-01, 3.8289653222319998e+02 },
    { 7.4000000000000003e-10, 6.2197461083299999e+00, 1.8837498197138189e+04 },
    { 7.2000000000000000e-10, 2.4976031588800001e+00, 4.5519534970588002e+03 },
    { 6.9999999999999996e-10, 5.5501102109899998e+00, 5.6613320491521999e+03 },
    { 6.2000000000000003e-10, 1.2136511931200000e+00, 1.0198033075026000e+04 },
    { 6.2000000000000003e-10, 1.9970495470700000e+00, 1.0228538017396000e+04 },
    { 5.7000000000000000e-10, 6.2155717729000004e-01, 4.1654963115967803e+04 },
    { 5.6000000000000003e-10, 1.6985541402000001e-01, 1.4919017853754600e+04 },
    { 5.3000000000000003e-10, 1.8008436556600000e+00, 4.5585172812187397e+04 },
    { 4.8999999999999996e-10, 3.4478903767500002e+00, 2.8286990484861199e+04 },
    { 4.8999999999999996e-10, 1.7473941091100000e+00, 3.5320606928113998e+03 },
    { 4.8000000000000000e-10, 2.5897654184899999e+00, 2.2779437246193800e+04 },
    { 4.6000000000000001e-10, 1.9698789311600000e+00, 4.2659819087599999e+02 },
    { 4.3999999999999998e-10, 3.3491742954000001e+00, 1.1272667476400200e+04 },
    { 4.3999999999999998e-10, 7.8654424918999999e-01, 2.0213271996984000e+04 },
    { 4.3000000000000001e-10, 5.2940347547500002e+00, 3.1491641605882000e+03 },
    { 3.7999999999999998e-10, 6.1928535233800002e+00, 2.9088811415985001e+04 },
    { 3.7000000000000001e-10, 4.1963193036800002e+00, 2.7511467873537200e+04 },
    { 3.7000000000000001e-10, 5.0556368361999997e+00, 5.3680451209540001e+02 },
    { 3.6000000000000000e-10, 4.3724612470000000e-02, 2.0400272772622200e+04 },
    { 3.6000000000000000e-10, 5.5441648847499998e+00, 5.2257741809380002e+02 },
    { 3.6000000000000000e-10, 3.2261848689199999e+00, 4.9515382508406998e+04 },
    { 3.4999999999999998e-10, 3.5900416392199999e+00, 2.1535949644515400e+04 },
    { 3.4000000000000001e-10, 4.1968527099199999e+00, 1.3341674311306801e+04 },
    { 3.3000000000000000e-10, 2.6315268825000002e+00, 1.6496361396202399e+04 },
    { 3.3000000000000000e-10, 6.0249054715900003e+00, 2.1202093703745999e+04 },
    { 3.1000000000000002e-10, 1.5749156970900000e+00, 3.2217200181080800e+04 },
    { 3.1000000000000002e-10, 1.2039243439200000e+00, 1.2592450019782600e+04 },
    { 3.1000000000000002e-10, 1.2730438322700000e+00, 3.3019021112204602e+04 },
    { 2.8999999999999998e-10, 3.6491152164099998e+00, 2.0452869412221800e+04 },
    { 2.8000000000000002e-10, 2.8339069791999999e-01, 2.9050783743349199e+04 },
    { 2.8000000000000002e-10, 3.7074838563100001e+00, 4.7320306273433998e+03 },
    { 2.7000000000000000e-10, 1.4626351085200000e+00, 8.0945216858326003e+03 },
};

static const VSOP87Term VSOP87A_VENUS_Z2[45] = {
    { 9.1480440000000002e-05, 3.3479100527200001e+00, 1.0213285546211000e+04 },
    { 1.6397700000000000e-06, 0.0000000000000000e+00, 0.0000000000000000e+00 },
    { 1.3554000000000001e-07, 5.3391431090400001e+00, 2.0426571092422000e+04 },
    { 6.8299999999999998e-09, 1.2793545780000000e-02, 1.0186987226411200e+04 },
    { 5.4000000000000004e-09, 4.1366677898399997e+00, 1.0239583866010800e+04 },
    { 4.7099999999999997e-09, 1.7679679210999999e+00, 3.0639856638632999e+04 },
    { 3.0399999999999998e-09, 6.0413349176200004e+00, 1.0988808157535001e+04 },
    { 3.0300000000000001e-09, 5.4544143559000002e-01, 2.3528661537717999e+03 },
    { 2.8200000000000002e-09, 4.0326649305700002e+00, 1.5773435424478000e+03 },
    { 2.7499999999999998e-09, 3.2625460349100002e+00, 1.1790629088658799e+04 },
    { 2.5899999999999999e-09, 5.1315489342099996e+00, 9.1539036160218002e+03 },
    { 2.4100000000000002e-09, 6.1715881242500004e+00, 9.4377629348869996e+03 },
    { 2.3800000000000001e-09, 2.8300843993200000e+00, 1.0404733812322600e+04 },
    { 1.1700000000000001e-09, 5.4278412191500003e+00, 5.5075532386674004e+03 },
    { 9.5000000000000003e-10, 5.9412563031999999e-01, 5.2969096509459996e+02 },
    { 9.4000000000000006e-10, 3.1570790345800002e+00, 1.8073704938650200e+04 },
    { 9.0999999999999996e-10, 6.0494607773400002e+00, 1.9651048481098002e+04 },
    { 8.6999999999999999e-10, 5.2403069930699999e+00, 1.3367972631106601e+04 },
    { 7.5000000000000000e-10, 2.3631944253000001e-01, 8.6359420037632008e+03 },
    { 7.1000000000000003e-10, 7.9640279433000005e-01, 1.0021837280099400e+04 },
    { 6.9000000000000000e-10, 1.1247272739900001e+00, 2.2003914634869801e+04 },
    { 6.2000000000000003e-10, 2.5293521151200000e+00, 9.6835945811164001e+03 },
    { 5.3000000000000003e-10, 5.2564157102299998e+00, 1.0596182078434200e+04 },
    { 5.0000000000000003e-10, 4.6259167881799996e+00, 1.5720838784878400e+04 },
    { 4.8000000000000000e-10, 3.8068223014100000e+00, 7.7552261132399997e+02 },
    { 4.0999999999999998e-10, 4.1928534127999999e+00, 1.4143495242430599e+04 },
    { 3.9000000000000000e-10, 3.3044524874099999e+00, 1.1322664098304400e+04 },
    { 3.7999999999999998e-10, 4.7178730671600002e+00, 7.0585984613153996e+03 },
    { 3.7999999999999998e-10, 1.8793222424200000e+00, 8.0182093112380005e+02 },
    { 3.4000000000000001e-10, 2.6240866672199998e+00, 1.9896880127327400e+04 },
    { 2.8999999999999998e-10, 4.7318389552999998e+00, 9.8303890139878004e+03 },
    { 2.5000000000000002e-10, 2.8590496471400000e+00, 9.4114646150871995e+03 },
    { 2.3000000000000001e-10, 2.9053782153699999e+00, 6.2830758499914000e+03 },
    { 2.1999999999999999e-10, 2.3111955597299998e+00, 1.8849227549974199e+04 },
    { 2.1000000000000000e-10, 3.1827807969999999e-01, 1.1015106477334801e+04 },
    { 1.8999999999999999e-10, 3.2510026208600000e+00, 2.0618019358533598e+04 },
    { 1.8000000000000000e-10, 4.9599857932600004e+00, 4.7057323075435997e+03 },
    { 1.7000000000000001e-10, 5.4117113206400003e+00, 2.6298319799800002e+01 },
    { 1.7000000000000001e-10, 2.6940501427900001e+00, 3.1283887650958000e+03 },
    { 1.5999999999999999e-10, 5.3259218562999999e-01, 1.0206171999210201e+04 },
    { 1.5999999999999999e-10, 5.8197012081499997e+00, 1.0220399093211799e+04 },
    { 1.5999999999999999e-10, 3.9410405171099998e+00, 1.0742976511305600e+04 },
    { 1.5000000000000000e-10, 4.6155538647799998e+00, 9.9999864507729999e+03 },
    { 1.5000000000000000e-10, 4.8806405423000001e+00, 1.0426584641649000e+04 },
    { 1.5000000000000000e-10, 6.1863693379100004e+00, 1.9144826611159999e+02 },
};

static const VSOP87Term VSOP87A_VENUS_Z3[10] = {
    { 2.7200499999999999e-06, 4.8764811614000001e+00, 1.0213285546211000e+04 },
    { 1.3656000000000000e-07, 3.1415926535900001e+00, 0.0000000000000000e+00 },
    { 1.7430000000000001e-08, 4.2985032723000000e-01, 2.0426571092422000e+04 },
    { 3.3000000000000000e-10, 1.5491424016599999e+00, 1.0186987226411200e+04 },
    { 2.5000000000000002e-10, 5.7874336881400001e+00, 1.0239583866010800e+04 },
    { 1.5000000000000000e-10, 2.6812223507999999e-01, 3.0639856638632999e+04 },
    { 1.0000000000000000e-10, 5.4882775335299998e+00, 9.4377629348869996e+03 },
    { 1.0000000000000000e-10, 1.9002773270100000e+00, 2.3528661537717999e+03 },
    { 8.9999999999999999e-11, 8.6239699725999996e-01, 1.0988808157535001e+04 },
    { 7.9999999999999995e-11, 4.6006915344200001e+00, 1.1790629088658799e+04 },
};

static const VSOP87Term VSOP87A_VENUS_Z4[3] = {
    { 6.1749999999999994e-08, 1.7121460360999999e-01, 1.0213285546211000e+04 },
    { 1.2500000000000000e-09, 3.1415926535900001e+00, 0.0000000000000000e+00 },
    { 4.0999999999999998e-10, 2.5036413692799999e+00, 2.0426571092422000e+04 },
};

static const VSOP87Term VSOP87A_VENUS_Z5[3] = {
    { 1.0800000000000000e-09, 1.6743973964100001e+00, 1.0213285546211000e+04 },
    { 2.5000000000000002e-10, 0.0000000000000000e+00, 0.0000000000000000e+00 },
    { 3.0000000000000000e-11, 3.6675112390300000e+00, 2.0426571092422000e+04 },
};

static const VSOP87Block VSOP87A_VENUS_X[6] = {
    { VSOP87A_VENUS_X0, 548 },
    { VSOP87A_VENUS_X1, 338 },
    { VSOP87A_VENUS_X2, 99 },
    { VSOP87A_VENUS_X3, 5 },
    { VSOP87A_VENUS_X4, 4 },
    { VSOP87A_VENUS_X5, 3 },
};

static const VSOP87Block VSOP87A_VENUS_Y[6] = {
    { VSOP87A_VENUS_Y0, 565 },
    { VSOP87A_VENUS_Y1, 325 },
    { VSOP87A_VENUS_Y2, 99 },
    { VSOP87A_VENUS_Y3, 5 },
    { VSOP87A_VENUS_Y4, 4 },
    { VSOP87A_VENUS_Y5, 3 },
};

static const VSOP87Block VSOP87A_VENUS_Z[6] = {
    { VSOP87A_VENUS_Z0, 190 },
    { VSOP87A_VENUS_Z1, 108 },
    { VSOP87A_VENUS_Z2, 45 },
    { VSOP87A_VENUS_Z3, 10 },
    { VSOP87A_VENUS_Z4, 3 },
    { VSOP87A_VENUS_Z5, 3 },
};

static const int32 VSOP87A_VENUS_TRUNC_COUNT[11][3][6] = {
    {
        { 548, 338, 99, 5, 4, 3 },
        { 565, 325, 99, 5, 4, 3 },
        { 190, 108, 45, 10, 3, 3 },
    },
    {
        { 164, 59, 13, 3, 0, 0 },
        { 169, 60, 13, 3, 0, 0 },
        { 64, 22, 3, 3, 1, 0 },
    },
    {
        { 97, 25, 6, 3, 0, 0 },
        { 96, 25, 6, 2, 0, 0 },
        { 35, 8, 3, 2, 1, 0 },
    },
    {
        { 74, 18, 4, 3, 0, 0 },
        { 72, 18, 4, 2, 0, 0 },
        { 27, 4, 3, 2, 0, 0 },
    },
    {
        { 36, 7, 3, 1, 0, 0 },
        { 38, 7, 3, 0, 0, 0 },
        { 9, 3, 2, 1, 0, 0 },
    },
    {
        { 29, 4, 3, 0, 0, 0 },
        { 29, 4, 3, 0, 0, 0 },
        { 5, 3, 2, 1, 0, 0 },
    },
    {
        { 13, 3, 3, 0, 0, 0 },
        { 13, 3, 2, 0, 0, 0 },
        { 3, 3, 1, 0, 0, 0 },
    },
    {
        { 8, 3, 0, 0, 0, 0 },
        { 8, 3, 1, 0, 0, 0 },
        { 3, 3, 1, 0, 0, 0 },
    },
    {
        { 3, 3, 0, 0, 0, 0 },
        { 3, 3, 0, 0, 0, 0 },
        { 3, 1, 1, 0, 0, 0 },
    },
    {
        { 3, 2, 0, 0, 0, 0 },
        { 3, 2, 0, 0, 0, 0 },
        { 3, 1, 0, 0, 0, 0 },
    },
    {
        { 3, 0, 0, 0, 0, 0 },
        { 3, 0, 0, 0, 0, 0 },
        { 1, 1, 0, 0, 0, 0 },
    },
};

