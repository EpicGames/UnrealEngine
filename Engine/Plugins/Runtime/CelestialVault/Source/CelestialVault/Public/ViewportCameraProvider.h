// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/**
 * 
 */
class CELESTIALVAULT_API IViewportCameraProvider
{
public:
	virtual ~IViewportCameraProvider() = default;

	virtual bool GetCamera(FVector& OutLocation, FRotator& OutRotation) const = 0;
};

//////////////////////////////////////////////////////////////////////////
// Default Runtime Implementation

class CELESTIALVAULT_API FRuntimeCameraProvider : public IViewportCameraProvider
{
public:
	virtual bool GetCamera(FVector& OutLocation, FRotator& OutRotation) const override;
};


// Global accessor
CELESTIALVAULT_API IViewportCameraProvider* GetCameraProvider();
CELESTIALVAULT_API void SetCameraProvider(IViewportCameraProvider* InProvider);
