// Copyright Epic Games, Inc. All Rights Reserved.

#include "CelestialVaultEditor.h"

#include "DaylightSavingsCustomization.h"
#include "DaylightSavings.h"
#include "ViewportCameraProvider.h"
#include "EditorViewportCameraProvider.h"
#include "PropertyEditorModule.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "FCelestialVaultEditorModule"


FEditorViewportCameraProvider FCelestialVaultEditorModule::EditorProvider;

void FCelestialVaultEditorModule::StartupModule()
{
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

	PropertyModule.RegisterCustomPropertyTypeLayout( FDaylightSavingsRule::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FDaylightSavingsCustomization::MakeInstance));

	SetCameraProvider(&EditorProvider);
}

void FCelestialVaultEditorModule::ShutdownModule()
{
	if (FModuleManager::Get().IsModuleLoaded("PropertyEditor"))
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

		PropertyModule.UnregisterCustomPropertyTypeLayout(FDaylightSavingsRule::StaticStruct()->GetFName());
	}

	SetCameraProvider(nullptr);
	
}

#undef LOCTEXT_NAMESPACE
    
IMPLEMENT_MODULE(FCelestialVaultEditorModule, CelestialVaultEditor)