// Copyright Epic Games, Inc. All Rights Reserved.


#include "CelestialVaultEditorUtilities.h"
#include "Engine/Texture2D.h"
#include "TextureResource.h"
#include "Engine/TextureDefines.h"

#include <LevelEditorViewport.h>



void UCelestialVaultEditorUtilities::GetViewportCursorInformation(bool& Focused, FVector2D& ScreenLocation, FVector& WorldLocation, FVector& WorldDirection)
{
	Focused = false;
	ScreenLocation = FVector2D(-1, -1);
	WorldLocation = FVector::ZeroVector;
	WorldDirection = FVector::ZeroVector;

	if (GCurrentLevelEditingViewportClient != nullptr && GCurrentLevelEditingViewportClient->Viewport->HasFocus())
	{
		FViewportCursorLocation cursorWL = GCurrentLevelEditingViewportClient->GetCursorWorldLocationFromMousePos();

		FIntPoint pos = cursorWL.GetCursorPos();
		ScreenLocation.X = pos.X;
		ScreenLocation.Y = pos.Y;
		WorldLocation = cursorWL.GetOrigin();
		WorldDirection = cursorWL.GetDirection();
		Focused = true;
	}
}

bool UCelestialVaultEditorUtilities::ComputeTextureMeanLuminance(UTexture2D* Texture, float& OutMean)
{
	OutMean = -1;
	
    if (!Texture || !Texture->GetPlatformData() || Texture->GetPlatformData()->Mips.Num() == 0)
        return false;

    FTexture2DMipMap& Mip = Texture->GetPlatformData()->Mips[0];

    const int32 Width = Mip.SizeX;
    const int32 Height = Mip.SizeY;

    if (Width == 0 || Height == 0)
        return false;

    const EPixelFormat Format = Texture->GetPixelFormat();
    if (Format == PF_G8 || Format == PF_B8G8R8A8)
    {
    	FByteBulkData* RawData = &Mip.BulkData;
    	const void* DataPtr = RawData->LockReadOnly();
    	if (!DataPtr)
    		return false;

    	const int64 PixelCount = static_cast<int64>(Width) * static_cast<int64>(Height);
    	double TotalLuminance = 0.0;

    	const bool bSRGB = Texture->SRGB;
    	if (Format == PF_G8)
    	{
    		const uint8* Pixels = static_cast<const uint8*>(DataPtr);

    		for (int64 i = 0; i < PixelCount; ++i)
    		{
    			float value = Pixels[i];
    			float linear;

    			if (bSRGB)
    			{
    				// Convert sRGB -> linear
    				linear = FLinearColor::FromSRGBColor(FColor(Pixels[i], Pixels[i], Pixels[i])).R;
    			}
    			else
    			{
    				constexpr float Inv255 = 1.0f / 255.0f;
    				linear = value * Inv255;
    			}

    			TotalLuminance += linear;
    		}
    	}
    	else if (Format == PF_B8G8R8A8)
    	{
    		const FColor* Pixels = static_cast<const FColor*>(DataPtr);

    		for (int64 i = 0; i < PixelCount; ++i)
    		{
    			FLinearColor linear;

    			if (bSRGB)
    			{
    				// Converts from sRGB to Linear
    				linear = FLinearColor(Pixels[i]);
    			}
    			else
    			{
    				linear = Pixels[i].ReinterpretAsLinear();
    			}

    			// Rec.709 luminance
    			double Lum = 0.2126 * linear.R + 0.7152 * linear.G + 0.0722 * linear.B;
    			TotalLuminance += Lum;
    		}
    	}

    	RawData->Unlock();
    	OutMean = static_cast<float>(TotalLuminance / static_cast<double>(PixelCount));
    	return true;
    }
	
    return false;
}
