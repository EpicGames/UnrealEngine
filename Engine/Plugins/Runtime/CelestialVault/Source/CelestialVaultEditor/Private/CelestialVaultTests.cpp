// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "CelestialDataTypes.h"
#include "CelestialMaths.h"
#include "Misc/AutomationTest.h"

IMPLEMENT_SIMPLE_AUTOMATION_TEST(CelestialVault_RADEC, "CelestialVault.RADEC",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool CelestialVault_RADEC::RunTest(const FString& Parameters)
{
	// Make the test pass by returning true, or fail by returning false.

	struct FRADECCase
	{
		float Latitude;
		float Longitude;
		double JulianDate;
		double ExpectedRAJ2000Degrees;
		double ExpectedDecJ2000Degrees;
		double ExpectedRADegrees;
		double ExpectedDecDegrees;

		FString ToString() const
		{
			return FString::Printf(TEXT("Latitude: %f, Longitude %f, JulianDate %f"), Latitude, Longitude, JulianDate);
		}
	};

	struct SpecificBodyCases
	{
		EVSOP87BodyType VSOP87BodyType;
		double RelativeErrorPercent;     // relative tolerance: error must be < RelativeErrorPercent% of the expected value
		double AbsoluteErrorArcsec;      // absolute tolerance (arcseconds): kicks in near 0 where relative is too strict
		TArray<FRADECCase> Cases;
	};

	// Test 1 : Montreal, Canada:  45, -73
	// Test 2 : Sydney, Australia: -34  

	TArray<FRADECCase> SunCases = {
		{45, -73, 2460676.5, 281.386045464, -23.024776939, 281.758238261, -22.999649364},
		{45, -73, 2460735.5, 341.814306234, -7.706069407, 342.137840036, -7.575724237},
		{45, -73, 2460796.5, 38.111756702, 14.97700725, 38.453710472, 15.087523803},
		{45, -73, 2460857.5, 99.883001289, 23.12308771, 100.264279309, 23.100965929},
		{45, -73, 2460919.5, 160.105053936, 8.389039907, 160.437009702, 8.257125123},
		{45, -73, 2460980.5, 216.087553478, -14.325153653, 216.434981067, -14.44103282},
		{45, -73, 2461041.5, 281.103856289, -23.044495936, 281.492843884, -23.018689028},
		{45, -73, 2461100.5, 341.57325084, -7.803296621, 341.911469983, -7.667175362},
		{45, -73, 2461161.5, 37.865835881, 14.898248512, 38.222740304, 15.013856167},
		{45, -73, 2461222.5, 99.614445723, 23.139620443, 100.012549675, 23.116780572},
		{45, -73, 2461284.5, 159.868969929, 8.48347413, 160.215486056, 8.345908207},
		{45, -73, 2461345.5, 215.833322363, -14.241169997, 216.195673401, -14.362153929},
		{45, -73, 2461406.5, 280.818176215, -23.064569911, 281.223916725, -23.037938031},
		{-34, 151, 2460676.5, 281.389003106, -23.022793189, 281.761278352, -22.997620694},
		{-34, 151, 2460735.5, 341.817115933, -7.703277044, 342.140724957, -7.572916434},
		{-34, 151, 2460796.5, 38.114431464, 14.980557375, 38.456479336, 15.091045295},
		{-34, 151, 2460857.5, 99.885852585, 23.126740913, 100.267228937, 23.104574039},
		{-34, 151, 2460919.5, 160.107693727, 8.392383495, 160.439735332, 8.260452611},
		{-34, 151, 2460980.5, 216.090123328, -14.322773922, 216.437641116, -14.438627027},
		{-34, 151, 2461041.5, 281.106813473, -23.042514072, 281.495883319, -23.016661915},
		{-34, 151, 2461100.5, 341.576062044, -7.800508026, 341.914356246, -7.664371052},
		{-34, 151, 2461161.5, 37.868510015, 14.901797321, 38.225508778, 15.017376297},
		{-34, 151, 2461222.5, 99.61729643, 23.143274608, 100.015499201, 23.120389323},
		{-34, 151, 2461284.5, 159.871610738, 8.486820359, 160.218212794, 8.349238075},
		{-34, 151, 2461345.5, 215.835891123, -14.238786257, 216.198332224, -14.359744095},
		{-34, 151, 2461406.5, 280.821132669, -23.062590093, 281.226955237, -23.035912641}

	};

	TArray<FRADECCase> MoonCases = {
		{45, -73, 2460676.5, 295.550065639, -26.51230532, 295.927202574, -26.455095983},
		{45, -73, 2460735.5, 353.514048429, -3.551922632, 353.832756801, -3.415063666},
		{45, -73, 2460796.5, 83.876266205, 28.033491408, 84.270931777, 28.050736742},
		{45, -73, 2460857.5, 169.35359541, 3.984104255, 169.681173507, 3.845367211},
		{45, -73, 2460919.5, 254.647315986, -28.870225506, 255.053915622, -28.910833615},
		{45, -73, 2460980.5, 338.736609293, -10.760909902, 339.081138338, -10.6265078},
		{45, -73, 2461041.5, 63.996073633, 25.95014798, 64.39915318, 26.016496377},
		{45, -73, 2461100.5, 132.450211142, 20.132070571, 132.832441864, 20.03344587},
		{45, -73, 2461161.5, 209.087022844, -16.823867898, 209.452500529, -16.955395741},
		{45, -73, 2461222.5, 292.708450583, -25.853336092, 293.122591602, -25.796509556},
		{45, -73, 2461284.5, 22.20780749, 13.014707786, 22.568272303, 13.154965913},
		{45, -73, 2461345.5, 120.013311159, 22.302517965, 120.413916386, 22.228326227},
		{45, -73, 2461406.5, 200.771229642, -14.157829304, 201.131860586, -14.298963907},
		{-34, 151, 2460676.5, 296.934791708, -25.683302772, 297.309111593, -25.623047945},
		{-34, 151, 2460735.5, 354.821687985, -2.308336389, 355.139971852, -2.171070119},
		{-34, 151, 2460796.5, 85.492190175, 29.083747564, 85.890473689, 29.097099188},
		{-34, 151, 2460857.5, 170.503832089, 5.100761189, 170.831756617, 4.961581974},
		{-34, 151, 2460919.5, 255.514251444, -27.361146396, 255.916449084, -27.399486143},
		{-34, 151, 2460980.5, 339.033496391, -9.325215012, 339.376400489, -9.190400865},
		{-34, 151, 2461041.5, 63.640354929, 26.468106165, 64.044554413, 26.535312125},
		{-34, 151, 2461100.5, 131.871847173, 20.834368828, 132.255851191, 20.736920983},
		{-34, 151, 2461161.5, 208.234139776, -15.441890914, 208.597092266, -15.574474162},
		{-34, 151, 2461222.5, 291.450810483, -24.571639361, 291.861650914, -24.517971035},
		{-34, 151, 2461284.5, 20.792794444, 14.196570564, 21.153693255, 14.338107425},
		{-34, 151, 2461345.5, 118.81639891, 23.785680852, 119.221892979, 23.714138022},
		{-34, 151, 2461406.5, 200.077876949, -13.292262924, 200.437446402, -13.434117874}

	};

	TArray<FRADECCase> MercuryCases = {
		{45, -73, 2460676.5, 258.701014892, -21.913687369, 259.070322426, -21.942479019},
		{45, -73, 2460735.5, 356.180203886, -1.406127045, 356.498165327, -1.268527823},
		{45, -73, 2460796.5, 15.076036878, 3.313005896, 15.397836546, 3.447883771},
		{45, -73, 2460857.5, 127.240303874, 19.62854359, 127.603259053, 19.545251021},
		{45, -73, 2460919.5, 149.447687516, 14.031099884, 149.790727404, 13.910637168},
		{45, -73, 2460980.5, 239.344215428, -23.411512087, 239.723596534, -23.485822225},
		{45, -73, 2461041.5, 268.12988567, -23.995778853, 268.522801319, -24.002204405},
		{45, -73, 2461100.5, 350.994560562, -0.30064899, 351.326518202, -0.158464807},
		{45, -73, 2461161.5, 24.444390399, 8.105111636, 24.78641071, 8.237964373},
		{45, -73, 2461222.5, 117.321369044, 18.512926451, 117.701705926, 18.447150224},
		{45, -73, 2461284.5, 164.315669782, 8.38759616, 164.66053524, 8.2463861},
		{45, -73, 2461345.5, 223.3539276, -18.193055818, 223.727684714, -18.301860008},
		{45, -73, 2461406.5, 280.50067475, -24.808595678, 280.911823191, -24.782794479},
		{-34, 151, 2460676.5, 258.702522778, -21.912212556, 259.071948008, -21.940978965},
		{-34, 151, 2460735.5, 356.183076167, -1.403394864, 356.501087326, -1.265792415},
		{-34, 151, 2460796.5, 15.077543936, 3.316292796, 15.39946052, 3.45116636},
		{-34, 151, 2460857.5, 127.24438019, 19.632115713, 127.607370534, 19.548773837},
		{-34, 151, 2460919.5, 149.449514774, 14.034080435, 149.792659912, 13.913596309},
		{-34, 151, 2460980.5, 239.34784024, -23.4091524, 239.727268596, -23.483407817},
		{-34, 151, 2461041.5, 268.131558021, -23.994564231, 268.524580916, -24.000954569},
		{-34, 151, 2461100.5, 350.998904025, -0.296303876, 351.330919986, -0.154117408},
		{-34, 151, 2461161.5, 24.446004081, 8.107876647, 24.788132727, 8.24071724},
		{-34, 151, 2461222.5, 117.326899818, 18.518372466, 117.707299048, 18.452546576},
		{-34, 151, 2461284.5, 164.317729625, 8.390024548, 164.662672096, 8.248798356},
		{-34, 151, 2461345.5, 223.358209924, -18.189752589, 223.732044492, -18.298517604},
		{-34, 151, 2461406.5, 280.502711282, -24.807318094, 280.913945742, -24.781471152}

	};

	TArray<FRADECCase> VenusCases = {
		{45, -73, 2460676.5, 330.058978942, -13.707094935, 330.392867308, -13.588919093},
		{45, -73, 2460735.5, 6.947739333, 10.302144032, 7.268453398, 10.44000749},
		{45, -73, 2460796.5, 359.384771101, 0.924797056, 359.705300047, 1.064026023},
		{45, -73, 2460857.5, 53.965358635, 16.614169177, 54.322618377, 16.698664674},
		{45, -73, 2460919.5, 129.637001686, 18.588172479, 129.99937902, 18.499597432},
		{45, -73, 2460980.5, 201.087304178, -7.278852528, 201.42026434, -7.412105644},
		{45, -73, 2461041.5, 279.664412844, -23.645478598, 280.05538195, -23.62322685},
		{45, -73, 2461100.5, 353.973989131, -4.085763061, 354.307134948, -3.942574394},
		{45, -73, 2461161.5, 65.995289329, 22.512813351, 66.38423237, 22.573651475},
		{45, -73, 2461222.5, 142.817846972, 16.55352469, 143.18226491, 16.437712863},
		{45, -73, 2461284.5, 199.804000711, -12.013178538, 200.154497946, -12.153127061},
		{45, -73, 2461345.5, 202.160581736, -14.52892252, 202.515476298, -14.666589513},
		{45, -73, 2461406.5, 231.601317442, -15.274568033, 231.978114084, -15.369021848},
		{-34, 151, 2460676.5, 330.06370795, -13.70295336, 330.397569245, -13.58474196},
		{-34, 151, 2460735.5, 6.957836126, 10.311388212, 7.27858562, 10.449227258},
		{-34, 151, 2460796.5, 359.386056042, 0.931664222, 359.706709561, 1.070892443},
		{-34, 151, 2460857.5, 53.965633699, 16.618383374, 54.323034646, 16.702875376},
		{-34, 151, 2460919.5, 129.63786726, 18.591096585, 130.000376999, 18.502506774},
		{-34, 151, 2460980.5, 201.08839478, -7.277206706, 201.421464503, -7.410450739},
		{-34, 151, 2461041.5, 279.666084864, -23.644374619, 280.057142082, -23.622080645},
		{-34, 151, 2461100.5, 353.975895714, -4.083973116, 354.309095112, -3.940775961},
		{-34, 151, 2461161.5, 65.997835732, 22.515010118, 66.386817728, 22.575795107},
		{-34, 151, 2461222.5, 142.821322512, 16.556305109, 143.185735739, 16.440452172},
		{-34, 151, 2461284.5, 199.810433209, -12.007777548, 200.160931415, -12.147693609},
		{-34, 151, 2461345.5, 202.167293255, -14.520980824, 202.522295804, -14.658624963},
		{-34, 151, 2461406.5, 231.601394536, -15.271440957, 231.978315877, -15.365894426}

	};
	TArray<FRADECCase> MarsCases = {
		{45, -73, 2460676.5, 124.752130317, 23.621856101, 125.128585618, 23.54266963},
		{45, -73, 2460735.5, 108.712537064, 25.894286567, 109.103997935, 25.851203412},
		{45, -73, 2460796.5, 128.113320974, 20.89033865, 128.48032252, 20.8052036},
		{45, -73, 2460857.5, 159.492537588, 9.72721132, 159.826064785, 9.595828052},
		{45, -73, 2460919.5, 194.426032143, -5.893242915, 194.755179046, -6.030825407},
		{45, -73, 2460980.5, 234.729150545, -19.893715678, 235.097463156, -19.977405294},
		{45, -73, 2461041.5, 283.488275413, -23.752242131, 283.878832947, -23.72061577},
		{45, -73, 2461100.5, 330.928417925, -13.062337646, 331.276840705, -12.937283835},
		{45, -73, 2461161.5, 15.090603999, 5.477819437, 15.428021914, 5.618480337},
		{45, -73, 2461222.5, 59.125663454, 20.100295002, 59.508972175, 20.176950325},
		{45, -73, 2461284.5, 104.203624574, 23.329403908, 104.606710009, 23.29474441},
		{45, -73, 2461345.5, 141.196096381, 16.879620538, 141.569813206, 16.763656997},
		{45, -73, 2461406.5, 162.310236775, 11.040331692, 162.670637884, 10.895064212},
		{-34, 151, 2460676.5, 124.746569791, 23.625534025, 125.122998101, 23.54640737},
		{-34, 151, 2460735.5, 108.712593913, 25.895735608, 109.103918052, 25.852651676},
		{-34, 151, 2460796.5, 128.115057455, 20.891713175, 128.481959676, 20.806544989},
		{-34, 151, 2460857.5, 159.494265726, 9.728687117, 159.827750392, 9.597282518},
		{-34, 151, 2460919.5, 194.427571775, -5.89190975, 194.756732778, -6.029478031},
		{-34, 151, 2460980.5, 234.730532768, -19.892736886, 235.09890366, -19.976384847},
		{-34, 151, 2461041.5, 283.489529439, -23.751431609, 283.880168877, -23.719761808},
		{-34, 151, 2461100.5, 330.929434872, -13.061307096, 331.277953029, -12.936233168},
		{-34, 151, 2461161.5, 15.091278519, 5.479320704, 15.428812399, 5.619975742},
		{-34, 151, 2461222.5, 59.125953109, 20.102229478, 59.509399976, 20.178876742},
		{-34, 151, 2461284.5, 104.203389266, 23.33167139, 104.606618773, 23.297017882},
		{-34, 151, 2461345.5, 141.194837151, 16.88228557, 141.568674923, 16.766341903},
		{-34, 151, 2461406.5, 162.306615641, 11.0438983, 162.66706909, 10.898655857}

	};

	TArray<FRADECCase> JupiterCases = {
		{45, -73, 2460676.5, 71.504233474, 21.740533712, 71.882588255, 21.787092219},
		{45, -73, 2460735.5, 70.472418256, 21.822329178, 70.848194084, 21.871306593},
		{45, -73, 2460796.5, 80.244647738, 22.894430445, 80.623270641, 22.920068387},
		{45, -73, 2460857.5, 94.859312337, 23.231059628, 95.241508879, 23.221131946},
		{45, -73, 2460919.5, 108.97774665, 22.256747445, 109.359453086, 22.2125826},
		{45, -73, 2460980.5, 116.537259646, 21.283569659, 116.920954126, 21.220703674},
		{45, -73, 2461041.5, 112.729769019, 22.034216083, 113.124776108, 21.978748746},
		{45, -73, 2461100.5, 106.197415482, 22.947323099, 106.597860168, 22.908007483},
		{45, -73, 2461161.5, 110.14383235, 22.538721957, 110.538018254, 22.490091304},
		{45, -73, 2461222.5, 122.065001495, 20.619851672, 122.448692213, 20.543546726},
		{45, -73, 2461284.5, 135.953015101, 17.323596706, 136.324530592, 17.218897853},
		{45, -73, 2461345.5, 146.496888475, 14.187816108, 146.862473697, 14.063830396},
		{45, -73, 2461406.5, 148.628512097, 13.705445176, 149.000373842, 13.575403953},
		{-34, 151, 2460676.5, 71.503923957, 21.740924954, 71.882151442, 21.787500417},
		{-34, 151, 2460735.5, 70.472892342, 21.822701318, 70.848563201, 21.871647921},
		{-34, 151, 2460796.5, 80.24529337, 22.894914128, 80.623916311, 22.92050116},
		{-34, 151, 2460857.5, 94.859747003, 23.231677417, 95.242042704, 23.221713223},
		{-34, 151, 2460919.5, 108.977723146, 22.257472687, 109.35956851, 22.213309415},
		{-34, 151, 2460980.5, 116.536631685, 21.284309133, 116.920407544, 21.221481853},
		{-34, 151, 2461041.5, 112.728987709, 22.034740641, 113.123931586, 21.979316928},
		{-34, 151, 2461100.5, 106.197458277, 22.947644546, 106.597766387, 22.908326222},
		{-34, 151, 2461161.5, 110.144416122, 22.53911313, 110.538532743, 22.490439414},
		{-34, 151, 2461222.5, 122.065572062, 20.620375698, 122.449304421, 20.544027063},
		{-34, 151, 2461284.5, 135.953265947, 17.324222082, 136.324902555, 17.21950669},
		{-34, 151, 2461345.5, 146.496618751, 14.188498823, 146.862324408, 14.064526107},
		{-34, 151, 2461406.5, 148.62773939, 13.70611726, 148.999620203, 13.576106579}

	};

	TArray<FRADECCase> SaturnCases = {
		{45, -73, 2460676.5, 346.193302806, -8.050496356, 346.516315545, -7.916945615},
		{45, -73, 2460735.5, 351.880411724, -5.582861784, 352.200104882, -5.446606235},
		{45, -73, 2460796.5, 358.500986928, -2.831250853, 358.821908937, -2.691853528},
		{45, -73, 2460857.5, 2.222137903, -1.474780324, 2.549998653, -1.33204677},
		{45, -73, 2460919.5, 0.673546322, -2.410673703, 1.008209223, -2.265152862},
		{45, -73, 2460980.5, 356.785859049, -4.077943276, 357.122205926, -3.932660525},
		{45, -73, 2461041.5, 357.047017102, -3.740863773, 357.380923704, -3.59658294},
		{45, -73, 2461100.5, 2.096879466, -1.41381077, 2.429009973, -1.269567422},
		{45, -73, 2461161.5, 8.922711402, 1.484106401, 9.257459525, 1.628004407},
		{45, -73, 2461222.5, 13.63962989, 3.252364079, 13.982543899, 3.397009509},
		{45, -73, 2461284.5, 13.270042469, 2.805805178, 13.619855137, 2.953632262},
		{45, -73, 2461345.5, 9.237158706, 1.060598365, 9.588152406, 1.211034063},
		{45, -73, 2461406.5, 8.272343096, 0.896636004, 8.621269095, 1.046433615},
		{-34, 151, 2460676.5, 346.193621799, -8.050172484, 346.51658057, -7.91660566},
		{-34, 151, 2460735.5, 351.880704822, -5.582587057, 352.200457007, -5.446320736},
		{-34, 151, 2460796.5, 358.501038887, -2.830968292, 358.822086332, -2.691570068},
		{-34, 151, 2460857.5, 2.221863668, -1.47445847, 2.549810055, -1.331727107},
		{-34, 151, 2460919.5, 0.673166094, -2.41031103, 1.007785136, -2.264794911},
		{-34, 151, 2460980.5, 356.785819259, -4.07756691, 357.12203957, -3.932285013},
		{-34, 151, 2461041.5, 357.04730844, -3.740534543, 357.381140464, -3.596247185},
		{-34, 151, 2461100.5, 2.09719922, -1.4135176, 2.429367286, -1.269271579},
		{-34, 151, 2461161.5, 8.922823983, 1.48441338, 9.257691836, 1.628310127},
		{-34, 151, 2461222.5, 13.639414062, 3.252705007, 13.982431163, 3.397354939},
		{-34, 151, 2461284.5, 13.269639873, 2.806159539, 13.619436384, 2.953993313},
		{-34, 151, 2461345.5, 9.237029455, 1.06095003, 9.587902937, 1.211386644},
		{-34, 151, 2461406.5, 8.272594443, 0.896958746, 8.621427259, 1.04675473}

	};
	TArray<FRADECCase> UranusCases = {
		{45, -73, 2460676.5, 50.958842926, 18.346349986, 51.318825303, 18.436480328},
		{45, -73, 2460735.5, 50.956594042, 18.367050045, 51.313555154, 18.456730273},
		{45, -73, 2460796.5, 53.700098475, 19.03511698, 54.05800871, 19.119029312},
		{45, -73, 2460857.5, 57.196313179, 19.805126797, 57.561966164, 19.882926551},
		{45, -73, 2460919.5, 59.003541739, 20.169385163, 59.378963968, 20.245164321},
		{45, -73, 2460980.5, 57.774044699, 19.923983813, 58.155234999, 20.003682483},
		{45, -73, 2461041.5, 55.356337996, 19.424577722, 55.737204005, 19.509579435},
		{45, -73, 2461100.5, 55.130638675, 19.393555365, 55.508521079, 19.478653132},
		{45, -73, 2461161.5, 57.762102328, 19.967560853, 58.140317539, 20.046555292},
		{45, -73, 2461222.5, 61.3433689, 20.666467347, 61.729034996, 20.738191542},
		{45, -73, 2461284.5, 63.384130701, 21.0266425, 63.779545996, 21.095279318},
		{45, -73, 2461345.5, 62.344686655, 20.850502341, 62.746536253, 20.922613596},
		{45, -73, 2461406.5, 59.850875075, 20.404145378, 60.252879078, 20.482288495},
		{-34, 151, 2460676.5, 50.958844129, 18.346446255, 51.318693008, 18.436576314},
		{-34, 151, 2460735.5, 50.956751986, 18.367168621, 51.313643708, 18.456814299},
		{-34, 151, 2460796.5, 53.700259837, 19.03528346, 54.058229257, 19.119158127},
		{-34, 151, 2460857.5, 57.196336334, 19.805327359, 57.562123533, 19.883121474},
		{-34, 151, 2460919.5, 59.00339039, 20.169572466, 59.378896493, 20.245386448},
		{-34, 151, 2460980.5, 57.773860934, 19.924113786, 58.154998258, 20.00385272},
		{-34, 151, 2461041.5, 55.356323202, 19.424670434, 55.737055222, 19.509675319},
		{-34, 151, 2461100.5, 55.130790635, 19.393668517, 55.508594573, 19.478731701},
		{-34, 151, 2461161.5, 57.76227081, 19.967724375, 58.140536269, 20.046678043},
		{-34, 151, 2461222.5, 61.343406076, 20.666668927, 61.72920544, 20.738383817},
		{-34, 151, 2461284.5, 63.383988665, 21.026834374, 63.77949674, 21.095505011},
		{-34, 151, 2461345.5, 62.344495378, 20.850635745, 62.746302422, 20.922790424},
		{-34, 151, 2461406.5, 59.850843649, 20.404235312, 60.252714134, 20.482385448}

	};
	TArray<FRADECCase> NeptuneCases = {
		{45, -73, 2460676.5, 357.71216588, -2.392439984, 358.031688446, -2.254080454},
		{45, -73, 2460735.5, 359.167991657, -1.736844271, 359.485909405, -1.598844141},
		{45, -73, 2460796.5, 1.192750699, -0.87507635, 1.513216448, -0.735692422},
		{45, -73, 2460857.5, 2.190625167, -0.493740773, 2.518549968, -0.351100833},
		{45, -73, 2460919.5, 1.458579764, -0.858962655, 1.79317016, -0.713503712},
		{45, -73, 2460980.5, 0.01761989, -1.483934501, 0.353506435, -1.338125072},
		{45, -73, 2461041.5, 359.743997579, -1.563440181, 0.077642097, -1.418669373},
		{45, -73, 2461100.5, 1.145398107, -0.928077552, 1.477557192, -0.783779112},
		{45, -73, 2461161.5, 3.180066345, -0.060552763, 3.514560624, 0.084785784},
		{45, -73, 2461222.5, 4.244408411, 0.348534061, 4.586370055, 0.496975551},
		{45, -73, 2461284.5, 3.576510811, 0.00953862, 3.925162416, 0.160911002},
		{45, -73, 2461345.5, 2.124410674, -0.621217264, 2.474530064, -0.469230717},
		{45, -73, 2461406.5, 1.776313758, -0.731732851, 2.124201953, -0.580757931},
		{-34, 151, 2460676.5, 357.712259273, -2.392335187, 358.031706669, -2.253971614},
		{-34, 151, 2460735.5, 359.168097684, -1.736745618, 359.486059641, -1.598742162},
		{-34, 151, 2460796.5, 1.192773389, -0.874977422, 1.513363432, -0.735593175},
		{-34, 151, 2460857.5, 2.1905386, -0.493637817, 2.518548918, -0.350998456},
		{-34, 151, 2460919.5, 1.458466232, -0.858855894, 1.793014396, -0.713398443},
		{-34, 151, 2460980.5, 0.01760119, -1.483826061, 0.353362411, -1.338017088},
		{-34, 151, 2461041.5, 359.744088264, -1.56333606, 0.077653644, -1.418562783},
		{-34, 151, 2461100.5, 1.145505624, -0.927978166, 1.477704261, -0.783678078},
		{-34, 151, 2461161.5, 3.180093342, -0.060452288, 3.514710832, 0.084886206},
		{-34, 151, 2461222.5, 4.244325178, 0.348638087, 4.586376047, 0.497080362},
		{-34, 151, 2461284.5, 3.576395753, 0.009644747, 3.925010117, 0.161017478},
		{-34, 151, 2461345.5, 2.124387088, -0.621110471, 2.474382116, -0.469124122},
		{-34, 151, 2461406.5, 1.7764016, -0.731629542, 2.124206803, -0.580653645}

	};

	TArray<SpecificBodyCases> AllTestCases = {
		// Body                        RelErr%  AbsErr"  Cases
		{EVSOP87BodyType::Sun,     0.1, 2.0, SunCases},
		{EVSOP87BodyType::Moon,    0.1, 2.0, MoonCases},
		{EVSOP87BodyType::Mercury, 0.1, 2.0, MercuryCases},
		{EVSOP87BodyType::Venus,   0.1, 2.0, VenusCases},
		{EVSOP87BodyType::Mars,    0.1, 2.0, MarsCases},
		{EVSOP87BodyType::Jupiter, 0.1, 2.0, JupiterCases},
		{EVSOP87BodyType::Saturn,  0.1, 2.0, SaturnCases},
		{EVSOP87BodyType::Uranus,  0.1, 2.0, UranusCases},
		{EVSOP87BodyType::Neptune, 0.1, 2.0, NeptuneCases}
	};

	bool success = true;

	for (SpecificBodyCases BodyCases : AllTestCases)
	{
		// Reset error statistics (tracked in arcseconds)
		double AverageErrorArcsec = 0.0;
		double MaxErrorArcsec = 0.0;
		int SampleCount = 0;

		// Tolerance: pass if absolute error <= max(relative tol, absolute tol)
		// This handles values near 0° cleanly: the absolute floor kicks in so that
		// e.g. a DEC of -0.15° isn't subject to an impossibly tight 0.1% = 0.54" tolerance.
		const double AbsTolDeg = BodyCases.AbsoluteErrorArcsec / 3600.0;
		const double RelTolFactor = BodyCases.RelativeErrorPercent / 100.0;

		// Get Body Name
		FString BodyName = FString("Unknown");
		if (UEnum* EnumClass = StaticEnum<EVSOP87BodyType>())
		{
			BodyName = EnumClass->GetNameStringByValue(static_cast<int64>(BodyCases.VSOP87BodyType));
		}

		// Helper: check one angle (degrees), record stats, emit error on failure
		auto CheckAngle = [&](const FRADECCase& TheCase, double Computed, double Expected, const TCHAR* Label)
		{
			double AbsErrDeg = FMath::Abs(Computed - Expected);
			double ToleranceDeg = FMath::Max(AbsTolDeg, FMath::Abs(Expected) * RelTolFactor);

			double AbsErrArcsec = AbsErrDeg * 3600.0;
			double TolArcsec = ToleranceDeg * 3600.0;

			++SampleCount;
			AverageErrorArcsec += (AbsErrArcsec - AverageErrorArcsec) / SampleCount;
			MaxErrorArcsec = FMath::Max(MaxErrorArcsec, AbsErrArcsec);

			if (AbsErrDeg > ToleranceDeg)
			{
				AddError(FString::Printf(TEXT("Case failed: %s error %.4f\" (tolerance %.4f\") - Body = %s, Case = %s"),
					Label, AbsErrArcsec, TolArcsec, *BodyName, *TheCase.ToString()));
				success = false;
			}
		};

		for (FRADECCase Case : BodyCases.Cases)
		{
			// Computed results
			double RAJ2000Hours = 0.0;         // topocentric J2000/ICRF — matches Horizons "Astrometric RA & Dec" with surface observer
			double DECJ2000Degrees = 0.0;
			double RAGeocentricHours = 0.0;    // geocentric of-date — used for 3D placement
			double DECGeocentricDegrees = 0.0;
			double RAHours = 0.0;              // topocentric of-date — matches Horizons "Apparent RA & Dec"
			double DECDegrees = 0.0;
			double DistanceBodyToEarthAU = 0.0;
			double DistanceBodyToSunAU = 0.0;
			double DistanceEarthToSunAU = 0.0;

			UCelestialMaths::GetBodyCelestialCoordinatesAU(Case.JulianDate, BodyCases.VSOP87BodyType, Case.Latitude, Case.Longitude, false, false, RAJ2000Hours, DECJ2000Degrees, RAHours, DECDegrees, RAGeocentricHours, DECGeocentricDegrees, DistanceBodyToEarthAU, DistanceBodyToSunAU, DistanceEarthToSunAU);

			CheckAngle(Case, RAJ2000Hours * 15.0, Case.ExpectedRAJ2000Degrees,  TEXT("RAJ2000"));
			CheckAngle(Case, DECJ2000Degrees,     Case.ExpectedDecJ2000Degrees, TEXT("DECJ2000"));
			CheckAngle(Case, RAHours * 15.0,      Case.ExpectedRADegrees,       TEXT("RA at Date"));
			CheckAngle(Case, DECDegrees,          Case.ExpectedDecDegrees,      TEXT("DEC at Date"));
		}

		AddInfo(FString::Printf(TEXT("RA/DEC Precision Test for %s — avg %.4f\", max %.4f\" (abs tol %.1f\", rel tol %.2f%%)"),
			*BodyName, AverageErrorArcsec, MaxErrorArcsec, BodyCases.AbsoluteErrorArcsec, BodyCases.RelativeErrorPercent));
	}

	return success;
}
