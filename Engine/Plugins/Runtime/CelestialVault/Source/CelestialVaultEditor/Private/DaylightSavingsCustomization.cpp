// Copyright Epic Games, Inc. All Rights Reserved.



#include "DaylightSavingsCustomization.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "DaylightSavings.h"
#include "IDetailChildrenBuilder.h"
#include "IPropertyUtilities.h"
#include "Widgets/Text/STextBlock.h"

// Add default functionality here for any IFDSTDayCustomization functions that are not pure virtual.
void FDaylightSavingsCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	// Capture the handle by value so we can use it in the lambda
	TSharedRef<IPropertyHandle> LocalHandle = PropertyHandle;

	HeaderRow
	.NameContent()
	[
		PropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	.MinDesiredWidth(200.f)
	[
		SNew(STextBlock)
		.Text_Lambda([LocalHandle]()
		{
			void* RawData = nullptr;
			if (LocalHandle->GetValueData(RawData) == FPropertyAccess::Success && RawData)
			{
				const FDaylightSavingsRule* DaylightSavingsRule = static_cast<const FDaylightSavingsRule*>(RawData);

				FString Caption = UDaylightSavings::ToString(*DaylightSavingsRule);

				// Specific to Celestial Vault
				// Ideally, we would like to display the date, but we need a "Year" property in the parent object.
				// And we also need to know the hemisphere, in case we are in the south hemisphere, and the End Date is on the next year!

				TOptional<FNeededDaylightSavingsValues> CelestialVaultValues = TryGetNeededDaylightSavingsValuesFromOuter(LocalHandle);
				if (CelestialVaultValues.IsSet())
				{
					int YearToUse = CelestialVaultValues.GetValue().Year;
					
					if (CelestialVaultValues.GetValue().Mode == EDaylightSavingsMode::SouthernHemisphere ||
						(CelestialVaultValues.GetValue().Mode == EDaylightSavingsMode::Automatic && CelestialVaultValues.GetValue().Latitude < 0 ))
					{
						// We are in the south hemisphere. But we need to increase the year only for the DaylightSavingsEndDay Property! 
						if (FProperty* DaylightSavingsRuleProp = LocalHandle->GetProperty())
						{
							const FName DaylightSavingsRulePropName = DaylightSavingsRuleProp->GetFName();
							if (DaylightSavingsRulePropName == "DaylightSavingsEnd")
							{
								YearToUse++;
							}
						}
					}

					FDateTime Day = UDaylightSavings::ToDate(*DaylightSavingsRule, YearToUse);
					Caption += Day.ToFormattedString(TEXT(" : (%d %b %Y)"));
					
				}
				return FText::FromString(Caption);
			}

			// In case of multi selection
			return FText();
		})
		.Font(IDetailLayoutBuilder::GetDetailFontItalic())
	];
}

void FDaylightSavingsCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	// Default behavior: show the struct’s children as normal when expanded
	uint32 NumChildren = 0;
	PropertyHandle->GetNumChildren(NumChildren);

	for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
	{
		ChildBuilder.AddProperty(PropertyHandle->GetChildHandle(ChildIndex).ToSharedRef());
	}
}

TSharedRef<IPropertyTypeCustomization> FDaylightSavingsCustomization::MakeInstance()
{
	return MakeShareable(new FDaylightSavingsCustomization);
}

TOptional<FNeededDaylightSavingsValues> FDaylightSavingsCustomization::TryGetNeededDaylightSavingsValuesFromOuter(const TSharedRef<IPropertyHandle>& PropertyHandle)
{
	TArray<UObject*> Outers;
	PropertyHandle->GetOuterObjects(Outers);

	UObject* Outer = Outers.Num() > 0 ? Outers[0] : nullptr;
	if (Outer)
	{
		FNeededDaylightSavingsValues Values;

		if (ReadInt32Property(Outer, TEXT("Year"), Values.Year) &&
			ReadDaylightSavingsModeProperty(Outer, TEXT("DaylightSavingsMode"), Values.Mode) &&
			ReadDoubleProperty(Outer, TEXT("Latitude"), Values.Latitude))
		{
			return Values;
		}    
	}
	return {};
}

bool FDaylightSavingsCustomization::ReadInt32Property(UObject* Object, FName PropertyName, int32& Out)
{
    if (Object)
    {
    	// Check Property Name
    	if (FProperty* Property = Object->GetClass()->FindPropertyByName(PropertyName))
    	{
    		// Check Property Type
    		if (const FIntProperty* IntProperty = CastField<FIntProperty>(Property))
    		{
    			Out = IntProperty->GetPropertyValue_InContainer(Object);
    			return true;
    		}
    	}    
    }
    return false;
}

bool FDaylightSavingsCustomization::ReadDoubleProperty(UObject* Object, FName PropertyName, double& Out)
{
	if (Object)
	{
		// Check Property Name
		if (FProperty* Property = Object->GetClass()->FindPropertyByName(PropertyName))
		{
			// Check Property Type
			if (const FDoubleProperty* DoubleProperty = CastField<FDoubleProperty>(Property))
			{
				Out = DoubleProperty->GetPropertyValue_InContainer(Object);
				return true;
			}
			if (const FFloatProperty* FloatProperty = CastField<FFloatProperty>(Property))
			{
				Out = static_cast<double>(FloatProperty->GetPropertyValue_InContainer(Object));
				return true;
			}
		}
	}
    return false;
}

bool FDaylightSavingsCustomization::ReadDaylightSavingsModeProperty(UObject* Object, FName PropertyName, EDaylightSavingsMode& Out)
{
    if (Object) 
    {
    	// Check Property Name
	    if (FProperty* Property = Object->GetClass()->FindPropertyByName(PropertyName))
    	{
	    	// Check Property Enum Type
    		if (const FEnumProperty* EnumProperty = CastField<FEnumProperty>(Property))
    		{
    			const void* ValuePtr = EnumProperty->ContainerPtrToValuePtr<void>(Object);
    			int64 Raw = EnumProperty->GetUnderlyingProperty()->GetSignedIntPropertyValue(ValuePtr);
    			Out = static_cast<EDaylightSavingsMode>(Raw);
    			return true;
    		}
    	}
    }
    return false;
}

