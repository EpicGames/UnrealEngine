// Copyright Epic Games, Inc. All Rights Reserved.


#include "EditorViewportCameraProvider.h"

#include "Editor.h"
#include "Subsystems/UnrealEditorSubsystem.h"

bool FEditorViewportCameraProvider::GetCamera(FVector& OutLocation, FRotator& OutRotation) const
{
	if (!GEditor) return false;

	// If we're playing (PIE), fallback to runtime behavior
	if (GEditor->GetPIEWorldContext())
	{
		static FRuntimeCameraProvider DefaultProvider;
		
		return DefaultProvider.GetCamera(OutLocation, OutRotation);
	}

	if (UUnrealEditorSubsystem* EditorSubsystem = GEditor->GetEditorSubsystem<UUnrealEditorSubsystem>())
	{
		return EditorSubsystem->GetLevelViewportCameraInfo(OutLocation, OutRotation);
	}
	return false;
}