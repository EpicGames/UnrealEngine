// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/Texture2D.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "CelestialVaultEditorUtilities.generated.h"

/**
 * 
 */
UCLASS()
class CELESTIALVAULTEDITOR_API UCelestialVaultEditorUtilities : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

	/**
     * Retrieve information about the viewport under the mouse cursor.
     * 
     * @param	Focused			If the Level editor not are in focus it will return false.
     * @param	ScreenLocation	Viewport-Space position of cursor.
     * @param	WorldLocation	Location of viewport origin (camera) in world space.
     * @param	WorldDirection	Direction of viewport (camera) in world space.
     */
	UFUNCTION(BlueprintCallable, Category = "Celestial Vault|Editor Utilities")
	static void GetViewportCursorInformation(bool& Focused, FVector2D& ScreenLocation, FVector& WorldLocation, FVector& WorldDirection);

	UFUNCTION(BlueprintCallable, Category = "Celestial Vault|Editor Utilities")
	static bool ComputeTextureMeanLuminance(UTexture2D* Texture, float& OutMean);
};
