// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DaylightSavings.h"
#include "IPropertyTypeCustomization.h"

enum class EDaylightSavingsMode : uint8;
class IPropertyHandle;

struct FNeededDaylightSavingsValues
{
	int32 Year = 0;
	EDaylightSavingsMode Mode = EDaylightSavingsMode::None;
	double Latitude = 0.0;
};


class CELESTIALVAULTEDITOR_API FDaylightSavingsCustomization : public IPropertyTypeCustomization
{
public:
	// Inherited from IPropertyTypeCustomization
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	

	// Utility function to create an instance of the Property Customization.
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

private:
	static TOptional<FNeededDaylightSavingsValues> TryGetNeededDaylightSavingsValuesFromOuter(const TSharedRef<IPropertyHandle>& PropertyHandle);
	static bool ReadInt32Property(UObject* Object, FName PropertyName, int32& Out);
	static bool ReadDoubleProperty(UObject* Object, FName PropertyName, double& Out);
	static bool ReadDaylightSavingsModeProperty(UObject* Object, FName PropertyName, EDaylightSavingsMode& Out);
};
