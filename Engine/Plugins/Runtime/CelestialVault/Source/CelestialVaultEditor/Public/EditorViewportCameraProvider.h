// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ViewportCameraProvider.h"

class FEditorViewportCameraProvider : public IViewportCameraProvider
{
public:
	virtual bool GetCamera(FVector& OutLocation, FRotator& OutRotation) const override;
};
